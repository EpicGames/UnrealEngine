{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Interchange AxF",
	"Description": "Imports AxF2UE files from X-Rite Pantora as Substrate materials via the Interchange framework.",
	"Category": "",
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Mac"
	],
	"Modules": [
		{
			"Name": "InterchangeAxF",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	],
	"Plugins": [
		{
			"Name": "Interchange",
			"Enabled": true
		},
		{
			"Name": "InterchangeAxFAssets",
			"Enabled": true
		}
	]
}