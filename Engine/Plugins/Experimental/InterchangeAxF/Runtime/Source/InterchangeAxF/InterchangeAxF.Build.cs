// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class InterchangeAxF : ModuleRules
{
    public InterchangeAxF(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		//OptimizeCode = CodeOptimization.Never;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
			}
            );


        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "Engine",
                "Slate",
                "SlateCore",
                "UnrealEd",
                "MaterialEditor",
                "InterchangeCore",
                "InterchangeEngine",
                "InterchangeImport",
                "InterchangeNodes",
                "InterchangeFactoryNodes",
                "InterchangePipelines",
                "InterchangeAxFAssets",
                "Json"
            }
            );
    }
}
