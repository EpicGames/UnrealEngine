// Copyright Epic Games, Inc. All Rights Reserved.

#include "AxFInterchangePipeline.h"
#include "AxFMaterialObjectNode.h"
#include "InterchangeGenericTexturePipeline.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangePipelineHelper.h"
#include "InterchangeSpecularProfileFactoryNode.h"
#include "InterchangeTexture2DFactoryNode.h"
#include "InterchangeTexture2DNode.h"
#include "InterchangeSpecularProfileNode.h"
#include "Material/InterchangeMaterialFactory.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Nodes/InterchangeSourceNode.h"

DEFINE_LOG_CATEGORY(LogAxFInterchangePipeline);

// Helper structure to differentiate scalar from vector parameters stored in
// UAxFMaterialObjectNode::PayloadData::ValuesMap
struct FMaterialParameterTypes
{
	TSet<FString> ScalarParameters;
	TSet<FString> VectorParameters;
};

// Map to store the names of the scalar and vector parameters of the predefined materials.
static TMap<FString, FMaterialParameterTypes> ParameterTypesPerMaterial;

static FString GetParameterNameFromUID(const FString& UID)
{
	FString LeftPart, ParameterName;
	UID.Split("_", &LeftPart, &ParameterName, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	return ParameterName;
}

UAxFInterchangePipeline::UAxFInterchangePipeline()
{
	TexturePipeline = CreateDefaultSubobject<UInterchangeGenericTexturePipeline>(TEXT("TexturePipeline"));
	TexturePipeline->bDetectNormalMapTexture = false;
	TexturePipeline->bAllowNonPowerOfTwo = true;
}

void UAxFInterchangePipeline::ExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer,
											  TArray<UInterchangeSourceData*> const& SourceDatas,
											  FString const& ContentBasePath)
{
	using namespace UE::Interchange::Materials;

	if (TexturePipeline)
	{
		TexturePipeline->ScriptedExecutePipeline(BaseNodeContainer, SourceDatas, ContentBasePath);
	}

	// Find all translated and factory nodes we need for this pipeline
	BaseNodeContainer->IterateNodes(
		[this](FString const& NodeUid, UInterchangeBaseNode* Node) {
			switch (Node->GetNodeContainerType())
			{
			case EInterchangeNodeContainerType::TranslatedAsset:
			{
				if (UAxFMaterialObjectNode* CustomDemoObjectNode = Cast<UAxFMaterialObjectNode>(Node))
				{
					AxFObjectNodes.Add(CustomDemoObjectNode);
				}
				else if (UInterchangeTexture2DNode* Texture2DNode = Cast<UInterchangeTexture2DNode>(Node))
				{
					float WidthMM = 0.f, HeightMM = 0.f;
					Texture2DNode->GetAttribute(TEXT("WidthMM"), WidthMM);
					Texture2DNode->GetAttribute(TEXT("HeightMM"), HeightMM);

					FString ParameterName = GetParameterNameFromUID(Texture2DNode->GetUniqueID());
					FAxFTextureMap& Pair = TexturesMap.FindOrAdd(ParameterName);
					Pair.SizeMM = FVector2f{ WidthMM, HeightMM };
				}
			}
			break;
			case EInterchangeNodeContainerType::FactoryData:
			{
				if (UInterchangeTexture2DFactoryNode* Texture2DFactoryNode = Cast<UInterchangeTexture2DFactoryNode>(Node))
				{
					Texture2DFactoryNodes.Add(Texture2DFactoryNode);
				}
			}
			break;
			default:;
			}
		});

	if (AxFObjectNodes.IsEmpty())
	{
		return;
	}
	TObjectPtr<UAxFMaterialObjectNode> AxFNode = AxFObjectNodes[0];
	if (!AxFNode)
	{
		UE_LOGF(LogAxFInterchangePipeline, Error, "AxF Interchange Node is not properly constructed!");
		return;
	}

	// Get Specular Profile node using string attribute set in AxFTranslator
	const UInterchangeSpecularProfileNode* SpecularProfileNode = nullptr;
	{
		if (FString SpecularProfileUID;
			AxFNode->GetStringAttribute(SubstrateMaterial::SpecularProfile.ToString(), SpecularProfileUID))
		{
			SpecularProfileNode = Cast<UInterchangeSpecularProfileNode>(BaseNodeContainer->GetNode(SpecularProfileUID));
			if (SpecularProfileNode)
			{
				CreateSpecularProfileFactoryNode(SpecularProfileNode, BaseNodeContainer, AxFNode);
			}
			else
			{
				UE_LOGF(LogAxFInterchangePipeline, Error, "Couldn't find the SpecularProfileNode: %ls", *SpecularProfileUID);
				return;
			}
		}
	}

	const TArray<EAxFFeature>& Features = AxFNode->PayloadData.UsedFeatures;
	const TMap<FString, FLinearColor>& ValuesMap = AxFNode->PayloadData.ValuesMap;

	FString BaseMaterialName;
	if (Features.Contains(EAxFFeature::CPA2))
	{
		BaseMaterialName += TEXT("CPA2");
		BaseMaterialName += (Features.Contains(EAxFFeature::Flakes) ? TEXT("_Flakes") : TEXT(""));
	}
	else
	{
		BaseMaterialName += "SVBRDF";
		{
			BaseMaterialName += (Features.Contains(EAxFFeature::Alpha) ? TEXT("_Alpha") : TEXT(""));
			BaseMaterialName += (Features.Contains(EAxFFeature::Heightmap) ? TEXT("_POM") : TEXT(""));
		}
	}

	FString FullBaseMaterialPath =
		TEXT("/InterchangeAxFAssets/BaseMaterials/") + BaseMaterialName + TEXT(".") + BaseMaterialName;

	if (!ensure(ParameterTypesPerMaterial.Contains(BaseMaterialName)))
	{
		UE_LOGF(LogAxFInterchangePipeline, Error, "Failed to create material instance %ls: %ls was not found", *AxFNode->GetDisplayLabel(), *FullBaseMaterialPath);
		return;
	}

	const FMaterialParameterTypes& ParameterTypes = ParameterTypesPerMaterial[BaseMaterialName];

	FString MaterialInstanceUID = TEXT("MI_") + AxFNode->GetDisplayLabel();

	UInterchangeMaterialInstanceFactoryNode* MaterialInstance = NewObject<UInterchangeMaterialInstanceFactoryNode>(BaseNodeContainer, FName(MaterialInstanceUID));
	BaseNodeContainer->SetupNode(MaterialInstance, MaterialInstanceUID, AxFNode->GetDisplayLabel(), EInterchangeNodeContainerType::FactoryData);

	for (const EAxFFeature Feature : Features)
	{
		const FString Value = UEnum::GetValueAsString(Feature);
		const FString AttributeKey = UInterchangeShaderPortsAPI::MakeInputValueKey(TEXT("Has") + Value);

		MaterialInstance->AddBooleanAttribute(AttributeKey, true);
	}

	TArray<FString> ParameterNames;
	ValuesMap.GetKeys(ParameterNames);
	constexpr float Epsilon = 1e-6f;
	for (FString const& Param : ParameterNames)
	{
		if (Param == TEXT("FlakePatchSizeMM"))
		{
			const FString XAttributeKey = UInterchangeShaderPortsAPI::MakeInputValueKey(TEXT("WidthMM"));
			const FString YAttributeKey = UInterchangeShaderPortsAPI::MakeInputValueKey(TEXT("HeightMM"));
			const FLinearColor& FlakePatchSizeMM = ValuesMap[Param];
			MaterialInstance->AddFloatAttribute(XAttributeKey, FlakePatchSizeMM.R + Epsilon);
			MaterialInstance->AddFloatAttribute(YAttributeKey, FlakePatchSizeMM.G + Epsilon);
		}

		// NOTE: Disable texture path in the case that we have a value for this parameter
		FString TextureToggleAttributeKey = UInterchangeShaderPortsAPI::MakeInputValueKey(TEXT("Has") + Param + TEXT("Texture"));
		MaterialInstance->AddBooleanAttribute(TextureToggleAttributeKey, false);

		const FString AttributeKey = UInterchangeShaderPortsAPI::MakeInputValueKey(Param);
		const FLinearColor& Value = ValuesMap[Param];
		if (ParameterTypes.ScalarParameters.Contains(Param))
		{
			MaterialInstance->AddFloatAttribute(AttributeKey, Value.R);
		}
		else
		{
			MaterialInstance->AddLinearColorAttribute(AttributeKey, Value);
		}
	}

	if (bUseTriplanarMappingByDefault)
	{
		FString GeoUVsToggleAttributeKey = UInterchangeShaderPortsAPI::MakeInputValueKey(TEXT("Use Geo UVs"));
		MaterialInstance->AddBooleanAttribute(GeoUVsToggleAttributeKey, false);
	}

	for (UInterchangeTexture2DFactoryNode* Texture2DFactoryNode : Texture2DFactoryNodes)
	{
		Texture2DFactoryNode->SetCustomSubPath(AxFNode->GetDisplayLabel());

		const FString Param = GetParameterNameFromUID(Texture2DFactoryNode->GetUniqueID());

		{
			FString AttributeKey = UInterchangeShaderPortsAPI::MakeInputValueKey(TEXT("Has") + Param + TEXT("Texture"));
			MaterialInstance->AddBooleanAttribute(AttributeKey, true);
		}

		{
			FString AttributeKey = UInterchangeShaderPortsAPI::MakeInputValueKey(Param + TEXT("Texture"));
			MaterialInstance->AddStringAttribute(AttributeKey, Texture2DFactoryNode->GetUniqueID());
		}

		if (Param == TEXT("HeightMap"))
		{
			FString AttributeKey = UInterchangeShaderPortsAPI::MakeInputValueKey(Param);
			MaterialInstance->AddStringAttribute(AttributeKey, Texture2DFactoryNode->GetUniqueID());
		}

		if (Param != TEXT("CPA2ColorTable"))
		{
			const FVector2f UVScale = TexturesMap.FindOrAdd(Param).SizeMM;
			FString UVScaleXAttributeKey = UInterchangeShaderPortsAPI::MakeInputValueKey(TEXT("WidthMM"));
			FString UVScaleYAttributeKey = UInterchangeShaderPortsAPI::MakeInputValueKey(TEXT("HeightMM"));
			MaterialInstance->AddFloatAttribute(UVScaleXAttributeKey, UVScale.X);
			MaterialInstance->AddFloatAttribute(UVScaleYAttributeKey, UVScale.Y);
			FString UVOffsetYAttributeKey = UInterchangeShaderPortsAPI::MakeInputValueKey(TEXT("OffsetOnY"));
			const float UVOffsetY = FMath::IsNearlyZero(UVScale.Y) ? 0.f : 1.0f - 100.0f / UVScale.Y;
			MaterialInstance->AddFloatAttribute(UVOffsetYAttributeKey, UVOffsetY);
		}

		MaterialInstance->AddFactoryDependencyUid(Texture2DFactoryNode->GetUniqueID());
	}
	Texture2DFactoryNodes.Empty();

	if (SpecularProfileNode != nullptr)
	{
		const FString SpecularProfileUid = SpecularProfileNode->GetUniqueID();
		MaterialInstance->AddStringAttribute(SubstrateMaterial::SpecularProfile.ToString(), SpecularProfileUid);
		MaterialInstance->AddFactoryDependencyUid(UInterchangeFactoryBaseNode::BuildFactoryNodeUid(SpecularProfileNode->GetUniqueID()));
	}

	MaterialInstance->SetCustomParent(FullBaseMaterialPath);
	MaterialInstance->MarkPackageDirty();
	MaterialInstance->PostEditChange();

	BaseNodeContainer->IterateNodesOfType<UInterchangeFactoryBaseNode>(
		[ReimportStrategyClosure = ReimportStrategy](const FString& NodeUid, UInterchangeFactoryBaseNode* FactoryNode)
		{
			// Make sure all factory nodes have the specified strategy
			FactoryNode->SetReimportStrategyFlags(ReimportStrategyClosure);
		}
	);
}

void UAxFInterchangePipeline::ExecutePostImportPipeline(
	UInterchangeBaseNodeContainer const* BaseNodeContainer, FString const& NodeKey, UObject* CreatedAsset,
	bool bIsAReimport)
{
	if (TexturePipeline)
	{
		TexturePipeline->ScriptedExecutePostImportPipeline(BaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
	}

	UMaterialInstanceConstant* Instance = Cast<UMaterialInstanceConstant>(CreatedAsset);
	if (Instance != nullptr && !AxFObjectNodes.IsEmpty())
	{
		TObjectPtr<UAxFMaterialObjectNode>* FoundData = AxFObjectNodes.FindByPredicate(
			[Instance](UAxFMaterialObjectNode const* Node) { return Node->GetDisplayLabel() == Instance->GetName(); });

		if (FoundData == nullptr)
		{
			UE_LOGF(
				LogAxFInterchangePipeline, Error,
				"Couldn't find appropriate AxF translated node to create Material Instance.")
				return;
		}

		UAxFMaterialObjectNode* AxFData = *FoundData;

		const TArray<EAxFFeature>& Features = AxFData->PayloadData.UsedFeatures;

		if (SpecularProfile != nullptr)
		{
			Instance->bOverrideSpecularProfile = true;
			Instance->SpecularProfileOverride = SpecularProfile;
			SpecularProfile = nullptr;
		}

		if (!Instance->MarkPackageDirty())
		{
			UE_LOGF(
				LogAxFInterchangePipeline, Warning,
				"Failed to mark package as dirty. Some changes to the material instance %ls might not be saved.",
				*Instance->GetName());
		}

		Instance->PostEditChange();

		AxFObjectNodes.RemoveSingle(AxFData);
	}

	Super::ExecutePostImportPipeline(BaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
}

void UAxFInterchangePipeline::ExecutePostFactoryPipeline(
	UInterchangeBaseNodeContainer const* BaseNodeContainer, FString const& NodeKey, UObject* CreatedAsset,
	bool bIsAReimport)
{
	if (TexturePipeline)
	{
		TexturePipeline->ScriptedExecutePostFactoryPipeline(BaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
	}

	FString ParameterName = GetParameterNameFromUID(NodeKey);

	UTexture2D* Texture2D = Cast<UTexture2D>(CreatedAsset);
	if (Texture2D != nullptr)
	{
		ETextureSourceFormat Format = Texture2D->Source.GetFormat();
		if (Format == TSF_BGRA8 || Format == TSF_G8 || Format == TSF_RGBA16)
		{
			Texture2D->CompressionSettings = TC_BC7;
		}
		else if (Format == TSF_RGBA32F)
		{
			Texture2D->CompressionSettings = TC_HDR_Compressed;
		}
		else if (Format == TSF_R32F || Format == TSF_R16F)
		{
			Texture2D->CompressionSettings = TC_HalfFloat;
		}
		else
		{
			UE_LOGF(
				LogAxFInterchangePipeline, Warning,
				"Unsupported texture source format! Can't set specific compression settings for %ls",
				*Texture2D->GetName());
		}

		Texture2D->MipGenSettings = TMGS_FromTextureGroup;
		Texture2D->PowerOfTwoMode = ETexturePowerOfTwoSetting::StretchToPowerOfTwo;
		if (ParameterName == TEXT("Normal") || ParameterName == TEXT("ClearCoatNormal"))
		{
			Texture2D->bFlipGreenChannel = true;
			Texture2D->CompressionSettings = TC_Normalmap;
		}
		Texture2D->SRGB = 0;

		if (!Texture2D->MarkPackageDirty())
		{
			UE_LOGF(
				LogAxFInterchangePipeline, Warning,
				"Failed to mark package as dirty. Some changes to the Texture2D %ls might not be saved.",
				*Texture2D->GetName());
		}
		Texture2D->PostEditChange();

		FAxFTextureMap& Pair = TexturesMap.FindOrAdd(ParameterName);
		Pair.Texture = Texture2D;
	}

	USpecularProfile* Profile = Cast<USpecularProfile>(CreatedAsset);
	if (Profile != nullptr)
	{
		SpecularProfile = Profile;
	}

	Super::ExecutePostFactoryPipeline(BaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
}

void UAxFInterchangePipeline::AdjustSettingsForContext(FInterchangePipelineContextParams const& ContextParams)
{
	if (TexturePipeline)
	{
		TexturePipeline->AdjustSettingsForContext(ContextParams);
	}

	// Check if the base materials are loaded
	static bool bRunOnce = []()
		{
			const FString BasePath = TEXT("/InterchangeAxFAssets/BaseMaterials/");
			const FString BaseMaterialNames[] = {
				TEXT("SVBRDF"),
				TEXT("SVBRDF_Alpha"),
				TEXT("SVBRDF_POM"),
				TEXT("SVBRDF_Alpha_POM"),
				TEXT("CPA2"),
				TEXT("CPA2_Flakes")
			};

			for (const FString& BaseMaterialName : BaseMaterialNames)
			{
				FSoftObjectPath UMaterialPath(FPaths::Combine(BasePath, BaseMaterialName) + TEXT(".") + BaseMaterialName);
				UObject* MaterialObject = UMaterialPath.TryLoad();
				if (MaterialObject == nullptr)
				{
					UE_LOGF(
						LogAxFInterchangePipeline, Error, "Failed to load base material from plugin Content: %ls",
						*UMaterialPath.ToString());
					continue;
				}

				UMaterial* Material = Cast<UMaterial>(MaterialObject);
				if (Material == nullptr)
				{
					UE_LOGF(
						LogAxFInterchangePipeline, Error, "Loaded asset is not a material: %ls",
						*UMaterialPath.ToString());
					continue;
				}

				// Caching the scalar and vector parameters per predefined materials
				FMaterialParameterTypes& ParameterTypes = ParameterTypesPerMaterial.Add(BaseMaterialName);
				{
					TArray<FMaterialParameterInfo> ParameterInfos;
					TArray<FGuid> ParameterIds;
					Material->GetAllScalarParameterInfo(ParameterInfos, ParameterIds);

					for (const FMaterialParameterInfo& Info : ParameterInfos)
					{
						ParameterTypes.ScalarParameters.Add(Info.Name.ToString());
					}
				}

				{
					TArray<FMaterialParameterInfo> ParameterInfos;
					TArray<FGuid> ParameterIds;
					Material->GetAllVectorParameterInfo(ParameterInfos, ParameterIds);

					for (const FMaterialParameterInfo& Info : ParameterInfos)
					{
						ParameterTypes.VectorParameters.Add(Info.Name.ToString());
					}
				}
			}

			return true;
		}();

	Super::AdjustSettingsForContext(ContextParams);
}

void UAxFInterchangePipeline::CreateSpecularProfileFactoryNode(const UInterchangeSpecularProfileNode* SpecularProfileNode, UInterchangeBaseNodeContainer* BaseNodeContainer, const TObjectPtr<UAxFMaterialObjectNode> AxFNode)
{
	const FString FactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(SpecularProfileNode->GetUniqueID());

	UInterchangeSpecularProfileFactoryNode* FactoryNode = NewObject<UInterchangeSpecularProfileFactoryNode>(BaseNodeContainer, NAME_None);

	BaseNodeContainer->SetupNode(FactoryNode, FactoryNodeUid, SpecularProfileNode->GetDisplayLabel(), EInterchangeNodeContainerType::FactoryData);

	FactoryNode->SetEnabled(true);
	FactoryNode->SetCustomSubPath(AxFNode->GetDisplayLabel());

	if (uint8 Format; SpecularProfileNode->GetCustomFormat(Format))
	{
		FactoryNode->SetCustomFormat(ESpecularProfileFormat{ Format });
	}

	if (FString TextureUid; SpecularProfileNode->GetCustomTexture(TextureUid))
	{
		if (BaseNodeContainer->GetNode(TextureUid))
		{
			FactoryNode->SetCustomTexture(TextureUid);
			FactoryNode->AddFactoryDependencyUid(UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TextureUid));
		}
	}

	FactoryNode->AddTargetNodeUid(SpecularProfileNode->GetUniqueID());
	SpecularProfileNode->AddTargetNodeUid(FactoryNode->GetUniqueID());
}

void UAxFInterchangePipeline::PreDialogCleanup(FName const PipelineStackName)
{
	if (TexturePipeline)
	{
		TexturePipeline->PreDialogCleanup(PipelineStackName);
	}
}

bool UAxFInterchangePipeline::IsSettingsAreValid(TOptional<FText>& OutInvalidReason) const
{
	if (TexturePipeline && !TexturePipeline->IsSettingsAreValid(OutInvalidReason))
	{
		return false;
	}

	return Super::IsSettingsAreValid(OutInvalidReason);
}

void UAxFInterchangePipeline::FilterPropertiesFromTranslatedData(
	UInterchangeBaseNodeContainer* InBaseNodeContainer)
{
	if (TexturePipeline)
	{
		TexturePipeline->FilterPropertiesFromTranslatedData(InBaseNodeContainer);
	}

	Super::FilterPropertiesFromTranslatedData(InBaseNodeContainer);
}

void UAxFInterchangePipeline::GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const
{
	if (TexturePipeline)
	{
		TexturePipeline->GetSupportAssetClasses(PipelineSupportAssetClasses);
	}

	PipelineSupportAssetClasses.Add(UMaterialInstance::StaticClass());
}