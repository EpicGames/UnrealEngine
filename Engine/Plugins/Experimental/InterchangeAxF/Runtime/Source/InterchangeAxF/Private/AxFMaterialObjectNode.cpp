// Copyright Epic Games, Inc. All Rights Reserved.

#include "AxFMaterialObjectNode.h"

UAxFMaterialObjectNode::UAxFMaterialObjectNode() {}

void UAxFMaterialObjectNode::InitializeAxFMaterialObjectNode(const FString &UniqueID, const FString &DisplayLabel)
{
    InitializeNode(UniqueID, DisplayLabel, EInterchangeNodeContainerType::TranslatedAsset);
}

FString UAxFMaterialObjectNode::GetTypeName() const
{
    FString TypeName = TEXT("AxFMaterialObjectNode");
    return TypeName;
}

const TOptional<FString> UAxFMaterialObjectNode::GetPayloadKey() const
{
    FString PayloadKey;

    UE::Interchange::FAttributeKey PayloadKeyAttributeName(TEXT("__PayloadKey__"));
    if (!Attributes->ContainAttribute(PayloadKeyAttributeName))
    {
        return TOptional<FString>();
    }

    UE::Interchange::FAttributeStorage::TAttributeHandle<FString> AttributeHandle =
        Attributes->GetAttributeHandle<FString>(PayloadKeyAttributeName);
    if (!AttributeHandle.IsValid())
    {
        return TOptional<FString>();
    }

    UE::Interchange::EAttributeStorageResult Result = AttributeHandle.Get(PayloadKey);
    if (!UE::Interchange::IsAttributeStorageResultSuccess(Result))
    {
        LogAttributeStorageErrors(Result, TEXT("UAxFMaterialObjectNode.GetPayLoadKey"), PayloadKeyAttributeName);
        return TOptional<FString>();
    }

    return TOptional(PayloadKey);
}

void UAxFMaterialObjectNode::SetPayloadKey(const FString &PayloadKey)
{
    UE::Interchange::FAttributeKey PayloadKeyAttributeName(TEXT("__PayloadKey__"));
    UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(PayloadKeyAttributeName, PayloadKey);
    if (!UE::Interchange::IsAttributeStorageResultSuccess(Result))
    {
        LogAttributeStorageErrors(Result, TEXT("UAxFMaterialObjectNode.SetPayLoadKey"), PayloadKeyAttributeName);
    }
}