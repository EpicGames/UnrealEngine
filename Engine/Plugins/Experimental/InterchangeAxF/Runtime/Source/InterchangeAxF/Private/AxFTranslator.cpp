// Copyright Epic Games, Inc. All Rights Reserved.

#include "AxFTranslator.h"
#include "AxFMaterialObjectNode.h"
#include "Engine/SpecularProfile.h"
#include "Engine/RendererSettings.h"
#include "ImageUtils.h"
#include "InterchangeSpecularProfileNode.h"
#include "InterchangeTexture2DNode.h"
#include "Modules/ModuleManager.h"
#include "InterchangeImportModule.h"
#include "InterchangeMaterialDefinitions.h"

#include "Serialization/JsonSerializer.h"

DEFINE_LOG_CATEGORY(LogAxFTranslator);

#include UE_INLINE_GENERATED_CPP_BY_NAME(AxFTranslator)

static bool GInterchangeEnableAxFImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnableAxFImport(
	TEXT("Interchange.FeatureFlags.Import.AxF"),
	GInterchangeEnableAxFImport,
	TEXT("Whether AxF Interchange support is enabled."),
	ECVF_Default);

bool UAxFTranslator::CanImportSourceData(const UInterchangeSourceData* InSourceData) const
{
	if (Super::CanImportSourceData(InSourceData))
	{
		if (IInterchangeImportModule::Get().IsSubstrateEnabled())
		{
			return true;
		}
		else
		{
			UE_LOGF(LogAxFTranslator, Warning, "Could not create material from %ls because Substrate materials are not enabled. Enable \"Substrate materials\" in Project Settings under Engine > Rendering.", *InSourceData->GetFilename());
		}
	}

	return false;
}

TArray<FString> UAxFTranslator::GetSupportedFormats() const
{
	TArray<FString> Formats;
	if (GInterchangeEnableAxFImport && IInterchangeImportModule::Get().IsSubstrateEnabled())
	{
		Formats.Add(TEXT("axf2ue;Intermediary AxF resource file"));
	}

	return Formats;
}

bool UAxFTranslator::Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const
{
	FString Filename = SourceData->GetFilename();

	FString Path, Name, Extension;
	FPaths::Split(Filename, Path, Name, Extension);

	FPaths::NormalizeFilename(Filename);
	if (!FPaths::FileExists(Filename))
	{
		return false;
	}

	FString DisplayLabel = FPaths::GetBaseFilename(Filename);
	FString NodeUID(Filename);
	UAxFMaterialObjectNode* AxFMaterialObjectNode = NewObject<UAxFMaterialObjectNode>(&BaseNodeContainer);
	if (!ensure(AxFMaterialObjectNode))
	{
		return false;
	}

	AxFMaterialObjectNode->InitializeAxFMaterialObjectNode(NodeUID, DisplayLabel);
	AxFMaterialObjectNode->SetPayloadKey(Filename);

	FString JSONString;
	if (!FFileHelper::LoadFileToString(JSONString, *SourceData->GetFilename()))
	{
		UE_LOGF(LogAxFTranslator, Error, "Failed to read JSON file at %ls", *SourceData->GetFilename());
		return {};
	}

	TSharedPtr<FJsonObject> JSON;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JSONString);
	if (!FJsonSerializer::Deserialize(Reader, JSON))
	{
		UE_LOGF(LogAxFTranslator, Error, "Failed to deserialize JSON read from file at %ls", *SourceData->GetFilename());
		return {};
	}

	static const TMap<FString, FString> ParameterNameMapping{
		{TEXT("alpha"), TEXT("Alpha")},
		{TEXT("anisotropicRotation"), TEXT("AnisotropicRotation")},
		{TEXT("clearCoatIOR"), TEXT("ClearCoatIOR")},
		{TEXT("clearCoatNormal"), TEXT("ClearCoatNormal")},
		{TEXT("clearCoatRoughness"), TEXT("ClearCoatRoughness")},
		{TEXT("clearCoatTransmissionColor"), TEXT("ClearCoatTransmissionColor")},
		{TEXT("cpaColorTable"), TEXT("CPA2ColorTable")},
		{TEXT("diffuseColor"), TEXT("DiffuseColor")},
		{TEXT("flakePatchSizeMM"), TEXT("FlakePatchSizeMM")},
		{TEXT("glintBrightness"), TEXT("GlintBrightness")},
		{TEXT("glintDensity"), TEXT("GlintDensity")},
		{TEXT("glintRoughness"), TEXT("GlintRoughness")},
		{TEXT("heightmap"), TEXT("HeightMap")},
		{TEXT("heightmapScale"), TEXT("HeightStrength")},
		{TEXT("heightmapReference"), TEXT("HeightReferencePlane")},
		{TEXT("f0"), TEXT("F0")},
		{TEXT("f90"), TEXT("F90")},
		{TEXT("normal"), TEXT("Normal")},
		{TEXT("roughness"), TEXT("Roughness")},
		{TEXT("sheenColor"), TEXT("SheenColor")},
		{TEXT("sheenRoughness"), TEXT("SheenRoughness")},
		{TEXT("transmissionColor"), TEXT("TransmissionColor")},
	};

	UE::Interchange::FAxFMaterialObjectData& AxFData = AxFMaterialObjectNode->PayloadData;
	for (const TPair<FString, FString>& Pair : ParameterNameMapping)
	{
		const FString& ParameterName = Pair.Key;
		const FString& UEParameterName = Pair.Value;
		bool ParameterPresent = false;

		if (const TArray<TSharedPtr<FJsonValue>>* TopLevelArray; JSON->TryGetArrayField(ParameterName, TopLevelArray))
		{
			ParameterPresent = true;

			TArray<FLinearColor> ColorsFromJSON;
			FLinearColor ValuesFromJSON(0.0f, 0.0f, 0.0f);
			const int32 TopLevelArrayNum = FMath::Clamp(TopLevelArray->Num(), 0, 4);

			for (size_t Index = 0; Index < TopLevelArrayNum; ++Index)
			{
				TSharedPtr<FJsonValue> JsonValue = (*TopLevelArray)[Index];
				if (JsonValue->Type != EJson::Number && JsonValue->Type != EJson::Array)
				{
					UE_LOGF(LogAxFTranslator, Error, "Invalid value type in array found");
					return {};
				}

				if (JsonValue->Type == EJson::Number)
				{
					ValuesFromJSON.Component(Index) = JsonValue->AsNumber();
				}
			}

			if (TopLevelArray->Num() == 1)
			{
				ValuesFromJSON = FLinearColor{ ValuesFromJSON.R, ValuesFromJSON.R, ValuesFromJSON.R };
			}

			AxFData.ValuesMap.Add(UEParameterName) = ValuesFromJSON;
		}
		else if (const TSharedPtr<FJsonObject>* ValueObject; JSON->TryGetObjectField(ParameterName, ValueObject))
		{
			ParameterPresent = true;

			float WidthMM{}, HeightMM{};
			if (!ValueObject->Get()->TryGetNumberField(TEXT("widthMM"), WidthMM))
			{
				UE_LOGF(LogAxFTranslator, Error, "Failed to find '%ls' property in AxF2UE file.", TEXT("widthMM"));
				return false;
			}
			if (!ValueObject->Get()->TryGetNumberField(TEXT("heightMM"), HeightMM))
			{
				UE_LOGF(LogAxFTranslator, Error, "Failed to find '%ls' property in AxF2UE file.", TEXT("heightMM"));
				return false;
			}

			if (UEParameterName == TEXT("FlakePatchSizeMM"))
			{
				AxFData.ValuesMap.Add(UEParameterName) = FLinearColor{ WidthMM, HeightMM, 0.0f };
				continue;
			}

			FString TexturePath{};
			if (!ValueObject->Get()->TryGetStringField(TEXT("path"), TexturePath))
			{
				UE_LOGF(LogAxFTranslator, Error, "Failed to find '%ls' property in AxF2UE file.", TEXT("path"));
				return false;
			}

			FString TextureUID = TEXT("T_") + DisplayLabel + TEXT("_") + UEParameterName;
			UInterchangeTexture2DNode* Texture2D = NewObject<UInterchangeTexture2DNode>(&BaseNodeContainer, FName(TextureUID));
			BaseNodeContainer.SetupNode(Texture2D, TextureUID, TextureUID, EInterchangeNodeContainerType::TranslatedAsset);
			Texture2D->SetPayLoadKey(Path + "|" + Name + "." + Extension + "|" + TexturePath);
			Texture2D->AddFloatAttribute(TEXT("WidthMM"), WidthMM);
			Texture2D->AddFloatAttribute(TEXT("HeightMM"), HeightMM);

			if (UEParameterName == TEXT("CPA2ColorTable"))
			{
				FString SpecProfileUID = TEXT("SPN_") + DisplayLabel;
				UInterchangeSpecularProfileNode* SpecularProfileNode = NewObject<UInterchangeSpecularProfileNode>(&BaseNodeContainer, FName(SpecProfileUID));
				BaseNodeContainer.SetupNode(SpecularProfileNode, SpecProfileUID, SpecProfileUID, EInterchangeNodeContainerType::TranslatedAsset);
				SpecularProfileNode->SetCustomFormat((uint8)ESpecularProfileFormat::HalfVector);
				SpecularProfileNode->SetCustomTexture(TextureUID);
				AxFMaterialObjectNode->AddStringAttribute(UE::Interchange::Materials::SubstrateMaterial::SpecularProfile.ToString(), SpecProfileUID);
			}
		}
		else if (bool ValueBool; JSON->TryGetBoolField(ParameterName, ValueBool))
		{
			ParameterPresent = ValueBool;
		}

		// If this parameter is present as either a value [array]
		// or a texture, check for optional AxF features to be used.
		if (ParameterPresent)
		{
			// Clear Coat
			if (UEParameterName == TEXT("ClearCoatTransmissionColor") || UEParameterName == TEXT("ClearCoatRoughness") ||
				UEParameterName == TEXT("ClearCoatIOR") || UEParameterName == TEXT("ClearCoatNormal"))
			{
				AxFData.UsedFeatures.AddUnique(EAxFFeature::ClearCoat);
			}

			// Sheen
			if (UEParameterName == TEXT("SheenColor") || UEParameterName == TEXT("SheenRoughness"))
			{
				AxFData.UsedFeatures.AddUnique(EAxFFeature::Sheen);
			}

			// Anisotropy
			if (UEParameterName == TEXT("Roughness") && AxFData.ValuesMap.Contains(UEParameterName))
			{
				if (const FLinearColor& RoughnessValues = AxFData.ValuesMap[UEParameterName]; RoughnessValues.R != RoughnessValues.G)
				{
					AxFData.UsedFeatures.AddUnique(EAxFFeature::Anisotropy);
				}
			}
			if (UEParameterName == TEXT("AnisotropicRotation"))
			{
				AxFData.UsedFeatures.AddUnique(EAxFFeature::Anisotropy);
			}

			// CPA2
			if (UEParameterName == TEXT("CPA2ColorTable"))
			{
				AxFData.UsedFeatures.AddUnique(EAxFFeature::CPA2);
			}

			// Alpha transparency
			if (UEParameterName == TEXT("Alpha"))
			{
				AxFData.UsedFeatures.AddUnique(EAxFFeature::Alpha);
			}

			// Transmission
			if (UEParameterName == TEXT("TransmissionColor"))
			{
				AxFData.UsedFeatures.AddUnique(EAxFFeature::Transmission);
			}

			// Heightmap
			if (UEParameterName == TEXT("HeightMap") || UEParameterName == TEXT("HeightStrength") || UEParameterName == TEXT("HeightReferencePlane"))
			{
				AxFData.UsedFeatures.AddUnique(EAxFFeature::Heightmap);
			}

			// Flakes
			if (UEParameterName == TEXT("GlintDensity") || UEParameterName == TEXT("GlintUVScale") ||
				UEParameterName == TEXT("GlintRoughness") || UEParameterName == TEXT("GlintBrightness"))
			{
				AxFData.UsedFeatures.AddUnique(EAxFFeature::Flakes);
			}
		}
	}

	BaseNodeContainer.AddNode(AxFMaterialObjectNode);

	return true;
}

TOptional<UE::Interchange::FImportImage>
UAxFTranslator::GetTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const
{
	FString BasePath, RightPart;
	if (!PayloadKey.Split("|", &BasePath, &RightPart))
	{
		UE_LOGF(LogAxFTranslator, Error, "Failed to properly split Interchange payload key!");
		return {};
	}

	FString JSONFileName, LocalTexturePath;
	if (!RightPart.Split("|", &JSONFileName, &LocalTexturePath))
	{
		UE_LOGF(LogAxFTranslator, Error, "Failed to properly split Interchange payload key!");
		return {};
	}

	// TODO: Abstract into separate function
	FString JSONString;
	FString JSONFilePath = FPaths::Combine(BasePath, JSONFileName);
	if (!FPaths::IsUnderDirectory(JSONFilePath, BasePath))
	{
		UE_LOGF(LogAxFTranslator, Error, "JSON file %ls is not under the directory %ls", *JSONFileName, *BasePath);
		return {};
	}
	if (!FFileHelper::LoadFileToString(JSONString, *JSONFilePath))
	{
		UE_LOGF(LogAxFTranslator, Error, "Failed to read JSON file at %ls", *JSONFilePath);
		return {};
	}

	TSharedPtr<FJsonObject> JSON;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JSONString);
	if (!FJsonSerializer::Deserialize(Reader, JSON))
	{
		UE_LOGF(LogAxFTranslator, Error, "Failed to deserialize JSON read from file at %ls", *JSONFilePath);
		return {};
	}

	TSharedPtr<TArray<uint8>> ByteArray(new TArray<uint8>());
	FString TexturePath = FPaths::Combine(BasePath, LocalTexturePath);
	if (!FPaths::IsUnderDirectory(TexturePath, BasePath))
	{
		UE_LOGF(LogAxFTranslator, Error, "Texture file file %ls is not under the directory %ls", *TexturePath, *BasePath);
		return {};
	}
	if (!FFileHelper::LoadFileToArray(*ByteArray, *TexturePath))
	{
		UE_LOGF(LogAxFTranslator, Error, "Failed to load texture from file: %ls", *TexturePath);
		return {};
	}

	FImage RawImage;
	if (FImageUtils::DecompressImage(ByteArray->GetData(), ByteArray->Num(), RawImage))
	{
		int32 Width = RawImage.GetWidth();
		int32 Height = RawImage.GetHeight();

		ETextureSourceFormat SourceFormat;
		if (RawImage.Format == ERawImageFormat::G8)
		{
			SourceFormat = TSF_G8;
		}
		else if (RawImage.Format == ERawImageFormat::BGRA8)
		{
			SourceFormat = TSF_BGRA8;
		}
		else if (RawImage.Format == ERawImageFormat::R32F)
		{
			SourceFormat = TSF_R32F;
		}
		else if (RawImage.Format == ERawImageFormat::RGBA32F)
		{
			SourceFormat = TSF_RGBA32F;
		}
		else
		{
			UE_LOGF(LogAxFTranslator, Warning, "Unsupported number of channels!");
			return {};
		}

		UE::Interchange::FImportImage Image;
		Image.Init2DWithParams(Width, Height, SourceFormat, false);
		FMemory::Memcpy(Image.RawData.GetData(), RawImage.RawData.GetData(), RawImage.RawData.NumBytes());

		return Image;
	}
	else
	{
		UE_LOGF(LogAxFTranslator, Error, "Texture asset could not be created: %ls", *TexturePath);
		return {};
	}
}

bool UAxFTranslator::SupportCompressedTexturePayloadData() const
{
	return IInterchangeTexturePayloadInterface::SupportCompressedTexturePayloadData();
}

TOptional<UE::Interchange::FImportImage>
UAxFTranslator::GetCompressedTexturePayloadData(const FString& PayloadKey,
												TOptional<FString>& AlternateTexturePath) const
{
	return IInterchangeTexturePayloadInterface::GetCompressedTexturePayloadData(PayloadKey, AlternateTexturePath);
}