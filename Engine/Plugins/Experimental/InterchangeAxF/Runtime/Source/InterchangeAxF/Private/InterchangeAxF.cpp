// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeAxF.h"
#include "AxFTranslator.h"
#include "Engine/Engine.h"
#include "Engine/RendererSettings.h"
#include "InterchangeManager.h"
#include "InterchangeProjectSettings.h"
#include "Misc/CoreDelegates.h"

#include <Modules/ModuleManager.h>

#define LOCTEXT_NAMESPACE "FInterchangeAxFModule"

DEFINE_LOG_CATEGORY(LogInterchangeAxF);

// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
void FInterchangeAxFModule::StartupModule()
{
	auto RegisterItems = []()
		{
			UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();

			// Register the translators
			InterchangeManager.RegisterTranslator(UAxFTranslator::StaticClass());

			// Add X-Rite's AxF translator to the Interchange engine's pipeline set
			if (UInterchangeProjectSettings* ProjectSettings = GetMutableDefault<UInterchangeProjectSettings>())
			{
				bool bPipelinesWereAdded = false;
				FInterchangeContentImportSettings& AssetImportSettings = ProjectSettings->ContentImportSettings;

				if (AssetImportSettings.DefaultPipelineStackOverride.Contains(EInterchangeTranslatorAssetType::Materials))
				{
					const FName DefaultMaterialsStack = AssetImportSettings.DefaultPipelineStackOverride[EInterchangeTranslatorAssetType::Materials];

					if (AssetImportSettings.PipelineStacks.Contains(DefaultMaterialsStack))
					{
						FInterchangeTranslatorPipelines TranslatorPipeplines;
						TranslatorPipeplines.Translator = TSoftClassPtr<UInterchangeTranslatorBase>(UAxFTranslator::StaticClass());
						TranslatorPipeplines.Pipelines.Add(FSoftObjectPath(TEXT("/InterchangeAxF/AxFInterchangePipeline.AxFInterchangePipeline")));
						{
							FInterchangePipelineStack& PipelineStack = AssetImportSettings.PipelineStacks[DefaultMaterialsStack];
							PipelineStack.PerTranslatorPipelines.Add(TranslatorPipeplines);
						}

						bPipelinesWereAdded = true;
						UE_LOGF(LogInterchangeAxF, Log, "Interchange AxF's pipelines added to default Material pipeline stack, %ls", *DefaultMaterialsStack.ToString());
					}
				}

				if (!AssetImportSettings.ShowImportDialogOverride.Contains(EInterchangeTranslatorAssetType::Materials))
				{
					FInterchangePerTranslatorDialogOverride AxFTranslatorImportDialogOverride{};
					AxFTranslatorImportDialogOverride.Translator = TSoftClassPtr<UInterchangeTranslatorBase>(UAxFTranslator::StaticClass());
					AxFTranslatorImportDialogOverride.bShowImportDialog = false;
					AxFTranslatorImportDialogOverride.bShowReimportDialog = false;
					FInterchangeDialogOverride& DialogOverride = AssetImportSettings.ShowImportDialogOverride.FindOrAdd(EInterchangeTranslatorAssetType::Materials);
					DialogOverride.PerTranslatorImportDialogOverride.Add(AxFTranslatorImportDialogOverride);
				}

				if (!bPipelinesWereAdded)
				{
					UE_LOGF(LogInterchangeAxF, Warning, "Cannot find Interchange default Materials' stack to add pipelines. Interchange AxF material import may not work properly.");
				}
			}
		};

	if (GEngine)
	{
		RegisterItems();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterItems);
	}
}

// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
// we call this function before unloading the module.
void FInterchangeAxFModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FInterchangeAxFModule, InterchangeAxF)
#undef LOCTEXT_NAMESPACE