// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/SpecularProfile.h"
#include "InterchangeGenericTexturePipeline.h"
#include "InterchangePipelineBase.h"
#include "InterchangeSpecularProfileNode.h"

#include "AxFInterchangePipeline.generated.h"

#define UE_API INTERCHANGEAXF_API

DECLARE_LOG_CATEGORY_EXTERN(LogAxFInterchangePipeline, Log, All);

class UInterchangeShaderGraphNode;
class UAxFMaterialObjectNode;
class UInterchangeBaseNodeContainer;
class UInterchangeSourceData;
class UInterchangeTexture2DFactoryNode;
class UInterchangeGenericTexturePipeline;
class USpecularProfile;

USTRUCT()
struct FAxFTextureMap
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UTexture2D> Texture;
	FVector2f SizeMM;
};

UCLASS(BlueprintType)
class UE_API UAxFInterchangePipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

public:
	UAxFInterchangePipeline();

	UPROPERTY(
		EditAnywhere, BlueprintReadWrite, Category = "General",
		meta = (StandAlonePipelineProperty = "True", PipelineInternalEditionData = "True"))
	FString PipelineDisplayName;

	/* Set the reimport strategy. */
	UPROPERTY(
		EditAnywhere, BlueprintReadWrite, Category = "Common",
		meta = (AdjustPipelineAndRefreshDetailOnChange = "True", PipelineInternalEditionData = "True"))
	EReimportStrategyFlags ReimportStrategy = EReimportStrategyFlags::ApplyNoProperties;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AxF")
	bool bUseTriplanarMappingByDefault = true;

protected:
	virtual void ExecutePipeline(
		UInterchangeBaseNodeContainer* BaseNodeContainer, TArray<UInterchangeSourceData*> const& SourceDatas,
		FString const& ContentBasePath) override;

	virtual void ExecutePostImportPipeline(
		UInterchangeBaseNodeContainer const* BaseNodeContainer, FString const& NodeKey, UObject* CreatedAsset,
		bool bIsAReimport) override;

	virtual void ExecutePostFactoryPipeline(
		UInterchangeBaseNodeContainer const* BaseNodeContainer, FString const& NodeKey, UObject* CreatedAsset,
		bool bIsAReimport) override;

	virtual void AdjustSettingsForContext(FInterchangePipelineContextParams const& ContextParams) override;

	// NOTE: This is a copied and modified version of UInterchangeGenericMaterialPipeline::CreateSpecularProfileFactoryNode().
	// This would be handled automatically, but for now the compromise is to just do what the Generic Material Pipeline does, 
	// in order to remove it as a dependency in the Interchange Pipeline Stack.
	void CreateSpecularProfileFactoryNode(const UInterchangeSpecularProfileNode* SpecularProfileNode,
										  UInterchangeBaseNodeContainer* BaseNodeContainer, TObjectPtr<UAxFMaterialObjectNode> AxFNode);

	virtual void PreDialogCleanup(FName const PipelineStackName) override;

	virtual bool IsSettingsAreValid(TOptional<FText>& OutInvalidReason) const override;

	virtual void FilterPropertiesFromTranslatedData(UInterchangeBaseNodeContainer* InBaseNodeContainer) override;

	virtual void GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const override;

private:
	UPROPERTY(Transient)
	TArray<TObjectPtr<UAxFMaterialObjectNode>> AxFObjectNodes;

	// Texture handling
	UPROPERTY(Transient)
	TArray<TObjectPtr<UInterchangeTexture2DFactoryNode>> Texture2DFactoryNodes;

	UPROPERTY()
	TMap<FString, FAxFTextureMap> TexturesMap;

	// Specular profile handling
	UPROPERTY()
	TObjectPtr<USpecularProfile> SpecularProfile;

	// Generic texture pipeline to be used to create the necessary texture assets the materials use
	UPROPERTY()
	TObjectPtr<UInterchangeGenericTexturePipeline> TexturePipeline;
};

#undef UE_API