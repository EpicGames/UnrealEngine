// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeBaseNode.h"
#include "AxFMaterialObjectNode.generated.h"

#define UE_API INTERCHANGEAXF_API

UENUM()
enum EAxFFeature : uint8
{
    Alpha,
    Anisotropy,
    ClearCoat,
    CPA2,
	Flakes,
    Heightmap,
    Sheen,
    Transmission
};

namespace UE
{
    namespace Interchange
    {
        struct UE_API FAxFMaterialObjectData
        {
            virtual ~FAxFMaterialObjectData() = default;
            FAxFMaterialObjectData() = default;
            FAxFMaterialObjectData(FAxFMaterialObjectData &&) = default;
            FAxFMaterialObjectData &operator=(FAxFMaterialObjectData &&) = default;
            FAxFMaterialObjectData(const FAxFMaterialObjectData &) = delete;
            FAxFMaterialObjectData &operator=(const FAxFMaterialObjectData &) = delete;

            TMap<FString, FLinearColor> ValuesMap;
            TArray<EAxFFeature> UsedFeatures;
        };
    } // namespace Interchange
} // namespace UE

UCLASS(BlueprintType)
class UE_API UAxFMaterialObjectNode : public UInterchangeBaseNode
{
    GENERATED_BODY()

    UAxFMaterialObjectNode();

public:
    UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AxFObject")
    void InitializeAxFMaterialObjectNode(const FString &UniqueID, const FString &DisplayLabel);

    virtual FString GetTypeName() const override;

    virtual const TOptional<FString> GetPayloadKey() const;

    virtual void SetPayloadKey(const FString &PayloadKey);

    UE::Interchange::FAxFMaterialObjectData PayloadData;
};

#undef UE_API