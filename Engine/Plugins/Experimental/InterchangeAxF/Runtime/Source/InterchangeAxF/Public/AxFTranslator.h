// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeTranslatorBase.h"
#include "Import/Public/Texture/InterchangeTexturePayloadInterface.h"

#include "AxFTranslator.generated.h"

#define UE_API INTERCHANGEAXF_API

DECLARE_LOG_CATEGORY_EXTERN(LogAxFTranslator, Log, All);

namespace UE::Interchange
{
    struct FAxFMaterialObjectData;
}

UCLASS(BlueprintType)
class UE_API UAxFTranslator
    : public UInterchangeTranslatorBase
    , public IInterchangeTexturePayloadInterface
{
	GENERATED_BODY()
public:

	virtual bool CanImportSourceData(const UInterchangeSourceData* InSourceData) const override;

	virtual TArray<FString> GetSupportedFormats() const override;

	virtual EInterchangeTranslatorAssetType GetSupportedAssetTypes() const override
	{
	    return EInterchangeTranslatorAssetType::Materials;
	}

	virtual bool Translate(UInterchangeBaseNodeContainer &BaseNodeContainer) const override;

	// IInterchangeTexturePayloadInterface
        virtual TOptional<UE::Interchange::FImportImage> GetTexturePayloadData(const FString &PayloadKey, TOptional<FString> &AlternateTexturePath) const override;
        virtual bool SupportCompressedTexturePayloadData() const override;
        virtual TOptional<UE::Interchange::FImportImage> GetCompressedTexturePayloadData(const FString &PayloadKey, TOptional<FString> &AlternateTexturePath) const override;
	// IInterchangeTexturePayloadInterface
};

#undef UE_API