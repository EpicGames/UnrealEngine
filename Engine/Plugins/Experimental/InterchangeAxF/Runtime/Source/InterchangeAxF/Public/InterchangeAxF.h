// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogInterchangeAxF, Log, All);

class FInterchangeAxFModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};