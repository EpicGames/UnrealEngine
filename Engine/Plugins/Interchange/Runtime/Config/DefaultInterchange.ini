[SectionsToSave]
bCanSaveAllSections=True

[CoreRedirects]
+EnumRedirects=(OldName="/Script/InterchangePipelines.EInterchangeMaterialImportOption",ValueChanges=(("DoNotImport","ImportAsMaterials")))
+EnumRedirects=(OldName="/Script/InterchangeNodes.EInterchangePropertyTracks",ValueChanges=(("Visibility","ActorHiddenInGame")))

; Rename Interchange CineCamera to PhysicalCamera FactoryNode
+ClassRedirects=(OldName="/Script/InterchangeFactoryNodes.InterchangeCineCameraFactoryNode", NewName="/Script/InterchangeFactoryNodes.InterchangePhysicalCameraFactoryNode")

; Rename MaterialX Material Expressions and Enums
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionAppend3Vector", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXAppend3Vector")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionAppend4Vector", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXAppend4Vector")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionBurn", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXBurn")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionDifference", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXDifference")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionDisjointOver", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXDisjointOver")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionDodge", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXDodge")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionFractal3D", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXFractal3D")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionIn", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXIn")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionLuminance", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXLuminance")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionMask", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXMask")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionMatte", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXMatte")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionMinus", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXMinus")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionOut", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXOut")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionOver", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXOver")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionOverlay", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXOverlay")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionPlace2D", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXPlace2D")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionPlus", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXPlus")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionPremult", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXPremult")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionRamp4", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXRamp4")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionRampLeftRight", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXRampLeftRight")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionRampTopBottom", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXRampTopBottom")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionRemap", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXRemap")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionRotate2D", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXRotate2D")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionScreen", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXScreen")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionSplitLeftRight", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXSplitLeftRight")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionSplitTopBottom", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXSplitTopBottom")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionTextureSampleParameterBlur", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXTextureSampleParameterBlur")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionUnpremult", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXUnpremult")
+EnumRedirects=(OldName="ETextureSampleBlurFilter", NewName="EMaterialXTextureSampleBlurFilter")
+EnumRedirects=(OldName="ELuminanceMode", NewName="EMaterialXLuminanceMode")
+EnumRedirects=(OldName="/Script/InterchangePipelines.EInterchangeMaterialXShaders>", NewName="/Script/InterchangeCommon.EInterchangeMaterialXShaders>")
+EnumRedirects=(OldName="/Script/InterchangePipelines.EInterchangeMaterialXBSDF>", NewName="/Script/InterchangeCommon.EInterchangeMaterialXBSDF>")
+EnumRedirects=(OldName="/Script/InterchangePipelines.EInterchangeMaterialXEDF>", NewName="/Script/InterchangeCommon.EInterchangeMaterialXEDF>")
+EnumRedirects=(OldName="/Script/InterchangePipelines.EInterchangeMaterialXVDF>", NewName="/Script/InterchangeCommon.EInterchangeMaterialXVDF>")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionMaterialXFractal3D", NewName="/Script/InterchangeImport.MaterialExpressionMaterialXFractal")

; Rename InterchangeAnimationTrackSetFactory to InterchangeLevelSequenceFactory
+ClassRedirects=(OldName="/Script/InterchangeImport.InterchangeAnimationTrackSetFactory", NewName="/Script/InterchangeImport.InterchangeLevelSequenceFactory")
+ClassRedirects=(OldName="/Script/InterchangeFactoryNodes.InterchangeAnimationTrackSetFactoryNode", NewName="/Script/InterchangeFactoryNodes.InterchangeLevelSequenceFactoryNode")

; Add proper suffix to Interchange predefined materials
+PackageRedirects=(OldName="/Interchange/Materials/PBRSurfaceMaterial", NewName="/InterchangeAssets/Materials/PBRSurfaceMaterial_MR")
+PackageRedirects=(OldName="/Interchange/Materials/ClearCoatMaterial", NewName="/InterchangeAssets/Materials/ClearCoatMaterial_MR")
+PackageRedirects=(OldName="/Interchange/Materials/SheenMaterial", NewName="/InterchangeAssets/Materials/SheenMaterial_MR")
+PackageRedirects=(OldName="/Interchange/Materials/SubsurfaceMaterial", NewName="/InterchangeAssets/Materials/SubsurfaceMaterial_MR")
+PackageRedirects=(OldName="/Interchange/Materials/ThinTranslucentMaterial", NewName="/InterchangeAssets/Materials/ThinTranslucentMaterial_MR")

[/Script/InterchangePipelines.GLTFPipelineSettings]
MaterialParents=(("M_GLTF", "/InterchangeAssets/gltf/Substrate/M_GLTF.M_GLTF"), ("MI_Default_Opaque", "/InterchangeAssets/gltf/MaterialInstances/MI_Default_Opaque.MI_Default_Opaque"), ("MI_Default_Mask", "/InterchangeAssets/gltf/MaterialInstances/MI_Default_Mask.MI_Default_Mask"), ("MI_Default_Blend", "/InterchangeAssets/gltf/MaterialInstances/MI_Default_Blend.MI_Default_Blend"), ("MI_Unlit_Opaque", "/InterchangeAssets/gltf/MaterialInstances/MI_Unlit_Opaque.MI_Unlit_Opaque"), ("MI_Unlit_Mask", "/InterchangeAssets/gltf/MaterialInstances/MI_Unlit_Mask.MI_Unlit_Mask"), ("MI_Unlit_Blend", "/InterchangeAssets/gltf/MaterialInstances/MI_Unlit_Blend.MI_Unlit_Blend"), ("MI_ClearCoat_Opaque", "/InterchangeAssets/gltf/MaterialInstances/MI_ClearCoat_Opaque.MI_ClearCoat_Opaque"), ("MI_ClearCoat_Mask", "/InterchangeAssets/gltf/MaterialInstances/MI_ClearCoat_Mask.MI_ClearCoat_Mask"), ("MI_ClearCoat_Blend", "/InterchangeAssets/gltf/MaterialInstances/MI_ClearCoat_Blend.MI_ClearCoat_Blend"), ("MI_Sheen_Opaque", "/InterchangeAssets/gltf/MaterialInstances/MI_Sheen_Opaque.MI_Sheen_Opaque"), ("MI_Sheen_Mask", "/InterchangeAssets/gltf/MaterialInstances/MI_Sheen_Mask.MI_Sheen_Mask"), ("MI_Sheen_Blend", "/InterchangeAssets/gltf/MaterialInstances/MI_Sheen_Blend.MI_Sheen_Blend"), ("MI_Transmission", "/InterchangeAssets/gltf/MaterialInstances/MI_Transmission.MI_Transmission"), ("MI_SpecularGlossiness_Opaque", "/InterchangeAssets/gltf/MaterialInstances/MI_SpecularGlossiness_Opaque.MI_SpecularGlossiness_Opaque"), ("MI_SpecularGlossiness_Mask", "/InterchangeAssets/gltf/MaterialInstances/MI_SpecularGlossiness_Mask.MI_SpecularGlossiness_Mask"), ("MI_SpecularGlossiness_Blend", "/InterchangeAssets/gltf/MaterialInstances/MI_SpecularGlossiness_Blend.MI_SpecularGlossiness_Blend"), ("MI_Default_Opaque_DS", "/InterchangeAssets/gltf/MaterialInstances/MI_Default_Opaque_DS.MI_Default_Opaque_DS"), ("MI_Default_Mask_DS", "/InterchangeAssets/gltf/MaterialInstances/MI_Default_Mask_DS.MI_Default_Mask_DS"), ("MI_Default_Blend_DS", "/InterchangeAssets/gltf/MaterialInstances/MI_Default_Blend_DS.MI_Default_Blend_DS"), ("MI_Unlit_Opaque_DS", "/InterchangeAssets/gltf/MaterialInstances/MI_Unlit_Opaque_DS.MI_Unlit_Opaque_DS"), ("MI_Unlit_Mask_DS", "/InterchangeAssets/gltf/MaterialInstances/MI_Unlit_Mask_DS.MI_Unlit_Mask_DS"), ("MI_Unlit_Blend_DS", "/InterchangeAssets/gltf/MaterialInstances/MI_Unlit_Blend_DS.MI_Unlit_Blend_DS"), ("MI_ClearCoat_Opaque_DS", "/InterchangeAssets/gltf/MaterialInstances/MI_ClearCoat_Opaque_DS.MI_ClearCoat_Opaque_DS"), ("MI_ClearCoat_Mask_DS", "/InterchangeAssets/gltf/MaterialInstances/MI_ClearCoat_Mask_DS.MI_ClearCoat_Mask_DS"), ("MI_ClearCoat_Blend_DS", "/InterchangeAssets/gltf/MaterialInstances/MI_ClearCoat_Blend_DS.MI_ClearCoat_Blend_DS"), ("MI_Sheen_Opaque_DS", "/InterchangeAssets/gltf/MaterialInstances/MI_Sheen_Opaque_DS.MI_Sheen_Opaque_DS"), ("MI_Sheen_Mask_DS", "/InterchangeAssets/gltf/MaterialInstances/MI_Sheen_Mask_DS.MI_Sheen_Mask_DS"), ("MI_Sheen_Blend_DS", "/InterchangeAssets/gltf/MaterialInstances/MI_Sheen_Blend_DS.MI_Sheen_Blend_DS"), ("MI_Transmission_DS", "/InterchangeAssets/gltf/MaterialInstances/MI_Transmission_DS.MI_Transmission_DS"), ("MI_SpecularGlossiness_Opaque_DS", "/InterchangeAssets/gltf/MaterialInstances/MI_SpecularGlossiness_Opaque_DS.MI_SpecularGlossiness_Opaque_DS"), ("MI_SpecularGlossiness_Mask_DS", "/InterchangeAssets/gltf/MaterialInstances/MI_SpecularGlossiness_Mask_DS.MI_SpecularGlossiness_Mask_DS"), ("MI_SpecularGlossiness_Blend_DS", "/InterchangeAssets/gltf/MaterialInstances/MI_SpecularGlossiness_Blend_DS.MI_SpecularGlossiness_Blend_DS"))

[/Script/InterchangePipelines.MaterialXPipelineSettings]
PredefinedSurfaceShaders=((OpenPBRSurface, "/InterchangeAssets/Functions/MX_OpenPBR_Opaque.MX_OpenPBR_Opaque"),(OpenPBRSurfaceTransmission, "/InterchangeAssets/Functions/MX_OpenPBR_Translucent.MX_OpenPBR_Translucent"),(StandardSurface, "/InterchangeAssets/Functions/MX_StandardSurface.MX_StandardSurface"),(StandardSurfaceTransmission, "/InterchangeAssets/Functions/MX_TransmissionSurface.MX_TransmissionSurface"),(SurfaceUnlit, "/InterchangeAssets/Functions/MX_SurfaceUnlit.MX_SurfaceUnlit"),(UsdPreviewSurface, "/InterchangeAssets/Functions/MX_UsdPreviewSurface.MX_UsdPreviewSurface"),(GltfPbr, "/InterchangeAssets/gltf/Substrate/MF_GLTFPBR.MF_GLTFPBR"),(Surface, "/InterchangeAssets/Functions/MX_Surface.MX_Surface"),(Displacement, "/InterchangeAssets/Functions/MX_Displacement.MX_Displacement"),(Volume, "/InterchangeAssets/Substrate/MX_SimpleVolume.MX_SimpleVolume"))
PredefinedBSDF=((OrenNayarDiffuse, "/InterchangeAssets/Functions/MX_OrenNayarBSDF.MX_OrenNayarBSDF"),(BurleyDiffuse,"/InterchangeAssets/Functions/MX_BurleyDiffuseBSDF.MX_BurleyDiffuseBSDF"),(Translucent,"/InterchangeAssets/Functions/MX_TranslucentBSDF.MX_TranslucentBSDF"),(Dielectric,"/InterchangeAssets/Functions/MX_DielectricBSDF.MX_DielectricBSDF"),(Conductor,"/InterchangeAssets/Functions/MX_ConductorBSDF.MX_ConductorBSDF"),(GeneralizedSchlick,"/InterchangeAssets/Functions/MX_GeneralizedSchlickBSDF.MX_GeneralizedSchlickBSDF"),(Subsurface,"/InterchangeAssets/Functions/MX_SubsurfaceBSDF.MX_SubsurfaceBSDF"),(Sheen,"/InterchangeAssets/Functions/MX_SheenBSDF.MX_SheenBSDF"),(ChiangHair,"/InterchangeAssets/Functions/MX_ChiangHairBSDF.MX_ChiangHairBSDF"))
PredefinedEDF=((Uniform, "/InterchangeAssets/Functions/MX_UniformEDF.MX_UniformEDF"),(Conical,"/InterchangeAssets/Functions/MX_ConicalEDF.MX_ConicalEDF"),(Measured,"/InterchangeAssets/Functions/MX_MeasuredEDF.MX_MeasuredEDF"))
PredefinedVDF=((Absorption, "/InterchangeAssets/Functions/MX_AbsorptionVDF.MX_AbsorptionVDF"),(Anisotropic,"/InterchangeAssets/Functions/MX_AnisotropicVDF.MX_AnisotropicVDF"))

[/Script/InterchangeImport.InterchangeFbxTranslatorProjectSettings]
bConvertScene=True
bForceFrontXAxis=False
bConvertSceneUnit=True
bKeepFbxNamespace=False