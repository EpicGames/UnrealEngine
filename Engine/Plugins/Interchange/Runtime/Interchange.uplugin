{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Interchange Framework",
	"Description": "The Interchange Framework plugin offers a customizable import and export system, with an extensible set of pipelines for handling common file types.",
	"Category": "Importers",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Mac"
	],
	"IsHidden": true,
	"Modules": [
		{
			"Name": "GLTFCore",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "InterchangeNodes",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "InterchangeFactoryNodes",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "InterchangeImport",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "InterchangeMessages",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "InterchangeExport",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "InterchangePipelines",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "InterchangeDispatcher",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "InterchangeCommon",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "InterchangeAnalytics",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "InterchangeCommonParser",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "InterchangeFbxParser",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			],
			"TargetAllowList": [
				"Editor",
				"Program"
			]
		}
	],
	"Plugins": [
		{
			"Name": "HairStrands",
			"Enabled": true
		},
		{
			"Name": "GeometryCache",
			"Enabled": true
		},
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "VariantManagerContent",
			"Enabled": true
		},
		{
			"Name": "VariantManager",
			"Enabled": true
		},
		{
			"Name": "InterchangeAssets",
			"Enabled": true
		}
	],
	"IsExperimentalVersion": false,
	"SupportedPrograms": [ "InterchangeWorker" ]
}
