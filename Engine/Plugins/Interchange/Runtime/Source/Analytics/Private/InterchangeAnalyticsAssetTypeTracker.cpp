// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeAnalyticsAssetTypeTracker.h"

TSet<UClass*> FInterchangeAnalyticsAssetTypeTracker::RegisteredAssetClasses;
TMap<UClass*, FString> FInterchangeAnalyticsAssetTypeTracker::AssetClassPathToDisplayNameMap;

void FInterchangeAnalyticsAssetTypeTracker::RegisterAssetType(UClass* AssetClass, const FString& DisplayName)
{
	if (!AssetClass)
	{
		return;
	}

	if (!RegisteredAssetClasses.Contains(AssetClass))
	{
		RegisteredAssetClasses.Emplace(AssetClass);
		AssetClassPathToDisplayNameMap.Emplace(AssetClass, DisplayName);
	}
}

TMap<FString, int32> FInterchangeAnalyticsAssetTypeTracker::GetAssetTypeFrequenceMap(const TArray<TObjectPtr<UObject>>& InObjects, bool bIncludeUntrackedClasses /*= false*/)
{
	TMap<FString, int32> FrequncyMap;
	AppendAssetTypeFrequenceMap(InObjects, FrequncyMap, bIncludeUntrackedClasses);
	return FrequncyMap;
}

void FInterchangeAnalyticsAssetTypeTracker::AppendAssetTypeFrequenceMap(const TArray<TObjectPtr<UObject>>& InObjects, TMap<FString, int32>& OutFreqMap, bool bIncludeUntrackedClasses /*= false*/)
{
	for (const TObjectPtr<UObject>& Object : InObjects)
	{
		if (!Object)
		{
			continue;
		}

		bool bAssetClassFound = false;
		for (const UClass* AssetClass : RegisteredAssetClasses)
		{
			if (Object->GetClass()->IsChildOf(AssetClass))
			{
				int32& Frequncy = OutFreqMap.FindOrAdd(AssetClassPathToDisplayNameMap.FindChecked(AssetClass));
				Frequncy++;
				bAssetClassFound = true;
				break;
			}
		}

		if (!bAssetClassFound && bIncludeUntrackedClasses)
		{
			int32& Frequency = OutFreqMap.FindOrAdd(TEXT("UntrackedAssets"));
			Frequency++;
		}
	}
}
