// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeAnalyticsHandlerDefault.h"
#include "EngineAnalytics.h"

#include "InterchangePipelineBase.h"
#include "InterchangeSourceData.h"
#include "Nodes/InterchangeSourceNode.h"

#include "Logging/LogCategory.h"

#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializerWriter.h"

#include "InterchangeAnalyticsAssetTypeTracker.h"

DEFINE_LOG_CATEGORY_STATIC(LogInterchangeAnalytics, Log, All);

namespace UE::Interchange::AnalyticsHelpers
{
	// Json writer subclass to allow us to avoid using a SharedPtr to write basic Json.
	class FAnalyticsJsonWriter : public TJsonStringWriter<TCondensedJsonPrintPolicy<TCHAR>>
	{
	public:
		explicit FAnalyticsJsonWriter(FString* Out) : TJsonStringWriter<TCondensedJsonPrintPolicy<TCHAR>>(Out, 0)
		{
		}
	};

	FJsonFragment GetWarningsErrorsMapAsJsonFragment(const TMap<FString, int32>& InFreqMap)
	{
		FString ReturnValue;
		FAnalyticsJsonWriter JsonWriter(&ReturnValue);
		JsonWriter.WriteArrayStart();
		for (const TPair<FString, int32>& FreqMapPair : InFreqMap)
		{
			JsonWriter.WriteObjectStart();
			JsonWriter.WriteValue(TEXT("MessageKey"), FreqMapPair.Key);
			JsonWriter.WriteValue(TEXT("MessageCount"), FreqMapPair.Value);
			JsonWriter.WriteObjectEnd();
		}
		JsonWriter.WriteArrayEnd();
		JsonWriter.Close();
		return FJsonFragment(MoveTemp(ReturnValue));
	}

	FJsonFragment GetPrimarySourceDataMetadataAsJsonFragment(const UInterchangeSourceData* InSourceData, const UInterchangeSourceNode* InSourceNode)
	{
		FString ReturnValue;
		FAnalyticsJsonWriter JsonWriter(&ReturnValue);

		TMap<FString, FString> ExtraInfoMap;
		if (InSourceNode)
		{
			InSourceNode->GetExtraInformation(ExtraInfoMap);
		}

		const FString DefaultValue("N/A");

		const FString ApplicationVendor = ExtraInfoMap.FindRef(FSourceNodeExtraInfoStaticData::GetApplicationVendorExtraInfoKey(), DefaultValue);
		const FString ApplicationName = ExtraInfoMap.FindRef(FSourceNodeExtraInfoStaticData::GetApplicationNameExtraInfoKey(), DefaultValue);
		const FString ApplicationVersion = ExtraInfoMap.FindRef(FSourceNodeExtraInfoStaticData::GetApplicationVersionExtraInfoKey(), DefaultValue);

		JsonWriter.WriteObjectStart();
		JsonWriter.WriteValue(TEXT("SourceExtension"), FPaths::GetExtension(InSourceData->GetFilename()));
		JsonWriter.WriteValue(TEXT("ApplicationVendor"), ApplicationVendor);
		JsonWriter.WriteValue(TEXT("ApplicationName"), ApplicationName);
		JsonWriter.WriteValue(TEXT("ApplicationVersion"), ApplicationVersion);
		JsonWriter.WriteObjectEnd();
		JsonWriter.Close();
		return FJsonFragment(MoveTemp(ReturnValue));
	}

	FJsonFragment GetAssetTypeFrequencyMapAsJsonFragment(const TMap<FString, int32>& InFreqMap)
	{
		FString ReturnValue;
		FAnalyticsJsonWriter JsonWriter(&ReturnValue);

		JsonWriter.WriteObjectStart();
		for (const TPair<FString, int32>& AssetTypeFreqPair : InFreqMap)
		{
			JsonWriter.WriteValue(AssetTypeFreqPair.Key, AssetTypeFreqPair.Value);
		}
		JsonWriter.WriteObjectEnd();
		JsonWriter.Close();
		return FJsonFragment(MoveTemp(ReturnValue));
	}

	bool ExtractNamespace(const FText& Text, FString& OutTextNamespaceId)
	{
		FText TextToUse = Text;

		TArray<FHistoricTextFormatData> TextHistory;
		FTextInspector::GetHistoricFormatData(Text, TextHistory);
		if (TextHistory.Num() > 0)
		{
			const FHistoricTextFormatData& FmtData = TextHistory[0];
			TextToUse = FmtData.SourceFmt.GetSourceText();
		}

		const TOptional<FString> TextNamespace = FTextInspector::GetNamespace(TextToUse);
		const TOptional<FString> TextKey = FTextInspector::GetKey(TextToUse);

		if (TextNamespace.IsSet() && TextKey.IsSet())
		{
			OutTextNamespaceId = FString::Printf(TEXT("%s_%s"), *TextNamespace.GetValue(), *TextKey.GetValue());
			return true;
		}

		OutTextNamespaceId = FString("UnknownError");
		return false;
	}

	void CollectResultsContainerWarningAndErrors(const UInterchangeResultsContainer* ResultContainer, TMap<FString, int32>&WarningTypes, TMap<FString, int32>&ErrorTypes)
	{
		TArray<UInterchangeResult*> InterchangeResults = ResultContainer->GetResults();

		for (const UInterchangeResult* InterchangeResult : InterchangeResults)
		{
			using namespace UE::Interchange::Private;

			switch (InterchangeResult->GetResultType())
			{
			case EInterchangeResultType::Success:
				break;
			case EInterchangeResultType::Warning:
			{
				FString OutWarningAttribValue;
				if (ExtractNamespace(InterchangeResult->GetText(), OutWarningAttribValue))
				{
					int32& Frequency = WarningTypes.FindOrAdd(OutWarningAttribValue);
					Frequency++;
				}
				else
				{
					UE_LOGF(LogInterchangeAnalytics, Error, "Failed to extract Analytic Attribute Value from %ls", *(InterchangeResult->GetText().ToString()));
				}
				break;
			}
			case EInterchangeResultType::Error:
			{
				FString OutErrorAttribValue;
				if (ExtractNamespace(InterchangeResult->GetText(), OutErrorAttribValue))
				{
					int32& Frequency = ErrorTypes.FindOrAdd(OutErrorAttribValue);
					Frequency++;
				}
				else
				{
					UE_LOGF(LogInterchangeAnalytics, Error, "Failed to extract Analytic Attribute Value from %ls", *(InterchangeResult->GetText().ToString()));
				}
				break;
			}
			}
		}
	}
}

void UInterchangeAnalyticsHandlerDefault::Execute_Append(const FString& Identifier, const TArray<FAnalyticsEventAttribute>& ToAdd)
{
	TArray<FAnalyticsEventAttribute>& Analytics = AnalyticsAttributes.FindOrAdd(Identifier);
	Analytics.Append(ToAdd);
}

void UInterchangeAnalyticsHandlerDefault::Execute_Add(const FString& Identifier, const FAnalyticsEventAttribute& Entry)
{
	TArray<FAnalyticsEventAttribute>& Analytics = AnalyticsAttributes.FindOrAdd(Identifier);
	Analytics.Add(Entry);
}

void UInterchangeAnalyticsHandlerDefault::Execute_Clear(const FString& Identifier)
{
	AnalyticsAttributes.Remove(Identifier);
}

void UInterchangeAnalyticsHandlerDefault::Execute_Flush()
{
	if (FEngineAnalytics::IsAvailable())
	{
		const FAnalyticsEventAttribute InterchangeAnalyticsIDAttribute = GetAnalyticsIDAttribute();

		for (TPair<FString, TArray<FAnalyticsEventAttribute>>& AnalyticsPerIdentifier : AnalyticsAttributes)
		{
			TArray<FAnalyticsEventAttribute> ModifiedAttributes;
			ModifiedAttributes.Reserve(AnalyticsPerIdentifier.Value.Num() + 1);
			ModifiedAttributes = MoveTemp(AnalyticsPerIdentifier.Value);
			ModifiedAttributes.Add(InterchangeAnalyticsIDAttribute);

			FEngineAnalytics::GetProvider().RecordEvent(AnalyticsPerIdentifier.Key, ModifiedAttributes);
		}

		AnalyticsAttributes.Empty();
	}
}

void UInterchangeAnalyticsHandlerDefault::SendImmediate(const FString& Identifier, TArray<FAnalyticsEventAttribute> Attributes) const
{
	if (FEngineAnalytics::IsAvailable())
	{
		Attributes.Add(GetAnalyticsIDAttribute());
		FEngineAnalytics::GetProvider().RecordEvent(Identifier, Attributes);
	}
}

void UInterchangeAnalyticsHandlerDefault::SendPipelineAnalytics(UInterchangePipelineBase& Pipeline, const int32 AsyncHelperUniqueId, const FString& ParentPipeline) const
{
	int32 PortFlags = 0;
	UClass* Class = Pipeline.GetClass();
	FString PipelineChainName = ParentPipeline.IsEmpty() ? Pipeline.GetName() : ParentPipeline + TEXT(".") + Pipeline.GetName();

	TArray<FAnalyticsEventAttribute> PipelineAttribs;
	PipelineAttribs.Add(FAnalyticsEventAttribute(TEXT("UniqueId"), AsyncHelperUniqueId));
	PipelineAttribs.Add(FAnalyticsEventAttribute(TEXT("Name"), PipelineChainName));
	PipelineAttribs.Add(FAnalyticsEventAttribute(TEXT("Class"), Class->GetName()));

	for (TPropertyValueIterator<FProperty> It(
			Pipeline.GetClass(),
			&Pipeline, 
			EPropertyValueIteratorFlags::FullRecursion, 
			EFieldIteratorFlags::ExcludeDeprecated); 
			It; 
			++It)
	{
		if (!Pipeline.CanAddPropertyToAnalytics(It))
		{
			It.SkipRecursiveProperty();
			continue;
		}

		const FProperty* Property = It.Key();
		const void* PropertyValue = It.Value();

		const FObjectProperty* SubObject = CastField<FObjectProperty>(Property);
		UInterchangePipelineBase* SubPipeline = SubObject ? Cast<UInterchangePipelineBase>(SubObject->GetObjectPropertyValue(PropertyValue)) : nullptr;

		if (SubPipeline)
		{
			// Save the settings if the referenced pipeline is a sub-object of ours
			if (SubPipeline->IsInOuter(&Pipeline))
			{
				//Go recursive with subObject, like if they are part of the same object
				SendPipelineAnalytics(*SubPipeline, AsyncHelperUniqueId, PipelineChainName);
			}
		}
		else
		{
			Pipeline.AddPropertyAnalytics(It, PipelineAttribs);
		}
	}

	SendImmediate(GetPipelineAnalyticsEventIdentifier(), PipelineAttribs);
}

void UInterchangeAnalyticsHandlerDefault::Send(const TArray<UInterchangePipelineBase*>& Pipelines, const int32 AsyncHelperUniqueId)
{
	if (!FEngineAnalytics::IsAvailable())
	{
		return;
	}

	for (UInterchangePipelineBase* Pipeline : Pipelines)
	{
		if (!Pipeline)
		{
			continue;
		}

		SendPipelineAnalytics(*Pipeline, AsyncHelperUniqueId, FString());
	}
}

void UInterchangeAnalyticsHandlerDefault::Send(const FInterchangeImportResultAnalyticsInfo& ImportResultAnalyticsInfo)
{
	if (ImportResultAnalyticsInfo.bCancel || !FEngineAnalytics::IsAvailable())
	{
		return;
	}

	using namespace UE::Interchange::AnalyticsHelpers;

	TArray<FAnalyticsEventAttribute> Attribs;
	//Set the unique id of this import
	Attribs.Add(FAnalyticsEventAttribute(TEXT("UniqueId"), ImportResultAnalyticsInfo.UniqueId));
	Attribs.Add(FAnalyticsEventAttribute(TEXT("IsCanceled"), ImportResultAnalyticsInfo.bCancel));

	{
		const UInterchangeSourceData* PrimarySourceData = ImportResultAnalyticsInfo.PrimarySourceData; 
		const UInterchangeSourceNode* PrimarySourceNode = ImportResultAnalyticsInfo.PrimarySourceNode;
		Attribs.Add(FAnalyticsEventAttribute(TEXT("PrimarySourceDataMetadata"), GetPrimarySourceDataMetadataAsJsonFragment(PrimarySourceData, PrimarySourceNode)));
	}

	int32 ImportedObjectCount = 0;
	TMap<FString, int32> AssetTypeFrequencyMap;
	constexpr bool bIncludeUntrackedAssets = true;
	for (const TPair<int32, TArray<TObjectPtr<UObject>>>& SourceIndexAndImportedAssets : ImportResultAnalyticsInfo.ImportedAssetsPerSourceIndex)
	{
		ImportedObjectCount += SourceIndexAndImportedAssets.Value.Num();
		FInterchangeAnalyticsAssetTypeTracker::AppendAssetTypeFrequenceMap(SourceIndexAndImportedAssets.Value, AssetTypeFrequencyMap, bIncludeUntrackedAssets);
	}
	
	for (const TPair<int32, TArray<TObjectPtr<UObject>>>& SourceIndexAndImportedSceneObjects : ImportResultAnalyticsInfo.ImportedSceneObjectsPerSourceIndex)
	{
		ImportedObjectCount += SourceIndexAndImportedSceneObjects.Value.Num();
		FInterchangeAnalyticsAssetTypeTracker::AppendAssetTypeFrequenceMap(SourceIndexAndImportedSceneObjects.Value, AssetTypeFrequencyMap, bIncludeUntrackedAssets);
	}

	Attribs.Add(FAnalyticsEventAttribute(TEXT("ImportObjectCount"), ImportedObjectCount));
	Attribs.Add(FAnalyticsEventAttribute(TEXT("ImportObjectDetails"), GetAssetTypeFrequencyMapAsJsonFragment(AssetTypeFrequencyMap)));

	//Report any warning or error message
	{
		TMap<FString, int32> WarningTypes;
		TMap<FString, int32> ErrorTypes;
		if (ImportResultAnalyticsInfo.AssetImportResultsContainer.IsSet())
		{
			CollectResultsContainerWarningAndErrors(ImportResultAnalyticsInfo.AssetImportResultsContainer.GetValue(), WarningTypes, ErrorTypes);
		}
		
		if (ImportResultAnalyticsInfo.SceneImportResultsContainer.IsSet())
		{
			CollectResultsContainerWarningAndErrors(ImportResultAnalyticsInfo.SceneImportResultsContainer.GetValue(), WarningTypes, ErrorTypes);
		}

		Attribs.Add(FAnalyticsEventAttribute(TEXT("WarningTypes"), GetWarningsErrorsMapAsJsonFragment(WarningTypes)));
		Attribs.Add(FAnalyticsEventAttribute(TEXT("ErrorTypes"), GetWarningsErrorsMapAsJsonFragment(ErrorTypes)));
	}
	
	SendImmediate(GetImportResultEventIdentifier(), Attribs);
}
