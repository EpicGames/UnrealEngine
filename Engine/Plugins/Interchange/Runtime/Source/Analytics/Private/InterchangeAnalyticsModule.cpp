// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeAnalyticsModule.h"

#include "Engine/Engine.h"
#include "Misc/CoreDelegates.h"

#include "InterchangeAnalyticsHandlerDefault.h"
#include "InterchangeManager.h"
#include "InterchangeAnalyticsAssetTypeTracker.h"

#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "Engine/Blueprint.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "GeometryCache.h"
#include "GroomAsset.h"
#include "LevelSequence.h"
#include "Materials/MaterialInterface.h"
#include "Sound/SoundWave.h"


class FInterchangeAnalyticsModule : public IInterchangeAnalyticsModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override 
	{
		auto RegisterAnalyticsClass = []() 
		{
			UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
			InterchangeManager.SetAnalyticsHandlerClass(UInterchangeAnalyticsHandlerDefault::StaticClass());

			// Track Asset Types with Interchange Analytics
			FInterchangeAnalyticsAssetTypeTracker::RegisterAssetType(UStaticMesh::StaticClass(), TEXT("StaticMeshes"));
			FInterchangeAnalyticsAssetTypeTracker::RegisterAssetType(USkeletalMesh::StaticClass(), TEXT("SkeletalMeshes"));
			FInterchangeAnalyticsAssetTypeTracker::RegisterAssetType(UMaterialInterface::StaticClass(), TEXT("Materials"));
			FInterchangeAnalyticsAssetTypeTracker::RegisterAssetType(UAnimSequence::StaticClass(), TEXT("Animations"));
			FInterchangeAnalyticsAssetTypeTracker::RegisterAssetType(USkeleton::StaticClass(), TEXT("Skeletons"));
			FInterchangeAnalyticsAssetTypeTracker::RegisterAssetType(USoundWave::StaticClass(), TEXT("Sounds"));
			FInterchangeAnalyticsAssetTypeTracker::RegisterAssetType(UTexture::StaticClass(), TEXT("Textures"));

			FInterchangeAnalyticsAssetTypeTracker::RegisterAssetType(UGroomAsset::StaticClass(), TEXT("Grooms"));
			FInterchangeAnalyticsAssetTypeTracker::RegisterAssetType(UGeometryCache::StaticClass(), TEXT("GeometryCaches"));
			
			FInterchangeAnalyticsAssetTypeTracker::RegisterAssetType(AActor::StaticClass(), TEXT("Actors"));
			FInterchangeAnalyticsAssetTypeTracker::RegisterAssetType(ULevelSequence::StaticClass(), TEXT("LevelSequences"));
			
			FInterchangeAnalyticsAssetTypeTracker::RegisterAssetType(UWorld::StaticClass(), TEXT("Levels"));
			FInterchangeAnalyticsAssetTypeTracker::RegisterAssetType(UBlueprint::StaticClass(), TEXT("PackedLevelActorBlueprints"));
		};

		if (GEngine)
		{
			RegisterAnalyticsClass();
		}
		else
		{
			PostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterAnalyticsClass);
		}
	}

	virtual void ShutdownModule() override 
	{
		FCoreDelegates::GetOnPostEngineInit().Remove(PostEngineInitHandle);
	}

private:
	FDelegateHandle PostEngineInitHandle;
};

IMPLEMENT_MODULE(FInterchangeAnalyticsModule, InterchangeAnalytics)
