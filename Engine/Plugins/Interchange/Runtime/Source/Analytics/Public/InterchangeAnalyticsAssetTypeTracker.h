// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"

#define UE_API INTERCHANGEANALYTICS_API

class FInterchangeAnalyticsAssetTypeTracker
{
public:
	UE_API static void RegisterAssetType(UClass* AssetClass, const FString& DisplayName);

	UE_API static TMap<FString, int32> GetAssetTypeFrequenceMap(const TArray<TObjectPtr<UObject>>& InObjects, bool bIncludeUntrackedClasses = false);

	UE_API static void AppendAssetTypeFrequenceMap(const TArray<TObjectPtr<UObject>>& InObjects, TMap<FString, int32>& OutFreqMap, bool bIncludeUntrackedClasses = false);

private:
    static TSet<UClass*> RegisteredAssetClasses;
    static TMap<UClass*, FString> AssetClassPathToDisplayNameMap;
};


#undef UE_API