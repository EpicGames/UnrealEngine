// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "InterchangeAnalyticsHandlerBase.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"

#include "Misc/Guid.h"
#include "AnalyticsEventAttribute.h"

#include "InterchangeAnalyticsHandlerDefault.generated.h"

#define UE_API INTERCHANGEANALYTICS_API

class UInterchangePipelineBase;

UCLASS(BlueprintType, Blueprintable, MinimalAPI)
class UInterchangeAnalyticsHandlerDefault : public UInterchangeAnalyticsHandlerBase
{
    GENERATED_BODY()
    
public:
	virtual void Send(const TArray<UInterchangePipelineBase*>& Pipelines, const int32 AsyncHelperUniqueId) override;

	virtual void Send(const FInterchangeImportResultAnalyticsInfo& ImportResultAnalyticsInfo) override;

protected:
	void SendImmediate(const FString& Identifier, TArray<FAnalyticsEventAttribute> Attributes) const;

    virtual void Execute_Append(const FString& Identifier, const TArray<FAnalyticsEventAttribute>& ToAdd) override;

    virtual void Execute_Add(const FString& Identifier, const FAnalyticsEventAttribute& Entry) override;

    virtual void Execute_Clear(const FString& Identifier) override;

	virtual void Execute_Flush() override;

private:
	void SendPipelineAnalytics(UInterchangePipelineBase& Pipeline, const int32 AsyncHelperUniqueId, const FString& ParentPipeline) const;

protected:
    //Analytics Attributes to share with the Translators. (to be sent when Import is finished)
    TMap<FString, TArray<FAnalyticsEventAttribute>> AnalyticsAttributes;
    FCriticalSection CriticalSection;
};


#undef UE_API