// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


class FInterchangeCommonModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FInterchangeCommonModule, InterchangeCommon)

