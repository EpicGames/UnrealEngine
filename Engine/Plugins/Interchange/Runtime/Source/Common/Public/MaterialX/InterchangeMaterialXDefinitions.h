// Copyright Epic Games, Inc. All Rights Reserved. 
#pragma once

#include "Math/Color.h"
#include "Math/MathFwd.h"
#include "Math/Vector.h"

// Interchange currently only brings in MaterialX when in the editor, so we
// define its namespace macros ourselves otherwise.
#if WITH_EDITOR

THIRD_PARTY_INCLUDES_START
#include "MaterialXCore/Library.h"
THIRD_PARTY_INCLUDES_END

#else

#define MATERIALX_NAMESPACE_BEGIN namespace MaterialX {
#define MATERIALX_NAMESPACE_END }

#endif


#include "InterchangeMaterialXDefinitions.generated.h"

UENUM(BlueprintType)
enum class EInterchangeMaterialXShaders : uint8
{
	/** Default settings for Open PBR Surface shader. */
	OpenPBRSurface,

	/** Open PBR Surface shader	used for translucency. */
	OpenPBRSurfaceTransmission,

	/** Default settings for Autodesk's Standard Surface shader. */
	StandardSurface,

	/** Standard Surface shader used for translucency. */
	StandardSurfaceTransmission,

	/** Shader used for unlit surfaces. */
	SurfaceUnlit,

	/** Default settings for USD's Surface shader. */
	UsdPreviewSurface,

	/** A surface shader constructed from scattering and emission distribution functions. */
	Surface,

	/** Shader used for displacement. */
	Displacement,

	/** Construct a volume shader describing a participating medium. */
	Volume,

	/** Default settings for glTF Surface shader. */
	GltfPBR,

	MaxShaderCount UMETA(hidden)
};

UENUM(BlueprintType)
/** Data type representing a Bidirectional Scattering Distribution Function. */
enum class EInterchangeMaterialXBSDF : uint8
{
	/** A BSDF node for diffuse reflections. */
	OrenNayarDiffuse,

	/** A BSDF node for Burley diffuse reflections. */
	BurleyDiffuse,

	/** A BSDF node for pure diffuse transmission. */
	Translucent,

	/** A reflection/transmission BSDF node based on a microfacet model and a Fresnel curve for dielectrics. */
	Dielectric,

	/** A reflection BSDF node based on a microfacet model and a Fresnel curve for conductors/metals. */
	Conductor,

	/** A reflection/transmission BSDF node based on a microfacet model and a generalized Schlick Fresnel curve. */
	GeneralizedSchlick,

	/** A subsurface scattering BSDF for true subsurface scattering. */
	Subsurface,

	/** A microfacet BSDF for the back-scattering properties of cloth-like materials. */
	Sheen,

	/* Constructs a hair BSDF based on the Chiang hair shading model [^Chiang2016]. This node does not support vertical layering. */
	ChiangHair,

	ThinFilm UE_DEPRECATED(5.7, "<thin_film_bsdf> has been removed from MaterialX 1.39.3, ThinFilm is therefore deprecated.") UMETA(hidden),

	MaxBSDFCount UMETA(hidden)
};

UENUM(BlueprintType)
/** Data type representing an Emission Distribution Function. */
enum class EInterchangeMaterialXEDF : uint8
{
	/** An EDF node for uniform emission. */
	Uniform,

	/** Constructs an EDF emitting light inside a cone around the normal direction. */
	Conical,

	/** Constructs an EDF emitting light according to a measured IES light profile. */
	Measured,

	MaxEDFCount UMETA(hidden)
};

UENUM(BlueprintType)
/** Data type representing a Volume Distribution Function. */
enum class EInterchangeMaterialXVDF : uint8
{
	/** Constructs a VDF for pure light absorption. */
	Absorption,

	/** Constructs a VDF scattering light for a participating medium, based on the Henyey-Greenstein phase function. */
	Anisotropic,

	MaxVDFCount UMETA(hidden)
};

namespace UE
{
	namespace Interchange
	{
		namespace MaterialX
		{
			inline constexpr uint8 IndexSurfaceShaders = 0;
			inline constexpr uint8 IndexBSDF = 1;
			inline constexpr uint8 IndexEDF = 2;
			inline constexpr uint8 IndexVDF = 3;

			namespace Attributes
			{
				inline constexpr const TCHAR* EnumType = TEXT("MaterialXEnumType");
				inline constexpr const TCHAR* EnumValue = TEXT("MaterialXEnumValue");
			}
		}
	}

	namespace Expressions
	{
		namespace Names
		{
			inline constexpr const TCHAR* Abs = TEXT("Abs");
			inline constexpr const TCHAR* Add = TEXT("Add");
			inline constexpr const TCHAR* AppendVector = TEXT("AppendVector");
			inline constexpr const TCHAR* Arccosine = TEXT("Arccosine");
			inline constexpr const TCHAR* Arcsine = TEXT("Arcsine");
			inline constexpr const TCHAR* Arctangent2 = TEXT("Arctangent2");
			inline constexpr const TCHAR* BlackBody = TEXT("BlackBody");
			inline constexpr const TCHAR* CameraVectorWS = TEXT("CameraVectorWS");
			inline constexpr const TCHAR* Ceil = TEXT("Ceil");
			inline constexpr const TCHAR* Clamp = TEXT("Clamp");
			inline constexpr const TCHAR* Contrast = TEXT("MaterialXContrast");
			inline constexpr const TCHAR* Cosine = TEXT("Cosine");
			inline constexpr const TCHAR* Crossproduct = TEXT("Crossproduct");
			inline constexpr const TCHAR* Desaturation = TEXT("Desaturation");
			inline constexpr const TCHAR* Divide = TEXT("Divide");
			inline constexpr const TCHAR* Distance = TEXT("Distance");
			inline constexpr const TCHAR* Dotproduct = TEXT("Dotproduct");
			inline constexpr const TCHAR* Exponential = TEXT("Exponential");
			inline constexpr const TCHAR* Floor = TEXT("Floor");
			inline constexpr const TCHAR* Frac = TEXT("Frac");
			inline constexpr const TCHAR* HsvToRgb = TEXT("HsvToRgb");
			inline constexpr const TCHAR* If = TEXT("If");
			inline constexpr const TCHAR* Logarithm = TEXT("Logarithm");
			inline constexpr const TCHAR* Length = TEXT("Length");
			inline constexpr const TCHAR* Lerp = TEXT("Lerp");
			inline constexpr const TCHAR* LocalPosition = TEXT("LocalPosition");
			inline constexpr const TCHAR* MaterialXAppend3Vector = TEXT("MaterialXAppend3Vector");
			inline constexpr const TCHAR* MaterialXAppend4Vector = TEXT("MaterialXAppend4Vector");
			inline constexpr const TCHAR* MaterialXBurn = TEXT("MaterialXBurn");
			inline constexpr const TCHAR* MaterialXDifference = TEXT("MaterialXDifference");
			inline constexpr const TCHAR* MaterialXDisjointover = TEXT("MaterialXDisjointover");
			inline constexpr const TCHAR* MaterialXDodge = TEXT("MaterialXDodge");
			inline constexpr const TCHAR* MaterialXFractal = TEXT("MaterialXFractal");
			inline constexpr const TCHAR* MaterialXIn = TEXT("MaterialXIn");
			inline constexpr const TCHAR* MaterialXLuminance = TEXT("MaterialXLuminance");
			inline constexpr const TCHAR* MaterialXMask = TEXT("MaterialXMask");
			inline constexpr const TCHAR* MaterialXMatte = TEXT("MaterialXMatte");
			inline constexpr const TCHAR* MaterialXMinus = TEXT("MaterialXMinus");
			inline constexpr const TCHAR* MaterialXMod = TEXT("MaterialXMod");
			inline constexpr const TCHAR* MaterialXOut = TEXT("MaterialXOut");
			inline constexpr const TCHAR* MaterialXOver = TEXT("MaterialXOver");
			inline constexpr const TCHAR* MaterialXOverlay = TEXT("MaterialXOverlay");
			inline constexpr const TCHAR* MaterialXPlus = TEXT("MaterialXPlus");
			inline constexpr const TCHAR* MaterialXPremult = TEXT("MaterialXPremult");
			inline constexpr const TCHAR* MaterialXRamp4 = TEXT("MaterialXRamp4");
			inline constexpr const TCHAR* MaterialXRampLeftRight = TEXT("MaterialXRampLeftRight");
			inline constexpr const TCHAR* MaterialXRampTopBottom = TEXT("MaterialXRampTopBottom");
			inline constexpr const TCHAR* MaterialXRange = TEXT("MaterialXRange");
			inline constexpr const TCHAR* MaterialXRemap = TEXT("MaterialXRemap");
			inline constexpr const TCHAR* MaterialXScreen = TEXT("MaterialXScreen");
			inline constexpr const TCHAR* MaterialXSplitLeftRight = TEXT("MaterialXSplitLeftRight");
			inline constexpr const TCHAR* MaterialXSplitTopBottom = TEXT("MaterialXSplitTopBottom");
			inline constexpr const TCHAR* MaterialXUnpremult = TEXT("MaterialXUnpremult");
			inline constexpr const TCHAR* Max = TEXT("Max");
			inline constexpr const TCHAR* Min = TEXT("Min");
			inline constexpr const TCHAR* Multiply = TEXT("Multiply");
			inline constexpr const TCHAR* Normalize = TEXT("Normalize");
			inline constexpr const TCHAR* Power = TEXT("Power");
			inline constexpr const TCHAR* RgbToHsv = TEXT("RgbToHsv");
			inline constexpr const TCHAR* RotateAboutAxis = TEXT("RotateAboutAxis");
			inline constexpr const TCHAR* Round = TEXT("Round");
			inline constexpr const TCHAR* Sign = TEXT("Sign");
			inline constexpr const TCHAR* Sine = TEXT("Sine");
			inline constexpr const TCHAR* SmoothStep = TEXT("SmoothStep");
			inline constexpr const TCHAR* SquareRoot = TEXT("SquareRoot");
			inline constexpr const TCHAR* SubstrateAdd = TEXT("SubstrateAdd");
			inline constexpr const TCHAR* SubstrateHorizontalMixing = TEXT("SubstrateHorizontalMixing");
			inline constexpr const TCHAR* SubstrateVerticalLayering = TEXT("SubstrateVerticalLayering");
			inline constexpr const TCHAR* SubstrateWeight = TEXT("SubstrateWeight");
			inline constexpr const TCHAR* Subtract = TEXT("Subtract");
			inline constexpr const TCHAR* Tangent = TEXT("Tangent");
			inline constexpr const TCHAR* Time = TEXT("Time");
			inline constexpr const TCHAR* Transform = TEXT("Transform");
			inline constexpr const TCHAR* TransformPosition = TEXT("TransformPosition");
			inline constexpr const TCHAR* VertexColor = TEXT("VertexColor");
			inline constexpr const TCHAR* VertexNormalWS = TEXT("VertexNormalWS");
			inline constexpr const TCHAR* VertexTangentWS = TEXT("VertexTangentWS");
			inline constexpr const TCHAR* WorldPosition = TEXT("WorldPosition");
		}

		namespace Inputs
		{
			inline constexpr const TCHAR* A = TEXT("A");
			inline constexpr const TCHAR* AbsoluteWorldPosition = TEXT("Absolute World Position");
			inline constexpr const TCHAR* AEqualsB = TEXT("AEqualsB");
			inline constexpr const TCHAR* AGreaterThanB = TEXT("AGreaterThanB");
			inline constexpr const TCHAR* Alpha = TEXT("Alpha");
			inline constexpr const TCHAR* ALessThanB = TEXT("ALessThanB");
			inline constexpr const TCHAR* Amount= TEXT("Amount");
			inline constexpr const TCHAR* Amplitude = TEXT("Amplitude");
			inline constexpr const TCHAR* B = TEXT("B");
			inline constexpr const TCHAR* Background = TEXT("Background");
			inline constexpr const TCHAR* Base = TEXT("Base");
			inline constexpr const TCHAR* Bottom = TEXT("Bottom");
			inline constexpr const TCHAR* C = TEXT("C");
			inline constexpr const TCHAR* Clamp = TEXT("Clamp");
			inline constexpr const TCHAR* Center = TEXT("Center");
			inline constexpr const TCHAR* Coordinate = TEXT("Coordinate");
			inline constexpr const TCHAR* Coordinates = TEXT("Coordinates");
			inline constexpr const TCHAR* D = TEXT("D");
			inline constexpr const TCHAR* Default = TEXT("Default");
			inline constexpr const TCHAR* Diminish = TEXT("Diminish");
			inline constexpr const TCHAR* Exponent = TEXT("Exponent");
			inline constexpr const TCHAR* Factor = TEXT("Factor");
			inline constexpr const TCHAR* Flatness = TEXT("Flatness");
			inline constexpr const TCHAR* Float2 = TEXT("Float2");
			inline constexpr const TCHAR* Float3 = TEXT("Float3");
			inline constexpr const TCHAR* Float4 = TEXT("Float4");
			inline constexpr const TCHAR* Foreground = TEXT("Foreground");
			inline constexpr const TCHAR* Fraction = TEXT("Fraction");
			inline constexpr const TCHAR* Gamma = TEXT("Gamma");
			inline constexpr const TCHAR* Height = TEXT("Height");
			inline constexpr const TCHAR* in1 = TEXT("in1");
			inline constexpr const TCHAR* in2 = TEXT("in2");
			inline constexpr const TCHAR* in3 = TEXT("in3");
			inline constexpr const TCHAR* in4 = TEXT("in4");
			inline constexpr const TCHAR* in5 = TEXT("in5");
			inline constexpr const TCHAR* in6 = TEXT("in6");
			inline constexpr const TCHAR* in7 = TEXT("in7");
			inline constexpr const TCHAR* in8 = TEXT("in8");
			inline constexpr const TCHAR* in9 = TEXT("in9");
			inline constexpr const TCHAR* in10 = TEXT("in10");
			inline constexpr const TCHAR* Input = TEXT("Input");
			inline constexpr const TCHAR* InputLow = TEXT("InputLow");
			inline constexpr const TCHAR* InputHigh = TEXT("InputHigh");
			inline constexpr const TCHAR* Lacunarity = TEXT("Lacunarity");
			inline constexpr const TCHAR* LuminanceFactors = TEXT("LuminanceFactors");
			inline constexpr const TCHAR* Max = TEXT("Max");
			inline constexpr const TCHAR* Min = TEXT("Min");
			inline constexpr const TCHAR* Mix = TEXT("Mix");
			inline constexpr const TCHAR* Normal = TEXT("Normal");
			inline constexpr const TCHAR* NormalizedRotationAxis = TEXT("NormalizedRotationAxis");
			inline constexpr const TCHAR* Octaves = TEXT("Octaves");
			inline constexpr const TCHAR* Offset = TEXT("Offset");
			inline constexpr const TCHAR* Pivot = TEXT("Pivot");
			inline constexpr const TCHAR* Position = TEXT("Position");
			inline constexpr const TCHAR* RayDirection = TEXT("Ray Direction");
			inline constexpr const TCHAR* RefractiveIndexOrigin = TEXT("Refractive Index Origin");
			inline constexpr const TCHAR* RefractiveIndexTarget = TEXT("Refractive Index Target");
			inline constexpr const TCHAR* RotationAngle = TEXT("RotationAngle");
			inline constexpr const TCHAR* Scale = TEXT("Scale");
			inline constexpr const TCHAR* SurfaceNormal = TEXT("Surface Normal");
			inline constexpr const TCHAR* SwitchValue = TEXT("SwitchValue");
			inline constexpr const TCHAR* TargetLow = TEXT("TargetLow");
			inline constexpr const TCHAR* TargetHigh = TEXT("TargetHigh");
			inline constexpr const TCHAR* Temp = TEXT("Temp");
			inline constexpr const TCHAR* Time = TEXT("Time");
			inline constexpr const TCHAR* Top = TEXT("Top");
			inline constexpr const TCHAR* Value = TEXT("Value");
			inline constexpr const TCHAR* VectorInput = TEXT("VectorInput");
			inline constexpr const TCHAR* Weight = TEXT("Weight");
			inline constexpr const TCHAR* X = TEXT("X");
			inline constexpr const TCHAR* Y = TEXT("Y");
		}
	}

	namespace MaterialFunctions
	{
		namespace Path
		{
			inline constexpr const TCHAR* BreakOutFloat2Components = TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/BreakOutFloat2Components.BreakOutFloat2Components");
			inline constexpr const TCHAR* BreakOutFloat3Components = TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/BreakOutFloat3Components.BreakOutFloat3Components");
			inline constexpr const TCHAR* BreakOutFloat4Components = TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/BreakOutFloat4Components.BreakOutFloat4Components");
			inline constexpr const TCHAR* DitherMask= TEXT("/InterchangeAssets/Substrate/MF_DitherMask.MF_DitherMask");
			inline constexpr const TCHAR* HeightToNormalSmooth = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Procedurals/HeightToNormalSmooth.HeightToNormalSmooth");
			inline constexpr const TCHAR* MxACEScgToRec709 = TEXT("/InterchangeAssets/Functions/MX_ACEScgToRec709.MX_ACEScgToRec709");
			inline constexpr const TCHAR* MxAdobeRGBToRec709 = TEXT("/InterchangeAssets/Functions/MX_AdobeRGBToRec709.MX_AdobeRGBToRec709");
			inline constexpr const TCHAR* MxArtisticIOR = TEXT("/InterchangeAssets/Functions/MX_Artistic_IOR.MX_Artistic_IOR");
			inline constexpr const TCHAR* MxAnd = TEXT("/InterchangeAssets/Functions/MX_And.MX_And");
			inline constexpr const TCHAR* MxCircle = TEXT("/InterchangeAssets/Functions/MX_Circle.MX_Circle");
			inline constexpr const TCHAR* MxCheckerboard = TEXT("/InterchangeAssets/Functions/MX_Checkerboard.MX_Checkerboard");
			inline constexpr const TCHAR* MxChiangHairAbsorptionFromColor = TEXT("/InterchangeAssets/Functions/MX_ChiangHairAbsorptionFromColor.MX_ChiangHairAbsorptionFromColor");
			inline constexpr const TCHAR* MxChiangHairRoughness = TEXT("/InterchangeAssets/Functions/MX_ChiangHairRoughness.MX_ChiangHairRoughness");
			inline constexpr const TCHAR* MxColorCorrect = TEXT("/InterchangeAssets/Functions/MX_ColorCorrect.MX_ColorCorrect");
			inline constexpr const TCHAR* MxDeonHairAbsorptionFromMelanin = TEXT("/InterchangeAssets/Functions/MX_DeonHairAbsorptionFromMelanin.MX_DeonHairAbsorptionFromMelanin");
			inline constexpr const TCHAR* MxGoochShade = TEXT("/InterchangeAssets/Functions/MX_GoochShade.MX_GoochShade");
			inline constexpr const TCHAR* MxHeightToNormalMap = TEXT("/InterchangeAssets/Functions/MX_HeightToNormal.MX_HeightToNormal");
			inline constexpr const TCHAR* MxHexTiledImage = TEXT("/InterchangeAssets/Functions/MX_HexTiledImage.MX_HexTiledImage");
			inline constexpr const TCHAR* MxHexTiledNormalMap= TEXT("/InterchangeAssets/Functions/MX_HexTiledNormalMap.MX_HexTiledNormalMap");
			inline constexpr const TCHAR* MxHsvAdjust = TEXT("/InterchangeAssets/Functions/MX_HSVAdjust.MX_HSVAdjust");
			inline constexpr const TCHAR* MxLatLongImage = TEXT("/InterchangeAssets/Functions/MX_LatLongImage.MX_LatLongImage");
			inline constexpr const TCHAR* MxLine = TEXT("/InterchangeAssets/Functions/MX_Line.MX_Line");
			inline constexpr const TCHAR* MxLinearAdobeRGBToRec709 = TEXT("/InterchangeAssets/Functions/MX_LinearAdobeRGBToRec709.MX_LinearAdobeRGBToRec709");
			inline constexpr const TCHAR* MxLinearDisplayP3ToRec709 = TEXT("/InterchangeAssets/Functions/MX_LinearDisplayP3ToRec709.MX_LinearDisplayP3ToRec709");
			inline constexpr const TCHAR* MxOr = TEXT("/InterchangeAssets/Functions/MX_Or.MX_Or");
			inline constexpr const TCHAR* MxNot = TEXT("/InterchangeAssets/Functions/MX_Not.MX_Not");
			inline constexpr const TCHAR* MxPlace2D = TEXT("/InterchangeAssets/Functions/MX_Place2D.MX_Place2D");
			inline constexpr const TCHAR* MxRamp = TEXT("/InterchangeAssets/Functions/MX_Ramp.MX_Ramp");
			inline constexpr const TCHAR* MxRampGradient = TEXT("/InterchangeAssets/Functions/MX_RampGradient.MX_RampGradient");
			inline constexpr const TCHAR* MxRandomFloat = TEXT("/InterchangeAssets/Functions/MX_RandomFloat.MX_RandomFloat");
			inline constexpr const TCHAR* MxRandomColor = TEXT("/InterchangeAssets/Functions/MX_RandomColor.MX_RandomColor");
			inline constexpr const TCHAR* MxRotate3D = TEXT("/InterchangeAssets/Functions/MX_Rotate3D.MX_Rotate3D");
			inline constexpr const TCHAR* MxRoughnesAnisotropy = TEXT("/InterchangeAssets/Functions/MX_Roughness_Anisotropy.MX_Roughness_Anisotropy");
			inline constexpr const TCHAR* MxRoughnessDual = TEXT("/InterchangeAssets/Functions/MX_Roughness_Dual.MX_Roughness_Dual");
			inline constexpr const TCHAR* MxSrgbDisplayP3ToRec709 = TEXT("/InterchangeAssets/Functions/MX_SrgbDisplayP3ToRec709.MX_SrgbDisplayP3ToRec709");
			inline constexpr const TCHAR* MxSrgbToRec709 = TEXT("/InterchangeAssets/Functions/MX_SrgbToRec709.MX_SrgbToRec709");
			inline constexpr const TCHAR* MxUnifiedNoise3D = TEXT("/InterchangeAssets/Functions/MX_UnifiedNoise3d.MX_UnifiedNoise3d");
			inline constexpr const TCHAR* MxTiledCircles = TEXT("/InterchangeAssets/Functions/MX_TiledCircles.MX_TiledCircles");
			inline constexpr const TCHAR* MxXor = TEXT("/InterchangeAssets/Functions/MX_Xor.MX_Xor");
			inline constexpr const TCHAR* NormalFromHeightMap = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Procedurals/NormalFromHeightmap.NormalFromHeightmap");
			inline constexpr const TCHAR* Refract = TEXT("/Engine/Functions/Engine_MaterialFunctions01/Vectors/Refract.Refract");
		}
	}
}

MATERIALX_NAMESPACE_BEGIN

	namespace OpenPBRSurface
	{
		namespace Input
		{
			inline constexpr const char* BaseWeight = "base_weight";
			inline constexpr const char* BaseColor = "base_color";
			inline constexpr const char* BaseDiffuseRoughness = "base_diffuse_roughness";
			inline constexpr const char* BaseMetalness = "base_metalness";
			inline constexpr const char* SpecularWeight = "specular_weight";
			inline constexpr const char* SpecularColor = "specular_color";
			inline constexpr const char* SpecularRoughness = "specular_roughness";
			inline constexpr const char* SpecularIOR = "specular_ior";
			inline constexpr const char* SpecularRoughnessAnisotropy = "specular_roughness_anisotropy";
			inline constexpr const char* TransmissionWeight = "transmission_weight";
			inline constexpr const char* TransmissionColor = "transmission_color";
			inline constexpr const char* TransmissionDepth = "transmission_depth";
			inline constexpr const char* TransmissionScatter = "transmission_scatter";
			inline constexpr const char* TransmissionScatterAnisotropy= "transmission_scatter_anisotropy";
			inline constexpr const char* TransmissionDispersionScale = "transmission_dispersion_scale";
			inline constexpr const char* TransmissionDispersionAbbeNumber= "transmission_dispersion_abbe_number";
			inline constexpr const char* SubsurfaceWeight = "subsurface_weight";
			inline constexpr const char* SubsurfaceColor = "subsurface_color";
			inline constexpr const char* SubsurfaceRadius = "subsurface_radius";
			inline constexpr const char* SubsurfaceRadiusScale = "subsurface_radius_scale";
			inline constexpr const char* SubsurfaceScatterAnisotropy = "subsurface_scatter_anisotropy";
			inline constexpr const char* FuzzWeight = "fuzz_weight";
			inline constexpr const char* FuzzColor = "fuzz_color";
			inline constexpr const char* FuzzRoughness = "fuzz_roughness";
			inline constexpr const char* CoatWeight = "coat_weight";
			inline constexpr const char* CoatColor = "coat_color";
			inline constexpr const char* CoatRoughness = "coat_roughness";
			inline constexpr const char* CoatRoughnessAnisotropy = "coat_roughness_anisotropy";
			inline constexpr const char* CoatRotation = "coat_rotation";
			inline constexpr const char* CoatIOR = "coat_ior";
			inline constexpr const char* CoatDarkening = "coat_darkening";
			inline constexpr const char* ThinFilmWeight= "thin_film_weight";
			inline constexpr const char* ThinFilmThickness = "thin_film_thickness";
			inline constexpr const char* ThinFilmIOR = "thin_film_ior";
			inline constexpr const char* EmissionLuminance = "emission_luminance";
			inline constexpr const char* EmissionColor = "emission_color";
			inline constexpr const char* GeometryOpacity = "geometry_opacity";
			inline constexpr const char* GeometryThinWalled = "geometry_thin_walled";
			inline constexpr const char* GeometryNormal = "geometry_normal";
			inline constexpr const char* GeometryCoatNormal = "geometry_coat_normal";
			inline constexpr const char* GeometryTangent = "geometry_tangent";
			inline constexpr const char* GeometryCoatTangent = "geometry_coat_tangent";
		}

		namespace DefaultValue
		{
			inline constexpr float BaseWeight = 1.f;
			inline constexpr FLinearColor BaseColor{ 0.8f, 0.8f, 0.8f };
			inline constexpr float BaseDiffuseRoughness = 0.f;
			inline constexpr float BaseMetalness = 0.f;
			inline constexpr float SpecularWeight = 1.f;
			inline constexpr FLinearColor SpecularColor{ 1, 1, 1 };
			inline constexpr float SpecularRoughness = 0.3f;
			inline constexpr float SpecularIOR = 1.5f;
			inline constexpr float SpecularRoughnessAnisotropy = 0.f;
			inline constexpr float TransmissionWeight = 0.f;
			inline constexpr FLinearColor TransmissionColor{ 1, 1, 1 };
			inline constexpr float TransmissionDepth = 0.f;
			inline constexpr FLinearColor TransmissionScatter{ 0, 0, 0 };
			inline constexpr float TransmissionScatterAnisotropy = 0.f;
			inline constexpr float TransmissionDispersionScale = 0.f;
			inline constexpr float TransmissionDispersionAbbeNumber = 20.f;
			inline constexpr float SubsurfaceWeight = 0.f;
			inline constexpr FLinearColor SubsurfaceColor{ 0.8f, 0.8f, 0.8f };
			inline constexpr float SubsurfaceRadius = 1.f;
			inline constexpr FLinearColor SubsurfaceRadiusScale{1, 0.5, 0.25};
			inline constexpr float SubsurfaceScatterAnisotropy = 0.f;
			inline constexpr float FuzzWeight = 0.f;
			inline constexpr FLinearColor FuzzColor{ 1, 1, 1 };
			inline constexpr float FuzzRoughness = 0.5f;
			inline constexpr float CoatWeight = 0.f;
			inline constexpr FLinearColor CoatColor{ 1, 1, 1 };
			inline constexpr float CoatRoughness = 0.f;
			inline constexpr float CoatRoughnessAnisotropy = 0.f;
			inline constexpr float CoatIOR = 1.6f;
			inline constexpr float CoatDarkening = 1.f;
			inline constexpr float ThinFilmWeight = 0.f;
			inline constexpr float ThinFilmThickness = 0.5f;
			inline constexpr float ThinFilmIOR = 1.4f;
			inline constexpr float EmissionLuminance = 0.f;
			inline constexpr FLinearColor EmissionColor{ 1, 1, 1 };
			inline constexpr float GeometryOpacity = 1.f;
			inline constexpr bool GeometryThinWalled = false;
			inline const FVector GeometryNormal{ 0, 0, 1 };
			inline const FVector GeometryCoatNormal{ 0, 0, 1 };
			inline const FVector GeometryTangent{ 0, 1, 0 };
			inline const FVector GeometryCoatTangent{ 0, 1, 0 };
		}
	}

	namespace StandardSurface
	{
		namespace Input
		{
			inline constexpr const char* Base = "base";
			inline constexpr const char* BaseColor = "base_color";
			inline constexpr const char* DiffuseRoughness = "diffuse_roughness";
			inline constexpr const char* Metalness = "metalness";
			inline constexpr const char* Specular = "specular";
			inline constexpr const char* SpecularColor = "specular_color";
			inline constexpr const char* SpecularRoughness = "specular_roughness";
			inline constexpr const char* SpecularIOR = "specular_IOR";
			inline constexpr const char* SpecularAnisotropy = "specular_anisotropy";
			inline constexpr const char* SpecularRotation = "specular_rotation";
			inline constexpr const char* Transmission = "transmission";
			inline constexpr const char* TransmissionColor = "transmission_color";
			inline constexpr const char* TransmissionDepth = "transmission_depth";
			inline constexpr const char* TransmissionScatter = "transmission_scatter";
			inline constexpr const char* TransmissionScatterAnisotropy = "transmission_scatter_anisotropy";
			inline constexpr const char* TransmissionDispersion = "transmission_dispersion";
			inline constexpr const char* TransmissionExtraRoughness = "transmission_extra_roughness";
			inline constexpr const char* Subsurface = "subsurface";
			inline constexpr const char* SubsurfaceColor = "subsurface_color";
			inline constexpr const char* SubsurfaceRadius = "subsurface_radius";
			inline constexpr const char* SubsurfaceScale = "subsurface_scale";
			inline constexpr const char* SubsurfaceAnisotropy = "subsurface_anisotropy";
			inline constexpr const char* Sheen = "sheen";
			inline constexpr const char* SheenColor = "sheen_color";
			inline constexpr const char* SheenRoughness = "sheen_roughness";
			inline constexpr const char* Coat = "coat";
			inline constexpr const char* CoatColor = "coat_color";
			inline constexpr const char* CoatRoughness = "coat_roughness";
			inline constexpr const char* CoatAnisotropy = "coat_anisotropy";
			inline constexpr const char* CoatRotation = "coat_rotation";
			inline constexpr const char* CoatIOR = "coat_IOR";
			inline constexpr const char* CoatNormal = "coat_normal";
			inline constexpr const char* CoatAffectColor = "coat_affect_color";
			inline constexpr const char* CoatAffectRoughness = "coat_affect_roughness";
			inline constexpr const char* ThinFilmThickness = "thin_film_thickness";
			inline constexpr const char* ThinFilmIOR = "thin_film_IOR";
			inline constexpr const char* Emission = "emission";
			inline constexpr const char* EmissionColor = "emission_color";
			inline constexpr const char* Opacity = "opacity";
			inline constexpr const char* ThinWalled = "thin_walled";
			inline constexpr const char* Normal = "normal";
			inline constexpr const char* Tangent = "tangent";
		}

		namespace DefaultValue
		{
			inline constexpr float Base = 1.f;
			inline constexpr FLinearColor BaseColor{ 0.8f, 0.8f, 0.8f };
			inline constexpr float DiffuseRoughness = 0.f;
			inline constexpr float Metalness = 0.f;
			inline constexpr float Specular = 1.f;
			inline constexpr FLinearColor SpecularColor{ 1, 1, 1 };
			inline constexpr float SpecularRoughness = 0.2f;
			inline constexpr float SpecularIOR = 1.5f;
			inline constexpr float SpecularAnisotropy = 0.f;
			inline constexpr float SpecularRotation = 0.f;
			inline constexpr float Transmission = 0.f;
			inline constexpr float TransmissionDepth = 0.f;
			inline constexpr float TransmissionScatterAnisotropy = 0.f;
			inline constexpr float TransmissionDispersion = 0.f;
			inline constexpr float TransmissionExtraRoughness = 0.f;
			inline constexpr float Subsurface = 0.f;
			inline constexpr FLinearColor SubsurfaceColor{ 1, 1, 1 };
			inline constexpr FLinearColor SubsurfaceRadius{ 1, 1, 1 };
			inline constexpr float SubsurfaceScale = 1.f;
			inline constexpr float SubsurfaceAnisotropy = 0.f;
			inline constexpr float Sheen = 0.f;
			inline constexpr FLinearColor SheenColor{ 1, 1, 1 };
			inline constexpr float SheenRoughness = 0.3f;
			inline constexpr float Coat = 0.f;
			inline constexpr FLinearColor CoatColor{ 1, 1, 1 };
			inline constexpr float CoatRoughness = 0.1f;
			inline constexpr float CoatAnisotropy = 0.f;
			inline constexpr float CoatRotation = 0.f;
			inline constexpr float CoatIOR = 1.5f;
			inline constexpr float CoatAffectColor = 0.f;
			inline constexpr float CoatAffectRoughness = 0.f;
			inline constexpr float ThinFilmThickness = 0.f;
			inline constexpr float ThinFilmIOR = 1.5f;
			inline constexpr FLinearColor TransmissionColor{ 1, 1, 1 };
			inline constexpr FLinearColor TransmissionScatter{ 0, 0, 0 };
			inline constexpr float Emission = 0.f;
			inline constexpr FLinearColor EmissionColor{ 1, 1, 1 };
			inline constexpr FLinearColor Opacity{ 1, 1, 1 };
			inline constexpr bool ThinWalled = false;
		}
	}

	namespace SurfaceUnlit
	{
		namespace Input
		{
			inline constexpr const char* Emission = "emission";
			inline constexpr const char* EmissionColor = "emission_color";
			inline constexpr const char* Transmission = "transmission";
			inline constexpr const char* TransmissionColor = "transmission_color";
			inline constexpr const char* Opacity = "opacity";
		}

		namespace DefaultValue
		{
			namespace Float
			{
				inline constexpr float Emission = 1.f;
				inline constexpr float Transmission = 0.f;
				inline constexpr float Opacity = 1.f;
			}

			namespace Color3
			{
				inline constexpr FLinearColor EmissionColor{ 1, 1, 1 };
				inline constexpr FLinearColor TransmissionColor{ 1, 1, 1 };
			}
		}
	}

	namespace Surface
	{
		namespace Input
		{
			inline constexpr const char* Bsdf = "bsdf";
			inline constexpr const char* Edf = "edf";
			inline constexpr const char* Opacity = "opacity";
		}

		namespace DefaultValue
		{
			namespace Float
			{
				inline constexpr float Opacity = 1.f;
			}
		}
	}

	namespace UsdPreviewSurface
	{
		namespace Input
		{
			inline constexpr const char* DiffuseColor = "diffuseColor";
			inline constexpr const char* EmissiveColor = "emissiveColor";
			inline constexpr const char* SpecularColor = "specularColor";
			inline constexpr const char* Metallic = "metallic";
			inline constexpr const char* Roughness = "roughness";
			inline constexpr const char* Clearcoat = "clearcoat";
			inline constexpr const char* ClearcoatRoughness = "clearcoatRoughness";
			inline constexpr const char* Opacity = "opacity";
			inline constexpr const char* OpacityThreshold = "opacityThreshold";
			inline constexpr const char* IOR = "ior";
			inline constexpr const char* Normal = "normal";
			inline constexpr const char* Displacement = "displacement";
			inline constexpr const char* Occlusion = "occlusion";
		}

		namespace DefaultValue
		{
			namespace Float
			{
				inline constexpr float Metallic = 0.f;
				inline constexpr float Roughness = 0.5f;
				inline constexpr float Clearcoat = 0.f;
				inline constexpr float ClearcoatRoughness = 0.01f;
				inline constexpr float Opacity = 1.f;
				inline constexpr float OpacityThreshold = 0.f;
				inline constexpr float IOR = 1.5f;
				inline constexpr float Displacement = 0.f;
				inline constexpr float Occlusion = 1.f;
			}

			namespace Color3
			{
				inline constexpr FLinearColor DiffuseColor{ 0.18f, 0.18f, 0.18f };
				inline constexpr FLinearColor EmissiveColor{ 0, 0, 0 };
				inline constexpr FLinearColor SpecularColor{ 0, 0, 0 };
			}
			
			namespace Vector3
			{
				inline const FVector3f Normal{ 0.f, 0.f, 1.f };
			}
		}
	}

	namespace GltfPbr
	{
		namespace Input
		{
			inline constexpr const char* BaseColor = "base_color";
			inline constexpr const char* Metallic = "metallic";
			inline constexpr const char* Roughness = "roughness";
			inline constexpr const char* Normal = "normal";
			inline constexpr const char* Tangent = "tangent";
			inline constexpr const char* Occlusion = "occlusion";
			inline constexpr const char* Transmission = "transmission";
			inline constexpr const char* Specular = "specular";
			inline constexpr const char* SpecularColor = "specular_color";
			inline constexpr const char* Ior = "ior";
			inline constexpr const char* Alpha = "alpha";
			inline constexpr const char* AlphaMode = "alpha_mode";
			inline constexpr const char* AlphaCutoff = "alpha_cutoff";
			inline constexpr const char* Iridescence = "iridescence";
			inline constexpr const char* IridescenceIor = "iridescence_ior";
			inline constexpr const char* IridescenceThickness = "iridescence_thickness";
			inline constexpr const char* SheenColor = "sheen_color";
			inline constexpr const char* SheenRoughness = "sheen_roughness";
			inline constexpr const char* Clearcoat = "clearcoat";
			inline constexpr const char* ClearcoatRoughness = "clearcoat_roughness";
			inline constexpr const char* ClearcoatNormal = "clearcoat_normal";
			inline constexpr const char* Emissive = "emissive";
			inline constexpr const char* EmissiveStrength = "emissive_strength";
			inline constexpr const char* Thickness = "thickness";
			inline constexpr const char* AttenuationDistance = "attenuation_distance";
			inline constexpr const char* AttenuationColor = "attenuation_color";
			inline constexpr const char* AnisotropyStrength = "anisotropy_strength";
			inline constexpr const char* AnisotropyRotation = "anisotropy_rotation";
			inline constexpr const char* Dispersion = "dispersion";
		}
	}

	namespace Lights
	{
		//There's no input per se in a Light, but we can find some common inputs among those lights
		namespace Input
		{
			inline constexpr const char* Color = "color";
			inline constexpr const char* Intensity = "intensity";
		}

		namespace PointLight
		{
			namespace Input
			{
				using namespace Lights::Input;
				inline constexpr const char* Position = "position";
				inline constexpr const char* DecayRate = "decay_rate";
			}
		}

		namespace DirectionalLight
		{
			namespace Input
			{
				using namespace Lights::Input;
				inline constexpr const char* Direction = "direction";
			}
		}

		namespace SpotLight
		{
			namespace Input
			{
				using namespace PointLight::Input;
				using namespace DirectionalLight::Input;
				inline constexpr const char* InnerAngle = "inner_angle";
				inline constexpr const char* OuterAngle = "outer_angle";
			}
		}
	}

	namespace Attributes
	{
		inline constexpr const char* IsVisited = "UE:IsVisited";
		inline constexpr const char* NewName = "UE:NewName";
		inline constexpr const char* ParentName = "UE:ParentName";
		inline constexpr const char* UniqueName = "UE:UniqueName";
		inline constexpr const char* GeomPropImage = "UE:GeomPropImage";
		inline constexpr const char* GeomPropSparseVolume = "UE:GeomPropSparseVolume";
	}

	namespace Category
	{
		// Math nodes
		inline constexpr const char* Absval = "absval";
		inline constexpr const char* Acos = "acos";
		inline constexpr const char* Add = "add";
		inline constexpr const char* ArrayAppend = "arrayappend";
		inline constexpr const char* Asin = "asin";
		inline constexpr const char* Atan2 = "atan2";
		inline constexpr const char* Ceil = "ceil";
		inline constexpr const char* Clamp = "clamp";
		inline constexpr const char* Cos = "cos";
		inline constexpr const char* CrossProduct = "crossproduct";
		inline constexpr const char* Determinant = "determinant";
		inline constexpr const char* Divide = "divide";
		inline constexpr const char* Distance = "distance";
		inline constexpr const char* DotProduct = "dotproduct";
		inline constexpr const char* Exp = "exp";
		inline constexpr const char* Floor = "floor";
		inline constexpr const char* Fract = "fract";
		inline constexpr const char* Invert = "invert";
		inline constexpr const char* InvertMatrix = "invertmatrix";
		inline constexpr const char* Ln = "ln";
		inline constexpr const char* Magnitude = "magnitude";
		inline constexpr const char* Max = "max";
		inline constexpr const char* Min = "min";
		inline constexpr const char* Modulo = "modulo";
		inline constexpr const char* Multiply = "multiply";
		inline constexpr const char* Normalize = "normalize";
		inline constexpr const char* NormalMap = "normalmap";
		inline constexpr const char* Place2D = "place2d";
		inline constexpr const char* Power = "power";
		inline constexpr const char* Reflect = "reflect";
		inline constexpr const char* Refract = "refract";
		inline constexpr const char* Rotate2D = "rotate2d";
		inline constexpr const char* Rotate3D = "rotate3d";
		inline constexpr const char* Round = "round";
		inline constexpr const char* SafePower = "safepower";
		inline constexpr const char* Sign = "sign";
		inline constexpr const char* Sin = "sin";
		inline constexpr const char* Sqrt = "sqrt";
		inline constexpr const char* Sub = "subtract";
		inline constexpr const char* Tan = "tan";
		inline constexpr const char* TransformMatrix = "transformmatrix";
		inline constexpr const char* TransformNormal = "transformnormal";
		inline constexpr const char* TransformPoint = "transformpoint";
		inline constexpr const char* TransformVector = "transformvector";
		inline constexpr const char* Transpose = "transpose";
		inline constexpr const char* TriangleWave = "trianglewave";
		// Compositing nodes
		inline constexpr const char* Burn = "burn";
		inline constexpr const char* Difference = "difference";
		inline constexpr const char* Disjointover = "disjointover";
		inline constexpr const char* Dodge = "dodge";
		inline constexpr const char* In = "in";
		inline constexpr const char* Inside = "inside";
		inline constexpr const char* Mask = "mask";
		inline constexpr const char* Matte = "matte";
		inline constexpr const char* Minus = "minus";
		inline constexpr const char* Mix = "mix";
		inline constexpr const char* Out = "out";
		inline constexpr const char* Outside = "outside";
		inline constexpr const char* Over = "over";
		inline constexpr const char* Overlay = "overlay";
		inline constexpr const char* Plus = "plus";
		inline constexpr const char* Premult = "premult";
		inline constexpr const char* Screen = "screen";
		inline constexpr const char* Unpremult = "unpremult";
		// Conditional nodes
		inline constexpr const char* And = "and";
		inline constexpr const char* IfGreater = "ifgreater";
		inline constexpr const char* IfGreaterEq = "ifgreatereq";
		inline constexpr const char* IfEqual = "ifequal";
		inline constexpr const char* Not = "not";
		inline constexpr const char* Or = "or";
		inline constexpr const char* Switch = "switch";
		inline constexpr const char* Xor = "xor";
		// Channel nodes
		inline constexpr const char* Convert = "convert";
		inline constexpr const char* Combine2 = "combine2";
		inline constexpr const char* Combine3 = "combine3";
		inline constexpr const char* Combine4 = "combine4";
		inline constexpr const char* Extract = "extract";
		inline constexpr const char* Separate2 = "separate2";
		inline constexpr const char* Separate3 = "separate3";
		inline constexpr const char* Separate4 = "separate4";
		// Procedural nodes 
		inline constexpr const char* Constant = "constant";
		// Procedural2D nodes
		inline constexpr const char* CellNoise2D = "cellnoise2d";
		inline constexpr const char* Checkerboard = "checkerboard";
		inline constexpr const char* Circle = "circle";
		inline constexpr const char* Fractal2D = "fractal2d";
		inline constexpr const char* Line = "line";
		inline constexpr const char* Noise2D = "noise2d";
		inline constexpr const char* Ramp4 = "ramp4";
		inline constexpr const char* Ramp = "ramp";
		inline constexpr const char* RampGradient = "ramp_gradient";
		inline constexpr const char* RampLR = "ramplr";
		inline constexpr const char* RampTB = "ramptb";
		inline constexpr const char* RandomFloat = "randomfloat";
		inline constexpr const char* SplitLR = "splitlr";
		inline constexpr const char* SplitTB = "splittb";
		inline constexpr const char* TiledCircles = "tiledcircles";
		inline constexpr const char* WorleyNoise2D = "worleynoise2d";
		// Procedural3D nodes 
		inline constexpr const char* CellNoise3D = "cellnoise3d";
		inline constexpr const char* Fractal3D = "fractal3d";
		inline constexpr const char* Noise3D = "noise3d";
		inline constexpr const char* RandomColor = "randomcolor";
		inline constexpr const char* UnifiedNoise3D = "unifiednoise3d";
		inline constexpr const char* WorleyNoise3D = "worleynoise3d";
		// Organization nodes 
		inline constexpr const char* Dot = "dot";
		// Texture nodes 
		inline constexpr const char* HexTiledImage = "hextiledimage";
		inline constexpr const char* HexTiledNormalMap = "hextilednormalmap";
		inline constexpr const char* Image = "image";
		inline constexpr const char* LatLongImage = "latlongimage";
		inline constexpr const char* TiledImage = "tiledimage";
		// Geometric nodes
		inline constexpr const char* Bitangent = "bitangent";
		inline constexpr const char* GeomColor = "geomcolor";
		inline constexpr const char* GeomPropValue = "geompropvalue";
		inline constexpr const char* Normal = "normal";
		inline constexpr const char* Position = "position";
		inline constexpr const char* Tangent = "tangent";
		inline constexpr const char* TexCoord = "texcoord";
		// Light nodes
		inline constexpr const char* DirectionalLight = "directional_light";
		inline constexpr const char* PointLight = "point_light";
		inline constexpr const char* SpotLight = "spot_light";
		// Adjustment nodes
		inline constexpr const char* ColorCorrect = "colorcorrect";
		inline constexpr const char* Contrast = "contrast";
		inline constexpr const char* HsvAdjust = "hsvadjust";
		inline constexpr const char* HsvToRgb = "hsvtorgb";
		inline constexpr const char* Luminance = "luminance";
		inline constexpr const char* Range = "range";
		inline constexpr const char* Remap = "remap";
		inline constexpr const char* RgbToHsv = "rgbtohsv";
		inline constexpr const char* Saturate = "saturate";
		inline constexpr const char* Smoothstep = "smoothstep";
		// Application
		inline constexpr const char* Time= "time";
		// PBR
		// BSDF
		inline constexpr const char* BurleyDiffuseBSDF= "burley_diffuse_bsdf";
		inline constexpr const char* ChiangHairBSDF = "chiang_hair_bsdf";
		inline constexpr const char* ConductorBSDF = "conductor_bsdf";
		inline constexpr const char* DielectricBSDF = "dielectric_bsdf";
		inline constexpr const char* GeneralizedSchlickBSDF= "generalized_schlick_bsdf";
		inline constexpr const char* OrenNayarDiffuseBSDF = "oren_nayar_diffuse_bsdf";
		inline constexpr const char* SheenBSDF = "sheen_bsdf";
		inline constexpr const char* SubsurfaceBSDF= "subsurface_bsdf";
		inline constexpr const char* TranslucentBSDF = "translucent_bsdf";
		// EDF
		inline constexpr const char* ConicalEDF = "conical_edf";
		inline constexpr const char* MeasuredEDF = "measured_edf";
		inline constexpr const char* UniformEDF = "uniform_edf";
		// VDF
		inline constexpr const char* AbsorptionVDF = "absorption_vdf";
		inline constexpr const char* AnisotropicVDF = "anisotropic_vdf";
		// PBR Utility Nodes
		inline constexpr const char* ArtisticIOR = "artistic_ior";
		inline constexpr const char* Blackbody = "blackbody";
		inline constexpr const char* ChiangHairAbsorptionFromColor = "chiang_hair_absorption_from_color";
		inline constexpr const char* ChiangHairRoughness = "chiang_hair_roughness";
		inline constexpr const char* DeonHairAbsorptionFromMelanin = "deon_hair_absorption_from_melanin";
		inline constexpr const char* Layer = "layer";
		inline constexpr const char* RoughnessAnisotropy = "roughness_anisotropy";
		inline constexpr const char* RoughnessDual = "roughness_dual";
		//Surface Shaders
		inline constexpr const char* GltfPbr = "gltf_pbr";
		inline constexpr const char* DisneyBSDF2012 = "disney_brdf_2012";
		inline constexpr const char* DisneyBSDF2015 = "disney_bsdf_2015";
		inline constexpr const char* OpenPBRSurface = "open_pbr_surface";
		inline constexpr const char* StandardSurface = "standard_surface";
		inline constexpr const char* Surface = "surface";
		inline constexpr const char* UsdPreviewSurface = "UsdPreviewSurface";
		// Shader
		inline constexpr const char* SurfaceUnlit = "surface_unlit";
		// Displacement Shader
		inline constexpr const char* Displacement = "displacement";
		// Convolution
		inline constexpr const char* Blur = "blur";
		inline constexpr const char* HeightToNormal = "heighttonormal";
		// ColorTransform
		inline constexpr const char* ACEScgToLinRec709 = "acescg_to_lin_rec709";
		inline constexpr const char* AdobeRgbToLinRec709 = "adobergb_to_lin_rec709";
		inline constexpr const char* G18Rec709ToLinRec709 = "g18_rec709_to_lin_rec709";
		inline constexpr const char* G22ApP1ToLinRec709 = "g22_ap1_to_lin_rec709";
		inline constexpr const char* LinAdobeRgbToLinRec709 = "lin_adobergb_to_lin_rec709";
		inline constexpr const char* LinDisplayP3ToLinRec709 = "lin_displayp3_to_lin_rec709";
		inline constexpr const char* Rec709DisplayToLinRec709 = "rec709_display_to_lin_rec709";
		inline constexpr const char* SrgbDisplayP3ToLinRec709 = "srgb_displayp3_to_lin_rec709";
		inline constexpr const char* SrgbTextureToLinRec709 = "srgb_texture_to_lin_rec709";
		// NPR
		inline constexpr const char* FacingRatio = "facingratio";
		inline constexpr const char* GoochShade = "gooch_shade";
		inline constexpr const char* ViewDirection = "viewdirection";
		// Volume shaders
		inline constexpr const char* Volume = "volume";
		// Unreal Engine nodes defined in Engine\Binaries\ThirdParty\MaterialX\libraries\Interchange\Interchange_defs.mtlx
		inline constexpr const char* SparseVolume = "sparse_volume";
	}

	namespace NodeDefinition
	{
		inline constexpr const char* OpenPBRSurface = "ND_open_pbr_surface_surfaceshader";
		inline constexpr const char* StandardSurface = "ND_standard_surface_surfaceshader";
		inline constexpr const char* SurfaceUnlit = "ND_surface_unlit";
		inline constexpr const char* UsdPreviewSurface = "ND_UsdPreviewSurface_surfaceshader";
		inline constexpr const char* GltfPbr = "ND_gltf_pbr_surfaceshader";
		inline constexpr const char* PointLight = "ND_point_light";
		inline constexpr const char* DirectionalLight = "ND_directional_light";
		inline constexpr const char* SpotLight = "ND_spot_light";
		inline constexpr const char* Surface = "ND_surface";
		inline constexpr const char* DisplacementFloat = "ND_displacement_float";
		inline constexpr const char* DisplacementVector3 = "ND_displacement_vector3";
		inline constexpr const char* MixSurfaceShader = "ND_mix_surfaceshader";
		inline constexpr const char* Volume = "ND_volume";
	}

	namespace Library
	{
		inline constexpr const char* Libraries = "libraries";
	}

	namespace Type
	{
		inline constexpr const char* Boolean = "boolean";
		inline constexpr const char* Integer = "integer";
		inline constexpr const char* Float = "float";
		inline constexpr const char* Color3 = "color3";
		inline constexpr const char* Color4 = "color4";
		inline constexpr const char* Vector2 = "vector2";
		inline constexpr const char* Vector3 = "vector3";
		inline constexpr const char* Vector4 = "vector4";
		inline constexpr const char* Matrix33 = "matrix33";
		inline constexpr const char* Matrix44 = "matrix44";
		inline constexpr const char* String = "string";
		inline constexpr const char* Filename = "filename";
		inline constexpr const char* GeomName = "geomname";
		inline constexpr const char* SurfaceShader = "surfaceshader";
		inline constexpr const char* DisplacementShader = "displacementshader";
		inline constexpr const char* VolumeShader = "volumeshader";
		inline constexpr const char* LightShader = "lightshader";
		inline constexpr const char* Material = "material";
		inline constexpr const char* None = "none";
		inline constexpr const char* IntegerArray = "integerarray";
		inline constexpr const char* FloatArray = "floatarray";
		inline constexpr const char* Color3Array = "color3array";
		inline constexpr const char* Color4Array = "color4array";
		inline constexpr const char* Vector2Array = "vector2array";
		inline constexpr const char* Vector3Array = "vector3array";
		inline constexpr const char* Vector4Array = "vector4array";
		inline constexpr const char* StringArray = "stringarray";
		inline constexpr const char* GeomNameArray = "geomnamearray";
		inline constexpr const char* BSDF = "BSDF";
		inline constexpr const char* EDF = "EDF";
		inline constexpr const char* VDF = "VDF";

	}

	namespace NodeGroup
	{
		namespace Texture2D
		{
			namespace Inputs
			{
				inline constexpr const char* File = "file";
				inline constexpr const char* Default = "default";
				inline constexpr const char* TexCoord = "texcoord";
				inline constexpr const char* FilterType = "filtertype";
				inline constexpr const char* FrameRange = "framerange";
				inline constexpr const char* FrameOffset = "frameoffset";
				inline constexpr const char* FrameEndAction = "frameendaction";
			}
		}

		namespace Math
		{
			template<typename T>
			inline const T NeutralZero = T{ 0 };

			template<typename T>
			inline const T NeutralOne = T{ 1 };
		}

		inline constexpr const char* Compositing = "compositing";
		inline constexpr const char* PBR = "pbr";
	}

	namespace Image
	{
		namespace Inputs
		{
			using namespace NodeGroup::Texture2D::Inputs;

			inline constexpr const char* Layer = "layer";
			inline constexpr const char* UAddressMode = "uaddressmode";
			inline constexpr const char* VAddressMode = "vaddressmode";
		}
	}

	namespace TiledImage
	{
		namespace Inputs
		{
			using namespace NodeGroup::Texture2D::Inputs;

			inline constexpr const char* UVTiling = "uvtiling";
			inline constexpr const char* UVOffset = "uvoffset";
			inline constexpr const char* RealWorldImageSize = "realworldimagesize";
			inline constexpr const char* RealWorldTileSize = "realworldtilesize";
		}
	}

MATERIALX_NAMESPACE_END
