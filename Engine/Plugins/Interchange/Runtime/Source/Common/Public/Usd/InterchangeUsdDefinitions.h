// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InterchangeUsdDefinitions.generated.h"

UENUM(BlueprintType)
enum class EInterchangeUsdPrimvar : uint8
{
	/** Store only the standard primvars such as UVs, VertexColors, etc.*/
	Standard = 0,

	/** Store only primvars in the Mesh Description used for baking to textures (basically <geompropvalue> node from MaterialX shadergraphs that are converted to <image>)*/
	Bake,

	/** Store all primvars in the MeshDescription */
	All
};

namespace UE::Interchange::USD
{
	inline const FString USDContextTag = TEXT("USD");

	// Name of a custom attribute added to translated nodes to contain their geometry purpose (proxy, render, guide, etc.)
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "No longer used, as geometry purpose is now handled on the translator and no longer needs to be delayed to the pipeline")
	inline const FString GeometryPurposeIdentifier = TEXT("USD_Geometry_Purpose");
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Prefixes we use to stash some primvar mapping information as custom attributes on mesh / material nodes, so that
	// the USD Pipeline can produce primvar-compatible materials
	inline const FString PrimvarUVIndexAttributePrefix = TEXT("USD_PrimvarUVIndex_");
	inline const FString ParameterToPrimvarAttributePrefix = TEXT("USD_ParameterPrimvar_");

	// Additional suffix we add to the UID of all primvar-compatible materials
	inline const FString CompatibleMaterialUidSuffix = TEXT("_USD_CompatibleMaterial_");

	// Some tokens we add to the material parameter name for USD material nodes. Put here because we need to use the
	// same tokens on the translator and the USD Pipeline, when computing primvar-compatible materials
	inline const FString UseTextureParameterPrefix = TEXT("Use");
	inline const FString UseTextureParameterSuffix = TEXT("Texture");
	inline const FString UVIndexParameterSuffix = TEXT("UVIndex");

	// In order to make it easy to find the skeleton root bone scene node UID from a prim path, for the root bone in particular
	// we always use an UID based on <skeleton prim path>/RootBoneUidSuffix (e.g. "\Bone\/MySkelRoot/MySkeleton/Root")
	//
	// Note that this is unrelated to the bone's *name*, which is still named after the actual USD bone name
	inline const FString RootBoneUidSuffix = TEXT("Root");

	// Prefix we add to the scene nodes UIDs we artificially produce for skeleton joints, to make sure we don't accidentally
	// produce an UID that is already used by a regular prim in the stage
	inline const FString BonePrefix = TEXT("\\Bone\\");

	// Flag indicating whether we should parse a UInterchangeShaderGraphNode on the InterchangeUSDPipeline.
	// This is now only used for the compatible primvar code
	inline const FString ParseMaterialIdentifier = TEXT("USD_MI_ParseMaterial");

	// Used for volumetric material parameters, whenever we assign an SVT to a material as a fallback due
	// to it's field name only
	inline const FString VolumeFieldNameMaterialParameterPrefix = TEXT("USD_FieldName_");

	// The USD Pipeline moves its SubdivisionLevel property value onto its Mesh factory nodes under this payload
	// attribute. The USD translator then retrieves this there and uses that subdivision level with OpenSubdiv for
	// the actual subdivision
	inline const FString SubdivisionLevelAttributeKey = TEXT("USD_SubdivisionLevel");

	// We don't fully support inputs:normalize for USD lights, but we at least move the attribute into the Interchange
	// factory node so that it's exposed for potential user pipelines
	inline const FString LightAPINormalizeAttributeKey = TEXT("USD_LightAPIInputsNormalize");

	// Name of the variant set that is used for describing LODs
	inline const FString LODString = TEXT("LOD");

	// Prefixes that should be used for asset node UIDs created from USD translation
	inline const FString AnimationPrefix = TEXT("\\Animation\\");
	inline const FString AnimationTrackPrefix = TEXT("\\AnimationTrack\\");
	inline const FString CameraPrefix = TEXT("\\Camera\\");
	inline const FString ComponentPrefix = TEXT("\\Component\\");
	inline const FString GroomPrefix = TEXT("\\Groom\\");
	inline const FString GroomBindingPrefix = TEXT("\\GroomBinding\\");
	inline const FString LightPrefix = TEXT("\\Light\\");
	inline const FString LODPrefix = TEXT("\\LOD\\");
	inline const FString MaterialPrefix = TEXT("\\Material\\");
	inline const FString MaterialReferencePrefix = TEXT("\\MaterialReference\\");
	inline const FString MeshPrefix = TEXT("\\Mesh\\");
	inline const FString MorphTargetPrefix = TEXT("\\MorphTarget\\");
	inline const FString PrimitiveShapePrefix = TEXT("\\PrimitiveShape\\");
	inline const FString SpatialAudioPrefix = TEXT("\\SpatialAudio\\");
	inline const FString TexturePrefix = TEXT("\\Texture\\");
	inline const FString VolumePrefix = TEXT("\\Volume\\");

	inline const FString LODContainerInstanceSuffix = TEXT("LODContainerInstance");
	inline const FString TwoSidedSuffix = TEXT("_TwoSided");

	inline const FString PrimPathAttributeKeyString = TEXT("_USDPrimPath_");

	// Attribute used to store the different properties of a material (Translucent, TwoSided, VT)
	inline const FString UsdReferenceMaterialProperties = TEXT("UsdReferenceMaterialProperties");

	// Attribute to avoid overwriting parent materials by the UsdPreviewSurface
	inline const FString UsdMaterialInstanceFromShaderGraph = TEXT("UsdMaterialInstanceFromShaderGraph");

	// Custom attribute keys used to describe SVT info extracted from the USD custom schemas.
	// We add these to the Volume nodes that the USD translator emits.
	namespace SparseVolumeTexture
	{
		inline const FString AttributesAFormat = TEXT("USD_AttributesA_Format");
		inline const FString AttributesBFormat = TEXT("USD_AttributesB_Format");

		inline const FString AttributesAChannelR = TEXT("USD_AttributesA_R");
		inline const FString AttributesAChannelG = TEXT("USD_AttributesA_G");
		inline const FString AttributesAChannelB = TEXT("USD_AttributesA_B");
		inline const FString AttributesAChannelA = TEXT("USD_AttributesA_A");
		inline const FString AttributesBChannelR = TEXT("USD_AttributesB_R");
		inline const FString AttributesBChannelG = TEXT("USD_AttributesB_G");
		inline const FString AttributesBChannelB = TEXT("USD_AttributesB_B");
		inline const FString AttributesBChannelA = TEXT("USD_AttributesB_A");
	}

	namespace Primvar
	{
		inline const FString Number = TEXT("USD_PrimvarNumber");
		// In case of "Number" primvars, the user should concatenate the Index to this attribute
		inline const FString Name = TEXT("USD_PrimvarName");
		inline const FString TangentSpace = TEXT("USD_PrimvarTangentSpace");
		// Attribute that informs about the UID of a ShaderNode of type TextureSample
		inline const FString ShaderNodeTextureSample = TEXT("USD_ShaderNodeTextureSample");
		// Attribute that informs about the UID of a ShaderNode of type SparseVolumeTextureSample
		inline const FString ShaderNodeSparseVolumeTextureSample = TEXT("USD_ShaderNodeSparseVolumeTextureSample");
		// Attribute that informs how we should handle primvars on MeshDescriptions
		inline const FString Import = TEXT("USD_Import_Primvars");
	}
}	 // namespace UE::Interchange::USD
