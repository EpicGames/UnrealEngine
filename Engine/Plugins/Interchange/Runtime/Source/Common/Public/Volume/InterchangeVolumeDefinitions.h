// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StaticArray.h"
#include "CoreMinimal.h"

#include "InterchangeVolumeDefinitions.generated.h"

UENUM(Blueprintable)
enum class EInterchangeSparseVolumeTextureFormat : uint8
{
	Unorm8 = 0,
	Float16 = 1,
	Float32 = 2,
};

namespace UE::Interchange::Volume
{
	const FString DensityGridName = TEXT("density");
	const FString GridNameAndComponentIndexSeparator = TEXT("_");

	const FString VolumetricMaterial = TEXT("Volumetric_Material");

	// These structs are direct copies of the ones from "OpenVDBImportOptions.h".
	// We do this instead of using them directly as that belongs to an editor-only module, and we can't
	// use #if WITH_EDITOR here as the FVolumePayloadKey shows up on the payload interface, which would
	// force every implementation to also use #if WITH_EDITOR

	// Describes what should go on a specific texture channel (e.g. AttributesA.Z) within a SparseVolumeTexture
	struct FComponentMapping
	{
		int32 SourceGridIndex = INDEX_NONE;
		int32 SourceComponentIndex = INDEX_NONE;
	};

	// Describes a specific texture (e.g. AttributesA) within a SparseVolumeTexture
	struct FTextureInfo
	{
		TStaticArray<FComponentMapping, 4> Mappings;
		EInterchangeSparseVolumeTextureFormat Format = EInterchangeSparseVolumeTextureFormat::Unorm8;
	};

	// Describes the full assignment info for a particular SparseVolumeTexture
	struct FAssignmentInfo
	{
		TStaticArray<FTextureInfo, 2> Attributes;
		bool bIsSequence = false;
	};
}	 // namespace UE::Interchange::Volume
