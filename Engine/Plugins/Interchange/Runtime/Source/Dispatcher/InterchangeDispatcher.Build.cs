// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class InterchangeDispatcher : ModuleRules
	{
		public InterchangeDispatcher(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"Sockets",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Json",
				}
			);

		}
	}
}
