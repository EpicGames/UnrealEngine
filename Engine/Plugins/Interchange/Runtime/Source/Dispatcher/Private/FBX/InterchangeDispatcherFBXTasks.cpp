// Copyright Epic Games, Inc. All Rights Reserved.

#include "FBX/InterchangeDispatcherFBXTasks.h"

#include "Dom/JsonObject.h"

namespace UE::Interchange
{
	const TCHAR* FJsonFBXLoadSourceCmd::TaskName = TEXT("LoadFBX");
	const TCHAR* FJsonFBXLoadSourceCmd::ConvertSceneJsonKey = TEXT("ConvertScene");
	const TCHAR* FJsonFBXLoadSourceCmd::ForceFrontXAxisJsonKey = TEXT("ForceFrontXAxis");
	const TCHAR* FJsonFBXLoadSourceCmd::ConvertSceneUnitJsonKey = TEXT("ConvertSceneUnit");
	const TCHAR* FJsonFBXLoadSourceCmd::KeepFbxNamespaceJsonKey = TEXT("KeepFbxNamespace");
	const TCHAR* FJsonFBXLoadSourceCmd::UseFbxClustersForBindPoseMatricesJsonKey = TEXT("UseFbxClustersForBindPoseMatrices");

	TSharedPtr<FJsonObject> FJsonFBXLoadSourceCmd::GetActionDataObject() const
	{
		TSharedPtr<FJsonObject> ActionDataObject = FJsonLoadSourceCmd::GetActionDataObject();
		ActionDataObject->SetBoolField(ConvertSceneJsonKey, bConvertScene);
		ActionDataObject->SetBoolField(ForceFrontXAxisJsonKey, bForceFrontXAxis);
		ActionDataObject->SetBoolField(ConvertSceneUnitJsonKey, bConvertSceneUnit);
		ActionDataObject->SetBoolField(KeepFbxNamespaceJsonKey, bKeepFbxNamespace);
		ActionDataObject->SetBoolField(UseFbxClustersForBindPoseMatricesJsonKey, bConsiderClusterBeforePoseForMeshBindPose);

		return ActionDataObject;
	}

	bool FJsonFBXLoadSourceCmd::IsActionDataObjectValid(const FJsonObject& ActionDataObject)
	{
		if (!(ActionDataObject.TryGetBoolField(ConvertSceneJsonKey, bConvertScene)))
		{
			return false;
		}
		if (!(ActionDataObject.TryGetBoolField(ForceFrontXAxisJsonKey, bForceFrontXAxis)))
		{
			return false;
		}
		if (!(ActionDataObject.TryGetBoolField(ConvertSceneUnitJsonKey, bConvertSceneUnit)))
		{
			return false;
		}
		if (!(ActionDataObject.TryGetBoolField(KeepFbxNamespaceJsonKey, bKeepFbxNamespace)))
		{
			return false;
		}

		if (!(ActionDataObject.TryGetBoolField(UseFbxClustersForBindPoseMatricesJsonKey, bConsiderClusterBeforePoseForMeshBindPose)))
		{
			return false;
		}

		return true;
	}
}//ns UE
