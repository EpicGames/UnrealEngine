// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeCommands.h"

#include "Serialization/Archive.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"

namespace UE
{
	namespace Interchange
	{
		int32 DispatcherCommandVersion::GetMajor()
		{
			return 20;
		}

		int32 DispatcherCommandVersion::GetMinor()
		{
			return 0;
		}

		int32 DispatcherCommandVersion::GetPatch()
		{
			return 0;
		}

		TSharedPtr<ICommand> CreateCommand(ECommandId CommandType)
		{
			switch (CommandType)
			{
				case ECommandId::Ping: return MakeShared<FPingCommand>();
				case ECommandId::Error: return MakeShared<FErrorCommand>();
				case ECommandId::BackPing: return MakeShared<FBackPingCommand>();
				case ECommandId::RunTask: return MakeShared<FRunTaskCommand>();
				case ECommandId::NotifyEndTask: return MakeShared<FCompletedTaskCommand>();
				case ECommandId::QueryTaskProgress: return MakeShared<FQueryTaskProgressCommand>();
				case ECommandId::CompletedQueryTaskProgress: return MakeShared<FCompletedQueryTaskProgressCommand>();
				case ECommandId::Terminate: return MakeShared<FTerminateCommand>();
			}
			return nullptr;
		}

		void SerializeCommand(ICommand& Command, TArray<uint8>& OutBuffer)
		{
			FMemoryWriter ArWriter(OutBuffer);
			uint8 type = static_cast<uint8>(Command.GetType());
			ArWriter << type;
			ArWriter << Command;
		}

		TSharedPtr<ICommand> DeserializeCommand(const TArray<uint8>& InBuffer)
		{
			FMemoryReader ArReader(InBuffer);
			uint8 type = 0;
			ArReader << type;
			if (TSharedPtr<ICommand> Command = CreateCommand(static_cast<ECommandId>(type)))
			{
				ArReader << *Command;
				return ArReader.IsError() ? nullptr : Command;
			}
			return nullptr;
		}

		void FErrorCommand::SerializeImpl(FArchive& Ar)
		{
			Ar << ErrorMessage;
		}

		void FRunTaskCommand::SerializeImpl(FArchive& Ar)
		{
			Ar << JsonDescription;
			Ar << TaskIndex;
		}

		void FCompletedTaskCommand::SerializeImpl(FArchive& Ar)
		{
			Ar << ProcessResult;
			Ar << JSonMessages;
			Ar << JSonResult;
			Ar << TaskIndex;
		}

		FQueryTaskProgressCommand::FQueryTaskProgressCommand(const TArray<int32>& Tasks)
		{
			TaskIndexes = Tasks;
		}
		
		void FQueryTaskProgressCommand::SerializeImpl(FArchive& Ar)
		{
			Ar << TaskIndexes;
		}

		FArchive& operator<<(FArchive& Ar, FCompletedQueryTaskProgressCommand::FTaskProgressData& A)
		{
			Ar << A.TaskIndex;
			Ar << A.TaskState;
			Ar << A.TaskProgress;
			return Ar;
		}

		void FCompletedQueryTaskProgressCommand::SerializeImpl(FArchive& Ar)
		{
			Ar << TaskStates;
		}
	}//ns Interchange
}//ns UE
