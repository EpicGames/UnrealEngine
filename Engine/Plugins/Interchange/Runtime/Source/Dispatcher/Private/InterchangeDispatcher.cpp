// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeDispatcher.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "InterchangeDispatcherConfig.h"
#include "InterchangeDispatcherLog.h"
#include "InterchangeDispatcherTask.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"

namespace UE
{
	namespace Interchange
	{

		FInterchangeDispatcher::FInterchangeDispatcher(const FString& InResultFolder, int32 DesiredWorkerCount)
			: ResultFolder(InResultFolder)
			, NextTaskIndex(0)
			, CompletedTaskCount(0)
		{
			// Set up number of workers to initiate based on main application's workload
			if (DesiredWorkerCount >= 0)
			{
				if (DesiredWorkerCount == 0)
				{
					DesiredWorkerCount = FPlatformMisc::NumberOfCores();
				}

				const int32 MaxNumberOfWorkers = FPlatformMisc::NumberOfCores();

				constexpr double RecommandedRamPerWorkers = 6.;
				constexpr double OneGigaByte = 1024. * 1024. * 1024.;
				const double AvailableRamGB = (double)(FPlatformMemory::GetStats().AvailablePhysical / OneGigaByte);

				const int32 RecommandedNumberOfWorkers = (int32)(AvailableRamGB / RecommandedRamPerWorkers + 0.5);

				int32 WorkerCount = FMath::Min(DesiredWorkerCount, FMath::Min(MaxNumberOfWorkers, RecommandedNumberOfWorkers));

				WorkerHandlers.SetNum(WorkerCount);
			}
		}

		int32 FInterchangeDispatcher::AddTask(const FString& JsonDescription)
		{
			if (!IsValid())
			{
				return INDEX_NONE;
			}

			FScopeLock Lock(&TaskPoolCriticalSection);
			int32 TaskIndex = TaskPool.Emplace(JsonDescription);
			TaskPool[TaskIndex].Index = TaskIndex;
			return TaskIndex;
		}
		int32 FInterchangeDispatcher::AddTask(const FString& JsonDescription, FInterchangeDispatcherTaskCompleted TaskCompledDelegate)
		{
			int32 TaskIndex = AddTask(JsonDescription);
			if (TaskIndex != INDEX_NONE)
			{
				TaskPool[TaskIndex].OnTaskCompleted = TaskCompledDelegate;
			}
			return TaskIndex;
		}

		TOptional<FTask> FInterchangeDispatcher::GetNextTask()
		{
			FScopeLock Lock(&TaskPoolCriticalSection);

			while (TaskPool.IsValidIndex(NextTaskIndex) && TaskPool[NextTaskIndex].State != ETaskState::UnTreated)
			{
				NextTaskIndex++;
			}

			if (!TaskPool.IsValidIndex(NextTaskIndex))
			{
				return TOptional<FTask>();
			}

			TaskPool[NextTaskIndex].State = ETaskState::Running;
			TaskPool[NextTaskIndex].RunningStateStartTime = FPlatformTime::Seconds();
			return TaskPool[NextTaskIndex++];
		}

		void FInterchangeDispatcher::SetTaskState(int32 TaskIndex, ETaskState TaskState, const FString& JsonResult, const TArray<FString>& JSonMessages)
		{
			FString JsonDescription;
			{
				FScopeLock Lock(&TaskPoolCriticalSection);

				if (!ensure(TaskPool.IsValidIndex(TaskIndex)))
				{
					return;
				}

				FTask& Task = TaskPool[TaskIndex];
				if (Task.State == ETaskState::ProcessOk || Task.State == ETaskState::ProcessFailed)
				{
					//Task was already processed, we cannot set its state after it was process
					return;
				}
				Task.State = TaskState;
				Task.JsonResult = JsonResult;
				Task.JsonMessages = JSonMessages;
				JsonDescription = Task.JsonDescription;

				if (TaskState == ETaskState::ProcessOk
					|| TaskState == ETaskState::ProcessFailed)
				{
					//Call the task completion delegate
					Task.OnTaskCompleted.ExecuteIfBound(TaskIndex);
					CompletedTaskCount++;
				}

				if (TaskState == ETaskState::UnTreated)
				{
					NextTaskIndex = TaskIndex;
				}
			}

			UE_CLOGF(TaskState == ETaskState::ProcessOk, LogInterchangeDispatcher, Verbose, "Json processed: %ls", *JsonDescription);
			UE_CLOGF(TaskState == ETaskState::UnTreated, LogInterchangeDispatcher, Warning, "Json resubmitted: %ls", *JsonDescription);
			UE_CLOGF(TaskState == ETaskState::ProcessFailed, LogInterchangeDispatcher, Error, "Json processing failure: %ls", *JsonDescription);
		}

		void FInterchangeDispatcher::GetTaskState(int32 TaskIndex, ETaskState& TaskState, double& TaskRunningStateStartTime)
		{
			FScopeLock Lock(&TaskPoolCriticalSection);

			if (!ensure(TaskPool.IsValidIndex(TaskIndex)))
			{
				return;
			}

			FTask& Task = TaskPool[TaskIndex];
			TaskState = Task.State;
			TaskRunningStateStartTime = Task.RunningStateStartTime;
		}

		void FInterchangeDispatcher::GetTaskState(int32 TaskIndex, ETaskState& TaskState, FString& JsonResult, TArray<FString>& JSonMessages)
		{
			FScopeLock Lock(&TaskPoolCriticalSection);

			if (!ensure(TaskPool.IsValidIndex(TaskIndex)))
			{
				return;
			}

			FTask& Task = TaskPool[TaskIndex];
			TaskState = Task.State;
			JsonResult = Task.JsonResult;
			JSonMessages = Task.JsonMessages;
		}

		void FInterchangeDispatcher::StartProcess()
		{
			//Start the process
			SpawnHandlers();
		}

		void FInterchangeDispatcher::StopProcess(bool bBlockUntilTerminated)
		{
			for (TUniquePtr<FInterchangeWorkerHandler>& WorkerHandler : WorkerHandlers)
			{
				if (WorkerHandler.IsValid() && WorkerHandler->IsAlive())
				{
					if (bBlockUntilTerminated)
					{
						WorkerHandler->StopBlocking();
					}
					else
					{
						WorkerHandler->Stop();
					}
				}
			}

			EmptyQueueTasks();
		}

		void FInterchangeDispatcher::TerminateProcess()
		{
			//Empty the cache folder
			if (IFileManager::Get().DirectoryExists(*ResultFolder))
			{
				const bool RequireExists = false;
				//Delete recursively folder's content
				const bool Tree = true;
				IFileManager::Get().DeleteDirectory(*ResultFolder, RequireExists, Tree);
			}
			//Terminate the process
			CloseHandlers();
		}

		void FInterchangeDispatcher::WaitAllTaskToCompleteExecution()
		{
			if (!IsValid())
			{
				UE_LOGF(LogInterchangeDispatcher, Error, "Cannot execute tasks before starting the process");
				return;
			}

			bool bLogRestartError = true;
			while (!IsOver())
			{

				if (!IsValid())
				{
					break;
				}

				FPlatformProcess::Sleep(0.1f);
			}

			if (!IsOver())
			{
				UE_LOGF(LogInterchangeDispatcher, Warning,
					   "Begin local processing. (Multi Process failed to consume all the tasks)\n" "See workers logs: %lsPrograms/InterchangeWorker/Saved/Logs", *FPaths::ConvertRelativePathToFull(FPaths::EngineDir()));
			}
			else
			{
				UE_LOGF(LogInterchangeDispatcher, Verbose, "Multi Process ended and consumed all the tasks");
			}
		}

		bool FInterchangeDispatcher::IsOver()
		{
			FScopeLock Lock(&TaskPoolCriticalSection);
			return CompletedTaskCount == TaskPool.Num();
		}

		bool FInterchangeDispatcher::IsValid()
		{
			for (TUniquePtr<FInterchangeWorkerHandler>& WorkerHandler : WorkerHandlers)
			{
				if (WorkerHandler.IsValid() && WorkerHandler->IsAlive())
				{
					return true;
				}
			}

			return false;
		}

		void FInterchangeDispatcher::SpawnHandlers()
		{
			for (TUniquePtr<FInterchangeWorkerHandler>& WorkerHandler : WorkerHandlers)
			{
				WorkerHandler = MakeUnique<FInterchangeWorkerHandler>(*this, ResultFolder);
			}
		}

		void FInterchangeDispatcher::CloseHandlers()
		{
			for (TUniquePtr<FInterchangeWorkerHandler>& WorkerHandler : WorkerHandlers)
			{
				if (WorkerHandler.IsValid() && WorkerHandler->IsAlive())
				{
					WorkerHandler->Stop();
					WorkerHandler.Reset();
				}
			}
		}

		void FInterchangeDispatcher::EmptyQueueTasks()
		{
			//Make sure all queue tasks are completed to process fail,
			//This ensure any wait on completed delegate like promise of a future is unblock.
			TOptional<FTask> NextTask = GetNextTask();
			while (NextTask.IsSet())
			{
				TArray<FString> GarbageMessages;
				SetTaskState(NextTask->Index, ETaskState::ProcessFailed, FString(), GarbageMessages);
				NextTask = GetNextTask();
			}
		}

		const FString& FInterchangeDispatcher::GetWorkerApplicationPath()
		{
			if (WorkerApplicationPath.IsEmpty())
			{
				WorkerApplicationPath = [&]()
					{

						const FString InterchangeWorkerApplicationName = GetWorkerApplicationName();
						FString Path = FPlatformProcess::GenerateApplicationPath(InterchangeWorkerApplicationName, FApp::GetBuildConfiguration());
						if (!FPaths::FileExists(Path))
						{
							//Force the development build if the path is not right
							Path = FPlatformProcess::GenerateApplicationPath(InterchangeWorkerApplicationName, EBuildConfiguration::Development);
						}

						if (!FPaths::FileExists(Path))
						{
							UE_LOGF(LogInterchangeDispatcher, Display, "InterchangeWorker executable not found. Expected location: %ls", *FPaths::ConvertRelativePathToFull(Path));
						}
						return Path;
					}();
			}

			return WorkerApplicationPath;
		}

	} //ns Interchange
}//ns UE