// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeDispatcherModule.h"

#include "InterchangeDispatcherLog.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"



#define LOCTEXT_NAMESPACE "FInterchangeDispatcherModule"

DEFINE_LOG_CATEGORY(LogInterchangeDispatcher);

FInterchangeDispatcherModule& FInterchangeDispatcherModule::Get()
{
	return FModuleManager::LoadModuleChecked< FInterchangeDispatcherModule >(INTERCHANGEDISPATCHER_MODULE_NAME);
}

bool FInterchangeDispatcherModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded(INTERCHANGEDISPATCHER_MODULE_NAME);
}

void FInterchangeDispatcherModule::StartupModule()
{
}

IMPLEMENT_MODULE(FInterchangeDispatcherModule, InterchangeDispatcher);

#undef LOCTEXT_NAMESPACE // "InterchangeDispatcherModule"

