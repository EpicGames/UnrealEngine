// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeDispatcherTask.h"

#include "CoreMinimal.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"

namespace UE
{
	namespace Interchange
	{

		TSharedPtr<FJsonObject> FJsonLoadSourceCmd::GetActionDataObject() const
		{
			TSharedPtr<FJsonObject> ActionDataObject = MakeShared<FJsonObject>();
			ActionDataObject->SetStringField(GetSourceFilenameJsonKey(), GetSourceFilename());

			return ActionDataObject;
		}

		FString FJsonLoadSourceCmd::ToJson() const
		{
			//Code should not do a ToJson if the data was not set before
			ensure(bIsDataInitialize);

			//CmdObject
			TSharedPtr<FJsonObject> CmdObject = MakeShared<FJsonObject>();
			CmdObject->SetStringField(GetCommandIDJsonKey(), GetAction());
			CmdObject->SetStringField(GetTranslatorIDJsonKey(), GetTranslatorID());
			
			//Get ActionDataObject
			TSharedPtr<FJsonObject> ActionDataObject = GetActionDataObject();

			//Add ActionDataObject
			CmdObject->SetObjectField(GetCommandDataJsonKey(), ActionDataObject);

			//Get Json string from CmdObject
			FString LoadSourceCmd;
			TSharedRef< TJsonWriter< TCHAR, TPrettyJsonPrintPolicy<TCHAR> > > JsonWriter = TJsonWriterFactory< TCHAR, TPrettyJsonPrintPolicy<TCHAR> >::Create(&LoadSourceCmd);
			if (!FJsonSerializer::Serialize(CmdObject.ToSharedRef(), JsonWriter))
			{
				//Error creating the json cmd string 
				return FString();
			}
			return LoadSourceCmd;
		}

		bool FJsonLoadSourceCmd::FromJson(const FString& JsonString)
		{
			TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(JsonString);

			TSharedPtr<FJsonObject> CmdObject;
			if (!FJsonSerializer::Deserialize(Reader, CmdObject) || !CmdObject.IsValid())
			{
				//Cannot read the json file
				return false;
			}
			FString JsonActionValue;
			if (!CmdObject->TryGetStringField(GetCommandIDJsonKey(), JsonActionValue))
			{
				//The json cmd id key is missing
				return false;
			}

			if (!JsonActionValue.Equals(GetAction()))
			{
				//This json do not represent a load command
				return false;
			}

			//Read the json
			if (!CmdObject->TryGetStringField(GetTranslatorIDJsonKey(), TranslatorID))
			{
				//Missing Load command translator ID
				return false;
			}
			const TSharedPtr<FJsonObject>* ActionDataObject = nullptr;
			if (!CmdObject->TryGetObjectField(GetCommandDataJsonKey(), ActionDataObject))
			{
				//Missing Load Action data object
				return false;
			}
			if (!((*ActionDataObject)->TryGetStringField(GetSourceFilenameJsonKey(), SourceFilename)))
			{
				return false;
			}

			//Since we filled the data from the json file, set the data has been initialized.
			bIsDataInitialize = IsActionDataObjectValid(*(*ActionDataObject));
			return bIsDataInitialize;
		}

		FString FJsonLoadSourceCmd::JsonResultParser::ToJson() const
		{
			TSharedPtr<FJsonObject> ResultObject = MakeShared<FJsonObject>();
			//CmdObject
			ResultObject->SetStringField(GetResultFilenameJsonKey(), GetResultFilename());

			FString JsonResult;
			TSharedRef< TJsonWriter< TCHAR, TPrettyJsonPrintPolicy<TCHAR> > > JsonWriter = TJsonWriterFactory< TCHAR, TPrettyJsonPrintPolicy<TCHAR> >::Create(&JsonResult);
			if (!FJsonSerializer::Serialize(ResultObject.ToSharedRef(), JsonWriter))
			{
				//Error creating the json cmd string 
				return FString();
			}
			return JsonResult;
		}

		bool FJsonLoadSourceCmd::JsonResultParser::FromJson(const FString& JsonString)
		{
			TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(JsonString);

			TSharedPtr<FJsonObject> ResultObject;
			if (!FJsonSerializer::Deserialize(Reader, ResultObject) || !ResultObject.IsValid())
			{
				//Cannot read the json file
				return false;
			}
			return ResultObject->TryGetStringField(GetResultFilenameJsonKey(), ResultFilename);
		}

		FString FJsonFetchPayloadCmd::ToJson() const
		{
			//Code should not do a ToJson if the data was not set before
			ensure(bIsDataInitialize);

			TSharedPtr<FJsonObject> CmdObject = MakeShared<FJsonObject>();
			TSharedPtr<FJsonObject> ActionDataObject = MakeShared<FJsonObject>();
			//CmdObject
			CmdObject->SetStringField(GetCommandIDJsonKey(), GetAction());
			CmdObject->SetStringField(GetTranslatorIDJsonKey(), GetTranslatorID());
			ActionDataObject->SetStringField(GetPayloadKeyJsonKey(), GetPayloadKey());
			CmdObject->SetObjectField(GetCommandDataJsonKey(), ActionDataObject);

			FString FetchPayloadCmd;
			TSharedRef< TJsonWriter< TCHAR, TPrettyJsonPrintPolicy<TCHAR> > > JsonWriter = TJsonWriterFactory< TCHAR, TPrettyJsonPrintPolicy<TCHAR> >::Create(&FetchPayloadCmd);
			if (!FJsonSerializer::Serialize(CmdObject.ToSharedRef(), JsonWriter))
			{
				//Error creating the json cmd string 
				return FString();
			}
			return FetchPayloadCmd;
		}

		bool FJsonFetchPayloadCmd::FromJson(const FString& JsonString)
		{
			TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(JsonString);

			TSharedPtr<FJsonObject> CmdObject;
			if (!FJsonSerializer::Deserialize(Reader, CmdObject) || !CmdObject.IsValid())
			{
				//Cannot read the json file
				return false;
			}
			FString JsonActionValue;
			if (!CmdObject->TryGetStringField(GetCommandIDJsonKey(), JsonActionValue))
			{
				//The json cmd id key is missing
				return false;
			}

			if (!JsonActionValue.Equals(GetAction()))
			{
				//This json do not represent a load command
				return false;
			}

			//Read the json
			if (!CmdObject->TryGetStringField(GetTranslatorIDJsonKey(), TranslatorID))
			{
				//Missing Load command translator ID
				return false;
			}
			const TSharedPtr<FJsonObject>* ActionDataObject = nullptr;
			if (!CmdObject->TryGetObjectField(GetCommandDataJsonKey(), ActionDataObject))
			{
				//Missing Load Action data object
				return false;
			}
			if (!((*ActionDataObject)->TryGetStringField(GetPayloadKeyJsonKey(), PayloadKey)))
			{
				return false;
			}

			//Since we filled the data from the json file, set the data has been initialize.
			bIsDataInitialize = true;
			return true;
		}

		FString FJsonFetchPayloadCmd::JsonResultParser::ToJson() const
		{
			TSharedPtr<FJsonObject> ResultObject = MakeShared<FJsonObject>();
			//CmdObject
			ResultObject->SetStringField(GetResultFilenameJsonKey(), GetResultFilename());

			FString JsonResult;
			TSharedRef< TJsonWriter< TCHAR, TPrettyJsonPrintPolicy<TCHAR> > > JsonWriter = TJsonWriterFactory< TCHAR, TPrettyJsonPrintPolicy<TCHAR> >::Create(&JsonResult);
			if (!FJsonSerializer::Serialize(ResultObject.ToSharedRef(), JsonWriter))
			{
				//Error creating the json cmd string 
				return FString();
			}
			return JsonResult;
		}

		bool FJsonFetchPayloadCmd::JsonResultParser::FromJson(const FString& JsonString)
		{
			TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(JsonString);

			TSharedPtr<FJsonObject> ResultObject;
			if (!FJsonSerializer::Deserialize(Reader, ResultObject) || !ResultObject.IsValid())
			{
				//Cannot read the json file
				return false;
			}
			return ResultObject->TryGetStringField(GetResultFilenameJsonKey(), ResultFilename);
		}

		
		FString FJsonFetchMeshPayloadCmd::ToJson() const
		{
			//Code should not do a ToJson if the data was not set before
			ensure(bIsDataInitialize);

			TSharedPtr<FJsonObject> CmdObject = MakeShared<FJsonObject>();
			TSharedPtr<FJsonObject> ActionDataObject = MakeShared<FJsonObject>();
			//CmdObject
			CmdObject->SetStringField(GetCommandIDJsonKey(), GetAction());
			CmdObject->SetStringField(GetTranslatorIDJsonKey(), GetTranslatorID());
			ActionDataObject->SetStringField(GetPayloadKeyJsonKey(), GetPayloadKey());

			//Bake settings
			ActionDataObject->SetStringField(GetMeshGlobalTransformJsonKey(), GetMeshGlobalTransform().ToString());

			CmdObject->SetObjectField(GetCommandDataJsonKey(), ActionDataObject);

			FString FetchPayloadCmd;
			TSharedRef< TJsonWriter< TCHAR, TPrettyJsonPrintPolicy<TCHAR> > > JsonWriter = TJsonWriterFactory< TCHAR, TPrettyJsonPrintPolicy<TCHAR> >::Create(&FetchPayloadCmd);
			if (!FJsonSerializer::Serialize(CmdObject.ToSharedRef(), JsonWriter))
			{
				//Error creating the json cmd string 
				return FString();
			}
			return FetchPayloadCmd;
		}

		bool FJsonFetchMeshPayloadCmd::FromJson(const FString& JsonString)
		{
			TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(JsonString);

			TSharedPtr<FJsonObject> CmdObject;
			if (!FJsonSerializer::Deserialize(Reader, CmdObject) || !CmdObject.IsValid())
			{
				//Cannot read the json file
				return false;
			}
			FString JsonActionValue;
			if (!CmdObject->TryGetStringField(GetCommandIDJsonKey(), JsonActionValue))
			{
				//The json cmd id key is missing
				return false;
			}

			if (!JsonActionValue.Equals(GetAction()))
			{
				//This json do not represent a load command
				return false;
			}

			//Read the json
			if (!CmdObject->TryGetStringField(GetTranslatorIDJsonKey(), TranslatorID))
			{
				//Missing Load command translator ID
				return false;
			}
			const TSharedPtr<FJsonObject>* ActionDataObject = nullptr;
			if (!CmdObject->TryGetObjectField(GetCommandDataJsonKey(), ActionDataObject))
			{
				//Missing Load Action data object
				return false;
			}
			if (!((*ActionDataObject)->TryGetStringField(GetPayloadKeyJsonKey(), PayloadKey)))
			{
				return false;
			}

			//Retrieve the mesh global transform
			FString TransformString;
			if (!((*ActionDataObject)->TryGetStringField(GetMeshGlobalTransformJsonKey(), TransformString)))
			{
				return false;
			}
			if (!MeshGlobalTransform.InitFromString(TransformString))
			{
				return false;
			}

			//Since we filled the data from the json file, set the data has been initialize.
			bIsDataInitialize = true;
			return true;
		}

		FString FJsonFetchAnimationQueriesCmd::ToJson() const
		{
			//Code should not do a ToJson if the data was not set before
			ensure(bIsDataInitialize);

			TSharedPtr<FJsonObject> CmdObject = MakeShared<FJsonObject>();
			
			TSharedPtr<FJsonObject> ActionDataObject = MakeShared<FJsonObject>();
			//CmdObject
			CmdObject->SetStringField(GetCommandIDJsonKey(), GetAction());
			CmdObject->SetStringField(GetTranslatorIDJsonKey(), GetTranslatorID());
			
			ActionDataObject->SetStringField(GetPayloadKeyJsonKey(), GetPayloadKey());

			//Queries:
			ActionDataObject->SetStringField(GetQueriesJsonStringKey(), QueriesJsonString);

			CmdObject->SetObjectField(GetCommandDataJsonKey(), ActionDataObject);

			FString FetchPayloadCmd;
			TSharedRef< TJsonWriter< TCHAR, TPrettyJsonPrintPolicy<TCHAR> > > JsonWriter = TJsonWriterFactory< TCHAR, TPrettyJsonPrintPolicy<TCHAR> >::Create(&FetchPayloadCmd);
			if (!FJsonSerializer::Serialize(CmdObject.ToSharedRef(), JsonWriter))
			{
				//Error creating the json cmd string 
				return FString();
			}
			return FetchPayloadCmd;
		}

		bool FJsonFetchAnimationQueriesCmd::FromJson(const FString& JsonString)
		{
			TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(JsonString);

			TSharedPtr<FJsonObject> CmdObject;
			if (!FJsonSerializer::Deserialize(Reader, CmdObject) || !CmdObject.IsValid())
			{
				//Cannot read the json file
				return false;
			}

			FString JsonActionValue;
			if (!CmdObject->TryGetStringField(GetCommandIDJsonKey(), JsonActionValue))
			{
				//The json cmd id key is missing
				return false;
			}

			if (!JsonActionValue.Equals(GetAction()))
			{
				//This json do not represent a load command
				return false;
			}

			//Read the json
			if (!CmdObject->TryGetStringField(GetTranslatorIDJsonKey(), TranslatorID))
			{
				//Missing Load command translator ID
				return false;
			}
			const TSharedPtr<FJsonObject>* ActionDataObject = nullptr;
			if (!CmdObject->TryGetObjectField(GetCommandDataJsonKey(), ActionDataObject))
			{
				//Missing Load Action data object
				return false;
			}
			if (!((*ActionDataObject)->TryGetStringField(GetPayloadKeyJsonKey(), PayloadKey)))
			{
				return false;
			}

			if (!((*ActionDataObject)->TryGetStringField(GetQueriesJsonStringKey(), QueriesJsonString)))
			{
				return false;
			}

			//Since we filled the data from the json file, set the data has been initialize.
			bIsDataInitialize = true;
			return true;
		}

		FString FJsonFetchAnimationQueriesCmd::JsonAnimationQueriesResultParser::ToJson() const
		{
			TSharedPtr<FJsonObject> ResultObject = MakeShared<FJsonObject>();

			for (const TPair<FString, FString>& HashToFilename : HashToFilenames)
			{
				ResultObject->SetStringField(HashToFilename.Key, HashToFilename.Value);
			}

			FString JsonResult;
			TSharedRef< TJsonWriter< TCHAR, TPrettyJsonPrintPolicy<TCHAR> > > JsonWriter = TJsonWriterFactory< TCHAR, TPrettyJsonPrintPolicy<TCHAR> >::Create(&JsonResult);
			if (!FJsonSerializer::Serialize(ResultObject.ToSharedRef(), JsonWriter))
			{
				//Error creating the json cmd string 
				return FString();
			}
			return JsonResult;
		}

		bool FJsonFetchAnimationQueriesCmd::JsonAnimationQueriesResultParser::FromJson(const FString& JsonString)
		{
			HashToFilenames.Empty();

			TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(JsonString);

			TSharedPtr<FJsonObject> ResultObject;
			if (!FJsonSerializer::Deserialize(Reader, ResultObject) || !ResultObject.IsValid())
			{
				//Cannot read the json file
				return false;
			}

			HashToFilenames.Reserve(ResultObject->Values.Num());

			for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Value : ResultObject->Values)
			{
				HashToFilenames.Add(FString(Value.Key), Value.Value->AsString());
			}

			return true;
		}
	} //ns Interchange
}//ns UE
