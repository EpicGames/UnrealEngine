// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeWorkerHandler.h"

#include "InterchangeCommands.h"
#include "InterchangeDispatcher.h"
#include "InterchangeDispatcherConfig.h"
#include "InterchangeDispatcherLog.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Sockets.h"
#include "SocketSubsystem.h"


namespace UE
{
	namespace Interchange
	{
		FInterchangeWorkerHandler::FInterchangeWorkerHandler(FInterchangeDispatcher& InDispatcher, FString& InResultFolder)
			: Dispatcher(InDispatcher)
			, WorkerState(EWorkerState::Uninitialized)
			, ErrorState(EWorkerErrorState::Ok)
			, ResultFolder(InResultFolder)
			, bShouldTerminate(false)
		{
			//Use a randomGuid to generate unique name
			const int32 UniqueID = (FPlatformTime::Cycles64() & 0x00000000EFFFFFFF);
			ThreadName = FString(TEXT("InterchangeWorkerHandler_")) + FString::FromInt(UniqueID);
			IOThread = FThread(*ThreadName, [this]() { Run(); });
			LastProgressMessageTime = FPlatformTime::Seconds();
		}

		FInterchangeWorkerHandler::~FInterchangeWorkerHandler()
		{
			StopBlocking();
		}

		void FInterchangeWorkerHandler::StartWorkerProcess()
		{
			uint32 WorkerProcessId = 0;

			ensure(ErrorState == EWorkerErrorState::Ok);
			FString ProcessorPath = Dispatcher.GetWorkerApplicationPath();
			if (FPaths::FileExists(ProcessorPath))
			{
				int32 ListenPort = NetworkInterface.GetListeningPort();
				if (ListenPort == 0)
				{
					ErrorState = EWorkerErrorState::ConnectionFailed_NotBound;
					return;
				}

				FString CommandToProcess;

				// Manually set worker BaseDir of worker process (as automatic deduction is broken in our case,
				// probably due to the abuse of ExeBinariesSubFolder in InterchangeWorker.Target.cs)
				// This fixes paths of logs generated by the worker.
				FString BaseDir = FPlatformProcess::BaseDir();
				CommandToProcess += TEXT(" -basedir=\"") + BaseDir + TEXT("\"");
				CommandToProcess += TEXT(" -ServerPID ") + FString::FromInt(FPlatformProcess::GetCurrentProcessId());
				CommandToProcess += TEXT(" -ServerPort ") + FString::FromInt(ListenPort);
				CommandToProcess += TEXT(" -InterchangeDispatcherVersion \"") + DispatcherCommandVersion::ToString() + TEXT('"');
				CommandToProcess += TEXT(" -ResultFolder \"") + ResultFolder + TEXT("\"");
				UE_LOGF(LogInterchangeDispatcher, Verbose, "CommandToProcess: %ls", *CommandToProcess);
				WorkerHandle = FPlatformProcess::CreateProc(*ProcessorPath, *CommandToProcess, true, false, false, &WorkerProcessId, 0, nullptr, nullptr);
			}

			if (!WorkerHandle.IsValid() || !FPlatformProcess::IsProcRunning(WorkerHandle) || !FPlatformProcess::IsApplicationRunning(WorkerProcessId))
			{
				ErrorState = EWorkerErrorState::WorkerProcess_CantCreate;
				return;
			}
		}

		void FInterchangeWorkerHandler::ValidateConnection()
		{
			if (!NetworkInterface.IsValid())
			{
				UE_LOGF(LogInterchangeDispatcher, Display, "NetworkInterface lost");
				WorkerState = EWorkerState::Closing;
				ErrorState = EWorkerErrorState::ConnectionLost;
			}
			else if (WorkerHandle.IsValid() && !FPlatformProcess::IsProcRunning(WorkerHandle))
			{
				UE_LOGF(LogInterchangeDispatcher, Display, "Worker lost");
				WorkerState = EWorkerState::Closing;
				ErrorState = EWorkerErrorState::WorkerProcess_Lost;
			}
			//Send periodic query progress for task that take long time to resolve
			const double MinimumProgressTime = 5.0;
			const double ProgressTickTime = 5.0;
			const double CurrentTime = FPlatformTime::Seconds();
			if(CurrentTime - LastProgressMessageTime > ProgressTickTime)
			{
				LastProgressMessageTime = CurrentTime;
				FScopeLock Lock(&CurrentTasksLock);
				TArray<int32> TasksToQuery;
				//Look all current tasks
				for (const int32 TaskIndex : CurrentTasks)
				{
					UE::Interchange::ETaskState TaskState;
					double TaskRunningStateStartTime;
					Dispatcher.GetTaskState(TaskIndex, TaskState, TaskRunningStateStartTime);
					
					if (TaskState == ETaskState::Running)
					{
						double RunningTime = FPlatformTime::Seconds() - TaskRunningStateStartTime;
						//Ask only for old task that run longer then the minimum progress time
						if (RunningTime > MinimumProgressTime)
						{
							TasksToQuery.Add(TaskIndex);
						}
					}
				}
				//Send the query command
				FQueryTaskProgressCommand QueryProgressCommand(TasksToQuery);

				if (CommandIO.SendCommand(QueryProgressCommand, Config::SendCommandTimeout_s))
				{
					UE_LOGF(LogInterchangeDispatcher, Verbose, "Query progress command sent");
				}
				else
				{
					// Cannot send a command to the worker, let set the Closing state
					WorkerState = EWorkerState::Closing;
					ErrorState = EWorkerErrorState::ConnectionLost_SendFailed;
				}
			}
		}

		void FInterchangeWorkerHandler::Run()
		{
			WorkerState = EWorkerState::Uninitialized;
			RunInternal();
			WorkerState = EWorkerState::Terminated;
			UE_CLOGF(ErrorState != EWorkerErrorState::Ok, LogInterchangeDispatcher, Display, "Handler ended with fault: %ls", EWorkerErrorStateAsString(ErrorState));
		}

		void FInterchangeWorkerHandler::KillAllCurrentTasks()
		{
			FScopeLock Lock(&CurrentTasksLock);
			//Fail all current tasks
			for (const int32 TaskIndex : CurrentTasks)
			{
				TArray<FString> GarbageMessages;
				Dispatcher.SetTaskState(TaskIndex, ETaskState::ProcessFailed, FString(), GarbageMessages);
			}
			CurrentTasks.Empty();
		}

		void FInterchangeWorkerHandler::RunInternal()
		{
			auto StartNewTask = [this]()->bool
			{
				// Fetch a new task
				TOptional<FTask> CurrentTask = Dispatcher.GetNextTask();
				bool bSucceed = false;
				if (CurrentTask.IsSet())
				{
					FRunTaskCommand NewTask(CurrentTask.GetValue());
					{
						FScopeLock Lock(&CurrentTasksLock);
						CurrentTasks.Add(NewTask.TaskIndex);
					}

					if (CommandIO.SendCommand(NewTask, Config::SendCommandTimeout_s))
					{
						UE_LOGF(LogInterchangeDispatcher, Verbose, "New task command sent");
						bSucceed = true;
					}
					else
					{
						// Signal that the Task was not processed
						TArray<FString> GarbageMessages;
						Dispatcher.SetTaskState(CurrentTask->Index, ETaskState::ProcessFailed, FString(), GarbageMessages);
						WorkerState = EWorkerState::Closing;
						ErrorState = EWorkerErrorState::ConnectionLost_SendFailed;
					}
				}
				return bSucceed;
			};

			auto SetTerminatedState = [this]()
			{
				//Always broadcast before setting the "terminated" state. API "IsAlive" is use by the dispatcher to reset the worker handler when worker handler state is "terminated"
				//Notify the dispatcher the worker handler is done so it can terminate queue tasks
				OnWorkerHandlerExitLoop.Broadcast();
				WorkerState = EWorkerState::Terminated;
			};

			while (IsAlive())
			{
				switch (WorkerState)
				{
					case EWorkerState::Uninitialized:
					{
						ErrorState = EWorkerErrorState::Ok;

						StartWorkerProcess();

						if (ErrorState != EWorkerErrorState::Ok)
						{
							SetTerminatedState();
							break;
						}

						// The Accept() call on the server blocks until a connection is initiated from a client
						static const FString SocketDescription = TEXT("InterchangeWorkerHandler");
						if (!NetworkInterface.Accept(SocketDescription, Config::AcceptTimeout_s))
						{
							ErrorState = EWorkerErrorState::ConnectionFailed_NoClient;
						}
						else
						{
							CommandIO.SetNetworkInterface(&NetworkInterface);
						}

						if (ErrorState != EWorkerErrorState::Ok)
						{
							WorkerState = EWorkerState::Closing;
							break;
						}

						WorkerState = EWorkerState::Idle;
						break;
					}

					case EWorkerState::Idle:
					{
						ValidateConnection();
						if (ErrorState == EWorkerErrorState::WorkerProcess_Lost)
						{
							WorkerState = EWorkerState::Closing;
						}
						else
						{
							//Start a task
							if (StartNewTask())
							{
								WorkerState = EWorkerState::Processing;
								break;
							}

							//Need to terminate?
							if (bShouldTerminate)
							{
								UE_LOGF(LogInterchangeDispatcher, Verbose, "Exit loop gracefully");
								WorkerState = EWorkerState::Closing;
							}
							// No task to execute, wait a little
							else if (WorkerState == EWorkerState::Idle)
							{
								FPlatformProcess::Sleep(static_cast<float>(Config::IdleLoopDelay));
							}
						}
						break;
					}

					case EWorkerState::Processing:
					{
						ValidateConnection();
						if (ErrorState == EWorkerErrorState::WorkerProcess_Lost)
						{
							WorkerState = EWorkerState::Closing;
						}
						else
						{
							// consume task
							if (TSharedPtr<ICommand> Command = CommandIO.GetNextCommand(Config::IdleLoopDelay))
							{
								ProcessCommand(*Command);
							}

							//Need to terminate?
							if (bShouldTerminate)
							{
								UE_LOGF(LogInterchangeDispatcher, Verbose, "Exit loop gracefully");
								WorkerState = EWorkerState::Closing;
							}
							// Task is being executed, sleep before rechecking
							else if (WorkerState == EWorkerState::Processing)
							{
								FPlatformProcess::Sleep(Config::ProcessingLoopDelay);
							}
						}
						break;
					}

					case EWorkerState::Closing:
					{
						// try to close the process gracefully
						if (WorkerHandle.IsValid())
						{
							bool CloseByCommand = Config::CloseProcessByCommand && CommandIO.IsValid() && FPlatformProcess::IsProcRunning(WorkerHandle);

							bool bClosed = false;
							if (CloseByCommand)
							{
								FTerminateCommand Terminate;
								CommandIO.SendCommand(Terminate, 0);

								for (int32 i = 0; i < int32(10. * Config::TerminateTimeout_s); ++i)
								{
									if (!FPlatformProcess::IsProcRunning(WorkerHandle))
									{
										bClosed = true;
										break;
									}
									FPlatformProcess::Sleep(0.1f);
								}
							}

							if (!bClosed)
							{
								FPlatformProcess::TerminateProc(WorkerHandle, true);
							}
						}

						// Process commands still in input queue
						CommandIO.Disconnect(0);
						while (TSharedPtr<ICommand> Command = CommandIO.GetNextCommand(0))
						{
							ProcessCommand(*Command);
						}

						//Make sure all running task are completed with error
						KillAllCurrentTasks();

						SetTerminatedState();
						break;
					}

					default:
					{
						ensureMsgf(false, TEXT("missing case handling"));
					}
				}
			}
		}

		void FInterchangeWorkerHandler::Stop()
		{
			bShouldTerminate = true;
		}

		void FInterchangeWorkerHandler::StopBlocking()
		{
			Stop();
			if (IOThread.IsJoinable())
			{
				IOThread.Join();
			}
		}

		bool FInterchangeWorkerHandler::IsAlive() const
		{
			return !EnumHasAnyFlags(EWorkerState(WorkerState), EWorkerState::Terminated);
		}

		void FInterchangeWorkerHandler::ProcessCommand(ICommand& Command)
		{
			switch (Command.GetType())
			{
				case ECommandId::Ping:
					ProcessCommand(StaticCast<FPingCommand&>(Command));
					break;
				
				case ECommandId::Error:
					ProcessCommand(StaticCast<FErrorCommand&>(Command));
					break;

				case ECommandId::NotifyEndTask:
					ProcessCommand(StaticCast<FCompletedTaskCommand&>(Command));
					break;

				case ECommandId::CompletedQueryTaskProgress:
					ProcessCommand(StaticCast<FCompletedQueryTaskProgressCommand&>(Command));
					break;

				default:
					break;
			}
		}

		void FInterchangeWorkerHandler::ProcessCommand(FPingCommand& PingCommand)
		{
			bPingCommandReceived = true;
			UE::Interchange::FBackPingCommand BackPing;
			CommandIO.SendCommand(BackPing, 0);
		}

		void FInterchangeWorkerHandler::ProcessCommand(FErrorCommand& ErrorCommand)
		{
			UE_LOGF(LogInterchangeDispatcher, Display, "%ls", *ErrorCommand.ErrorMessage);
			this->Dispatcher.SetInterchangeWorkerFatalError(ErrorCommand.ErrorMessage);
			FTerminateCommand Terminate;
			CommandIO.SendCommand(Terminate, 0);
		}

		void FInterchangeWorkerHandler::ProcessCommand(FCompletedTaskCommand& CompletedTaskCommand)
		{
			{
				FScopeLock Lock(&CurrentTasksLock);
				CurrentTasks.Remove(CompletedTaskCommand.TaskIndex);
			}
			Dispatcher.SetTaskState(CompletedTaskCommand.TaskIndex, CompletedTaskCommand.ProcessResult, CompletedTaskCommand.JSonResult, CompletedTaskCommand.JSonMessages);

			WorkerState = EWorkerState::Idle;
		}

		void FInterchangeWorkerHandler::ProcessCommand(FCompletedQueryTaskProgressCommand& CompletedQueryTaskProgressCommand)
		{
			for (const FCompletedQueryTaskProgressCommand::FTaskProgressData& TaskData : CompletedQueryTaskProgressCommand.TaskStates)
			{
				if (TaskData.TaskState != ETaskState::Running)
				{
					ETaskState CurrentTaskState;
					double TaskRunningStateStartTime;
					Dispatcher.GetTaskState(TaskData.TaskIndex, CurrentTaskState, TaskRunningStateStartTime);
					if (CurrentTaskState == ETaskState::Running)
					{
						//This is a stale task, we must complete it failed. This can happen if a completed task message is skip (lost, not process, etc...)
						//We must ensure we complete the task with a failed so if someone wait for it it will be release.
						FString EmptyGarbage1;
						TArray<FString> EmptyGarbage2;
						Dispatcher.SetTaskState(TaskData.TaskIndex, ETaskState::ProcessFailed, EmptyGarbage1, EmptyGarbage2);
					}
				}
			}
		}

		const TCHAR* FInterchangeWorkerHandler::EWorkerErrorStateAsString(EWorkerErrorState Error)
		{
			switch (Error)
			{
				case EWorkerErrorState::Ok: return TEXT("Ok");
				case EWorkerErrorState::ConnectionFailed_NotBound: return TEXT("Connection failed (socket not bound)");
				case EWorkerErrorState::ConnectionFailed_NoClient: return TEXT("Connection failed (No connection from client)");
				case EWorkerErrorState::ConnectionLost: return TEXT("Connection lost");
				case EWorkerErrorState::ConnectionLost_SendFailed: return TEXT("Connection lost (send failed)");
				case EWorkerErrorState::WorkerProcess_CantCreate: return TEXT("Worker process issue (cannot create worker process)");
				case EWorkerErrorState::WorkerProcess_Lost: return TEXT("Worker process issue (worker lost)");
				default: return TEXT("unknown");
			}
		}

	} //ns Interchange
}//ns UE
