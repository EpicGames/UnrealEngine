// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeDispatcherTask.h"

namespace UE::Interchange
{

	class FJsonFBXLoadSourceCmd : public FJsonLoadSourceCmd
	{
	public:
		FJsonFBXLoadSourceCmd() : FJsonLoadSourceCmd() {}

		FJsonFBXLoadSourceCmd(const FString& InTranslatorID
			, const FString& InSourceFilename
			, const bool InbConvertScene
			, const bool InbForceFrontXAxis
			, const bool InbConvertSceneUnit
			, const bool InbKeepFbxNamespace
			, const bool InbConsiderClusterBeforePoseForMeshBindPose)
			: FJsonLoadSourceCmd(InTranslatorID, InSourceFilename)
			, bConvertScene(InbConvertScene)
			, bForceFrontXAxis(InbForceFrontXAxis)
			, bConvertSceneUnit(InbConvertSceneUnit)
			, bKeepFbxNamespace(InbKeepFbxNamespace)
			, bConsiderClusterBeforePoseForMeshBindPose(InbConsiderClusterBeforePoseForMeshBindPose)
		{
		}

		INTERCHANGEDISPATCHER_API virtual TSharedPtr<FJsonObject> GetActionDataObject() const override;
		INTERCHANGEDISPATCHER_API virtual bool IsActionDataObjectValid(const FJsonObject& ActionDataObject) override;
		
		bool GetDoesConvertScene() const
		{
			//Code should not do query data if the data was not set before
			ensure(bIsDataInitialize);
			return bConvertScene;
		}

		bool GetDoesForceFrontXAxis() const
		{
			//Code should not do query data if the data was not set before
			ensure(bIsDataInitialize);
			return bForceFrontXAxis;
		}

		bool GetDoesConvertSceneUnit() const
		{
			//Code should not do query data if the data was not set before
			ensure(bIsDataInitialize);
			return bConvertSceneUnit;
		}

		bool GetDoesKeepFbxNamespace() const
		{
			//Code should not do query data if the data was not set before
			ensure(bIsDataInitialize);
			return bKeepFbxNamespace;
		}

		bool GetDoesUseFbxClustersForBindPoseMatrices() const
		{
			//Code should not do query data if the data was not set before
			ensure(bIsDataInitialize);
			return bConsiderClusterBeforePoseForMeshBindPose;
		}

		INTERCHANGEDISPATCHER_API static const TCHAR* TaskName;

	private:
		bool bConvertScene = true;
		bool bForceFrontXAxis = false;
		bool bConvertSceneUnit = true;
		bool bKeepFbxNamespace = false;
		bool bConsiderClusterBeforePoseForMeshBindPose = false;

		static const TCHAR* ConvertSceneJsonKey;
		static const TCHAR* ForceFrontXAxisJsonKey;
		static const TCHAR* ConvertSceneUnitJsonKey;
		static const TCHAR* KeepFbxNamespaceJsonKey;
		static const TCHAR* UseFbxClustersForBindPoseMatricesJsonKey;
	};
}//ns UE
