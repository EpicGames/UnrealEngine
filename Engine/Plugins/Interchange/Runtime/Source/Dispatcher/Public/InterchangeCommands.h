// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeDispatcherTask.h"
#include "Serialization/Archive.h"

#define UE_API INTERCHANGEDISPATCHER_API

class FArchive;

namespace UE
{
	namespace Interchange
	{

		class DispatcherCommandVersion
		{
		public:
			//Major version should be updated when there is an existing API that has been change.
			static UE_API int32 GetMajor();
			//Minor version must be updated when there is an addition to the API.
			static UE_API int32 GetMinor();
			//Patch version should be update if there is some bug fixes in the private code.
			static UE_API int32 GetPatch();

			//LWCDisabled version tell if the code was compile with serialization compatibility.
			static bool GetLWCDisabled()
			{
				// LWC_TODO: This is now redundant, and will be removed in the future. 
				return false;
			}

			/** Return the version in string format "Major.Minor.Patch" */
			static FString ToString()
			{
				static FString Version = FString::FromInt(GetMajor()) + TEXT(".") + FString::FromInt(GetMinor()) + TEXT(".") + FString::FromInt(GetPatch()) + TEXT(".") + FString::FromInt(GetLWCDisabled() ? 1 : 0);
				return Version;
			}

			static bool FromString(const FString& VersionStr, int32& OutMajor, int32& OutMinor, int32& OutPatch, bool& bOutLWCDisabled)
			{
				TArray<FString> Tokens;
				VersionStr.ParseIntoArray(Tokens, TEXT("."));
				if (Tokens.Num() != 4)
				{
					return false;
				}
				OutMajor = FCString::Atoi(*(Tokens[0]));
				OutMinor = FCString::Atoi(*(Tokens[1]));
				OutPatch = FCString::Atoi(*(Tokens[2]));
				bOutLWCDisabled = FCString::Atoi(*(Tokens[3])) != 0;

				return true;
			}

			/**
			 * We consider having the same major and minor version will make the API fully compatible.
			 * Patch is only a hint in case you need a particular fixes
			 * @param Major - The Major version you want to test against the compile Major version
			 * @param Minor - The Minor version you want to test against the compile Minor version
			 * @param Patch - In case we need it later, not use currently
			 *
			 * @return true if Major and Minor version equal the compile one, InPatch is only there in case we need it in a future release
			 */
			static bool IsAPICompatible(int32 Major, int32 Minor, int32 Patch, bool bLWCDisabled)
			{
				return (GetMajor() == Major && GetMinor() == Minor && GetLWCDisabled() == bLWCDisabled);
			}
		};

		enum class ECommandId : uint8
		{
			Invalid,
			Error,
			Ping,
			BackPing,
			RunTask,
			NotifyEndTask,
			QueryTaskProgress,
			CompletedQueryTaskProgress,
			Terminate,
			Last
		};

		class ICommand
		{
		public:
			virtual ~ICommand() = default;

			virtual ECommandId GetType() const = 0;

			friend void operator<<(FArchive& Ar, ICommand& C) { C.SerializeImpl(Ar); }

		protected:
			virtual void SerializeImpl(FArchive&) {}
		};



		// Create a new command from its type
		TSharedPtr<ICommand> CreateCommand(ECommandId CommandType);

		// Converts a command into a byte buffer
		void SerializeCommand(ICommand& Command, TArray<uint8>& OutBuffer);

		// Converts byte buffer back into a Command
		// returns nullptr in case of error
		TSharedPtr<ICommand> DeserializeCommand(const TArray<uint8>& InBuffer);



		class FTerminateCommand : public ICommand
		{
		public:
			virtual ECommandId GetType() const override { return ECommandId::Terminate; }
		};

		class FErrorCommand : public ICommand
		{
		public:
			virtual ECommandId GetType() const override { return ECommandId::Error; }
			
		protected:
			UE_API virtual void SerializeImpl(FArchive&) override;
		public:
			FString ErrorMessage;
		};

		class FPingCommand : public ICommand
		{
		public:
			virtual ECommandId GetType() const override { return ECommandId::Ping; }
		};


		class FBackPingCommand : public ICommand
		{
		public:
			virtual ECommandId GetType() const override { return ECommandId::BackPing; }
		};

		class FRunTaskCommand : public ICommand
		{
		public:
			FRunTaskCommand() = default;
			FRunTaskCommand(const FTask& Task) : JsonDescription(Task.JsonDescription), TaskIndex(Task.Index) {}
			virtual ECommandId GetType() const override { return ECommandId::RunTask; }

		protected:
			UE_API virtual void SerializeImpl(FArchive&) override;

		public:
			FString JsonDescription;
			int32 TaskIndex = -1;
		};

		class FCompletedTaskCommand : public ICommand
		{
		public:
			virtual ECommandId GetType() const override { return ECommandId::NotifyEndTask; }

		protected:
			UE_API virtual void SerializeImpl(FArchive&) override;

		public:
			ETaskState ProcessResult = ETaskState::Unknown;
			FString JSonResult;
			TArray<FString> JSonMessages;
			int32 TaskIndex = INDEX_NONE;
		};

		class FQueryTaskProgressCommand : public ICommand
		{
		public:
			FQueryTaskProgressCommand() = default;
			UE_API FQueryTaskProgressCommand(const TArray<int32>& Tasks);
			virtual ECommandId GetType() const override { return ECommandId::QueryTaskProgress; }

		protected:
			UE_API virtual void SerializeImpl(FArchive&) override;

		public:
			TArray<int32> TaskIndexes;
		};

		class FCompletedQueryTaskProgressCommand : public ICommand
		{
		public:
			FCompletedQueryTaskProgressCommand() = default;
			virtual ECommandId GetType() const override { return ECommandId::CompletedQueryTaskProgress; }

		protected:
			UE_API virtual void SerializeImpl(FArchive&) override;

		public:
			struct FTaskProgressData
			{
				int32 TaskIndex;
				ETaskState TaskState;
				float TaskProgress;

				friend FArchive& operator<<(FArchive& Ar, FTaskProgressData& A);
			};
			TArray<FTaskProgressData> TaskStates;
		};
	} //ns Interchange
}//ns UE

#undef UE_API
