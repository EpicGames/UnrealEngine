// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "HAL/Thread.h"
#include "InterchangeDispatcherTask.h"
#include "InterchangeWorkerHandler.h"

#define UE_API INTERCHANGEDISPATCHER_API

namespace UE
{
	namespace Interchange
	{

		// Handle a list of tasks, and a set of external workers to consume them.
		class  FInterchangeDispatcher
		{
		public:
			// If WorkerCount == 0, the dispatcher will determine the number of worker processes to start
			// based on main application's workload, threads and memory wise.
			UE_API FInterchangeDispatcher(const FString& InResultFolder, int32 WorkerCount = 1);
			virtual ~FInterchangeDispatcher() { TerminateProcess(); }

			UE_API int32 AddTask(const FString& JsonDescription);
			UE_API int32 AddTask(const FString& JsonDescription, FInterchangeDispatcherTaskCompleted TaskCompledDelegate);
			UE_API TOptional<FTask> GetNextTask();
			UE_API void SetTaskState(int32 TaskIndex, ETaskState TaskState, const FString& JsonResult, const TArray<FString>& JSonMessages);
			UE_API void GetTaskState(int32 TaskIndex, ETaskState& TaskState, double& TaskRunningStateStartTime);
			UE_API void GetTaskState(int32 TaskIndex, ETaskState& TaskState, FString& JsonResult, TArray<FString>& JSonMessages);

			UE_API void StartProcess();
			UE_API virtual void StopProcess(bool bBlockUntilTerminated);
			UE_API virtual void TerminateProcess();
			UE_API void WaitAllTaskToCompleteExecution();
			UE_API bool IsOver();

			virtual const TCHAR* GetWorkerApplicationName()
			{
				return TEXT("InterchangeWorker");
			}

			const FString& GetWorkerApplicationPath();

			void SetInterchangeWorkerFatalError(FString& ErrorMessage)
			{
				InterchangeWorkerFatalError = MoveTemp(ErrorMessage);
			}

			FString GetInterchangeWorkerFatalError()
			{
				return InterchangeWorkerFatalError;
			}

			UE_API bool IsValid();
		
		protected:
			UE_API void CloseHandlers();

			/** Path where the result files are dump */
			FString ResultFolder;
			TArray<FTask> TaskPool;

		private:
			void SpawnHandlers();
			void EmptyQueueTasks();

			// Tasks
			FCriticalSection TaskPoolCriticalSection;
			int32 NextTaskIndex;
			int32 CompletedTaskCount;

			FString InterchangeWorkerFatalError;

			// Workers
			FString WorkerApplicationPath;
			TArray<TUniquePtr<FInterchangeWorkerHandler>> WorkerHandlers;
			TUniquePtr<FInterchangeWorkerHandler> WorkerHandler_Deprecated = nullptr;
		};

	} //ns Interchange
}//ns UE

#undef UE_API
