// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Modules/ModuleInterface.h"

#define INTERCHANGEDISPATCHER_MODULE_NAME TEXT("InterchangeDispatcher")

/**
 * This module allows out-of-process Interchange translators in case a third-party SDK is not thread-safe.
 */
class FInterchangeDispatcherModule : public IModuleInterface
{
public:
	static FInterchangeDispatcherModule& Get();
	static bool IsAvailable();

private:
	virtual void StartupModule() override;
};
