// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeCommands.h"

#include "CoreMinimal.h"
#include "Containers/Queue.h"
#include "HAL/CriticalSection.h"
#include "Sockets.h"

#define UE_API INTERCHANGEDISPATCHER_API

namespace UE
{
	namespace Interchange
	{

		enum class SocketErrorCode
		{
			NoError = 0,
			Error_Create,
			Error_Bind,
			Error_Listen,
			UnableToReadOnSocket,
			UnableToSendData,
			CouldNotStartWSA,
			UnableToGetLocalAddress,
			ConnectionToServerFailed,
		};


		struct FMessageHeader
		{
			int32 ByteSize = -1;

			friend void operator<< (FArchive& Ar, FMessageHeader& H)
			{
				FString Guard = Ar.IsLoading() ? TEXT("") : TEXT("guard");
				Ar << Guard;
				Ar << H.ByteSize;
				ensure(Guard == TEXT("guard"));
			}
		};


		class FNetworkNode
		{
		public:
			UE_API virtual ~FNetworkNode();

			// Send a byte buffer as an atomic message
			// returns true when a message is successfully sent
			UE_API bool SendMessage(const TArray<uint8>& Buffer, double Timeout_s);

			// Receive a byte buffer as an atomic message
			// returns true when a message is fetched. In that case, OutBuffer contains the message.
			UE_API bool ReceiveMessage(TArray<uint8>& OutBuffer, double Timeout_s);

			bool IsValid() const { return !bReadError && !bWriteError; }

		protected:
			struct FMessage
			{
				FMessageHeader Header;
				TArray<uint8> Content;
			};
			UE_API FSocket* CreateInternalSocket(const FString& Description);
			UE_API void CloseSocket(FSocket*& Socket);
			UE_API bool IsConnected();

		protected:
			FSocket* ConnectedSocket = nullptr;
			SocketErrorCode ConnectedSocketError = SocketErrorCode::NoError;

			FMessage IncommingMessage;

			bool bReadError = false;
			bool bWriteError = false;

			FCriticalSection SendReceiveCriticalSection;
		};



		// Create, Bind, Listen, Accept sequence
		class FNetworkServerNode : public FNetworkNode
		{
		public:
			UE_API FNetworkServerNode();
			UE_API ~FNetworkServerNode();
			UE_API int32 GetListeningPort();
			UE_API bool Accept(const FString& Description, double Timeout_s);

		private:
			FSocket* ListeningSocket;
		};



		// Create, Connect sequence
		class FNetworkClientNode : public FNetworkNode
		{
		public:
			UE_API bool Connect(const FString& Description, int32 ServerPort, double Timeout_s);
		};



		class FCommandQueue
		{
		public:
			UE_API void SetNetworkInterface(FNetworkNode* InNetworkInterface);

			UE_API TSharedPtr<ICommand> GetNextCommand(double Timeout_s);
			UE_API bool SendCommand(ICommand& Commmand, double Timeout_s);

			bool IsValid() { return NetworkInterface ? NetworkInterface->IsValid() : true; }

			UE_API void Disconnect(double Timeout_s);

		private:
			UE_API bool Poll(double Timeout_s);
			TQueue<TSharedPtr<ICommand>> InCommands;

		private:
			FNetworkNode* NetworkInterface = nullptr;
		};


	} //ns Interchange
}//ns UE

#undef UE_API
