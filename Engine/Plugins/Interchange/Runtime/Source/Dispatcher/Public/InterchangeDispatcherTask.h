// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FJsonObject;

namespace UE
{
	namespace Interchange
	{
		struct FTask;
		struct FAnimationPayloadQuery;
	}
}

DECLARE_DELEGATE_OneParam(FInterchangeDispatcherTaskCompleted, int32 TaskIndex);

namespace UE
{
	namespace Interchange
	{
		enum class ETaskState
		{
			Unknown,
			Running,
			UnTreated,
			ProcessOk,
			ProcessFailed,
		};

		struct FTask
		{
			FTask() = delete;

			FTask(const FString& InJsonDescription)
			{
				JsonDescription = InJsonDescription;
				State = ETaskState::UnTreated;
			}

			FString JsonDescription;
			int32 Index = -1;
			ETaskState State = ETaskState::Unknown;
			FString JsonResult;
			TArray<FString> JsonMessages;
			FInterchangeDispatcherTaskCompleted OnTaskCompleted;
			double RunningStateStartTime = 0;
		};

		/**
		 * Json cmd helper to be able to read and write a FTask::JsonDescription
		 */
		class IJsonCmdBase
		{
		public:
			virtual ~IJsonCmdBase() = default;

			virtual FString GetAction() const = 0;
			virtual FString GetTranslatorID() const = 0;
			virtual FString ToJson() const = 0;

			/**
			 * Return false if the JsonString do not match the command, true otherwise.
			 */
			virtual bool FromJson(const FString& JsonString) = 0;

			static FString GetCommandIDJsonKey()
			{
				static const FString Key = TEXT("CmdID");
				return Key;
			}
			static FString GetTranslatorIDJsonKey()
			{
				static const FString Key = TEXT("TranslatorID");
				return Key;
			}
			static FString GetCommandDataJsonKey()
			{
				static const FString Key = TEXT("CmdData");
				return Key;
			}

		protected:
			//Use this member to know if the data is initialize before using it
			bool bIsDataInitialize = false;
		};

		class FJsonLoadSourceCmd : public IJsonCmdBase
		{
		public:
			FJsonLoadSourceCmd()
			{
				bIsDataInitialize = false;
			}

			virtual ~FJsonLoadSourceCmd() = default;

			FJsonLoadSourceCmd(const FString& InTranslatorID
				, const FString& InSourceFilename)
				: TranslatorID(InTranslatorID)
				, SourceFilename(InSourceFilename)
			{
				bIsDataInitialize = true;
			}

			virtual FString GetAction() const override
			{
				static const FString LoadString = TEXT("LoadSource");
				return LoadString;
			}

			virtual FString GetTranslatorID() const override
			{
				//Code should not do query data if the data was not set before
				ensure(bIsDataInitialize);
				return TranslatorID;
			}

			INTERCHANGEDISPATCHER_API virtual FString ToJson() const;
			INTERCHANGEDISPATCHER_API virtual bool FromJson(const FString& JsonString);

			INTERCHANGEDISPATCHER_API virtual TSharedPtr<FJsonObject> GetActionDataObject() const;
			virtual bool IsActionDataObjectValid(const FJsonObject& ActionDataObject)
			{
				return true;
			}

			FString GetSourceFilename() const
			{
				//Code should not do query data if the data was not set before
				ensure(bIsDataInitialize);
				return SourceFilename;
			}

			static FString GetSourceFilenameJsonKey()
			{
				static const FString Key = TEXT("SourceFile");
				return Key;
			}

			/**
			 * Use this class helper to create the cmd result json string and to read it
			 */
			class JsonResultParser
			{
			public:
				FString GetResultFilename() const
				{
					return ResultFilename;
				}
				void SetResultFilename(const FString& InResultFilename)
				{
					ResultFilename = InResultFilename;
				}
				INTERCHANGEDISPATCHER_API FString ToJson() const;
				INTERCHANGEDISPATCHER_API bool FromJson(const FString& JsonString);

				static FString GetResultFilenameJsonKey()
				{
					const FString Key = TEXT("ResultFile");
					return Key;
				}
			private:
				FString ResultFilename = FString();
			};

		private:
			FString TranslatorID = FString();
			FString SourceFilename = FString();
		};

		class FJsonFetchPayloadCmd : public IJsonCmdBase
		{
		public:
			FJsonFetchPayloadCmd()
			{
				bIsDataInitialize = false;
			}

			FJsonFetchPayloadCmd(const FString& InTranslatorID, const FString& InPayloadKey)
				: TranslatorID(InTranslatorID)
				, PayloadKey(InPayloadKey)
			{
				
				bIsDataInitialize = true;
			}

			virtual FString GetAction() const override
			{
				static const FString LoadString = TEXT("Payload");
				return LoadString;
			}

			virtual FString GetTranslatorID() const override
			{
				//Code should not do query data if the data was not set before
				ensure(bIsDataInitialize);
				return TranslatorID;
			}

			INTERCHANGEDISPATCHER_API virtual FString ToJson() const;
			INTERCHANGEDISPATCHER_API virtual bool FromJson(const FString& JsonString);

			FString GetPayloadKey() const
			{
				//Code should not do query data if the data was not set before
				ensure(bIsDataInitialize);
				return PayloadKey;
			}

			static FString GetPayloadKeyJsonKey()
			{
				static const FString Key = TEXT("PayloadKey");
				return Key;
			}

			/**
			 * Use this class helper to create the cmd result json string and to read it
			 */
			class JsonResultParser
			{
			public:
				FString GetResultFilename() const
				{
					return ResultFilename;
				}
				void SetResultFilename(const FString& InResultFilename)
				{
					ResultFilename = InResultFilename;
				}
				INTERCHANGEDISPATCHER_API FString ToJson() const;
				INTERCHANGEDISPATCHER_API bool FromJson(const FString& JsonString);

				static FString GetResultFilenameJsonKey()
				{
					const FString Key = TEXT("ResultFile");
					return Key;
				}
			private:
				FString ResultFilename = FString();
			};

		protected:
			FString TranslatorID = FString();
			FString PayloadKey = FString();
		};

		//Mesh payload require a transform to bake the mesh to avoid degenerate triangle when importing a small mesh scale by a scene node.
		class FJsonFetchMeshPayloadCmd : public FJsonFetchPayloadCmd
		{
		public:
			FJsonFetchMeshPayloadCmd()
			{
				check(!bIsDataInitialize);
			}

			FJsonFetchMeshPayloadCmd(const FString& InTranslatorID
				, const FString& InPayloadKey
				, const FTransform& InMeshGlobalTransform)
				: FJsonFetchPayloadCmd(InTranslatorID, InPayloadKey)
				, MeshGlobalTransform(InMeshGlobalTransform)
			{}

			INTERCHANGEDISPATCHER_API virtual FString ToJson() const override;
			INTERCHANGEDISPATCHER_API virtual bool FromJson(const FString& JsonString) override;

			FTransform GetMeshGlobalTransform() const
			{
				ensure(bIsDataInitialize);
				return MeshGlobalTransform;
			}

			static FString GetMeshGlobalTransformJsonKey()
			{
				static const FString Key = TEXT("GlobalMeshTransform");
				return Key;
			}
		protected:
			FTransform MeshGlobalTransform = FTransform::Identity;
		};

		class FJsonFetchAnimationQueriesCmd : public FJsonFetchPayloadCmd
		{
		public:
			FJsonFetchAnimationQueriesCmd()
			{
				check(!bIsDataInitialize);
			}

			FJsonFetchAnimationQueriesCmd(const FString& InTranslatorID, const FString& InQueriesJsonString) 
				: FJsonFetchPayloadCmd(InTranslatorID, TEXT("AnimationQueries"))
				, QueriesJsonString(InQueriesJsonString)
			{
			}

			INTERCHANGEDISPATCHER_API virtual FString ToJson() const override;
			INTERCHANGEDISPATCHER_API virtual bool FromJson(const FString& JsonString) override;

			static FString GetQueriesJsonStringKey()
			{
				static const FString Key = TEXT("QueriesJsonString");
				return Key;
			}

			FString GetQueriesJsonString() const
			{
				return QueriesJsonString;
			}

			/**
			 * Use this class helper to create the cmd result json string and to read it
			 */
			class JsonAnimationQueriesResultParser
			{
			public:
				TMap<FString, FString> GetResultFilename() const
				{
					return HashToFilenames;
				}
				void SetHashToFilenames(const TMap<FString, FString>& InHashToFilenames)
				{
					HashToFilenames = InHashToFilenames;
				}

				INTERCHANGEDISPATCHER_API FString ToJson() const;
				INTERCHANGEDISPATCHER_API bool FromJson(const FString& JsonString);

				static FString GetHashToFilenamesKey()
				{
					const FString Key = TEXT("HashToFilenames");
					return Key;
				}

				const TMap<FString, FString>& GetHashToFilenames() const
				{
					return HashToFilenames;
				}
			private:
				TMap<FString, FString> HashToFilenames;
			};

		protected:
			FString QueriesJsonString;
		};
	} //ns Interchange
}//ns UE
