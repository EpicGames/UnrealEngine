// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Thread.h"
#include "InterchangeCommands.h"
#include "InterchangeDispatcherNetworking.h"
#include "InterchangeDispatcherTask.h"

#define UE_API INTERCHANGEDISPATCHER_API

namespace UE
{
	namespace Interchange
	{
		namespace Dispatcher
		{
			class FTaskProcessCommand;
		}

		class FInterchangeDispatcher;

		enum class EWorkerState : uint8
		{
			Uninitialized = 0,
			Processing    = 1 << 0,   // send task and receive result
			Idle          = 1 << 1,   // waiting for a new task
			Closing       = 1 << 2,   // in the process of terminating
			Terminated    = 1 << 3,   // worker does not run
		};
		ENUM_CLASS_FLAGS(EWorkerState);

		//Handle a Worker by socket communication
		class FInterchangeWorkerHandler
		{
			enum class EWorkerErrorState
			{
				Ok,
				ConnectionFailed_NotBound,
				ConnectionFailed_NoClient,
				ConnectionLost,
				ConnectionLost_SendFailed,
				WorkerProcess_CantCreate,
				WorkerProcess_Lost,
				WorkerProcess_Crashed,
			};

		public:
			UE_API FInterchangeWorkerHandler(FInterchangeDispatcher& InDispatcher, FString& InResultFolder);
			UE_API ~FInterchangeWorkerHandler();

			UE_API void Run();
			UE_API bool IsAlive() const;
			UE_API void Stop();
			UE_API void StopBlocking();

			bool IsPingCommandReceived() { return bPingCommandReceived; }

			FSimpleMulticastDelegate OnWorkerHandlerExitLoop;
		protected:
			UE_API void ProcessCommand(ICommand& Command);
		private:
			UE_API void RunInternal();
			UE_API void StartWorkerProcess();
			UE_API void ValidateConnection();

			UE_API void ProcessCommand(FPingCommand& PingCommand);
			UE_API void ProcessCommand(FErrorCommand& ErrorCommand);
			UE_API void ProcessCommand(FCompletedTaskCommand& RunTaskCommand);
			UE_API void ProcessCommand(FCompletedQueryTaskProgressCommand& CompletedQueryTaskProgressCommand);
			UE_API const TCHAR* EWorkerErrorStateAsString(EWorkerErrorState e);

			UE_API void KillAllCurrentTasks();

		private:
			FInterchangeDispatcher& Dispatcher;

			// Send and receive commands
			FNetworkServerNode NetworkInterface;
			FCommandQueue CommandIO;
			FThread IOThread;
			FString ThreadName;

			// External process
			FProcHandle WorkerHandle;
			TAtomic<EWorkerState> WorkerState;
			EWorkerErrorState ErrorState;

			// self
			FString ResultFolder;
			FCriticalSection CurrentTasksLock;
			TArray<int32> CurrentTasks;
			bool bShouldTerminate;
			double LastProgressMessageTime;

			//When the worker start, it send a ping command. This flag is turn on when we receive the ping command
			bool bPingCommandReceived = false;

			friend Dispatcher::FTaskProcessCommand;
		};
	} //ns Interchange
}//ns UE

#undef UE_API
