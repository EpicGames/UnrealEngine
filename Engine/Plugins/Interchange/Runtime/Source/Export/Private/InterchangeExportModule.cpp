// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeExportModule.h"

#include "CoreMinimal.h"
#include "Engine/Engine.h"
#include "InterchangeManager.h"
#include "InterchangeTextureWriter.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"


class FInterchangeExportModule : public IInterchangeExportModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FInterchangeExportModule, InterchangeExport)



void FInterchangeExportModule::StartupModule()
{
	auto RegisterItems = []()
	{
		UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
		//Register the Writers
		InterchangeManager.RegisterWriter(UInterchangeTextureWriter::StaticClass());
	};

	if (GEngine)
	{
		RegisterItems();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterItems);
	}
}


void FInterchangeExportModule::ShutdownModule()
{
	
}



