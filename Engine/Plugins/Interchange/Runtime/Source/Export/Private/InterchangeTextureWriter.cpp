// Copyright Epic Games, Inc. All Rights Reserved. 
#include "InterchangeTextureWriter.h"

#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeTextureWriter)

bool UInterchangeTextureWriter::Export(UInterchangeBaseNodeContainer* BaseNodeContainer) const
{
	return false;
}

