// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeWriterBase.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeTextureWriter.generated.h"

#define UE_API INTERCHANGEEXPORT_API

UCLASS(MinimalAPI, BlueprintType, Experimental)
class UInterchangeTextureWriter : public UInterchangeWriterBase
{
	GENERATED_BODY()
public:
	
	/**
	 * Export all nodes of type FTextureNode hold by the BaseNodeContainer
	 * @param BaseNodeContainer - Contain nodes describing what to export
	 * @return true if the writer can export the nodes, false otherwise.
	 */
	UE_API virtual bool Export(UInterchangeBaseNodeContainer* BaseNodeContainer) const override;
};


#undef UE_API
