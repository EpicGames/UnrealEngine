// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class InterchangeFactoryNodes : ModuleRules
	{
		public InterchangeFactoryNodes(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"CinematicCamera",
					"Core",
					"CoreUObject",
					"Engine",
					"InterchangeCommon",
					"InterchangeCore",
					"InterchangeEngine",
					"InterchangeNodes",
					"TextureUtilitiesCommon",
					"VariantManagerContent",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"HairStrandsCore",
					"GeometryCache",
					"LevelSequence",
				}
			);
		}
	}
}
