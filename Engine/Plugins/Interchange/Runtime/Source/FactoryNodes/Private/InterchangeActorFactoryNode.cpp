// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeActorFactoryNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeActorFactoryNode)

#if WITH_ENGINE
#include "Components/SceneComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "GameFramework/Actor.h"
#endif

UClass* UInterchangeActorFactoryNode::GetObjectClass() const
{
#if WITH_ENGINE
	FString ActorClassName;
	if (GetCustomActorClassName(ActorClassName))
	{
		UClass* ActorClass = FindObject<UClass>(nullptr, *ActorClassName);
		if (ActorClass->IsChildOf<AActor>())
		{
			return ActorClass;
		}
	}

	return AActor::StaticClass();
#else
	return nullptr;
#endif
}

bool UInterchangeActorFactoryNode::GetCustomGlobalTransform(FTransform& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GlobalTransform, FTransform);
}

bool UInterchangeActorFactoryNode::SetCustomGlobalTransform(const FTransform& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeActorFactoryNode, GlobalTransform, FTransform, USceneComponent);
}

bool UInterchangeActorFactoryNode::GetCustomLocalTransform(FTransform& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(LocalTransform, FTransform);
}

bool UInterchangeActorFactoryNode::SetCustomLocalTransform(const FTransform& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeActorFactoryNode, LocalTransform, FTransform, USceneComponent);
}

bool UInterchangeActorFactoryNode::GetCustomComponentVisibility(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ComponentVisibility, bool);
}

bool UInterchangeActorFactoryNode::SetCustomComponentVisibility(bool AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeActorFactoryNode, ComponentVisibility, bool, USceneComponent);
}

bool UInterchangeActorFactoryNode::GetCustomActorVisibility(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ActorVisibility, bool);
}

bool UInterchangeActorFactoryNode::SetCustomActorVisibility(bool AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeActorFactoryNode, ActorVisibility, bool, USceneComponent);
}

bool UInterchangeActorFactoryNode::GetCustomActorClassName(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ActorClassName, FString);
}

bool UInterchangeActorFactoryNode::SetCustomActorClassName(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ActorClassName, FString);
}

bool UInterchangeActorFactoryNode::GetCustomMobility(uint8& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Mobility, uint8);
}

bool UInterchangeActorFactoryNode::SetCustomMobility(const uint8& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Mobility, uint8);
}

bool UInterchangeActorFactoryNode::ApplyCustomGlobalTransformToAsset(UObject* Asset) const
{
	FTransform GlobalTransform;
	if (GetCustomGlobalTransform(GlobalTransform))
	{
		if (USceneComponent* Component = Cast<USceneComponent>(Asset))
		{
			Component->SetWorldTransform(GlobalTransform);
			return true;
		}
	}

	return false;
}

bool UInterchangeActorFactoryNode::FillCustomGlobalTransformFromAsset(UObject* Asset)
{
	if (const USceneComponent* Component = Cast<USceneComponent>(Asset))
	{
		FTransform GlobalTransform = Component->GetComponentToWorld();
		return SetCustomGlobalTransform(GlobalTransform, false);
	}

	return false;
}

bool UInterchangeActorFactoryNode::ApplyCustomLocalTransformToAsset(UObject* Asset) const
{
	FTransform LocalTransform;
	if (GetCustomLocalTransform(LocalTransform))
	{
		if (USceneComponent* Component = Cast<USceneComponent>(Asset))
		{
			Component->SetRelativeTransform(LocalTransform);
			return true;
		}
	}

	return false;
}

bool UInterchangeActorFactoryNode::FillCustomLocalTransformFromAsset(UObject* Asset)
{
	if (const USceneComponent* Component = Cast<USceneComponent>(Asset))
	{
		FTransform LocalTransform = Component->GetRelativeTransform();
		return this->SetCustomLocalTransform(LocalTransform, false);
	}

	return false;
}

bool UInterchangeActorFactoryNode::ApplyCustomComponentVisibilityToAsset(UObject* Asset) const
{
	bool bVisible = true;
	if (GetCustomComponentVisibility(bVisible))
	{
		if (USceneComponent* Component = Cast<USceneComponent>(Asset))
		{
			Component->SetVisibility(bVisible);
			return true;
		}
	}

	return false;
}

bool UInterchangeActorFactoryNode::FillCustomComponentVisibilityFromAsset(UObject* Asset)
{
	if (const USceneComponent* Component = Cast<USceneComponent>(Asset))
	{
		const bool bVisible = Component->GetVisibleFlag();
		const bool bAddApplyDelegate = false;
		return SetCustomComponentVisibility(bVisible, bAddApplyDelegate);
	}

	return false;
}

bool UInterchangeActorFactoryNode::ApplyCustomActorVisibilityToAsset(UObject* Asset) const
{
	bool bVisible = true;
	if (GetCustomActorVisibility(bVisible))
	{
		AActor* Actor = nullptr;
		if (USceneComponent* Component = Cast<USceneComponent>(Asset))
		{
			Actor = Component->GetOwner();
		}
		else
		{
			Actor = Cast<AActor>(Asset);
		}

		if (Actor)
		{
			Actor->SetActorHiddenInGame(!bVisible);
			return true;
		}
	}

	return false;
}

bool UInterchangeActorFactoryNode::FillCustomActorVisibilityFromAsset(UObject* Asset)
{
	AActor* Actor = nullptr;
	if (USceneComponent* Component = Cast<USceneComponent>(Asset))
	{
		Actor = Component->GetOwner();
	}
	else
	{
		Actor = Cast<AActor>(Asset);
	}

	if (Actor)
	{
#if WITH_EDITOR
		const bool bVisible = !Actor->IsHiddenEd() && !Actor->IsHidden();
#else
		const bool bVisible = !Actor->IsHidden();
#endif	  // WITH_EDITOR

		const bool bAddApplyDelegate = false;
		return SetCustomActorVisibility(bVisible, bAddApplyDelegate);
	}

	return false;
}

void UInterchangeActorFactoryNode::CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object)
{
	Super::CopyWithObject(SourceNode, Object);

	if (const UInterchangeActorFactoryNode* ActorFactoryNode = Cast<UInterchangeActorFactoryNode>(SourceNode))
	{
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangeActorFactoryNode, GlobalTransform, FTransform, USceneComponent::StaticClass())
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangeActorFactoryNode, LocalTransform, FTransform, USceneComponent::StaticClass())
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangeActorFactoryNode, ComponentVisibility, bool, USceneComponent::StaticClass())
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangeActorFactoryNode, ActorVisibility, bool, USceneComponent::StaticClass())
	}
}

UInterchangeActorFactoryNode::UInterchangeActorFactoryNode()
{
	LayerNames.Initialize(Attributes.ToSharedRef(), TEXT("__LayerNames__"));
	Tags.Initialize(Attributes.ToSharedRef(), TEXT("__Tags__"));
	ComponentUids.Initialize(Attributes.ToSharedRef(), TEXT("__ComponentUids__"));
}

void UInterchangeActorFactoryNode::GetLayerNames(TArray<FString>& OutLayerNames) const
{
	LayerNames.GetItems(OutLayerNames);
}

bool UInterchangeActorFactoryNode::AddLayerName(const FString& LayerName)
{
	return LayerNames.AddItem(LayerName);
}

bool UInterchangeActorFactoryNode::AddLayerNames(const TArray<FString>& InLayerNames)
{
	bool bSuccess = true;
	for (const FString& LayerName : InLayerNames)
	{
		bSuccess &= LayerNames.AddItem(LayerName);
	}

	return bSuccess;
}

bool UInterchangeActorFactoryNode::RemoveLayerName(const FString& LayerName)
{
	return LayerNames.RemoveItem(LayerName);
}

void UInterchangeActorFactoryNode::GetTags(TArray<FString>& OutTags) const
{
	Tags.GetItems(OutTags);
}

bool UInterchangeActorFactoryNode::AddTag(const FString& Tag)
{
	return Tags.AddItem(Tag);
}

bool UInterchangeActorFactoryNode::AddTags(const TArray<FString>& InTags)
{
	bool bSuccess = true;
	for (const FString& Tag : InTags)
	{
		bSuccess &= Tags.AddItem(Tag);
	}

	return bSuccess;
}

bool UInterchangeActorFactoryNode::RemoveTag(const FString& Tag)
{
	return Tags.RemoveItem(Tag);
}

void UInterchangeActorFactoryNode::OnRestoreAllCustomAttributeDelegates()
{
	REFILL_CUSTOM_ATTRIBUTE_APPLY_DELEGATE(GlobalTransform, FTransform)
	REFILL_CUSTOM_ATTRIBUTE_APPLY_DELEGATE(LocalTransform, FTransform)
	REFILL_CUSTOM_ATTRIBUTE_APPLY_DELEGATE(ComponentVisibility, bool)
	REFILL_CUSTOM_ATTRIBUTE_APPLY_DELEGATE(ActorVisibility, bool)
}

bool UInterchangeActorFactoryNode::AddComponentUid(const FString& ComponentUid)
{
	return ComponentUids.AddItem(ComponentUid);
}

void UInterchangeActorFactoryNode::GetComponentUids(TArray<FString>& OutComponentUids) const
{
	ComponentUids.GetItems(OutComponentUids);
}
