// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeAnimSequenceFactoryNode.h"
#include "Animation/AnimSequence.h"
#include "InterchangeAnimationTrackSetNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeAnimSequenceFactoryNode)

namespace UE::Interchange::Animation
{
	FFrameRate ConvertSampleRatetoFrameRate(double SampleRate)
	{
		double IntegralPart = FMath::FloorToDouble(SampleRate);
		double FractionalPart = SampleRate - IntegralPart;
		const int32 Tolerance = 1000000;
		double Divisor = static_cast<double>(FMath::GreatestCommonDivisor(FMath::RoundToInt(FractionalPart * Tolerance), Tolerance));
		int32 Denominator = static_cast<int32>(Tolerance / Divisor);
		int32 Numerator = static_cast<int32>(IntegralPart * Denominator + FMath::RoundToDouble(FractionalPart * Tolerance) / Divisor);
		check(Denominator != 0);
		return FFrameRate(Numerator, Denominator);
	}
}

namespace UE::Interchange
{
	const FAttributeKey& FAnimSequenceNodeStaticData::GetAnimatedMorphTargetDependenciesKey()
	{
		static FAttributeKey AttributeKey(TEXT("__AnimatedMeshMorphTargetDependencies__"));
		return AttributeKey;
	}
	
	const FAttributeKey& FAnimSequenceNodeStaticData::GetAnimatedAttributeCurveNamesKey()
	{
		static FAttributeKey AttributeKey(TEXT("__AnimatedAttributeCurveNames__"));
		return AttributeKey;
	}
	
	const FAttributeKey& FAnimSequenceNodeStaticData::GetAnimatedAttributeStepCurveNamesKey()
	{
		static FAttributeKey AttributeKey(TEXT("__AnimatedAttributeStepCurveNames__"));
		return AttributeKey;
	}

	const FAttributeKey& FAnimSequenceNodeStaticData::GetAnimatedMaterialCurveSuffixesKey()
	{
		static FAttributeKey AttributeKey(TEXT("__AnimatedMaterialCurveSuffixes__"));
		return AttributeKey;
	}

	const FAttributeKey& FAnimSequenceNodeStaticData::GetSceneNodeAnimationPayloadKeyUidMapKey()
	{
		static FAttributeKey AttributeKey(TEXT("__SceneNodeAnimationPayloadKeyUidMap__"));
		return AttributeKey;
	}

	const FAttributeKey& FAnimSequenceNodeStaticData::GetSceneNodeAnimationPayloadKeyTypeMapKey()
	{
		static FAttributeKey AttributeKey(TEXT("__SceneNodeAnimationPayloadKeyTypeMap__"));
		return AttributeKey;
	}

	const FAttributeKey& FAnimSequenceNodeStaticData::GetMorphTargetNodePayloadKeyUidMapKey()
	{
		static FAttributeKey AttributeKey(TEXT("__MorphTargetNodePayloadKeyUidMap__"));
		return AttributeKey;
	}

	const FAttributeKey& FAnimSequenceNodeStaticData::GetMorphTargetNodePayloadKeyTypeMapKey()
	{
		static FAttributeKey AttributeKey(TEXT("__MorphTargetNodePayloadKeyTypeMap__"));
		return AttributeKey;
	}
}//ns UE::Interchange


UInterchangeAnimSequenceFactoryNode::UInterchangeAnimSequenceFactoryNode()
{
	AnimatedMorphTargetDependencies.Initialize(Attributes, UE::Interchange::FAnimSequenceNodeStaticData::GetAnimatedMorphTargetDependenciesKey().ToString());
	AnimatedAttributeCurveNames.Initialize(Attributes, UE::Interchange::FAnimSequenceNodeStaticData::GetAnimatedAttributeCurveNamesKey().ToString());
	AnimatedAttributeStepCurveNames.Initialize(Attributes, UE::Interchange::FAnimSequenceNodeStaticData::GetAnimatedAttributeStepCurveNamesKey().ToString());
	AnimatedMaterialCurveSuffixes.Initialize(Attributes, UE::Interchange::FAnimSequenceNodeStaticData::GetAnimatedMaterialCurveSuffixesKey().ToString());

	SceneNodeAnimationPayLoadKeyUidMap.Initialize(Attributes.ToSharedRef(), UE::Interchange::FAnimSequenceNodeStaticData::GetSceneNodeAnimationPayloadKeyUidMapKey().ToString());
	SceneNodeAnimationPayLoadKeyTypeMap.Initialize(Attributes.ToSharedRef(), UE::Interchange::FAnimSequenceNodeStaticData::GetSceneNodeAnimationPayloadKeyTypeMapKey().ToString());

	MorphTargetNodePayloadKeyUidMap.Initialize(Attributes.ToSharedRef(), UE::Interchange::FAnimSequenceNodeStaticData::GetMorphTargetNodePayloadKeyUidMapKey().ToString());
	MorphTargetNodePayloadKeyTypeMap.Initialize(Attributes.ToSharedRef(), UE::Interchange::FAnimSequenceNodeStaticData::GetMorphTargetNodePayloadKeyTypeMapKey().ToString());
}

void UInterchangeAnimSequenceFactoryNode::InitializeAnimSequenceNode(const FString& UniqueID, const FString& DisplayLabel, UInterchangeBaseNodeContainer* NodeContainer)
{
	NodeContainer->SetupNode(this, UniqueID, DisplayLabel, EInterchangeNodeContainerType::FactoryData);
}

FString UInterchangeAnimSequenceFactoryNode::GetTypeName() const
{
	const FString TypeName = TEXT("AnimSequenceNode");
	return TypeName;
}

#if WITH_EDITOR

FString UInterchangeAnimSequenceFactoryNode::GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	FString KeyDisplayName = NodeAttributeKey.ToString();
	if (NodeAttributeKey == Macro_CustomSkeletonFactoryNodeUidKey)
	{
		KeyDisplayName = TEXT("Skeleton Uid");
	}
	else if (NodeAttributeKey == Macro_CustomSkeletonSoftObjectPathKey)
	{
		KeyDisplayName = TEXT("Specified Existing Skeleton");
	}
	else if (NodeAttributeKey == UE::Interchange::FAnimSequenceNodeStaticData::GetAnimatedMorphTargetDependenciesKey())
	{
		KeyDisplayName = TEXT("Animated Morph Targets Count");
		return KeyDisplayName;
	}
	else if (KeyDisplayName.StartsWith(UE::Interchange::FAnimSequenceNodeStaticData::GetAnimatedMorphTargetDependenciesKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Animated Morph Targets Index "), KeyDisplayName);
	}
	else if (NodeAttributeKey == UE::Interchange::FAnimSequenceNodeStaticData::GetAnimatedAttributeCurveNamesKey())
	{
		KeyDisplayName = TEXT("Animated Attribute Curvse Count");
		return KeyDisplayName;
	}
	else if (KeyDisplayName.StartsWith(UE::Interchange::FAnimSequenceNodeStaticData::GetAnimatedAttributeCurveNamesKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Animated Attribute Curves Index "), KeyDisplayName);
	}
	else if (NodeAttributeKey == UE::Interchange::FAnimSequenceNodeStaticData::GetAnimatedAttributeStepCurveNamesKey())
	{
		KeyDisplayName = TEXT("Animated Attribute Step Curves Count");
		return KeyDisplayName;
	}
	else if (KeyDisplayName.StartsWith(UE::Interchange::FAnimSequenceNodeStaticData::GetAnimatedAttributeStepCurveNamesKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Animated Attribute Step Curves Index "), KeyDisplayName);
	}
	else if (NodeAttributeKey == UE::Interchange::FAnimSequenceNodeStaticData::GetAnimatedMaterialCurveSuffixesKey())
	{
		KeyDisplayName = TEXT("Animated Material Curve suffixes Count");
		return KeyDisplayName;
	}
	else if (KeyDisplayName.StartsWith(UE::Interchange::FAnimSequenceNodeStaticData::GetAnimatedMaterialCurveSuffixesKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Animated Material Curve Suffixes Index "), KeyDisplayName);
	}
	else if (NodeAttributeKey == UE::Interchange::FAnimSequenceNodeStaticData::GetSceneNodeAnimationPayloadKeyUidMapKey())
	{
		KeyDisplayName = TEXT("Scene Node Animation Payload Key Uid Count");
		return KeyDisplayName;
	}
	else if (KeyDisplayName.StartsWith(UE::Interchange::FAnimSequenceNodeStaticData::GetSceneNodeAnimationPayloadKeyUidMapKey().ToString()))
	{
		FString MapKeyIndex = UE::Interchange::FAnimSequenceNodeStaticData::GetSceneNodeAnimationPayloadKeyUidMapKey().ToString() + TEXT("_KeyIndex_");
		FString MapKey = UE::Interchange::FAnimSequenceNodeStaticData::GetSceneNodeAnimationPayloadKeyUidMapKey().ToString() + TEXT("_Key_");

		int32 IndexPosition = 0;
		if (KeyDisplayName.StartsWith(MapKeyIndex))
		{
			KeyDisplayName = TEXT("Scene Node Animation Payload Key Uid Key ");
			IndexPosition = KeyDisplayName.Find(MapKeyIndex) + MapKeyIndex.Len();
		}
		else if (KeyDisplayName.StartsWith(MapKey))
		{
			KeyDisplayName = TEXT("Scene Node Animation Payload Key Uid Value ");
			IndexPosition = KeyDisplayName.Find(MapKey) + MapKey.Len();
		}

		if (IndexPosition < KeyDisplayName.Len())
		{
			KeyDisplayName += KeyDisplayName.RightChop(IndexPosition);
		}
		return KeyDisplayName;
	}
	else if (NodeAttributeKey == UE::Interchange::FAnimSequenceNodeStaticData::GetSceneNodeAnimationPayloadKeyTypeMapKey())
	{
		KeyDisplayName = TEXT("Scene Node Animation Payload Key Type Count");
		return KeyDisplayName;
	}
	else if (KeyDisplayName.StartsWith(UE::Interchange::FAnimSequenceNodeStaticData::GetSceneNodeAnimationPayloadKeyTypeMapKey().ToString()))
	{
		FString MapKeyIndex = UE::Interchange::FAnimSequenceNodeStaticData::GetSceneNodeAnimationPayloadKeyTypeMapKey().ToString() + TEXT("_KeyIndex_");
		FString MapKey = UE::Interchange::FAnimSequenceNodeStaticData::GetSceneNodeAnimationPayloadKeyTypeMapKey().ToString() + TEXT("_Key_");

		int32 IndexPosition = 0;
		if (KeyDisplayName.StartsWith(MapKeyIndex))
		{
			KeyDisplayName = TEXT("Scene Node Animation Payload Key Type Key ");
			IndexPosition = KeyDisplayName.Find(MapKeyIndex) + MapKeyIndex.Len();
		}
		else if (KeyDisplayName.StartsWith(MapKey))
		{
			KeyDisplayName = TEXT("Scene Node Animation Payload Key Type Value ");
			IndexPosition = KeyDisplayName.Find(MapKey) + MapKey.Len();
		}

		if (IndexPosition < KeyDisplayName.Len())
		{
			KeyDisplayName += KeyDisplayName.RightChop(IndexPosition);
		}
		return KeyDisplayName;
	}
	else if (NodeAttributeKey == UE::Interchange::FAnimSequenceNodeStaticData::GetMorphTargetNodePayloadKeyUidMapKey())
	{
		KeyDisplayName = TEXT("Morph Target Node Payload Key Uid Count");
		return KeyDisplayName;
	}
	else if (KeyDisplayName.StartsWith(UE::Interchange::FAnimSequenceNodeStaticData::GetMorphTargetNodePayloadKeyUidMapKey().ToString()))
	{
		FString MapKeyIndex = UE::Interchange::FAnimSequenceNodeStaticData::GetMorphTargetNodePayloadKeyUidMapKey().ToString() + TEXT("_KeyIndex_");
		FString MapKey = UE::Interchange::FAnimSequenceNodeStaticData::GetMorphTargetNodePayloadKeyUidMapKey().ToString() + TEXT("_Key_");

		int32 IndexPosition = 0;
		if (KeyDisplayName.StartsWith(MapKeyIndex))
		{
			KeyDisplayName = TEXT("Morph Target Node Payload Key Uid Key ");
			IndexPosition = KeyDisplayName.Find(MapKeyIndex) + MapKeyIndex.Len();
		}
		else if (KeyDisplayName.StartsWith(MapKey))
		{
			KeyDisplayName = TEXT("Morph Target Node Payload Key Uid Value ");
			IndexPosition = KeyDisplayName.Find(MapKey) + MapKey.Len();
		}

		if (IndexPosition < KeyDisplayName.Len())
		{
			KeyDisplayName += KeyDisplayName.RightChop(IndexPosition);
		}
		return KeyDisplayName;
	}
	else if (NodeAttributeKey == UE::Interchange::FAnimSequenceNodeStaticData::GetMorphTargetNodePayloadKeyTypeMapKey())
	{
		KeyDisplayName = TEXT("Morph Target Node Payload Key Type Count");
		return KeyDisplayName;
	}
	else if (KeyDisplayName.StartsWith(UE::Interchange::FAnimSequenceNodeStaticData::GetMorphTargetNodePayloadKeyTypeMapKey().ToString()))
	{
		FString MapKeyIndex = UE::Interchange::FAnimSequenceNodeStaticData::GetMorphTargetNodePayloadKeyTypeMapKey().ToString() + TEXT("_KeyIndex_");
		FString MapKey = UE::Interchange::FAnimSequenceNodeStaticData::GetMorphTargetNodePayloadKeyTypeMapKey().ToString() + TEXT("_Key_");

		int32 IndexPosition = 0;
		if (KeyDisplayName.StartsWith(MapKeyIndex))
		{
			KeyDisplayName = TEXT("Morph Target Node Payload Key Type Key ");
			IndexPosition = KeyDisplayName.Find(MapKeyIndex) + MapKeyIndex.Len();
		}
		else if (KeyDisplayName.StartsWith(MapKey))
		{
			KeyDisplayName = TEXT("Morph Target Node Payload Key Type Value ");
			IndexPosition = KeyDisplayName.Find(MapKey) + MapKey.Len();
		}

		if (IndexPosition < KeyDisplayName.Len())
		{
			KeyDisplayName += KeyDisplayName.RightChop(IndexPosition);
		}
		return KeyDisplayName;
	}
	else
	{
		KeyDisplayName = Super::GetKeyDisplayName(NodeAttributeKey);
	}
	return KeyDisplayName;
}

FString UInterchangeAnimSequenceFactoryNode::GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	const FString NodeAttributeKeyString = NodeAttributeKey.ToString();
	if (NodeAttributeKeyString.StartsWith(UE::Interchange::FAnimSequenceNodeStaticData::GetAnimatedMorphTargetDependenciesKey().ToString()))
	{
		return TEXT("Animated Morph Target Dependencies");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FAnimSequenceNodeStaticData::GetAnimatedAttributeCurveNamesKey().ToString()))
	{
		return TEXT("Animated Attribute Curve Names");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FAnimSequenceNodeStaticData::GetAnimatedAttributeStepCurveNamesKey().ToString()))
	{
		return TEXT("Animated Attribute Step Curve Names");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FAnimSequenceNodeStaticData::GetAnimatedMaterialCurveSuffixesKey().ToString()))
	{
		return TEXT("Animated Material Curve Suffixes");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FAnimSequenceNodeStaticData::GetSceneNodeAnimationPayloadKeyUidMapKey().ToString())
		|| NodeAttributeKeyString.StartsWith(UE::Interchange::FAnimSequenceNodeStaticData::GetSceneNodeAnimationPayloadKeyTypeMapKey().ToString()))
	{
		return TEXT("Scene Node Animation Payload Keys");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FAnimSequenceNodeStaticData::GetMorphTargetNodePayloadKeyUidMapKey().ToString())
		|| NodeAttributeKeyString.StartsWith(UE::Interchange::FAnimSequenceNodeStaticData::GetMorphTargetNodePayloadKeyTypeMapKey().ToString()))
	{
		return TEXT("Morph Target Node Payload Keys");
	}
	else
	{
		return Super::GetAttributeCategory(NodeAttributeKey);
	}
}

bool UInterchangeAnimSequenceFactoryNode::ShouldHideAttribute(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	if (UserInterfaceContext == EInterchangeNodeUserInterfaceContext::Preview)
	{
		if (NodeAttributeKey == Macro_CustomSkeletonFactoryNodeUidKey)
		{
			return true;
		}
	}

	return Super::ShouldHideAttribute(NodeAttributeKey);
}

#endif //WITH_EDITOR

UClass* UInterchangeAnimSequenceFactoryNode::GetObjectClass() const
{
	return UAnimSequence::StaticClass();
}

bool UInterchangeAnimSequenceFactoryNode::GetCustomSkeletonFactoryNodeUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SkeletonFactoryNodeUid, FString);
}

bool UInterchangeAnimSequenceFactoryNode::SetCustomSkeletonFactoryNodeUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SkeletonFactoryNodeUid, FString);
}

bool UInterchangeAnimSequenceFactoryNode::GetCustomImportBoneTracks(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ImportBoneTracks, bool);
}

bool UInterchangeAnimSequenceFactoryNode::SetCustomImportBoneTracks(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ImportBoneTracks, bool);
}

bool UInterchangeAnimSequenceFactoryNode::GetCustomImportBoneTracksSampleRate(double& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ImportBoneTracksSampleRate, double);
}

bool UInterchangeAnimSequenceFactoryNode::SetCustomImportBoneTracksSampleRate(const double& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ImportBoneTracksSampleRate, double);
}

bool UInterchangeAnimSequenceFactoryNode::GetCustomImportBoneTracksRangeStart(double& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ImportBoneTracksRangeStart, double);
}

bool UInterchangeAnimSequenceFactoryNode::SetCustomImportBoneTracksRangeStart(const double& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ImportBoneTracksRangeStart, double);
}

bool UInterchangeAnimSequenceFactoryNode::GetCustomImportBoneTracksRangeStop(double& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ImportBoneTracksRangeStop, double);
}

bool UInterchangeAnimSequenceFactoryNode::SetCustomImportBoneTracksRangeStop(const double& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ImportBoneTracksRangeStop, double);
}

bool UInterchangeAnimSequenceFactoryNode::GetCustomImportAttributeCurves(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ImportAttributeCurves, bool);
}

bool UInterchangeAnimSequenceFactoryNode::SetCustomImportAttributeCurves(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ImportAttributeCurves, bool);
}

bool UInterchangeAnimSequenceFactoryNode::GetCustomDoNotImportCurveWithZero(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DoNotImportCurveWithZero, bool);
}

bool UInterchangeAnimSequenceFactoryNode::SetCustomAddCurveMetadataToSkeleton(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AddCurveMetadataToSkeleton, bool);
}

bool UInterchangeAnimSequenceFactoryNode::GetCustomAddCurveMetadataToSkeleton(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AddCurveMetadataToSkeleton, bool);
}

bool UInterchangeAnimSequenceFactoryNode::SetCustomDoNotImportCurveWithZero(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DoNotImportCurveWithZero, bool);
}

bool UInterchangeAnimSequenceFactoryNode::GetCustomRemoveCurveRedundantKeys(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(RemoveCurveRedundantKeys, bool);
}

bool UInterchangeAnimSequenceFactoryNode::SetCustomRemoveCurveRedundantKeys(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(RemoveCurveRedundantKeys, bool);
}

bool UInterchangeAnimSequenceFactoryNode::GetCustomDeleteExistingMorphTargetCurves(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DeleteExistingMorphTargetCurves, bool);
}

bool UInterchangeAnimSequenceFactoryNode::SetCustomDeleteExistingMorphTargetCurves(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DeleteExistingMorphTargetCurves, bool);
}

bool UInterchangeAnimSequenceFactoryNode::GetCustomSkeletonSoftObjectPath(FSoftObjectPath& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SkeletonSoftObjectPath, FSoftObjectPath)
}

bool UInterchangeAnimSequenceFactoryNode::SetCustomSkeletonSoftObjectPath(const FSoftObjectPath& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SkeletonSoftObjectPath, FSoftObjectPath)
}


int32 UInterchangeAnimSequenceFactoryNode::GetAnimatedAttributeCurveNamesCount() const
{
	return AnimatedAttributeCurveNames.GetCount();
}

void UInterchangeAnimSequenceFactoryNode::GetAnimatedAttributeCurveNames(TArray<FString>& OutAttributeCurveNames) const
{
	AnimatedAttributeCurveNames.GetItems(OutAttributeCurveNames);
}

void UInterchangeAnimSequenceFactoryNode::GetAnimatedAttributeCurveName(const int32 Index, FString& OutAttributeCurveName) const
{
	AnimatedAttributeCurveNames.GetItem(Index, OutAttributeCurveName);
}

bool UInterchangeAnimSequenceFactoryNode::SetAnimatedAttributeCurveName(const FString& AttributeCurveName)
{
	return AnimatedAttributeCurveNames.AddItem(AttributeCurveName);
}

bool UInterchangeAnimSequenceFactoryNode::RemoveAnimatedAttributeCurveName(const FString& AttributeCurveName)
{
	return AnimatedAttributeCurveNames.RemoveItem(AttributeCurveName);
}

bool UInterchangeAnimSequenceFactoryNode::GetCustomMaterialDriveParameterOnCustomAttribute(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(MaterialDriveParameterOnCustomAttribute, bool);
}

bool UInterchangeAnimSequenceFactoryNode::SetCustomMaterialDriveParameterOnCustomAttribute(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(MaterialDriveParameterOnCustomAttribute, bool);
}

int32 UInterchangeAnimSequenceFactoryNode::GetAnimatedMaterialCurveSuffixesCount() const
{
	return AnimatedMaterialCurveSuffixes.GetCount();
}

void UInterchangeAnimSequenceFactoryNode::GetAnimatedMaterialCurveSuffixes(TArray<FString>& OutMaterialCurveSuffixes) const
{
	AnimatedMaterialCurveSuffixes.GetItems(OutMaterialCurveSuffixes);
}

void UInterchangeAnimSequenceFactoryNode::GetAnimatedMaterialCurveSuffixe(const int32 Index, FString& OutMaterialCurveSuffixe) const
{
	AnimatedMaterialCurveSuffixes.GetItem(Index, OutMaterialCurveSuffixe);
}

bool UInterchangeAnimSequenceFactoryNode::SetAnimatedMaterialCurveSuffixe(const FString& MaterialCurveSuffixe)
{
	return AnimatedMaterialCurveSuffixes.AddItem(MaterialCurveSuffixe);
}

bool UInterchangeAnimSequenceFactoryNode::RemoveAnimatedMaterialCurveSuffixe(const FString& MaterialCurveSuffixe)
{
	return AnimatedMaterialCurveSuffixes.RemoveItem(MaterialCurveSuffixe);
}


int32 UInterchangeAnimSequenceFactoryNode::GetAnimatedAttributeStepCurveNamesCount() const
{
	return AnimatedAttributeStepCurveNames.GetCount();
}

void UInterchangeAnimSequenceFactoryNode::GetAnimatedAttributeStepCurveNames(TArray<FString>& OutAttributeStepCurveNames) const
{
	AnimatedAttributeStepCurveNames.GetItems(OutAttributeStepCurveNames);
}

void UInterchangeAnimSequenceFactoryNode::GetAnimatedAttributeStepCurveName(const int32 Index, FString& OutAttributeStepCurveName) const
{
	AnimatedAttributeStepCurveNames.GetItem(Index, OutAttributeStepCurveName);
}

bool UInterchangeAnimSequenceFactoryNode::SetAnimatedAttributeStepCurveName(const FString& AttributeStepCurveName)
{
	return AnimatedAttributeStepCurveNames.AddItem(AttributeStepCurveName);
}

bool UInterchangeAnimSequenceFactoryNode::RemoveAnimatedAttributeStepCurveName(const FString& AttributeStepCurveName)
{
	return AnimatedAttributeStepCurveNames.RemoveItem(AttributeStepCurveName);
}


bool UInterchangeAnimSequenceFactoryNode::GetCustomDeleteExistingCustomAttributeCurves(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DeleteExistingCustomAttributeCurves, bool);
}

bool UInterchangeAnimSequenceFactoryNode::SetCustomDeleteExistingCustomAttributeCurves(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DeleteExistingCustomAttributeCurves, bool);
}

bool UInterchangeAnimSequenceFactoryNode::GetCustomDeleteExistingNonCurveCustomAttributes(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DeleteExistingNonCurveCustomAttributes, bool);
}

bool UInterchangeAnimSequenceFactoryNode::SetCustomDeleteExistingNonCurveCustomAttributes(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DeleteExistingNonCurveCustomAttributes, bool);
}

void UInterchangeAnimSequenceFactoryNode::GetSceneNodeAnimationPayloadKeys(TMap<FString, FInterchangeAnimationPayLoadKey>& OutSceneNodeAnimationPayloadKeys) const
{
	TMap<FString, FString> SceneNodeToPayLoadUidMap = SceneNodeAnimationPayLoadKeyUidMap.ToMap();
	TMap<FString, uint8> SceneNodeToPayLoadTypeMap = SceneNodeAnimationPayLoadKeyTypeMap.ToMap();

	OutSceneNodeAnimationPayloadKeys.Reserve(SceneNodeToPayLoadTypeMap.Num());

	for (const TPair<FString, FString>& SceneNodePayloadUidPair : SceneNodeToPayLoadUidMap)
	{
		uint8* Value = SceneNodeToPayLoadTypeMap.Find(SceneNodePayloadUidPair.Key);
		if (Value)
		{
			OutSceneNodeAnimationPayloadKeys.Add(SceneNodePayloadUidPair.Key, FInterchangeAnimationPayLoadKey(SceneNodePayloadUidPair.Value, (EInterchangeAnimationPayLoadType)*Value));
		}
	}
}

void UInterchangeAnimSequenceFactoryNode::SetAnimationPayloadKeysForSceneNodeUids(const TMap<FString, FString>& SceneNodeAnimationPayloadKeyUids, const TMap<FString, uint8>& SceneNodeAnimationPayloadKeyTypes)
{
	for (const TPair<FString, FString>& SceneNodePayLoadUidPair : SceneNodeAnimationPayloadKeyUids)
	{
		SceneNodeAnimationPayLoadKeyUidMap.SetKeyValue(SceneNodePayLoadUidPair.Key, SceneNodePayLoadUidPair.Value);
	}

	for (const TPair<FString, uint8>& SceneNodePayLoadTypePair : SceneNodeAnimationPayloadKeyTypes)
	{
		SceneNodeAnimationPayLoadKeyTypeMap.SetKeyValue(SceneNodePayLoadTypePair.Key, SceneNodePayLoadTypePair.Value);
	}
}

void UInterchangeAnimSequenceFactoryNode::GetMorphTargetNodeAnimationPayloadKeys(TMap<FString, FInterchangeAnimationPayLoadKey>& OutMorphTargetAnimationPayloads) const
{
	TMap<FString, FString> MorphTargetToPayLoadUidMap = MorphTargetNodePayloadKeyUidMap.ToMap();
	TMap<FString, uint8> MorphTargetToPayLoadTypeMap = MorphTargetNodePayloadKeyTypeMap.ToMap();

	OutMorphTargetAnimationPayloads.Reserve(MorphTargetToPayLoadTypeMap.Num());

	for (const TPair<FString, FString>& SceneNodePayloadUidPair : MorphTargetToPayLoadUidMap)
	{
		uint8* Value = MorphTargetToPayLoadTypeMap.Find(SceneNodePayloadUidPair.Key);
		if (Value)
		{
			OutMorphTargetAnimationPayloads.Add(SceneNodePayloadUidPair.Key, FInterchangeAnimationPayLoadKey(SceneNodePayloadUidPair.Value, (EInterchangeAnimationPayLoadType)*Value));
		}
	}

}

void UInterchangeAnimSequenceFactoryNode::SetAnimationPayloadKeysForMorphTargetNodeUids(const TMap<FString, FString>& MorphTargetAnimationPayloadKeyUids, const TMap<FString, uint8>& MorphTargetAnimationPayloadKeyTypes)
{
	for (const TPair<FString, FString>& MorphTargetPayLoadUidPair : MorphTargetAnimationPayloadKeyUids)
	{
		MorphTargetNodePayloadKeyUidMap.SetKeyValue(MorphTargetPayLoadUidPair.Key, MorphTargetPayLoadUidPair.Value);
	}

	for (const TPair<FString, uint8>& MorphTargetPayLoadTypePair : MorphTargetAnimationPayloadKeyTypes)
	{
		MorphTargetNodePayloadKeyTypeMap.SetKeyValue(MorphTargetPayLoadTypePair.Key, MorphTargetPayLoadTypePair.Value);
	}
}

/************************************************************************/
/* Automation tests                                                     */
/************************************************************************/
#if WITH_EDITOR
#if WITH_DEV_AUTOMATION_TESTS

#include "Misc/AutomationTest.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FInterchangeAnimSequenceTest, "System.Runtime.Interchange.ConvertSampleRatetoFrameRate", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FInterchangeAnimSequenceTest::RunTest(const FString& Parameters)
{
	FFrameRate FrameRate;
	FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(120.0);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 120.0 to FFrameRate"), FrameRate.Numerator, 120);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 120.0 to FFrameRate"), FrameRate.Denominator, 1);

	FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(100.0);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 100.0 to FFrameRate"), FrameRate.Numerator, 100);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 100.0 to FFrameRate"), FrameRate.Denominator, 1);

	FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(60.0);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 60.0 to FFrameRate"), FrameRate.Numerator, 60);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 60.0 to FFrameRate"), FrameRate.Denominator, 1);

	FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(50.0);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 50.0 to FFrameRate"), FrameRate.Numerator, 50);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 50.0 to FFrameRate"), FrameRate.Denominator, 1);

	FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(48.0);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 48.0 to FFrameRate"), FrameRate.Numerator, 48);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 48.0 to FFrameRate"), FrameRate.Denominator, 1);

	FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(30.0);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 30.0 to FFrameRate"), FrameRate.Numerator, 30);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 30.0 to FFrameRate"), FrameRate.Denominator, 1);

	FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(29.97);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 29.97 to FFrameRate"), FrameRate.Numerator, 2997);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 29.97 to FFrameRate"), FrameRate.Denominator, 100);

	FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(25.0);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 25.0 to FFrameRate"), FrameRate.Numerator, 25);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 25.0 to FFrameRate"), FrameRate.Denominator, 1);

	FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(24.0);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 24.0 to FFrameRate"), FrameRate.Numerator, 24);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 24.0 to FFrameRate"), FrameRate.Denominator, 1);

	FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(23.976);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 23.976 to FFrameRate"), FrameRate.Numerator, 2997);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 23.976 to FFrameRate"), FrameRate.Denominator, 125);

	FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(96.0);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 96.0 to FFrameRate"), FrameRate.Numerator, 96);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 96.0 to FFrameRate"), FrameRate.Denominator, 1);

	FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(72.0);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 72.0 to FFrameRate"), FrameRate.Numerator, 72);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 72.0 to FFrameRate"), FrameRate.Denominator, 1);

	FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(59.94);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 59.94 to FFrameRate"), FrameRate.Numerator, 2997);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 59.94 to FFrameRate"), FrameRate.Denominator, 50);

	FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(119.88);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 119.88 to FFrameRate"), FrameRate.Numerator, 2997);
	TestEqual(TEXT("`ConvertSampleRateToFrameRate` Error converting 119.88 to FFrameRate"), FrameRate.Denominator, 25);

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
#endif //WITH_EDITOR

