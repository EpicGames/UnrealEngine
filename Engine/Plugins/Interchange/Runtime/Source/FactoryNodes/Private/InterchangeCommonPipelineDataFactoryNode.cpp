// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeCommonPipelineDataFactoryNode.h"

#include "InterchangeSceneNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeCommonPipelineDataFactoryNode)

namespace UE::Interchange::CommonPipelineData
{
	FString GetCommonPipelineDataUniqueID()
	{
		static FString StaticUid = TEXT("CommonPipelineDataFactoryNode");
		return StaticUid;
	}
}

UInterchangeCommonPipelineDataFactoryNode* UInterchangeCommonPipelineDataFactoryNode::FindOrCreateUniqueInstance(UInterchangeBaseNodeContainer* NodeContainer)
{
	const FString StaticUid = UE::Interchange::CommonPipelineData::GetCommonPipelineDataUniqueID();
	UInterchangeCommonPipelineDataFactoryNode* CommonPipelineDataFactoryNode = Cast<UInterchangeCommonPipelineDataFactoryNode>(NodeContainer->GetFactoryNode(StaticUid));
	if (!CommonPipelineDataFactoryNode)
	{
		CommonPipelineDataFactoryNode = NewObject<UInterchangeCommonPipelineDataFactoryNode>(NodeContainer, NAME_None);
		NodeContainer->SetupNode(CommonPipelineDataFactoryNode, StaticUid, StaticUid, EInterchangeNodeContainerType::FactoryData);
	}
	return CommonPipelineDataFactoryNode;
}
UInterchangeCommonPipelineDataFactoryNode* UInterchangeCommonPipelineDataFactoryNode::GetUniqueInstance(const UInterchangeBaseNodeContainer* NodeContainer)
{
	static FString StaticUid = UE::Interchange::CommonPipelineData::GetCommonPipelineDataUniqueID();
	return Cast<UInterchangeCommonPipelineDataFactoryNode>(NodeContainer->GetFactoryNode(StaticUid));
}

bool UInterchangeCommonPipelineDataFactoryNode::GetCustomGlobalOffsetTransform(FTransform& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GlobalOffsetTransform, FTransform);
}

UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Common Pipeline Data")
bool UInterchangeCommonPipelineDataFactoryNode::SetCustomGlobalOffsetTransform(const UInterchangeBaseNodeContainer* NodeContainer, const FTransform& AttributeValue)
{
	auto ImplementationFunction = [this, &NodeContainer, &AttributeValue]()
		{
			IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(GlobalOffsetTransform, FTransform);
		};
	if (ImplementationFunction())
	{
		//Reset all scene node container cache
		UInterchangeSceneNode::ResetAllGlobalTransformCaches(NodeContainer);
		return true;
	}
	return false;
}

bool UInterchangeCommonPipelineDataFactoryNode::GetBakeMeshes(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(BakeMeshes, bool);
}

bool UInterchangeCommonPipelineDataFactoryNode::SetBakeMeshes(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(BakeMeshes, bool);
}

bool UInterchangeCommonPipelineDataFactoryNode::GetBakePivotMeshes(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(BakePivotMeshes, bool);
}

bool UInterchangeCommonPipelineDataFactoryNode::SetBakePivotMeshes(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(BakePivotMeshes, bool);
}