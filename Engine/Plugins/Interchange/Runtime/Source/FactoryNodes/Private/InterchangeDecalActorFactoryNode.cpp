// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeDecalActorFactoryNode.h"
#include "Components/DecalComponent.h"

#if WITH_ENGINE
#include "Engine/DecalActor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeDecalActorFactoryNode)

UClass* UInterchangeDecalActorFactoryNode::GetObjectClass() const
{
#if WITH_ENGINE
	return ADecalActor::StaticClass();
#else
	return nullptr;
#endif
}

void UInterchangeDecalActorFactoryNode::CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object)
{
	Super::CopyWithObject(SourceNode, Object);

	if (const UInterchangeDecalActorFactoryNode* SourceFactoryNode = Cast<UInterchangeDecalActorFactoryNode>(SourceNode))
	{
		UClass* Class = UDecalComponent::StaticClass();
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(SourceFactoryNode, UInterchangeDecalActorFactoryNode, SortOrder, int32, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(SourceFactoryNode, UInterchangeDecalActorFactoryNode, DecalSize, FVector, Class)
	}
}

bool UInterchangeDecalActorFactoryNode::GetCustomSortOrder(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SortOrder, int32);
}

UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
bool UInterchangeDecalActorFactoryNode::SetCustomSortOrder(const int32& AttributeValue, bool bAddApplyDelegate /*= true*/)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeDecalActorFactoryNode, SortOrder, int32, UDecalComponent);
}

UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
bool UInterchangeDecalActorFactoryNode::GetCustomDecalSize(FVector& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DecalSize, FVector);
}

UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
bool UInterchangeDecalActorFactoryNode::SetCustomDecalSize(const FVector& AttributeValue, bool bAddApplyDelegate /*= true*/)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeDecalActorFactoryNode, DecalSize, FVector, UDecalComponent);
}

UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
bool UInterchangeDecalActorFactoryNode::GetCustomDecalMaterialPathName(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DecalMaterialPathName, FString);
}

UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
bool UInterchangeDecalActorFactoryNode::SetCustomDecalMaterialPathName(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DecalMaterialPathName, FString);
}
