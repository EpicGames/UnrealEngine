// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeDecalMaterialFactoryNode.h"
#include "Materials/MaterialInstanceConstant.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeDecalMaterialFactoryNode)

UClass* UInterchangeDecalMaterialFactoryNode::GetObjectClass() const
{
#if WITH_ENGINE
	return UMaterialInstanceConstant::StaticClass();
#else
	return nullptr;
#endif
}

bool UInterchangeDecalMaterialFactoryNode::GetCustomDiffuseTexturePath(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DiffuseTexturePath, FString);
}

bool UInterchangeDecalMaterialFactoryNode::SetCustomDiffuseTexturePath(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DiffuseTexturePath, FString);
}

bool UInterchangeDecalMaterialFactoryNode::GetCustomNormalTexturePath(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(NormalTexturePath, FString);
}

bool UInterchangeDecalMaterialFactoryNode::SetCustomNormalTexturePath(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(NormalTexturePath, FString);
}
