// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeFactoryNodesModule.h"

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

class FInterchangeFactoryNodesModule : public IInterchangeFactoryNodesModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FInterchangeFactoryNodesModule, InterchangeFactoryNodes)



void FInterchangeFactoryNodesModule::StartupModule()
{

}


void FInterchangeFactoryNodesModule::ShutdownModule()
{

}



