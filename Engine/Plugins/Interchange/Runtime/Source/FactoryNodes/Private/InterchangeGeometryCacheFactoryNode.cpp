// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeGeometryCacheFactoryNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "GeometryCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGeometryCacheFactoryNode)

namespace UE::Interchange
{
	const FAttributeKey& FGeometryCacheNodeStaticData::GetSceneNodeAnimationPayloadKeyUidMapKey()
	{
		static FAttributeKey AttributeKey(TEXT("__SceneNodeAnimationPayloadKeyUidMap__"));
		return AttributeKey;
	}
}

UInterchangeGeometryCacheFactoryNode::UInterchangeGeometryCacheFactoryNode()
{
	SceneNodeAnimationPayloadKeyUidMap.Initialize(Attributes.ToSharedRef(), UE::Interchange::FGeometryCacheNodeStaticData::GetSceneNodeAnimationPayloadKeyUidMapKey().ToString());
	AssetClass = nullptr;
}

void UInterchangeGeometryCacheFactoryNode::InitializeGeometryCacheNode(const FString& UniqueID, const FString& DisplayLabel, const FString& InAssetClass, UInterchangeBaseNodeContainer* NodeContainer)
{
	bIsNodeClassInitialized = false;
	NodeContainer->SetupNode(this, UniqueID, DisplayLabel, EInterchangeNodeContainerType::FactoryData);

	FString OperationName = GetTypeName() + TEXT(".SetAssetClassName");
	InterchangePrivateNodeBase::SetCustomAttribute<FString>(*Attributes, ClassNameAttributeKey, OperationName, InAssetClass);
	FillAssetClassFromAttribute();
}

FString UInterchangeGeometryCacheFactoryNode::GetTypeName() const
{
	const FString TypeName = TEXT("GeometryCacheNode");
	return TypeName;
}

UClass* UInterchangeGeometryCacheFactoryNode::GetObjectClass() const
{
	ensure(bIsNodeClassInitialized);
	return AssetClass.Get() != nullptr ? AssetClass.Get() : UGeometryCache::StaticClass();
}

void UInterchangeGeometryCacheFactoryNode::FillAssetClassFromAttribute()
{
#if WITH_ENGINE
	FString OperationName = GetTypeName() + TEXT(".GetAssetClassName");
	FString ClassName;
	InterchangePrivateNodeBase::GetCustomAttribute<FString>(*Attributes, ClassNameAttributeKey, OperationName, ClassName);
	if (ClassName.Equals(UGeometryCache::StaticClass()->GetName()))
	{
		AssetClass = UGeometryCache::StaticClass();
		bIsNodeClassInitialized = true;
	}
#endif
}

bool UInterchangeGeometryCacheFactoryNode::SetNodeClassFromClassAttribute()
{
	if (!bIsNodeClassInitialized)
	{
		FillAssetClassFromAttribute();
	}
	return bIsNodeClassInitialized;
}

bool UInterchangeGeometryCacheFactoryNode::GetCustomFlattenTracks(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(FlattenTracks, bool);
}

bool UInterchangeGeometryCacheFactoryNode::SetCustomFlattenTracks(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(FlattenTracks, bool);
}

bool UInterchangeGeometryCacheFactoryNode::GetCustomPositionPrecision(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(PositionPrecision, float);
}

bool UInterchangeGeometryCacheFactoryNode::SetCustomPositionPrecision(const float& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(PositionPrecision, float);
}

bool UInterchangeGeometryCacheFactoryNode::GetCustomNumBitsForUVs(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(NumBitsForUVs, int32);
}

bool UInterchangeGeometryCacheFactoryNode::SetCustomNumBitsForUVs(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(NumBitsForUVs, int32);
}

bool UInterchangeGeometryCacheFactoryNode::GetCustomStartFrame(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(StartFrame, int32);
}

bool UInterchangeGeometryCacheFactoryNode::SetCustomStartFrame(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(StartFrame, int32);
}

bool UInterchangeGeometryCacheFactoryNode::GetCustomEndFrame(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(EndFrame, int32);
}

bool UInterchangeGeometryCacheFactoryNode::SetCustomEndFrame(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(EndFrame, int32);
}

bool UInterchangeGeometryCacheFactoryNode::GetCustomMotionVectorsImport(EInterchangeMotionVectorsHandling& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(MotionVectors, EInterchangeMotionVectorsHandling);
}

bool UInterchangeGeometryCacheFactoryNode::SetCustomMotionVectorsImport(EInterchangeMotionVectorsHandling AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(MotionVectors, EInterchangeMotionVectorsHandling)
}

bool UInterchangeGeometryCacheFactoryNode::GetCustomApplyConstantTopologyOptimization(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ApplyConstantTopologyOptimization, bool);
}

bool UInterchangeGeometryCacheFactoryNode::SetCustomApplyConstantTopologyOptimization(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ApplyConstantTopologyOptimization, bool);
}

bool UInterchangeGeometryCacheFactoryNode::GetCustomStoreImportedVertexNumbers(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(StoreImportedVertexNumbers, bool);
}

bool UInterchangeGeometryCacheFactoryNode::SetCustomStoreImportedVertexNumbers(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(StoreImportedVertexNumbers, bool);
}

bool UInterchangeGeometryCacheFactoryNode::GetCustomOptimizeIndexBuffers(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(OptimizeIndexBuffers, bool);
}

bool UInterchangeGeometryCacheFactoryNode::SetCustomOptimizeIndexBuffers(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(OptimizeIndexBuffers, bool);
}

void UInterchangeGeometryCacheFactoryNode::GetSceneNodeAnimationPayloadKeys(TMap<FString, FString>& OutSceneNodeAnimationPayloadKeyUids) const
{
	OutSceneNodeAnimationPayloadKeyUids = SceneNodeAnimationPayloadKeyUidMap.ToMap();
}

bool UInterchangeGeometryCacheFactoryNode::SetAnimationPayloadKeyForSceneNodeUid(const FString& SceneNodeUid, const FString& InUniqueId)
{
	return SceneNodeAnimationPayloadKeyUidMap.SetKeyValue(SceneNodeUid, InUniqueId);
}
