// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeGroomBindingFactoryNode.h"

#include "GroomBindingAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGroomBindingFactoryNode)

FString UInterchangeGroomBindingFactoryNode::GetTypeName() const
{
	const FString TypeName = TEXT("GroomBindingFactoryNode");
	return TypeName;
}

UClass* UInterchangeGroomBindingFactoryNode::GetObjectClass() const
{
	return UGroomBindingAsset::StaticClass();
}

bool UInterchangeGroomBindingFactoryNode::GetNumInterpolationPoints(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(NumInterpolationPoints, int32);
}

bool UInterchangeGroomBindingFactoryNode::SetNumInterpolationPoints(int32 AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(NumInterpolationPoints, int32);
}
