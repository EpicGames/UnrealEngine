// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeGroomCacheFactoryNode.h"

#include "GroomCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGroomCacheFactoryNode)

FString UInterchangeGroomCacheFactoryNode::GetTypeName() const
{
	const FString TypeName = TEXT("GroomCacheFactoryNode");
	return TypeName;
}

UClass* UInterchangeGroomCacheFactoryNode::GetObjectClass() const
{
	return UGroomCache::StaticClass();
}

bool UInterchangeGroomCacheFactoryNode::GetCustomStartFrame(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(StartFrame, int32);
}

bool UInterchangeGroomCacheFactoryNode::SetCustomStartFrame(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(StartFrame, int32);
}

bool UInterchangeGroomCacheFactoryNode::GetCustomEndFrame(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(EndFrame, int32);
}

bool UInterchangeGroomCacheFactoryNode::SetCustomEndFrame(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(EndFrame, int32);
}

bool UInterchangeGroomCacheFactoryNode::GetCustomNumFrames(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(NumFrames, int32);
}

bool UInterchangeGroomCacheFactoryNode::SetCustomNumFrames(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(NumFrames, int32);
}

bool UInterchangeGroomCacheFactoryNode::GetCustomFrameRate(double& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(FrameRate, double);
}

bool UInterchangeGroomCacheFactoryNode::SetCustomFrameRate(const double& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(FrameRate, double);
}

bool UInterchangeGroomCacheFactoryNode::GetCustomGroomCacheAttributes(EInterchangeGroomCacheAttributes& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GroomCacheAttributes, EInterchangeGroomCacheAttributes);
}

bool UInterchangeGroomCacheFactoryNode::SetCustomGroomCacheAttributes(const EInterchangeGroomCacheAttributes& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(GroomCacheAttributes, EInterchangeGroomCacheAttributes);
}

bool UInterchangeGroomCacheFactoryNode::GetCustomGroomCacheImportType(EInterchangeGroomCacheImportType& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ImportType, EInterchangeGroomCacheImportType);
}

bool UInterchangeGroomCacheFactoryNode::SetCustomGroomCacheImportType(const EInterchangeGroomCacheImportType& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ImportType, EInterchangeGroomCacheImportType);
}

bool UInterchangeGroomCacheFactoryNode::GetCustomGroomAssetPath(FSoftObjectPath& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GroomAssetPath, FSoftObjectPath);
}

bool UInterchangeGroomCacheFactoryNode::SetCustomGroomAssetPath(const FSoftObjectPath& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(GroomAssetPath, FSoftObjectPath);
}
