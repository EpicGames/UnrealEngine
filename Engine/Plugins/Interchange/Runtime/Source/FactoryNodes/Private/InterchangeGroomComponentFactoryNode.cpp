// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeGroomComponentFactoryNode.h"

#include "GroomComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGroomComponentFactoryNode)

FString UInterchangeGroomComponentFactoryNode::GetTypeName() const
{
	return TEXT("UInterchangeGroomComponentFactoryNode");
}

UClass* UInterchangeGroomComponentFactoryNode::GetObjectClass() const
{
	return UGroomComponent::StaticClass();
}

bool UInterchangeGroomComponentFactoryNode::GetGroomDependencyUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GroomUid, FString);
}

bool UInterchangeGroomComponentFactoryNode::SetGroomDependencyUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(GroomUid, FString);
}

bool UInterchangeGroomComponentFactoryNode::GetGroomBindingDependencyUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GroomBindingUid, FString);
}

bool UInterchangeGroomComponentFactoryNode::SetGroomBindingDependencyUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(GroomBindingUid, FString);
}
