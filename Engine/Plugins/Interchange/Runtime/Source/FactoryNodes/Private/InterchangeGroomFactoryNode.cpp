// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeGroomFactoryNode.h"

#include "GroomAsset.h"
#include "GroomAssetInterpolation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGroomFactoryNode)

FString UInterchangeGroomFactoryNode::GetTypeName() const
{
	const FString TypeName = TEXT("GroomFactoryNode");
	return TypeName;
}

UClass* UInterchangeGroomFactoryNode::GetObjectClass() const
{
	return UGroomAsset::StaticClass();
}

#define GET_ATTRIBUTE(Struct, AttributeName, AttributeType) \
	static const FString AttributeName##OperationName = GetTypeName() + TEXT(".Get" #AttributeName); \
	if (!InterchangePrivateNodeBase::GetCustomAttribute<AttributeType>(*Attributes, Macro_Custom##AttributeName##Key, AttributeName##OperationName, AttributeValue.Struct.AttributeName)) return false;

#define SET_ATTRIBUTE(Struct, AttributeName, AttributeType) \
	static const FString AttributeName##OperationName = GetTypeName() + TEXT(".Set" #AttributeName); \
	if (!InterchangePrivateNodeBase::SetCustomAttribute<AttributeType>(*Attributes, Macro_Custom##AttributeName##Key, AttributeName##OperationName, AttributeValue.Struct.AttributeName)) return false;

bool UInterchangeGroomFactoryNode::GetCustomGroupInterpolationSettings(FHairGroupsInterpolation& AttributeValue) const
{
	GET_ATTRIBUTE(DecimationSettings, CurveDecimation, float);
	GET_ATTRIBUTE(DecimationSettings, VertexDecimation, float);
	GET_ATTRIBUTE(InterpolationSettings, GuideType, EGroomGuideType);
	GET_ATTRIBUTE(InterpolationSettings, HairToGuideDensity, float);
	GET_ATTRIBUTE(InterpolationSettings, RiggedGuideNumCurves, int32);
	GET_ATTRIBUTE(InterpolationSettings, RiggedGuideNumPoints, int32);
	GET_ATTRIBUTE(InterpolationSettings, InterpolationQuality, EHairInterpolationQuality);
	GET_ATTRIBUTE(InterpolationSettings, InterpolationDistance, EHairInterpolationWeight);
	GET_ATTRIBUTE(InterpolationSettings, bRandomizeGuide, bool);
	GET_ATTRIBUTE(InterpolationSettings, bUseUniqueGuide, bool);

	return true;
}

bool UInterchangeGroomFactoryNode::SetCustomGroupInterpolationSettings(const FHairGroupsInterpolation& AttributeValue)
{
	SET_ATTRIBUTE(DecimationSettings, CurveDecimation, float);
	SET_ATTRIBUTE(DecimationSettings, VertexDecimation, float);
	SET_ATTRIBUTE(InterpolationSettings, GuideType, EGroomGuideType);
	SET_ATTRIBUTE(InterpolationSettings, HairToGuideDensity, float);
	SET_ATTRIBUTE(InterpolationSettings, RiggedGuideNumCurves, int32);
	SET_ATTRIBUTE(InterpolationSettings, RiggedGuideNumPoints, int32);
	SET_ATTRIBUTE(InterpolationSettings, InterpolationQuality, EHairInterpolationQuality);
	SET_ATTRIBUTE(InterpolationSettings, InterpolationDistance, EHairInterpolationWeight);
	SET_ATTRIBUTE(InterpolationSettings, bRandomizeGuide, bool);
	SET_ATTRIBUTE(InterpolationSettings, bUseUniqueGuide, bool);

	return true;
}

#undef SET_ATTRIBUTE
#undef GET_ATTRIBUTE
