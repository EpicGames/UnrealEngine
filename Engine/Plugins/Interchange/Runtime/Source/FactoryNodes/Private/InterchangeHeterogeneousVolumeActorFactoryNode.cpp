// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeHeterogeneousVolumeActorFactoryNode.h"

#include "Components/HeterogeneousVolumeComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeHeterogeneousVolumeActorFactoryNode)

UClass* UInterchangeHeterogeneousVolumeActorFactoryNode::GetObjectClass() const
{
#if WITH_ENGINE
	return AHeterogeneousVolume::StaticClass();
#else
	return nullptr;
#endif
}

bool UInterchangeHeterogeneousVolumeActorFactoryNode::GetCustomVolumetricMaterialUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(MaterialDependency, FString)
}

bool UInterchangeHeterogeneousVolumeActorFactoryNode::SetCustomVolumetricMaterialUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(MaterialDependency, FString)
}
