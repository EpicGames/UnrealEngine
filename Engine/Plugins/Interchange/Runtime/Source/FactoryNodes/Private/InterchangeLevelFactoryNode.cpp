// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeLevelFactoryNode.h"

#include "Engine/Level.h"
#include "Engine/World.h"
#include "Math/Box.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeLevelFactoryNode)

UInterchangeLevelFactoryNode::UInterchangeLevelFactoryNode()
{
	CustomActorFactoryNodeUids.Initialize(Attributes, TEXT("ActorFactoryNodeUids"));
}

UClass* UInterchangeLevelFactoryNode::GetObjectClass() const
{
	return UWorld::StaticClass();
}

int32 UInterchangeLevelFactoryNode::GetCustomActorFactoryNodeUidCount() const
{
	return CustomActorFactoryNodeUids.GetCount();
}

void UInterchangeLevelFactoryNode::GetCustomActorFactoryNodeUids(TArray<FString>& OutActorFactoryNodeUids) const
{
	CustomActorFactoryNodeUids.GetItems(OutActorFactoryNodeUids);
}

void UInterchangeLevelFactoryNode::GetCustomActorFactoryNodeUid(const int32 Index, FString& OutActorFactoryNodeUid) const
{
	CustomActorFactoryNodeUids.GetItem(Index, OutActorFactoryNodeUid);
}

bool UInterchangeLevelFactoryNode::AddCustomActorFactoryNodeUid(const FString& ActorFactoryNodeUid)
{
	return CustomActorFactoryNodeUids.AddItem(ActorFactoryNodeUid);
}

bool UInterchangeLevelFactoryNode::RemoveCustomActorFactoryNodeUid(const FString& ActorFactoryNodeUid)
{
	return CustomActorFactoryNodeUids.RemoveItem(ActorFactoryNodeUid);
}

bool UInterchangeLevelFactoryNode::GetCustomSceneImportAssetFactoryNodeUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SceneImportAssetFactoryNodeUid, FString);
}

bool UInterchangeLevelFactoryNode::SetCustomSceneImportAssetFactoryNodeUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SceneImportAssetFactoryNodeUid, FString);
}

bool UInterchangeLevelFactoryNode::GetCustomShouldCreateLevel(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ShouldCreateLevel, bool);
}

bool UInterchangeLevelFactoryNode::SetCustomShouldCreateLevel(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ShouldCreateLevel, bool);
}

bool UInterchangeLevelFactoryNode::GetCustomCreateWorldPartitionLevel(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(CreateWorldPartitionLevel, bool);
}

bool UInterchangeLevelFactoryNode::SetCustomCreateWorldPartitionLevel(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(CreateWorldPartitionLevel, bool);
}
