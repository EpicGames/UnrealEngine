// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeLevelInstanceActorFactoryNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeLevelInstanceActorFactoryNode)

bool UInterchangeLevelInstanceActorFactoryNode::GetCustomLevelReference(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(LevelReference, FString);
}

bool UInterchangeLevelInstanceActorFactoryNode::SetCustomLevelReference(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(LevelReference, FString);
}
