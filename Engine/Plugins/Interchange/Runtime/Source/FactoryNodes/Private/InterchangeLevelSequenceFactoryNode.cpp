// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeLevelSequenceFactoryNode.h"

#include "LevelSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeLevelSequenceFactoryNode)

UInterchangeLevelSequenceFactoryNode::UInterchangeLevelSequenceFactoryNode()
{
	CustomAnimationTrackUids.Initialize(Attributes, TEXT("AnimationTrackUids"));
}

UClass* UInterchangeLevelSequenceFactoryNode::GetObjectClass() const
{
	return ULevelSequence::StaticClass();
}

int32 UInterchangeLevelSequenceFactoryNode::GetCustomAnimationTrackUidCount() const
{
	return CustomAnimationTrackUids.GetCount();
}

void UInterchangeLevelSequenceFactoryNode::GetCustomAnimationTrackUids(TArray<FString>& OutAnimationTrackUids) const
{
	CustomAnimationTrackUids.GetItems(OutAnimationTrackUids);
}

void UInterchangeLevelSequenceFactoryNode::GetCustomAnimationTrackUid(const int32 Index, FString& OutAnimationTrackUid) const
{
	CustomAnimationTrackUids.GetItem(Index, OutAnimationTrackUid);
}

bool UInterchangeLevelSequenceFactoryNode::AddCustomAnimationTrackUid(const FString& AnimationTrackUid)
{
	return CustomAnimationTrackUids.AddItem(AnimationTrackUid);
}

bool UInterchangeLevelSequenceFactoryNode::RemoveCustomAnimationTrackUid(const FString& AnimationTrackUid)
{
	return CustomAnimationTrackUids.RemoveItem(AnimationTrackUid);
}

bool UInterchangeLevelSequenceFactoryNode::GetCustomFrameRate(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(FrameRate, float);
}

bool UInterchangeLevelSequenceFactoryNode::SetCustomFrameRate(const float& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(FrameRate, float);
}
