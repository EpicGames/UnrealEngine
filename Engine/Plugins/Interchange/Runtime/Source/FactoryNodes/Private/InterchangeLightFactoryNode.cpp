// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeLightFactoryNode.h"

#include "Components/DirectionalLightComponent.h"
#include "Components/RectLightComponent.h"
#include "Components/SkyLightComponent.h"
#include "Components/SpotLightComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeLightFactoryNode)

bool UInterchangeBaseLightFactoryNode::GetCustomLightColor(FColor& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(LightColor, FColor)
}

bool UInterchangeBaseLightFactoryNode::SetCustomLightColor(const FColor& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeBaseLightFactoryNode, LightColor, FColor, ULightComponentBase)
}

bool UInterchangeBaseLightFactoryNode::GetCustomIntensity(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Intensity, float)
}

bool UInterchangeBaseLightFactoryNode::SetCustomIntensity(float AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeBaseLightFactoryNode, Intensity, float, ULightComponentBase)
}

bool UInterchangeBaseLightFactoryNode::GetCustomTemperature(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Temperature, float)
}

bool UInterchangeBaseLightFactoryNode::SetCustomTemperature(float AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeBaseLightFactoryNode, Temperature, float, ULightComponent)
}

bool UInterchangeBaseLightFactoryNode::GetCustomUseTemperature(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(bUseTemperature, bool)
}

bool UInterchangeBaseLightFactoryNode::SetCustomUseTemperature(bool AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeBaseLightFactoryNode, bUseTemperature, bool, ULightComponent)
}

void UInterchangeBaseLightFactoryNode::CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object)
{
	Super::CopyWithObject(SourceNode, Object);

	if (const UInterchangeBaseLightFactoryNode* LightFactoryNode = Cast<UInterchangeBaseLightFactoryNode>(SourceNode))
	{
		COPY_NODE_DELEGATES(LightFactoryNode, LightColor, FColor, ULightComponentBase)
		COPY_NODE_DELEGATES(LightFactoryNode, Intensity, float, ULightComponentBase)
		COPY_NODE_DELEGATES(LightFactoryNode, Temperature, float, ULightComponent)
		COPY_NODE_DELEGATES(LightFactoryNode, bUseTemperature, bool, ULightComponent)
	}
}

void UInterchangeBaseLightFactoryNode::OnRestoreAllCustomAttributeDelegates()
{
	Super::OnRestoreAllCustomAttributeDelegates();
	REFILL_CUSTOM_ATTRIBUTE_APPLY_DELEGATE(LightColor, FColor);
	REFILL_CUSTOM_ATTRIBUTE_APPLY_DELEGATE(Intensity, float);
	REFILL_CUSTOM_ATTRIBUTE_APPLY_DELEGATE(Temperature, float);
	REFILL_CUSTOM_ATTRIBUTE_APPLY_DELEGATE_EXPLICIT(Macro_CustombUseTemperatureKey.Key, GetCustomUseTemperature, SetCustomUseTemperature, bool);
}

bool UInterchangeDirectionalLightFactoryNode::GetCustomLightSourceAngle(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(LightSourceAngle, float)
}

bool UInterchangeDirectionalLightFactoryNode::SetCustomLightSourceAngle(float AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeDirectionalLightFactoryNode, LightSourceAngle, float, UDirectionalLightComponent)
}

void UInterchangeDirectionalLightFactoryNode::CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object)
{
	Super::CopyWithObject(SourceNode, Object);

	if (const UInterchangeDirectionalLightFactoryNode* LightFactoryNode = Cast<UInterchangeDirectionalLightFactoryNode>(SourceNode))
	{
		COPY_NODE_DELEGATES(LightFactoryNode, LightSourceAngle, float, UDirectionalLightComponent)
	}
}

void UInterchangeDirectionalLightFactoryNode::OnRestoreAllCustomAttributeDelegates()
{
	Super::OnRestoreAllCustomAttributeDelegates();
	REFILL_CUSTOM_ATTRIBUTE_APPLY_DELEGATE(LightSourceAngle, float);
}

bool UInterchangeLightFactoryNode::GetCustomIntensityUnits(ELightUnits& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(IntensityUnits, ELightUnits)
}

bool UInterchangeLightFactoryNode::SetCustomIntensityUnits(ELightUnits AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeLightFactoryNode, IntensityUnits, ELightUnits, ULocalLightComponent)
}

bool UInterchangeLightFactoryNode::GetCustomAttenuationRadius(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttenuationRadius, float)
}

bool UInterchangeLightFactoryNode::SetCustomAttenuationRadius(float AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeLightFactoryNode, AttenuationRadius, float, ULocalLightComponent)
}

bool UInterchangeLightFactoryNode::GetCustomIESTexture(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(IESTexture, FString);
}

bool UInterchangeLightFactoryNode::SetCustomIESTexture(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(IESTexture, FString)
}

bool UInterchangeLightFactoryNode::GetCustomUseIESBrightness(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(UseIESBrightness, bool);
}

bool UInterchangeLightFactoryNode::SetCustomUseIESBrightness(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(UseIESBrightness, bool);
}

bool UInterchangeLightFactoryNode::GetCustomIESBrightnessScale(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(IESBrightnessScale, float);
}

bool UInterchangeLightFactoryNode::SetCustomIESBrightnessScale(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(IESBrightnessScale, float);
}

bool UInterchangeLightFactoryNode::GetCustomRotation(FRotator& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Rotation, FRotator);
}

bool UInterchangeLightFactoryNode::SetCustomRotation(const FRotator& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Rotation, FRotator);
}

void UInterchangeLightFactoryNode::CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object)
{
	Super::CopyWithObject(SourceNode, Object);

	if (const UInterchangeLightFactoryNode* LightFactoryNode = Cast<UInterchangeLightFactoryNode>(SourceNode))
	{
		COPY_NODE_DELEGATES(LightFactoryNode, IntensityUnits, ELightUnits, ULocalLightComponent)
		COPY_NODE_DELEGATES(LightFactoryNode, AttenuationRadius, float, ULocalLightComponent)
	}
}

void UInterchangeLightFactoryNode::OnRestoreAllCustomAttributeDelegates()
{
	Super::OnRestoreAllCustomAttributeDelegates();
	REFILL_CUSTOM_ATTRIBUTE_APPLY_DELEGATE(IntensityUnits, ELightUnits)
	REFILL_CUSTOM_ATTRIBUTE_APPLY_DELEGATE(AttenuationRadius, float)
	REFILL_CUSTOM_ATTRIBUTE_APPLY_DELEGATE(UseIESBrightness, bool)
}

bool UInterchangePointLightFactoryNode::GetCustomUseInverseSquaredFalloff(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(bUseInverseSquaredFalloff, bool)
}

bool UInterchangePointLightFactoryNode::SetCustomUseInverseSquaredFalloff(bool AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangePointLightFactoryNode, bUseInverseSquaredFalloff, bool, UPointLightComponent)
}

bool UInterchangePointLightFactoryNode::GetCustomLightFalloffExponent(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(LightFalloffExponent, float)
}

bool UInterchangePointLightFactoryNode::SetCustomLightFalloffExponent(float AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangePointLightFactoryNode, LightFalloffExponent, float, UPointLightComponent)
}

bool UInterchangePointLightFactoryNode::GetCustomSourceRadius(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceRadius, float)
}

bool UInterchangePointLightFactoryNode::SetCustomSourceRadius(float AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangePointLightFactoryNode, SourceRadius, float, UPointLightComponent)
}

void UInterchangePointLightFactoryNode::CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object)
{
	Super::CopyWithObject(SourceNode, Object);

	if (const UInterchangePointLightFactoryNode* LightFactoryNode = Cast<UInterchangePointLightFactoryNode>(SourceNode))
	{
		COPY_NODE_DELEGATES(LightFactoryNode, bUseInverseSquaredFalloff, bool, UPointLightComponent)
		COPY_NODE_DELEGATES(LightFactoryNode, LightFalloffExponent, float, UPointLightComponent)
		COPY_NODE_DELEGATES(LightFactoryNode, SourceRadius, float, UPointLightComponent)
	}
}

bool UInterchangeSpotLightFactoryNode::GetCustomInnerConeAngle(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(InnerConeAngle, float)
}

bool UInterchangeSpotLightFactoryNode::SetCustomInnerConeAngle(float AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeSpotLightFactoryNode, InnerConeAngle, float, USpotLightComponent)
}

bool UInterchangeSpotLightFactoryNode::GetCustomOuterConeAngle(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(OuterConeAngle, float)
}

bool UInterchangeSpotLightFactoryNode::SetCustomOuterConeAngle(float AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeSpotLightFactoryNode, OuterConeAngle, float, USpotLightComponent)
}

void UInterchangeSpotLightFactoryNode::CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object)
{
	Super::CopyWithObject(SourceNode, Object);

	if (const UInterchangeSpotLightFactoryNode* LightFactoryNode = Cast<UInterchangeSpotLightFactoryNode>(SourceNode))
	{
		COPY_NODE_DELEGATES(LightFactoryNode, InnerConeAngle, float, USpotLightComponent)
		COPY_NODE_DELEGATES(LightFactoryNode, OuterConeAngle, float, USpotLightComponent)
	}
}

bool UInterchangeRectLightFactoryNode::GetCustomSourceWidth(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceWidth, float)
}

bool UInterchangeRectLightFactoryNode::SetCustomSourceWidth(float AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeRectLightFactoryNode, SourceWidth, float, URectLightComponent)
}

bool UInterchangeRectLightFactoryNode::GetCustomSourceHeight(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceHeight, float)
}

bool UInterchangeRectLightFactoryNode::SetCustomSourceHeight(float AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeRectLightFactoryNode, SourceHeight, float, URectLightComponent)
}

void UInterchangeRectLightFactoryNode::CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object)
{
	Super::CopyWithObject(SourceNode, Object);

	if (const UInterchangeRectLightFactoryNode* LightFactoryNode = Cast<UInterchangeRectLightFactoryNode>(SourceNode))
	{
		COPY_NODE_DELEGATES(LightFactoryNode, SourceWidth, float, URectLightComponent)
		COPY_NODE_DELEGATES(LightFactoryNode, SourceHeight, float, URectLightComponent)
	}
}

void UInterchangeRectLightFactoryNode::OnRestoreAllCustomAttributeDelegates()
{
	Super::OnRestoreAllCustomAttributeDelegates();

	REFILL_CUSTOM_ATTRIBUTE_APPLY_DELEGATE(SourceWidth, float)
	REFILL_CUSTOM_ATTRIBUTE_APPLY_DELEGATE(SourceHeight, float)
}

bool UInterchangeSkyLightFactoryNode::GetCustomCubemapDependency(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(CubemapDependency, FString)
}

bool UInterchangeSkyLightFactoryNode::SetCustomCubemapDependency(FString AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(CubemapDependency, FString)
}

bool UInterchangeSkyLightFactoryNode::GetCustomSourceType(EInterchangeSkyLightSourceType& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceType, EInterchangeSkyLightSourceType)
}

bool UInterchangeSkyLightFactoryNode::SetCustomSourceType(EInterchangeSkyLightSourceType AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SourceType, EInterchangeSkyLightSourceType)
}
