// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeMaterialFactoryNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeMaterialFactoryNode)

#if WITH_ENGINE

#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInterface.h"

#endif

FString UInterchangeBaseMaterialFactoryNode::GetMaterialFactoryNodeUidFromMaterialNodeUid(const FString& TranslatedNodeUid)
{
	FString NewUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TranslatedNodeUid);
	return NewUid;
}

bool UInterchangeBaseMaterialFactoryNode::GetCustomIsMaterialImportEnabled(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(IsMaterialImportEnabled, bool);
}

bool UInterchangeBaseMaterialFactoryNode::SetCustomIsMaterialImportEnabled(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(IsMaterialImportEnabled, bool);
}

FString UInterchangeMaterialFactoryNode::GetTypeName() const
{
	const FString TypeName = TEXT("MaterialFactoryNode");
	return TypeName;
}

UClass* UInterchangeMaterialFactoryNode::GetObjectClass() const
{
#if WITH_ENGINE
	return UMaterial::StaticClass();
#else
	return nullptr;
#endif
}

bool UInterchangeMaterialFactoryNode::GetBaseColorConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::PBRMR::Parameters::BaseColor.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToBaseColor(const FString& AttributeValue)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::PBRMR::Parameters::BaseColor.ToString(), AttributeValue);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToBaseColor(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::PBRMR::Parameters::BaseColor.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetMetallicConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::PBRMR::Parameters::Metallic.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToMetallic(const FString& AttributeValue)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::PBRMR::Parameters::Metallic.ToString(), AttributeValue);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToMetallic(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::PBRMR::Parameters::Metallic.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetSpecularConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::PBRMR::Parameters::Specular.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToSpecular(const FString& ExpressionNodeUid)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::PBRMR::Parameters::Specular.ToString(), ExpressionNodeUid);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToSpecular(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::PBRMR::Parameters::Specular.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetRoughnessConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::PBRMR::Parameters::Roughness.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToRoughness(const FString& ExpressionNodeUid)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::PBRMR::Parameters::Roughness.ToString(), ExpressionNodeUid);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToRoughness(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::PBRMR::Parameters::Roughness.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetAnisotropyConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::PBRMR::Parameters::Anisotropy.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToAnisotropy(const FString& ExpressionNodeUid)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::PBRMR::Parameters::Anisotropy.ToString(), ExpressionNodeUid);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToAnisotropy(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::PBRMR::Parameters::Anisotropy.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetEmissiveColorConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::Common::Parameters::EmissiveColor.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToEmissiveColor(const FString& ExpressionNodeUid)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::Common::Parameters::EmissiveColor.ToString(), ExpressionNodeUid);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToEmissiveColor(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::PBRMR::Parameters::EmissiveColor.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetNormalConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::Common::Parameters::Normal.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToNormal(const FString& ExpressionNodeUid)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::Common::Parameters::Normal.ToString(), ExpressionNodeUid);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToNormal(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::PBRMR::Parameters::Normal.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetTangentConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::Common::Parameters::Tangent.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToTangent(const FString& ExpressionNodeUid)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::Common::Parameters::Tangent.ToString(), ExpressionNodeUid);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToTangent(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::PBRMR::Parameters::Tangent.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetSubsurfaceConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::Subsurface::Parameters::SubsurfaceColor.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToSubsurface(const FString& ExpressionNodeUid)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::Subsurface::Parameters::SubsurfaceColor.ToString(), ExpressionNodeUid);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToSubsurface(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::Subsurface::Parameters::SubsurfaceColor.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetOpacityConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::Common::Parameters::Opacity.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToOpacity(const FString& AttributeValue)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::Common::Parameters::Opacity.ToString(), AttributeValue);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToOpacity(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::PBRMR::Parameters::Opacity.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetOcclusionConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::Common::Parameters::Occlusion.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToOcclusion(const FString& AttributeValue)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::Common::Parameters::Occlusion.ToString(), AttributeValue);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToOcclusion(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::PBRMR::Parameters::Occlusion.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetRefractionConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::Common::Parameters::IndexOfRefraction.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToRefraction(const FString& AttributeValue)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::Common::Parameters::IndexOfRefraction.ToString(), AttributeValue);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToRefraction(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::Common::Parameters::IndexOfRefraction.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetClearCoatConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::ClearCoat::Parameters::ClearCoat.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToClearCoat(const FString& AttributeValue)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::ClearCoat::Parameters::ClearCoat.ToString(), AttributeValue);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToClearCoat(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::ClearCoat::Parameters::ClearCoat.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetClearCoatRoughnessConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::ClearCoat::Parameters::ClearCoatRoughness.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToClearCoatRoughness(const FString& AttributeValue)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::ClearCoat::Parameters::ClearCoatRoughness.ToString(), AttributeValue);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToClearCoatRoughness(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::ClearCoat::Parameters::ClearCoatRoughness.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetClearCoatNormalConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::ClearCoat::Parameters::ClearCoatNormal.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToClearCoatNormal(const FString& AttributeValue)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::ClearCoat::Parameters::ClearCoatNormal.ToString(), AttributeValue);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToClearCoatNormal(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::ClearCoat::Parameters::ClearCoatNormal.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetTransmissionColorConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::ThinTranslucent::Parameters::TransmissionColor.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToTransmissionColor(const FString& AttributeValue)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::ThinTranslucent::Parameters::TransmissionColor.ToString(), AttributeValue);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToTransmissionColor(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::ThinTranslucent::Parameters::TransmissionColor.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetSurfaceCoverageConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::ThinTranslucent::Parameters::SurfaceCoverage.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToSurfaceCoverage(const FString& ExpressionUid)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::ThinTranslucent::Parameters::SurfaceCoverage.ToString(), ExpressionUid);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToSurfaceCoverage(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::ThinTranslucent::Parameters::SurfaceCoverage.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetFuzzColorConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::Sheen::Parameters::SheenColor.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToFuzzColor(const FString& AttributeValue)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::Sheen::Parameters::SheenColor.ToString(), AttributeValue);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToFuzzColor(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::Sheen::Parameters::SheenColor.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetClothConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::Sheen::Parameters::SheenRoughness.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToCloth(const FString& AttributeValue)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::Sheen::Parameters::SheenRoughness.ToString(), AttributeValue);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToCloth(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::Sheen::Parameters::SheenRoughness.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetDisplacementConnection(FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, UE::Interchange::Materials::Common::Parameters::Displacement.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::ConnectToDisplacement(const FString& AttributeValue)
{
	return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(this, UE::Interchange::Materials::Common::Parameters::Displacement.ToString(), AttributeValue);
}

bool UInterchangeMaterialFactoryNode::ConnectOutputToDisplacement(const FString& ExpressionNodeUid, const FString& OutputName)
{
	return UInterchangeShaderPortsAPI::ConnectOuputToInputByName(this, UE::Interchange::Materials::Common::Parameters::Displacement.ToString(), ExpressionNodeUid, OutputName);
}

bool UInterchangeMaterialFactoryNode::GetCustomShadingModel(TEnumAsByte<EMaterialShadingModel>& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ShadingModel, TEnumAsByte<EMaterialShadingModel>);
}

bool UInterchangeMaterialFactoryNode::SetCustomShadingModel(const TEnumAsByte<EMaterialShadingModel>& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeMaterialFactoryNode, ShadingModel, TEnumAsByte<EMaterialShadingModel>, UMaterial);
}

bool UInterchangeMaterialFactoryNode::GetCustomTranslucencyLightingMode(TEnumAsByte<ETranslucencyLightingMode>& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(TranslucencyLightingMode, TEnumAsByte<ETranslucencyLightingMode>);
}

bool UInterchangeMaterialFactoryNode::SetCustomTranslucencyLightingMode(const TEnumAsByte<ETranslucencyLightingMode>& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeMaterialFactoryNode, TranslucencyLightingMode, TEnumAsByte<ETranslucencyLightingMode>, UMaterial);
}

bool UInterchangeMaterialFactoryNode::GetCustomBlendMode(TEnumAsByte<EBlendMode>& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(BlendMode, TEnumAsByte<EBlendMode>);
}

bool UInterchangeMaterialFactoryNode::SetCustomBlendMode(const TEnumAsByte<EBlendMode>& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeMaterialFactoryNode, BlendMode, TEnumAsByte<EBlendMode>, UMaterial);
}

bool UInterchangeMaterialFactoryNode::GetCustomTwoSided(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(TwoSided, bool);
}

bool UInterchangeMaterialFactoryNode::SetCustomTwoSided(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeMaterialFactoryNode, TwoSided, bool, UMaterial);
}

bool UInterchangeMaterialFactoryNode::GetCustomOpacityMaskClipValue(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(OpacityMaskClipValue, float);
}

bool UInterchangeMaterialFactoryNode::SetCustomOpacityMaskClipValue(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeMaterialFactoryNode, OpacityMaskClipValue, float, UMaterial);
}

bool UInterchangeMaterialFactoryNode::GetCustomRefractionMethod(TEnumAsByte<ERefractionMode>& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(RefractionMethod, TEnumAsByte<ERefractionMode>);
}

bool UInterchangeMaterialFactoryNode::SetCustomRefractionMethod(const TEnumAsByte<ERefractionMode>& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeMaterialFactoryNode, RefractionMethod, TEnumAsByte<ERefractionMode>, UMaterial);
}

bool UInterchangeMaterialFactoryNode::GetCustomScreenSpaceReflections(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ScreenSpaceReflections, bool);
}

bool UInterchangeMaterialFactoryNode::SetCustomScreenSpaceReflections(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ScreenSpaceReflections, bool);
}

bool UInterchangeMaterialFactoryNode::GetCustomDisplacementCenter(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DisplacementCenter, float);
}

bool UInterchangeMaterialFactoryNode::SetCustomDisplacementCenter(float AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DisplacementCenter, float);
}

bool UInterchangeMaterialFactoryNode::GetCustomDisplacementMagnitude(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DisplacementMagnitude, float);
}

bool UInterchangeMaterialFactoryNode::SetCustomDisplacementMagnitude(float AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DisplacementMagnitude, float);
}

FString UInterchangeMaterialExpressionFactoryNode::GetTypeName() const
{
	const FString TypeName = TEXT("MaterialExpressionFactoryNode");
	return TypeName;
}

bool UInterchangeMaterialExpressionFactoryNode::GetCustomExpressionClassName(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ExpressionClassName, FString);
}

bool UInterchangeMaterialExpressionFactoryNode::SetCustomExpressionClassName(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ExpressionClassName, FString);
}

FString UInterchangeMaterialInstanceFactoryNode::GetTypeName() const
{
	const FString TypeName = TEXT("MaterialInstanceFactoryNode");
	return TypeName;
}

UClass* UInterchangeMaterialInstanceFactoryNode::GetObjectClass() const
{
#if WITH_ENGINE
	FString InstanceClassName;
	if (GetCustomInstanceClassName(InstanceClassName))
	{
		UClass* InstanceClass = FindObject<UClass>(nullptr, *InstanceClassName);
		if (InstanceClass->IsChildOf<UMaterialInstance>())
		{
			return InstanceClass;
		}
	}

	return UMaterialInstanceConstant::StaticClass();
#else
	return nullptr;
#endif
}

bool UInterchangeMaterialInstanceFactoryNode::GetCustomInstanceClassName(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(InstanceClassName, FString);
}

bool UInterchangeMaterialInstanceFactoryNode::SetCustomInstanceClassName(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(InstanceClassName, FString);
}

bool UInterchangeMaterialInstanceFactoryNode::GetCustomParent(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Parent, FString);
}

bool UInterchangeMaterialInstanceFactoryNode::SetCustomParent(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Parent, FString);
}

bool UInterchangeMaterialInstanceFactoryNode::GetCustomBlendMode(TEnumAsByte<EBlendMode>& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(BlendMode, TEnumAsByte<EBlendMode>);
}

bool UInterchangeMaterialInstanceFactoryNode::SetCustomBlendMode(TEnumAsByte<EBlendMode> AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(BlendMode, TEnumAsByte<EBlendMode>);
}

bool UInterchangeMaterialInstanceFactoryNode::GetCustomTwoSided(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(TwoSided, bool);
}

bool UInterchangeMaterialInstanceFactoryNode::SetCustomTwoSided(bool AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(TwoSided, bool);
}

bool UInterchangeMaterialInstanceFactoryNode::GetCustomEnableTessellation(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(EnableTessellation, bool);
}

bool UInterchangeMaterialInstanceFactoryNode::SetCustomEnableTessellation(bool AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(EnableTessellation, bool);
}

FString UInterchangeMaterialReferenceFactoryNode::GetTypeName() const
{
	const FString TypeName = TEXT("MaterialReferenceFactoryNode");
	return TypeName;
}

UClass* UInterchangeMaterialReferenceFactoryNode::GetObjectClass() const
{
#if WITH_ENGINE
	return UMaterialInterface::StaticClass();
#else
	return nullptr;
#endif
}

FString UInterchangeMaterialFunctionCallExpressionFactoryNode::GetTypeName() const
{
	const FString TypeName = TEXT("MaterialFunctionCallExpressionFactoryNode");
	return TypeName;
}

bool UInterchangeMaterialFunctionCallExpressionFactoryNode::GetCustomMaterialFunctionDependency(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(MaterialFunctionDependency, FString);
}

bool UInterchangeMaterialFunctionCallExpressionFactoryNode::SetCustomMaterialFunctionDependency(const FString& AttributeValue)
{

	auto ImplementationFunction = [this, &AttributeValue]()
		{
			IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(MaterialFunctionDependency, FString);
		};
	if (ImplementationFunction())
	{
		AddFactoryDependencyUid(AttributeValue);
		return true;
	}
	return false;
}

FString UInterchangeMaterialFunctionFactoryNode::GetTypeName() const
{
	const FString TypeName = TEXT("MaterialFunctionFactoryNode");
	return TypeName;
}

UClass* UInterchangeMaterialFunctionFactoryNode::GetObjectClass() const
{
#if WITH_ENGINE
	return UMaterialFunction::StaticClass();
#else
	return nullptr;
#endif
}

bool UInterchangeMaterialFunctionFactoryNode::GetInputConnection(const FString& InputName, FString& ExpressionNodeUid, FString& OutputName) const
{
	return UInterchangeShaderPortsAPI::GetInputConnection(this, InputName, ExpressionNodeUid, OutputName);
}
