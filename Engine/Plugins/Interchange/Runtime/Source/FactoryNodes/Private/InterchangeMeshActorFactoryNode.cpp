// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeMeshActorFactoryNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeMeshActorFactoryNode)

#if WITH_ENGINE
	#include "GameFramework/Actor.h"
#endif

UInterchangeMeshActorFactoryNode::UInterchangeMeshActorFactoryNode()
{
	SlotMaterialDependencies.Initialize(Attributes.ToSharedRef(), TEXT("__SlotMaterialDependencies__"));
}

bool UInterchangeMeshActorFactoryNode::SetCustomInstancedAssetFactoryNodeUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(InstancedAssetFactoryNodeUid, FString);
}

bool UInterchangeMeshActorFactoryNode::GetCustomInstancedAssetFactoryNodeUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(InstancedAssetFactoryNodeUid, FString);
}

void UInterchangeMeshActorFactoryNode::GetSlotMaterialDependencies(TMap<FString, FString>& OutMaterialDependencies) const
{
	OutMaterialDependencies = SlotMaterialDependencies.ToMap();
}

bool UInterchangeMeshActorFactoryNode::GetSlotMaterialDependencyUid(const FString& SlotName, FString& OutMaterialDependency) const
{
	return SlotMaterialDependencies.GetValue(SlotName, OutMaterialDependency);
}

bool UInterchangeMeshActorFactoryNode::SetSlotMaterialDependencyUid(const FString& SlotName, const FString& MaterialDependencyUid)
{
	return SlotMaterialDependencies.SetKeyValue(SlotName, MaterialDependencyUid);
}

bool UInterchangeMeshActorFactoryNode::RemoveSlotMaterialDependencyUid(const FString& SlotName)
{
	return SlotMaterialDependencies.RemoveKey(SlotName);
}

bool UInterchangeMeshActorFactoryNode::SetCustomAnimationAssetUidToPlay(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AnimationAssetUidToPlay, FString);
}
bool UInterchangeMeshActorFactoryNode::GetCustomAnimationAssetUidToPlay(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AnimationAssetUidToPlay, FString);
}

bool UInterchangeMeshActorFactoryNode::GetCustomGeometricTransform(FTransform& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GeometricTransform, FTransform);
}

bool UInterchangeMeshActorFactoryNode::SetCustomGeometricTransform(const FTransform& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(GeometricTransform, FTransform);
}