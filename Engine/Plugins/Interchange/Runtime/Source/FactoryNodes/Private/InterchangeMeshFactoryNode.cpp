// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeMeshFactoryNode.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include <string>

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeMeshFactoryNode)

//The static mesh source model which contain the build settings are editor only
#if WITH_EDITOR

#define IMPLEMENT_MESH_BUILD_VALUE_TO_ASSET(AttributeName, AttributeType, PropertyName)	\
bool bResult = false;																\
AttributeType ValueData;															\
if (GetAttribute<AttributeType>(Macro_Custom##AttributeName##Key.Key, ValueData))	\
{																					\
	if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Asset))					\
	{																				\
		for(int32 LodIndex = 0; LodIndex < SkeletalMesh->GetLODNum(); ++LodIndex)	\
		{																			\
			if (FSkeletalMeshLODInfo* LodInfo = SkeletalMesh->GetLODInfo(LodIndex))	\
			{																		\
				LodInfo->BuildSettings.PropertyName = ValueData;					\
				bResult = true;														\
			}																		\
		}																			\
	}																				\
	else if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))					\
	{																				\
		for(int32 LodIndex = 0; LodIndex < StaticMesh->GetNumSourceModels(); ++LodIndex) \
		{																			\
			if (StaticMesh->IsSourceModelValid(LodIndex))							\
			{																		\
				StaticMesh->GetSourceModel(LodIndex).BuildSettings.PropertyName = ValueData;	\
				bResult = true;														\
			}																		\
		}																			\
	}																				\
}																					\
return bResult;

#define IMPLEMENT_MESH_BUILD_ASSET_TO_VALUE(AttributeName, AttributeType, PropertyName)	\
if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))								\
{																					\
	if (StaticMesh->GetNumSourceModels() > 0)										\
	{																				\
		return SetAttribute<AttributeType>(Macro_Custom##AttributeName##Key.Key, StaticMesh->GetSourceModel(0).BuildSettings.PropertyName);	\
	}																				\
}																					\
else if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Asset))					\
{																					\
	if (FSkeletalMeshLODInfo* LodInfo = SkeletalMesh->GetLODInfo(0))				\
	{																				\
		return SetAttribute<AttributeType>(Macro_Custom##AttributeName##Key.Key, LodInfo->BuildSettings.PropertyName);	\
	}																				\
}																					\
return false;

#else //WITH_EDITOR

#define IMPLEMENT_MESH_BUILD_VALUE_TO_ASSET(AttributeName, AttributeType, PropertyName)	\
return false;

#define IMPLEMENT_MESH_BUILD_ASSET_TO_VALUE(AttributeName, AttributeType, PropertyName)	\
return false;
#endif // else WITH_EDITOR



namespace UE::Interchange
{
	static constexpr const TCHAR PayloadPrefix[] = TEXT("__PAYLOAD_PREFIX__");

	const FAttributeKey& FMeshFactoryNodeStaticData::GetLodDependenciesBaseKey()
	{
		static FAttributeKey LodDependencies_BaseKey = FAttributeKey(TEXT("Lod_Dependencies"));
		return LodDependencies_BaseKey;
	}

	const FAttributeKey& FMeshFactoryNodeStaticData::GetSlotMaterialDependencyBaseKey()
	{
		static FAttributeKey SlotMaterialDependency_BaseKey(TEXT("__SlotMaterialDependency__"));
		return SlotMaterialDependency_BaseKey;
	}

	const FAttributeKey& FMeshFactoryNodeStaticData::GetAssemblyPartDependencyBaseKey()
	{
		static FAttributeKey SlotMaterialDependency_BaseKey(TEXT("__AssemblyPartDependency__"));
		return SlotMaterialDependency_BaseKey;
	}

} // namespace UE::Interchange


UInterchangeMeshFactoryNode::UInterchangeMeshFactoryNode()
{
	LodDependencies.Initialize(Attributes, UE::Interchange::FMeshFactoryNodeStaticData::GetLodDependenciesBaseKey().ToString());
	SlotMaterialDependencies.Initialize(Attributes.ToSharedRef(), UE::Interchange::FMeshFactoryNodeStaticData::GetSlotMaterialDependencyBaseKey().ToString());
	AssemblyPartDependencies.Initialize(Attributes.ToSharedRef(), UE::Interchange::FMeshFactoryNodeStaticData::GetAssemblyPartDependencyBaseKey().ToString());
}

#if WITH_EDITOR

FString UInterchangeMeshFactoryNode::GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	FString KeyDisplayName = NodeAttributeKey.ToString();
	FString NodeAttributeKeyString = NodeAttributeKey.ToString();
	if (NodeAttributeKey == UE::Interchange::FMeshFactoryNodeStaticData::GetLodDependenciesBaseKey())
	{
		return KeyDisplayName = TEXT("LOD Dependencies Count");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FMeshFactoryNodeStaticData::GetLodDependenciesBaseKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("LOD Dependencies Index "), NodeAttributeKeyString);
	}
	else if (NodeAttributeKey == UE::Interchange::FMeshFactoryNodeStaticData::GetSlotMaterialDependencyBaseKey())
	{
		return KeyDisplayName = TEXT("Material Dependencies Count");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FMeshFactoryNodeStaticData::GetSlotMaterialDependencyBaseKey().ToString()))
	{
		FString MapKeyIndex = UE::Interchange::FMeshFactoryNodeStaticData::GetSlotMaterialDependencyBaseKey().ToString() + TEXT("_KeyIndex_");
		FString MapKey = UE::Interchange::FMeshFactoryNodeStaticData::GetSlotMaterialDependencyBaseKey().ToString() + TEXT("_Key_");

		int32 IndexPosition = 0;
		if (NodeAttributeKeyString.StartsWith(MapKeyIndex))
		{
			KeyDisplayName = TEXT("Material Dependencies Key ");
			IndexPosition = NodeAttributeKeyString.Find(MapKeyIndex) + MapKeyIndex.Len();
		}
		else if (NodeAttributeKeyString.StartsWith(MapKey))
		{
			KeyDisplayName = TEXT("Material Dependencies Value ");
			IndexPosition = NodeAttributeKeyString.Find(MapKey) + MapKey.Len();
		}
		
		if (IndexPosition < NodeAttributeKeyString.Len())
		{
			KeyDisplayName += NodeAttributeKeyString.RightChop(IndexPosition);
		}
		return KeyDisplayName;
	}
	else if (NodeAttributeKey == UE::Interchange::FMeshFactoryNodeStaticData::GetAssemblyPartDependencyBaseKey())
	{
		return KeyDisplayName = TEXT("Nanite Assembly Part Dependencies Count");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FMeshFactoryNodeStaticData::GetAssemblyPartDependencyBaseKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Nanite Assembly Part Dependencies Index "), NodeAttributeKeyString);
	}
	return Super::GetKeyDisplayName(NodeAttributeKey);
}

FString UInterchangeMeshFactoryNode::GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	if (NodeAttributeKey.ToString().StartsWith(UE::Interchange::FMeshFactoryNodeStaticData::GetLodDependenciesBaseKey().ToString()))
	{
		return TEXT("LOD Dependencies");
	}
	else if (NodeAttributeKey.ToString().StartsWith(UE::Interchange::FMeshFactoryNodeStaticData::GetSlotMaterialDependencyBaseKey().ToString()))
	{
		return TEXT("Materials Dependencies");
	}
	else if (NodeAttributeKey.ToString().StartsWith(UE::Interchange::FMeshFactoryNodeStaticData::GetAssemblyPartDependencyBaseKey().ToString()))
	{
		return TEXT("Nanite Assembly Part Dependencies");
	}
	else
	{
		return Super::GetAttributeCategory(NodeAttributeKey);
	}
}

bool UInterchangeMeshFactoryNode::ShouldHideAttribute(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	const FString NodeAttributeString = NodeAttributeKey.ToString();
	if (NodeAttributeString.StartsWith(UE::Interchange::FMeshFactoryNodeStaticData::GetLodDependenciesBaseKey().ToString()))
	{
		return true;
	}
	return Super::ShouldHideAttribute(NodeAttributeKey);
}

#endif //WITH_EDITOR

const FString& UInterchangeMeshFactoryNode::GetMeshSocketPrefix()
{
	static FString StaticMeshSocketPrefix = TEXT("SOCKET_");
	return StaticMeshSocketPrefix;
}

int32 UInterchangeMeshFactoryNode::GetLodDataCount() const
{
	return LodDependencies.GetCount();
}

void UInterchangeMeshFactoryNode::GetLodDataUniqueIds(TArray<FString>& OutLodDataUniqueIds) const
{
	LodDependencies.GetItems(OutLodDataUniqueIds);
}

bool UInterchangeMeshFactoryNode::AddLodDataUniqueId(const FString& LodDataUniqueId)
{
	return LodDependencies.AddItem(LodDataUniqueId);
}

bool UInterchangeMeshFactoryNode::RemoveLodDataUniqueId(const FString& LodDataUniqueId)
{
	return LodDependencies.RemoveItem(LodDataUniqueId);
}

bool UInterchangeMeshFactoryNode::GetCustomVertexColorReplace(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(VertexColorReplace, bool)
}

bool UInterchangeMeshFactoryNode::SetCustomVertexColorReplace(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(VertexColorReplace, bool)
}

bool UInterchangeMeshFactoryNode::GetCustomVertexColorIgnore(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(VertexColorIgnore, bool)
}

bool UInterchangeMeshFactoryNode::SetCustomVertexColorIgnore(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(VertexColorIgnore, bool)
}

bool UInterchangeMeshFactoryNode::GetCustomVertexColorOverride(FColor& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(VertexColorOverride, FColor)
}

bool UInterchangeMeshFactoryNode::SetCustomVertexColorOverride(const FColor& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(VertexColorOverride, FColor)
}

bool UInterchangeMeshFactoryNode::GetCustomKeepSectionsSeparate(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(KeepSectionsSeparate, bool)
}

bool UInterchangeMeshFactoryNode::SetCustomKeepSectionsSeparate(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(KeepSectionsSeparate, bool)
}

bool UInterchangeMeshFactoryNode::GetCustomImportSockets(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ImportSockets, bool)
}
bool UInterchangeMeshFactoryNode::SetCustomImportSockets(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ImportSockets, bool)
}

void UInterchangeMeshFactoryNode::GetSlotMaterialDependencies(TMap<FString, FString>& OutMaterialDependencies) const
{
	bool bKeepSectionsSeparate = false;
	GetCustomKeepSectionsSeparate(bKeepSectionsSeparate);
	if (bKeepSectionsSeparate)
	{
		OutMaterialDependencies = SlotMaterialDependencies.ToMap();
	}
	else
	{
		TMap<FString, FString> MaterialDependencies = SlotMaterialDependencies.ToMap();
		TSet<FString> MaterialUids;
		MaterialUids.Reserve(MaterialDependencies.Num());
		OutMaterialDependencies.Empty(MaterialDependencies.Num());
		for (TPair<FString, FString>& MaterialDependency : MaterialDependencies)
		{
			if (!MaterialUids.Contains(MaterialDependency.Value))
			{
				OutMaterialDependencies.Add(MaterialDependency);
				MaterialUids.Add(MaterialDependency.Value);
			}
		}
	}
}

bool UInterchangeMeshFactoryNode::GetSlotMaterialDependencyUid(const FString& SlotName, FString& OutMaterialDependency) const
{
	return SlotMaterialDependencies.GetValue(SlotName, OutMaterialDependency);
}

bool UInterchangeMeshFactoryNode::SetSlotMaterialDependencyUid(const FString& SlotName, const FString& MaterialDependencyUid)
{
	return SlotMaterialDependencies.SetKeyValue(SlotName, MaterialDependencyUid);
}

bool UInterchangeMeshFactoryNode::RemoveSlotMaterialDependencyUid(const FString& SlotName)
{
	if (SlotMaterialDependencies.RemoveKey(SlotName))
	{
		SlotMaterialDependencies.RebuildCache();
		return true;
	}
	return false;
}

bool UInterchangeMeshFactoryNode::ResetSlotMaterialDependencies()
{
	SlotMaterialDependencies.Empty();
	return true;
}

int32 UInterchangeMeshFactoryNode::GetAssemblyPartDependenciesCount() const
{
	return AssemblyPartDependencies.GetCount();
}

void UInterchangeMeshFactoryNode::GetAssemblyPartDependencies(TMap<FString, FString>& OutAssemblyPartDependencies) const
{
	OutAssemblyPartDependencies = AssemblyPartDependencies.ToMap();
}

bool UInterchangeMeshFactoryNode::SetAssemblyPartDependencyUid(const FString& MeshUid, const FString& AssemblyPartDependencyUid)
{
	return AssemblyPartDependencies.SetKeyValue(MeshUid, AssemblyPartDependencyUid);
}

bool UInterchangeMeshFactoryNode::RemoveAssemblyPartDependencyUid(const FString& MeshUid)
{
	if (AssemblyPartDependencies.RemoveKey(MeshUid))
	{
		AssemblyPartDependencies.RebuildCache();
		return true;
	}
	return false;
}

bool UInterchangeMeshFactoryNode::ResetAssemblyDependencies()
{
	AssemblyPartDependencies.Empty();
	return true;
}

bool UInterchangeMeshFactoryNode::GetCustomLODGroup(FName& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(LODGroup, FName)
}

bool UInterchangeMeshFactoryNode::SetCustomLODGroup(const FName& AttributeValue, bool bAddApplyDelegate)
{
	//The LOD Group must be setup directly by the factory with the static mesh API
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(LODGroup, FName)
}

bool UInterchangeMeshFactoryNode::GetCustomRecomputeNormals(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(RecomputeNormals, bool)
}

bool UInterchangeMeshFactoryNode::SetCustomRecomputeNormals(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeMeshFactoryNode, RecomputeNormals, bool);
}

bool UInterchangeMeshFactoryNode::ApplyCustomRecomputeNormalsToAsset(UObject* Asset) const
{
	IMPLEMENT_MESH_BUILD_VALUE_TO_ASSET(RecomputeNormals, bool, bRecomputeNormals);
}

bool UInterchangeMeshFactoryNode::FillCustomRecomputeNormalsFromAsset(UObject* Asset)
{
	IMPLEMENT_MESH_BUILD_ASSET_TO_VALUE(RecomputeNormals, bool, bRecomputeNormals);
}

bool UInterchangeMeshFactoryNode::GetCustomRecomputeTangents(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(RecomputeTangents, bool)
}

bool UInterchangeMeshFactoryNode::SetCustomRecomputeTangents(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeMeshFactoryNode, RecomputeTangents, bool);
}

bool UInterchangeMeshFactoryNode::ApplyCustomRecomputeTangentsToAsset(UObject* Asset) const
{
	IMPLEMENT_MESH_BUILD_VALUE_TO_ASSET(RecomputeTangents, bool, bRecomputeTangents);
}

bool UInterchangeMeshFactoryNode::FillCustomRecomputeTangentsFromAsset(UObject* Asset)
{
	IMPLEMENT_MESH_BUILD_ASSET_TO_VALUE(RecomputeTangents, bool, bRecomputeTangents);
}

bool UInterchangeMeshFactoryNode::GetCustomUseMikkTSpace(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(UseMikkTSpace, bool)
}

bool UInterchangeMeshFactoryNode::SetCustomUseMikkTSpace(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeMeshFactoryNode, UseMikkTSpace, bool);
}

bool UInterchangeMeshFactoryNode::ApplyCustomUseMikkTSpaceToAsset(UObject* Asset) const
{
	IMPLEMENT_MESH_BUILD_VALUE_TO_ASSET(UseMikkTSpace, bool, bUseMikkTSpace);
}

bool UInterchangeMeshFactoryNode::FillCustomUseMikkTSpaceFromAsset(UObject* Asset)
{
	IMPLEMENT_MESH_BUILD_ASSET_TO_VALUE(UseMikkTSpace, bool, bUseMikkTSpace);
}

bool UInterchangeMeshFactoryNode::GetCustomComputeWeightedNormals(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ComputeWeightedNormals, bool)
}

bool UInterchangeMeshFactoryNode::SetCustomComputeWeightedNormals(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeMeshFactoryNode, ComputeWeightedNormals, bool);
}

bool UInterchangeMeshFactoryNode::ApplyCustomComputeWeightedNormalsToAsset(UObject* Asset) const
{
	IMPLEMENT_MESH_BUILD_VALUE_TO_ASSET(ComputeWeightedNormals, bool, bComputeWeightedNormals);
}

bool UInterchangeMeshFactoryNode::FillCustomComputeWeightedNormalsFromAsset(UObject* Asset)
{
	IMPLEMENT_MESH_BUILD_ASSET_TO_VALUE(ComputeWeightedNormals, bool, bComputeWeightedNormals);
}

bool UInterchangeMeshFactoryNode::GetCustomUseHighPrecisionTangentBasis(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(UseHighPrecisionTangentBasis, bool)
}

bool UInterchangeMeshFactoryNode::SetCustomUseHighPrecisionTangentBasis(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeMeshFactoryNode, UseHighPrecisionTangentBasis, bool);
}

bool UInterchangeMeshFactoryNode::ApplyCustomUseHighPrecisionTangentBasisToAsset(UObject* Asset) const
{
	IMPLEMENT_MESH_BUILD_VALUE_TO_ASSET(UseHighPrecisionTangentBasis, bool, bUseHighPrecisionTangentBasis);
}

bool UInterchangeMeshFactoryNode::FillCustomUseHighPrecisionTangentBasisFromAsset(UObject* Asset)
{
	IMPLEMENT_MESH_BUILD_ASSET_TO_VALUE(UseHighPrecisionTangentBasis, bool, bUseHighPrecisionTangentBasis);
}

bool UInterchangeMeshFactoryNode::GetCustomUseFullPrecisionUVs(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(UseFullPrecisionUVs, bool)
}

bool UInterchangeMeshFactoryNode::SetCustomUseFullPrecisionUVs(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeMeshFactoryNode, UseFullPrecisionUVs, bool);
}

bool UInterchangeMeshFactoryNode::ApplyCustomUseFullPrecisionUVsToAsset(UObject* Asset) const
{
	IMPLEMENT_MESH_BUILD_VALUE_TO_ASSET(UseFullPrecisionUVs, bool, bUseFullPrecisionUVs);
}

bool UInterchangeMeshFactoryNode::FillCustomUseFullPrecisionUVsFromAsset(UObject* Asset)
{
	IMPLEMENT_MESH_BUILD_ASSET_TO_VALUE(UseFullPrecisionUVs, bool, bUseFullPrecisionUVs);
}

bool UInterchangeMeshFactoryNode::GetCustomUseBackwardsCompatibleF16TruncUVs(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(UseBackwardsCompatibleF16TruncUVs, bool)
}

bool UInterchangeMeshFactoryNode::SetCustomUseBackwardsCompatibleF16TruncUVs(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeMeshFactoryNode, UseBackwardsCompatibleF16TruncUVs, bool);
}

bool UInterchangeMeshFactoryNode::ApplyCustomUseBackwardsCompatibleF16TruncUVsToAsset(UObject* Asset) const
{
	IMPLEMENT_MESH_BUILD_VALUE_TO_ASSET(UseBackwardsCompatibleF16TruncUVs, bool, bUseBackwardsCompatibleF16TruncUVs);
}
bool UInterchangeMeshFactoryNode::FillCustomUseBackwardsCompatibleF16TruncUVsFromAsset(UObject* Asset)
{
	IMPLEMENT_MESH_BUILD_ASSET_TO_VALUE(UseBackwardsCompatibleF16TruncUVs, bool, bUseBackwardsCompatibleF16TruncUVs);
}

bool UInterchangeMeshFactoryNode::GetCustomRemoveDegenerates(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(RemoveDegenerates, bool)
}

bool UInterchangeMeshFactoryNode::SetCustomRemoveDegenerates(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeMeshFactoryNode, RemoveDegenerates, bool);
}

bool UInterchangeMeshFactoryNode::ApplyCustomRemoveDegeneratesToAsset(UObject* Asset) const
{
	IMPLEMENT_MESH_BUILD_VALUE_TO_ASSET(RemoveDegenerates, bool, bRemoveDegenerates);
}

bool UInterchangeMeshFactoryNode::FillCustomRemoveDegeneratesFromAsset(UObject* Asset)
{
	IMPLEMENT_MESH_BUILD_ASSET_TO_VALUE(RemoveDegenerates, bool, bRemoveDegenerates);
}

void UInterchangeMeshFactoryNode::CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object)
{
	Super::CopyWithObject(SourceNode, Object);

	if (const UInterchangeMeshFactoryNode* MeshFactoryNode = Cast<UInterchangeMeshFactoryNode>(SourceNode))
	{
#if WITH_ENGINE
		// Make sure the class is properly set when we compile with engine, this will set the bIsNodeClassInitialized to true.
		SetNodeClassFromClassAttribute();
#endif

		UClass* Class = GetObjectClass();
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(MeshFactoryNode, UInterchangeMeshFactoryNode, RecomputeNormals, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(MeshFactoryNode, UInterchangeMeshFactoryNode, RecomputeTangents, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(MeshFactoryNode, UInterchangeMeshFactoryNode, UseMikkTSpace, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(MeshFactoryNode, UInterchangeMeshFactoryNode, ComputeWeightedNormals, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(MeshFactoryNode, UInterchangeMeshFactoryNode, UseHighPrecisionTangentBasis, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(MeshFactoryNode, UInterchangeMeshFactoryNode, UseFullPrecisionUVs, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(MeshFactoryNode, UInterchangeMeshFactoryNode, UseBackwardsCompatibleF16TruncUVs, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(MeshFactoryNode, UInterchangeMeshFactoryNode, RemoveDegenerates, bool, Class)
	}
}

bool UInterchangeMeshFactoryNode::GetPayloadKeyStringAttribute(const FString& PayloadAttributeKey, FString& Value)
{
	return GetStringAttribute(UE::Interchange::PayloadPrefix + PayloadAttributeKey, Value);
}

bool UInterchangeMeshFactoryNode::AddPayloadKeyStringAttribute(const FString& PayloadAttributeKey, const FString& Value)
{
	return AddStringAttribute(UE::Interchange::PayloadPrefix + PayloadAttributeKey, Value);
}

bool UInterchangeMeshFactoryNode::GetPayloadKeyFloatAttribute(const FString& PayloadAttributeKey, float& Value)
{
	return GetFloatAttribute(UE::Interchange::PayloadPrefix + PayloadAttributeKey, Value);
}

bool UInterchangeMeshFactoryNode::AddPayloadKeyFloatAttribute(const FString& PayloadAttributeKey, float Value)
{
	return AddFloatAttribute(UE::Interchange::PayloadPrefix + PayloadAttributeKey, Value);
}

bool UInterchangeMeshFactoryNode::GetPayloadKeyInt32Attribute(const FString& PayloadAttributeKey, int32& Value)
{
	return GetInt32Attribute(UE::Interchange::PayloadPrefix + PayloadAttributeKey, Value);
}

bool UInterchangeMeshFactoryNode::AddPayloadKeyInt32Attribute(const FString& PayloadAttributeKey, int32 Value)
{
	return AddInt32Attribute(UE::Interchange::PayloadPrefix + PayloadAttributeKey, Value);
}

bool UInterchangeMeshFactoryNode::GetPayloadKeyBooleanAttribute(const FString& PayloadAttributeKey, bool& Value)
{
	return GetBooleanAttribute(UE::Interchange::PayloadPrefix + PayloadAttributeKey, Value);
}

bool UInterchangeMeshFactoryNode::AddPayloadKeyBooleanAttribute(const FString& PayloadAttributeKey, bool Value)
{
	return AddBooleanAttribute(UE::Interchange::PayloadPrefix + PayloadAttributeKey, Value);
}

bool UInterchangeMeshFactoryNode::GetPayloadKeyDoubleAttribute(const FString& PayloadAttributeKey, double& Value)
{
	return GetDoubleAttribute(UE::Interchange::PayloadPrefix + PayloadAttributeKey, Value);
}

bool UInterchangeMeshFactoryNode::AddPayloadKeyDoubleAttribute(const FString& PayloadAttributeKey, double Value)
{
	return AddDoubleAttribute(UE::Interchange::PayloadPrefix + PayloadAttributeKey, Value);
}

void UInterchangeMeshFactoryNode::CopyPayloadKeyStorageAttributes(const UInterchangeBaseNode* SourceNode, UE::Interchange::FAttributeStorage& DestinationStorage)
{
	TArray<UE::Interchange::FAttributeKey> AttributeKeys;
	SourceNode->GetAttributeKeys(AttributeKeys);

	TArray<TPair<UE::Interchange::FAttributeKey, UE::Interchange::FAttributeKey>> PayloadKeys;
	PayloadKeys.Reserve(AttributeKeys.Num());
	for (const UE::Interchange::FAttributeKey& AttributeKey : AttributeKeys)
	{
		constexpr size_t Length = std::char_traits<TCHAR>::length(UE::Interchange::PayloadPrefix);
		if (AttributeKey.Key.StartsWith(UE::Interchange::PayloadPrefix, Length))
		{
			UE::Interchange::FAttributeKey PayloadKey = AttributeKey;
			PayloadKey.Key.RemoveFromStart(UE::Interchange::PayloadPrefix, Length);
			PayloadKeys.Emplace(AttributeKey, MoveTemp(PayloadKey));
		}
	}
	PayloadKeys.Shrink();
	
	UInterchangeBaseNode::CopyStorageAttributes(SourceNode, DestinationStorage, PayloadKeys);
}
