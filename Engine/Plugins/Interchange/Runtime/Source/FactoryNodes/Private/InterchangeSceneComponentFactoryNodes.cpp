// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeSceneComponentFactoryNodes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSceneComponentFactoryNodes)

#if WITH_ENGINE
#include "Components/SceneComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "GameFramework/Actor.h"
#endif


UInterchangeSceneComponentFactoryNode::UInterchangeSceneComponentFactoryNode()
{
	ComponentUids.Initialize(Attributes, TEXT("__ComponentUids__Key"));
}

FString UInterchangeSceneComponentFactoryNode::GetTypeName() const
{
	return TEXT("UInterchangeSceneComponentFactoryNode");
}

UClass* UInterchangeSceneComponentFactoryNode::GetObjectClass() const
{
#if WITH_ENGINE
	return USceneComponent::StaticClass();
#else
	return nullptr;
#endif
}


bool UInterchangeSceneComponentFactoryNode::AddComponentUid(const FString& ComponentUid)
{
	return ComponentUids.AddItem(ComponentUid);
}

void UInterchangeSceneComponentFactoryNode::GetComponentUids(TArray<FString>& OutComponentUids) const
{
	ComponentUids.GetItems(OutComponentUids);
}


bool UInterchangeSceneComponentFactoryNode::GetCustomLocalTransform(FTransform& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(LocalTransform, FTransform);
}

bool UInterchangeSceneComponentFactoryNode::SetCustomLocalTransform(const FTransform& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(LocalTransform, FTransform);
}


bool UInterchangeSceneComponentFactoryNode::GetCustomComponentVisibility(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ComponentVisibility, bool);
}

bool UInterchangeSceneComponentFactoryNode::SetCustomComponentVisibility(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ComponentVisibility, bool);
}



UInterchangeInstancedStaticMeshComponentFactoryNode ::UInterchangeInstancedStaticMeshComponentFactoryNode ()
{
	InstanceTransforms.Initialize(Attributes, TEXT("__InstanceTransforms__"));
}

FString UInterchangeInstancedStaticMeshComponentFactoryNode ::GetTypeName() const
{
	return TEXT("UInterchangeInstancedStaticMeshComponentFactoryNode ");
}

UClass* UInterchangeInstancedStaticMeshComponentFactoryNode ::GetObjectClass() const
{
#if WITH_ENGINE
	bool bHierarchicalISM = false;
	GetCustomHierarchicalISM(bHierarchicalISM);
	if (bHierarchicalISM)
	{
		return UHierarchicalInstancedStaticMeshComponent::StaticClass();
	}
	else
	{
		return UInstancedStaticMeshComponent::StaticClass();
	}
#else
	return nullptr;
#endif
}

void UInterchangeInstancedStaticMeshComponentFactoryNode ::AddInstanceTransform(const FTransform& InstanceTransform)
{
	InstanceTransforms.AddItem(InstanceTransform);
}

void UInterchangeInstancedStaticMeshComponentFactoryNode ::AddInstanceTransforms(const TArray<FTransform>& InInstanceTransforms)
{
	for (const FTransform& InstanceTransform : InInstanceTransforms)
	{
		InstanceTransforms.AddItem(InstanceTransform);
	}
}

void UInterchangeInstancedStaticMeshComponentFactoryNode ::GetInstanceTransforms(TArray<FTransform>& OutInstanceTransforms) const
{
	InstanceTransforms.GetItems(OutInstanceTransforms);
}

bool UInterchangeInstancedStaticMeshComponentFactoryNode ::GetCustomInstancedAssetUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AssetInstanceUid, FString);
}

bool UInterchangeInstancedStaticMeshComponentFactoryNode ::SetCustomInstancedAssetUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AssetInstanceUid, FString);
}

bool UInterchangeInstancedStaticMeshComponentFactoryNode ::GetCustomHierarchicalISM(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(HierarchicalISM, bool);
}

bool UInterchangeInstancedStaticMeshComponentFactoryNode ::SetCustomHierarchicalISM(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(HierarchicalISM, bool);
}