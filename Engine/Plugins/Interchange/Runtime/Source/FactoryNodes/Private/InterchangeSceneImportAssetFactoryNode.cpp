// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeSceneImportAssetFactoryNode.h"

#include "InterchangeSceneImportAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSceneImportAssetFactoryNode)

UClass* UInterchangeSceneImportAssetFactoryNode::GetObjectClass() const
{
	return UInterchangeSceneImportAsset::StaticClass();
}
