// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeSceneVariantSetsFactoryNode.h"

#include "LevelVariantSets.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSceneVariantSetsFactoryNode)

UInterchangeSceneVariantSetsFactoryNode::UInterchangeSceneVariantSetsFactoryNode()
{
	CustomVariantSetUids.Initialize(Attributes, TEXT("VariantSetUids"));
}

UClass* UInterchangeSceneVariantSetsFactoryNode::GetObjectClass() const
{
	return ULevelVariantSets::StaticClass();
}

int32 UInterchangeSceneVariantSetsFactoryNode::GetCustomVariantSetUidCount() const
{
	return CustomVariantSetUids.GetCount();
}

void UInterchangeSceneVariantSetsFactoryNode::GetCustomVariantSetUids(TArray<FString>& OutVariantSetUids) const
{
	CustomVariantSetUids.GetItems(OutVariantSetUids);
}

void UInterchangeSceneVariantSetsFactoryNode::GetCustomVariantSetUid(const int32 Index, FString& OutVariantSetUid) const
{
	CustomVariantSetUids.GetItem(Index, OutVariantSetUid);
}

bool UInterchangeSceneVariantSetsFactoryNode::AddCustomVariantSetUid(const FString& VariantSetUid)
{
	return CustomVariantSetUids.AddItem(VariantSetUid);
}

bool UInterchangeSceneVariantSetsFactoryNode::RemoveCustomVariantSetUid(const FString& VariantSetUid)
{
	return CustomVariantSetUids.RemoveItem(VariantSetUid);
}
