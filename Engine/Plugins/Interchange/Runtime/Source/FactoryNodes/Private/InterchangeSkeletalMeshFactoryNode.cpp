// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeSkeletalMeshFactoryNode.h"

#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSkeletalMeshFactoryNode)

#if WITH_EDITOR

#define IMPLEMENT_SKELETAL_BUILD_VALUE_TO_ASSET(AttributeName, AttributeType, PropertyName)	\
bool bResult = false;																\
AttributeType ValueData;															\
if (GetAttribute<AttributeType>(Macro_Custom##AttributeName##Key.Key, ValueData))	\
{																					\
	if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Asset))					\
	{																				\
		for(int32 LodIndex = 0; LodIndex < SkeletalMesh->GetLODNum(); ++LodIndex)	\
		{																			\
			if (FSkeletalMeshLODInfo* LodInfo = SkeletalMesh->GetLODInfo(LodIndex))	\
			{																		\
				LodInfo->BuildSettings.PropertyName = ValueData;					\
				bResult = true;														\
			}																		\
		}																			\
	}																				\
}																					\
return bResult;

#define IMPLEMENT_SKELETALMESH_BUILD_ASSET_TO_VALUE(AttributeName, AttributeType, PropertyName)					\
if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Asset))						\
{																					\
	if (FSkeletalMeshLODInfo* LodInfo = SkeletalMesh->GetLODInfo(0))				\
	{																				\
		return SetAttribute<AttributeType>(Macro_Custom##AttributeName##Key.Key, LodInfo->BuildSettings.PropertyName);	\
	}																				\
}																					\
return false;

#else //WITH_EDITOR

#define IMPLEMENT_SKELETAL_BUILD_VALUE_TO_ASSET(AttributeName, AttributeType, PropertyName)	\
return false;

#define IMPLEMENT_SKELETALMESH_BUILD_ASSET_TO_VALUE(AttributeName, AttributeType, PropertyName)					\
return false;

#endif //else WITH_EDIOTR

UInterchangeSkeletalMeshFactoryNode::UInterchangeSkeletalMeshFactoryNode()
{
	AssetClass = nullptr;
}

void UInterchangeSkeletalMeshFactoryNode::InitializeSkeletalMeshNode(const FString& UniqueID, const FString& DisplayLabel, const FString& InAssetClass, UInterchangeBaseNodeContainer* NodeContainer)
{
	bIsNodeClassInitialized = false;
	NodeContainer->SetupNode(this, UniqueID, DisplayLabel, EInterchangeNodeContainerType::FactoryData);

	FString OperationName = GetTypeName() + TEXT(".SetAssetClassName");
	InterchangePrivateNodeBase::SetCustomAttribute<FString>(*Attributes, ClassNameAttributeKey, OperationName, InAssetClass);
	FillAssetClassFromAttribute();
}

FString UInterchangeSkeletalMeshFactoryNode::GetTypeName() const
{
	const FString TypeName = TEXT("SkeletalMeshNode");
	return TypeName;
}

UClass* UInterchangeSkeletalMeshFactoryNode::GetObjectClass() const
{
	ensure(bIsNodeClassInitialized);
	return AssetClass.Get() != nullptr ? AssetClass.Get() : USkeletalMesh::StaticClass();
}

bool UInterchangeSkeletalMeshFactoryNode::GetCustomSkeletonSoftObjectPath(FSoftObjectPath& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SkeletonSoftObjectPath, FSoftObjectPath)
}

bool UInterchangeSkeletalMeshFactoryNode::SetCustomSkeletonSoftObjectPath(const FSoftObjectPath& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SkeletonSoftObjectPath, FSoftObjectPath)
}

bool UInterchangeSkeletalMeshFactoryNode::GetCustomImportMorphTarget(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ImportMorphTarget, bool)
}

bool UInterchangeSkeletalMeshFactoryNode::SetCustomImportMorphTarget(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ImportMorphTarget, bool)
}

bool UInterchangeSkeletalMeshFactoryNode::GetCustomAddCurveMetadataToSkeleton(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AddCurveMetadataToSkeleton, bool);
}

bool UInterchangeSkeletalMeshFactoryNode::SetCustomAddCurveMetadataToSkeleton(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AddCurveMetadataToSkeleton, bool);
}

bool UInterchangeSkeletalMeshFactoryNode::GetCustomImportVertexAttributes(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ImportVertexAttributes, bool)
}

bool UInterchangeSkeletalMeshFactoryNode::SetCustomImportVertexAttributes(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ImportVertexAttributes, bool)
}

bool UInterchangeSkeletalMeshFactoryNode::GetCustomCreatePhysicsAsset(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(CreatePhysicsAsset, bool)
}

bool UInterchangeSkeletalMeshFactoryNode::SetCustomCreatePhysicsAsset(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(CreatePhysicsAsset, bool)
}

bool UInterchangeSkeletalMeshFactoryNode::GetCustomPhysicAssetSoftObjectPath(FSoftObjectPath& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(PhysicAssetSoftObjectPath, FSoftObjectPath)
}

bool UInterchangeSkeletalMeshFactoryNode::SetCustomPhysicAssetSoftObjectPath(const FSoftObjectPath& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(PhysicAssetSoftObjectPath, FSoftObjectPath)
}

bool UInterchangeSkeletalMeshFactoryNode::GetCustomUseHighPrecisionSkinWeights(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(UseHighPrecisionSkinWeights, bool)
}
bool UInterchangeSkeletalMeshFactoryNode::SetCustomUseHighPrecisionSkinWeights(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeSkeletalMeshFactoryNode, UseHighPrecisionSkinWeights, bool);
}
bool UInterchangeSkeletalMeshFactoryNode::ApplyCustomUseHighPrecisionSkinWeightsToAsset(UObject* Asset) const
{
	IMPLEMENT_SKELETAL_BUILD_VALUE_TO_ASSET(UseHighPrecisionSkinWeights, bool, bUseHighPrecisionSkinWeights);
}
bool UInterchangeSkeletalMeshFactoryNode::FillCustomUseHighPrecisionSkinWeightsFromAsset(UObject* Asset)
{
	IMPLEMENT_SKELETALMESH_BUILD_ASSET_TO_VALUE(UseHighPrecisionSkinWeights, bool, bUseHighPrecisionSkinWeights);
}

bool UInterchangeSkeletalMeshFactoryNode::GetCustomThresholdPosition(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ThresholdPosition, float)
}
bool UInterchangeSkeletalMeshFactoryNode::SetCustomThresholdPosition(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeSkeletalMeshFactoryNode, ThresholdPosition, float);
}
bool UInterchangeSkeletalMeshFactoryNode::ApplyCustomThresholdPositionToAsset(UObject* Asset) const
{
	IMPLEMENT_SKELETAL_BUILD_VALUE_TO_ASSET(ThresholdPosition, float, ThresholdPosition);
}
bool UInterchangeSkeletalMeshFactoryNode::FillCustomThresholdPositionFromAsset(UObject* Asset)
{
	IMPLEMENT_SKELETALMESH_BUILD_ASSET_TO_VALUE(ThresholdPosition, float, ThresholdPosition);
}

bool UInterchangeSkeletalMeshFactoryNode::GetCustomThresholdTangentNormal(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ThresholdTangentNormal, float)
}
bool UInterchangeSkeletalMeshFactoryNode::SetCustomThresholdTangentNormal(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeSkeletalMeshFactoryNode, ThresholdTangentNormal, float);
}
bool UInterchangeSkeletalMeshFactoryNode::ApplyCustomThresholdTangentNormalToAsset(UObject* Asset) const
{
	IMPLEMENT_SKELETAL_BUILD_VALUE_TO_ASSET(ThresholdTangentNormal, float, ThresholdTangentNormal);
}
bool UInterchangeSkeletalMeshFactoryNode::FillCustomThresholdTangentNormalFromAsset(UObject* Asset)
{
	IMPLEMENT_SKELETALMESH_BUILD_ASSET_TO_VALUE(ThresholdTangentNormal, float, ThresholdTangentNormal);
}

bool UInterchangeSkeletalMeshFactoryNode::GetCustomThresholdUV(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ThresholdUV, float)
}
bool UInterchangeSkeletalMeshFactoryNode::SetCustomThresholdUV(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeSkeletalMeshFactoryNode, ThresholdUV, float);
}
bool UInterchangeSkeletalMeshFactoryNode::ApplyCustomThresholdUVToAsset(UObject* Asset) const
{
	IMPLEMENT_SKELETAL_BUILD_VALUE_TO_ASSET(ThresholdUV, float, ThresholdUV);
}
bool UInterchangeSkeletalMeshFactoryNode::FillCustomThresholdUVFromAsset(UObject* Asset)
{
	IMPLEMENT_SKELETALMESH_BUILD_ASSET_TO_VALUE(ThresholdUV, float, ThresholdUV);
}

bool UInterchangeSkeletalMeshFactoryNode::GetCustomMorphThresholdPosition(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(MorphThresholdPosition, float)
}
bool UInterchangeSkeletalMeshFactoryNode::SetCustomMorphThresholdPosition(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeSkeletalMeshFactoryNode, MorphThresholdPosition, float);
}
bool UInterchangeSkeletalMeshFactoryNode::ApplyCustomMorphThresholdPositionToAsset(UObject* Asset) const
{
	IMPLEMENT_SKELETAL_BUILD_VALUE_TO_ASSET(MorphThresholdPosition, float, MorphThresholdPosition);
}
bool UInterchangeSkeletalMeshFactoryNode::FillCustomMorphThresholdPositionFromAsset(UObject* Asset)
{
	IMPLEMENT_SKELETALMESH_BUILD_ASSET_TO_VALUE(MorphThresholdPosition, float, MorphThresholdPosition);
}

bool UInterchangeSkeletalMeshFactoryNode::GetCustomBoneInfluenceLimit(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(BoneInfluenceLimit, int32)
}
bool UInterchangeSkeletalMeshFactoryNode::SetCustomBoneInfluenceLimit(const int32& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeSkeletalMeshFactoryNode, BoneInfluenceLimit, int32);
}
bool UInterchangeSkeletalMeshFactoryNode::ApplyCustomBoneInfluenceLimitToAsset(UObject* Asset) const
{
	IMPLEMENT_SKELETAL_BUILD_VALUE_TO_ASSET(BoneInfluenceLimit, int32, BoneInfluenceLimit);
}
bool UInterchangeSkeletalMeshFactoryNode::FillCustomBoneInfluenceLimitFromAsset(UObject* Asset)
{
	IMPLEMENT_SKELETALMESH_BUILD_ASSET_TO_VALUE(BoneInfluenceLimit, int32, BoneInfluenceLimit);
}

bool UInterchangeSkeletalMeshFactoryNode::GetCustomMergeMorphTargetShapeWithSameName(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(MergeMorphTargetShapeWithSameName, bool)
}
bool UInterchangeSkeletalMeshFactoryNode::SetCustomMergeMorphTargetShapeWithSameName(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(MergeMorphTargetShapeWithSameName, bool)
}

bool UInterchangeSkeletalMeshFactoryNode::GetCustomImportContentType(EInterchangeSkeletalMeshContentType& AttributeValue) const
{
	FString OperationName = GetTypeName() + TEXT(".GetImportContentType");
	uint8 EnumRawValue = 0;
	const bool bResult = InterchangePrivateNodeBase::GetCustomAttribute<uint8>(*Attributes, Macro_CustomImportContentTypeKey, OperationName, EnumRawValue);
	if (bResult)
	{
		AttributeValue = static_cast<EInterchangeSkeletalMeshContentType>(EnumRawValue);
	}
	return bResult;
}

bool UInterchangeSkeletalMeshFactoryNode::SetCustomImportContentType(const EInterchangeSkeletalMeshContentType& AttributeValue)
{
	FString OperationName = GetTypeName() + TEXT(".SetImportContentType");
	uint8 EnumRawValue = static_cast<uint8>(AttributeValue);
	return InterchangePrivateNodeBase::SetCustomAttribute<uint8>(*Attributes, Macro_CustomImportContentTypeKey, OperationName, EnumRawValue);
}

void UInterchangeSkeletalMeshFactoryNode::AppendAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::AppendAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UInterchangeSkeletalMeshFactoryNode::AppendAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::AppendAssetRegistryTags(Context);
#if WITH_EDITORONLY_DATA
	EInterchangeSkeletalMeshContentType ContentType;
	if (GetCustomImportContentType(ContentType))
	{
		auto ImportContentTypeToString = [](const EInterchangeSkeletalMeshContentType value)-> FString
		{
			switch (value)
			{
			case EInterchangeSkeletalMeshContentType::All:
				return NSSkeletalMeshSourceFileLabels::GeoAndSkinningMetaDataValue();
			case EInterchangeSkeletalMeshContentType::Geometry:
				return NSSkeletalMeshSourceFileLabels::GeometryMetaDataValue();
			case EInterchangeSkeletalMeshContentType::SkinningWeights:
				return NSSkeletalMeshSourceFileLabels::SkinningMetaDataValue();
			}
			return NSSkeletalMeshSourceFileLabels::GeoAndSkinningMetaDataValue();
		};

		FString EnumString = ImportContentTypeToString(ContentType);
		Context.AddTag(FAssetRegistryTag(NSSkeletalMeshSourceFileLabels::GetSkeletalMeshLastImportContentTypeMetadataKey(), EnumString, FAssetRegistryTag::TT_Hidden));
	}
#endif
}

void UInterchangeSkeletalMeshFactoryNode::FillAssetClassFromAttribute()
{
#if WITH_ENGINE
	FString OperationName = GetTypeName() + TEXT(".GetAssetClassName");
	FString ClassName;
	InterchangePrivateNodeBase::GetCustomAttribute<FString>(*Attributes, ClassNameAttributeKey, OperationName, ClassName);
	if (ClassName.Equals(USkeletalMesh::StaticClass()->GetName()))
	{
		AssetClass = USkeletalMesh::StaticClass();
		bIsNodeClassInitialized = true;
	}
#endif
}

bool UInterchangeSkeletalMeshFactoryNode::SetNodeClassFromClassAttribute()
{
	if (!bIsNodeClassInitialized)
	{
		FillAssetClassFromAttribute();
	}
	return bIsNodeClassInitialized;
}

void UInterchangeSkeletalMeshFactoryNode::CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object)
{
	Super::CopyWithObject(SourceNode, Object);

	if (const UInterchangeSkeletalMeshFactoryNode* MeshFactoryNode = Cast<UInterchangeSkeletalMeshFactoryNode>(SourceNode))
	{
		UClass* Class = GetObjectClass();

		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(MeshFactoryNode, UInterchangeSkeletalMeshFactoryNode, UseHighPrecisionSkinWeights, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(MeshFactoryNode, UInterchangeSkeletalMeshFactoryNode, ThresholdPosition, float, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(MeshFactoryNode, UInterchangeSkeletalMeshFactoryNode, ThresholdTangentNormal, float, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(MeshFactoryNode, UInterchangeSkeletalMeshFactoryNode, ThresholdUV, float, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(MeshFactoryNode, UInterchangeSkeletalMeshFactoryNode, MorphThresholdPosition, float, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(MeshFactoryNode, UInterchangeSkeletalMeshFactoryNode, BoneInfluenceLimit, int32, Class)
	}
}
