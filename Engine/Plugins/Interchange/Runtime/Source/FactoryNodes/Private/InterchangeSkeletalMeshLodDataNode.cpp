// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeSkeletalMeshLodDataNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSkeletalMeshLodDataNode)

//Interchange namespace
namespace UE::Interchange
{
	const FAttributeKey& FSkeletalMeshNodeLodDataStaticData::GetLODMeshDataArrayBaseKey()
	{
		static FAttributeKey LODMeshDataArray_BaseKey(TEXT("__LODMeshDataArray__Key"));
		return LODMeshDataArray_BaseKey;
	}
}//ns UE::Interchange

UInterchangeSkeletalMeshLodDataNode::UInterchangeSkeletalMeshLodDataNode()
{
	LODMeshDataArray.Initialize(Attributes, UE::Interchange::FSkeletalMeshNodeLodDataStaticData::GetLODMeshDataArrayBaseKey().ToString());
}

/**
	* Return the node type name of the class. This is used when reporting errors.
	*/
FString UInterchangeSkeletalMeshLodDataNode::GetTypeName() const
{
	const FString TypeName = TEXT("SkeletalMeshLodDataNode");
	return TypeName;
}

#if WITH_EDITOR

FString UInterchangeSkeletalMeshLodDataNode::GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	FString KeyDisplayName = NodeAttributeKey.ToString();
	const FString NodeAttributeKeyString = KeyDisplayName;
	if (NodeAttributeKey == UE::Interchange::FSkeletalMeshNodeLodDataStaticData::GetLODMeshDataArrayBaseKey())
	{
		return KeyDisplayName = TEXT("Mesh count");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FSkeletalMeshNodeLodDataStaticData::GetLODMeshDataArrayBaseKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Mesh index "), NodeAttributeKeyString);
	}
	else if (NodeAttributeKey == Macro_CustomSkeletonUidKey)
	{
		return KeyDisplayName = TEXT("Skeleton factory node");
	}
	return Super::GetKeyDisplayName(NodeAttributeKey);
}

FString UInterchangeSkeletalMeshLodDataNode::GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	if (NodeAttributeKey.ToString().StartsWith(UE::Interchange::FSkeletalMeshNodeLodDataStaticData::GetLODMeshDataArrayBaseKey().ToString()))
	{
		return FString(TEXT("Meshes"));
	}
	return Super::GetAttributeCategory(NodeAttributeKey);
}

#endif //WITH_EDITOR

bool UInterchangeSkeletalMeshLodDataNode::GetCustomSkeletonUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SkeletonUid, FString);
}

bool UInterchangeSkeletalMeshLodDataNode::SetCustomSkeletonUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SkeletonUid, FString)
}

int32 UInterchangeSkeletalMeshLodDataNode::GetMeshUidsCount() const
{
	return LODMeshDataArray.GetCount();
}

void UInterchangeSkeletalMeshLodDataNode::GetMeshUids(TArray<FString>& OutMeshUids) const
{
	TArray<FInterchangeLODMeshData> LODMeshDataArrayArray;
	LODMeshDataArray.GetItems(LODMeshDataArrayArray);

	OutMeshUids.Empty();
	OutMeshUids.Reserve(LODMeshDataArrayArray.Num());
	for (const FInterchangeLODMeshData& Pair : LODMeshDataArrayArray)
	{
		OutMeshUids.Add(Pair.MeshUid);
	}
}

bool UInterchangeSkeletalMeshLodDataNode::AddMeshUid(const FString& MeshUid)
{
	return AddLODMeshData(FInterchangeLODMeshData(MeshUid, FTransform::Identity));
}

void UInterchangeSkeletalMeshLodDataNode::GetLODMeshDataArray(TArray<FInterchangeLODMeshData>& OutLODMeshDataArray) const
{
	LODMeshDataArray.GetItems(OutLODMeshDataArray);
}

bool UInterchangeSkeletalMeshLodDataNode::AddLODMeshData(const FInterchangeLODMeshData& LODMeshData)
{
	return LODMeshDataArray.AddItem(LODMeshData);
}

bool UInterchangeSkeletalMeshLodDataNode::RemoveAllMeshes()
{
	return LODMeshDataArray.RemoveAllItems();
}

void UInterchangeSkeletalMeshLodDataNode::RemoveMeshUid(const FString& MeshUid)
{
	TArray<FInterchangeLODMeshData> LODMeshDataArrayArray;
	LODMeshDataArray.GetItems(LODMeshDataArrayArray);
	LODMeshDataArray.RemoveAllItems();

	for (const FInterchangeLODMeshData& LODMeshData : LODMeshDataArrayArray)
	{
		if (LODMeshData.MeshUid != MeshUid)
		{
			LODMeshDataArray.AddItem(LODMeshData);
		}
	}
}

bool UInterchangeSkeletalMeshLodDataNode::IsEditorOnlyDataDefined()
{
#if WITH_EDITORONLY_DATA
	return true;
#else
	return false;
#endif
}

