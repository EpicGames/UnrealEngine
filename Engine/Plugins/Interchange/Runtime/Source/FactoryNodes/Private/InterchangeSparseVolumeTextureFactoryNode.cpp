// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeSparseVolumeTextureFactoryNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSparseVolumeTextureFactoryNode)

#if WITH_ENGINE
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#endif

FString UInterchangeSparseVolumeTextureFactoryNode::GetTypeName() const
{
	const FString TypeName = TEXT("SparseVolumeTextureFactoryNode");
	return TypeName;
}

UClass* UInterchangeSparseVolumeTextureFactoryNode::GetObjectClass() const
{
#if WITH_ENGINE
	return USparseVolumeTexture::StaticClass();
#else
	return nullptr;
#endif
}

bool UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesAFormat(EInterchangeSparseVolumeTextureFormat& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttributesADataType, EInterchangeSparseVolumeTextureFormat);
}

bool UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAFormat(EInterchangeSparseVolumeTextureFormat AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AttributesADataType, EInterchangeSparseVolumeTextureFormat);
}

bool UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesBFormat(EInterchangeSparseVolumeTextureFormat& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttributesBDataType, EInterchangeSparseVolumeTextureFormat);
}

bool UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesBFormat(EInterchangeSparseVolumeTextureFormat AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AttributesBDataType, EInterchangeSparseVolumeTextureFormat);
}

bool UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesAChannelX(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttributesAChannelX, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAChannelX(FString AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AttributesAChannelX, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesAChannelY(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttributesAChannelY, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAChannelY(FString AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AttributesAChannelY, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesAChannelZ(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttributesAChannelZ, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAChannelZ(FString AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AttributesAChannelZ, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesAChannelW(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttributesAChannelW, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAChannelW(FString AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AttributesAChannelW, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesBChannelX(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttributesBChannelX, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesBChannelX(FString AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AttributesBChannelX, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesBChannelY(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttributesBChannelY, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesBChannelY(FString AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AttributesBChannelY, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesBChannelZ(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttributesBChannelZ, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesBChannelZ(FString AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AttributesBChannelZ, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesBChannelW(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttributesBChannelW, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesBChannelW(FString AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AttributesBChannelW, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::GetCustomAnimationID(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AnimationID, FString);
}

bool UInterchangeSparseVolumeTextureFactoryNode::SetCustomAnimationID(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AnimationID, FString);
}
