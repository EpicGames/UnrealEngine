// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeSpecularProfileFactoryNode.h"
#include "Engine/SpecularProfile.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSpecularProfileFactoryNode)

FString UInterchangeSpecularProfileFactoryNode::GetTypeName() const
{
	return TEXT("SpecularProfileFactoryNode");
}

UClass* UInterchangeSpecularProfileFactoryNode::GetObjectClass() const
{
	return USpecularProfile::StaticClass();
}

bool UInterchangeSpecularProfileFactoryNode::SetCustomFormat(ESpecularProfileFormat AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Format, ESpecularProfileFormat);
}

bool UInterchangeSpecularProfileFactoryNode::GetCustomFormat(ESpecularProfileFormat& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Format, ESpecularProfileFormat);
}

bool UInterchangeSpecularProfileFactoryNode::GetCustomTexture(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Texture, FString);
}

bool UInterchangeSpecularProfileFactoryNode::SetCustomTexture(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Texture, FString)
}
