// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeStaticMeshFactoryNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeStaticMeshFactoryNode)

// Verify that our Interchange-level Nanite enums match the engine enums value-for-value
static_assert((int32)EInterchangeNaniteShapePreservation::None == (int32)ENaniteShapePreservation::None);
static_assert((int32)EInterchangeNaniteShapePreservation::PreserveArea == (int32)ENaniteShapePreservation::PreserveArea);
static_assert((int32)EInterchangeNaniteShapePreservation::Voxelize == (int32)ENaniteShapePreservation::Voxelize);
static_assert((int32)EInterchangeNaniteGenerateFallback::PlatformDefault == (int32)ENaniteGenerateFallback::PlatformDefault);
static_assert((int32)EInterchangeNaniteGenerateFallback::Enabled == (int32)ENaniteGenerateFallback::Enabled);
static_assert((int32)EInterchangeNaniteFallbackTarget::Auto == (int32)ENaniteFallbackTarget::Auto);
static_assert((int32)EInterchangeNaniteFallbackTarget::PercentTriangles == (int32)ENaniteFallbackTarget::PercentTriangles);
static_assert((int32)EInterchangeNaniteFallbackTarget::RelativeError == (int32)ENaniteFallbackTarget::RelativeError);

#if WITH_EDITOR

//We set the build value to all LODs
#define IMPLEMENT_STATICMESH_BUILD_VALUE_TO_ASSET(AttributeName, AttributeType, PropertyName)	\
bool bResult = false;																\
AttributeType ValueData;															\
if (GetAttribute<AttributeType>(Macro_Custom##AttributeName##Key.Key, ValueData))	\
{																					\
	if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))							\
	{																				\
		for(int32 LodIndex = 0; LodIndex < StaticMesh->GetNumSourceModels(); ++LodIndex) \
		{																			\
			if (StaticMesh->IsSourceModelValid(LodIndex))							\
			{																		\
				StaticMesh->GetSourceModel(LodIndex).BuildSettings.PropertyName = ValueData;	\
				bResult = true;														\
			}																		\
		}																			\
	}																				\
}																					\
return bResult;

//We take only the lod zero to get back the value
#define IMPLEMENT_STATICMESH_BUILD_ASSET_TO_VALUE(AttributeName, AttributeType, PropertyName)	\
if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))								\
{																					\
	if (StaticMesh->GetNumSourceModels() > 0)										\
	{																				\
		return SetAttribute<AttributeType>(Macro_Custom##AttributeName##Key.Key, StaticMesh->GetSourceModel(0).BuildSettings.PropertyName);	\
	}																				\
}																					\
return false;

#define IMPLEMENT_NANITE_VALUE_TO_ASSET(AttributeName, AttributeType, PropertyName)	\
bool bResult = false;																\
AttributeType ValueData;															\
if (GetAttribute<AttributeType>(Macro_Custom##AttributeName##Key.Key, ValueData))	\
{																					\
	if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))							\
	{																				\
		StaticMesh->GetNaniteSettings().PropertyName = ValueData;					\
		bResult = true;																\
	}																				\
}																					\
return bResult;

#define IMPLEMENT_NANITE_ASSET_TO_VALUE(AttributeName, AttributeType, PropertyName)	\
if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))								\
{																					\
	return SetAttribute<AttributeType>(Macro_Custom##AttributeName##Key.Key, StaticMesh->GetNaniteSettings().PropertyName);	\
}																					\
return false;

#define IMPLEMENT_NANITE_ENUM_VALUE_TO_ASSET(AttributeName, InterchangeEnumType, EngineEnumType, PropertyName)	\
bool bResult = false;																\
int32 ValueData;																	\
if (GetAttribute<int32>(Macro_Custom##AttributeName##Key.Key, ValueData))			\
{																					\
	if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))							\
	{																				\
		StaticMesh->GetNaniteSettings().PropertyName = static_cast<EngineEnumType>(ValueData);	\
		bResult = true;																\
	}																				\
}																					\
return bResult;

#define IMPLEMENT_NANITE_ENUM_ASSET_TO_VALUE(AttributeName, InterchangeEnumType, EngineEnumType, PropertyName)	\
if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))								\
{																					\
	return SetAttribute<int32>(Macro_Custom##AttributeName##Key.Key, static_cast<int32>(StaticMesh->GetNaniteSettings().PropertyName));	\
}																					\
return false;

#else //WITH_EDITOR

#define IMPLEMENT_STATICMESH_BUILD_VALUE_TO_ASSET(AttributeName, AttributeType, PropertyName)	\
return false;

#define IMPLEMENT_STATICMESH_BUILD_ASSET_TO_VALUE(AttributeName, AttributeType, PropertyName)	\
return false;

#define IMPLEMENT_NANITE_VALUE_TO_ASSET(AttributeName, AttributeType, PropertyName)	\
return false;

#define IMPLEMENT_NANITE_ASSET_TO_VALUE(AttributeName, AttributeType, PropertyName)	\
return false;

#define IMPLEMENT_NANITE_ENUM_VALUE_TO_ASSET(AttributeName, InterchangeEnumType, EngineEnumType, PropertyName)	\
return false;

#define IMPLEMENT_NANITE_ENUM_ASSET_TO_VALUE(AttributeName, InterchangeEnumType, EngineEnumType, PropertyName)	\
return false;

#endif // else WITH_EDITOR

namespace UE
{
	namespace Interchange
	{
		const FAttributeKey& FStaticMeshNodeStaticData::GetLODScreenSizeBaseKey()
		{
			static FAttributeKey LODScreenSize_BaseKey(TEXT("__LODScreenSize__"));
			return LODScreenSize_BaseKey;
		}

		const FAttributeKey& FStaticMeshNodeStaticData::GetSocketUidsBaseKey()
		{
			static FAttributeKey SocketUids_BaseKey = FAttributeKey(TEXT("SocketUids"));
			return SocketUids_BaseKey;
		}
	} // namespace Interchange
} // namespace UE


UInterchangeStaticMeshFactoryNode::UInterchangeStaticMeshFactoryNode()
{
#if WITH_ENGINE
	AssetClass = nullptr;
#endif
	LODScreenSizes.Initialize(Attributes.ToSharedRef(), UE::Interchange::FStaticMeshNodeStaticData::GetLODScreenSizeBaseKey().ToString());
	SocketUids.Initialize(Attributes, UE::Interchange::FStaticMeshNodeStaticData::GetSocketUidsBaseKey().ToString());
}

void UInterchangeStaticMeshFactoryNode::InitializeStaticMeshNode(const FString& UniqueID, const FString& DisplayLabel, const FString& InAssetClass, UInterchangeBaseNodeContainer* NodeContainer)
{
	bIsNodeClassInitialized = false;
	NodeContainer->SetupNode(this, UniqueID, DisplayLabel, EInterchangeNodeContainerType::FactoryData);

	FString OperationName = GetTypeName() + TEXT(".SetAssetClassName");
	InterchangePrivateNodeBase::SetCustomAttribute<FString>(*Attributes, ClassNameAttributeKey, OperationName, InAssetClass);
	FillAssetClassFromAttribute();
}

FString UInterchangeStaticMeshFactoryNode::GetTypeName() const
{
	const FString TypeName = TEXT("StaticMeshNode");
	return TypeName;
}

UClass* UInterchangeStaticMeshFactoryNode::GetObjectClass() const
{
	ensure(bIsNodeClassInitialized);
#if WITH_ENGINE
	return AssetClass.Get() != nullptr ? AssetClass.Get() : UStaticMesh::StaticClass();
#else
	return nullptr;
#endif
}

void UInterchangeStaticMeshFactoryNode::CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object)
{
	Super::CopyWithObject(SourceNode, Object);

	if (const UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode = Cast<UInterchangeStaticMeshFactoryNode>(SourceNode))
	{
		UClass* Class = GetObjectClass();
#if WITH_EDITORONLY_DATA
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, BuildNanite, bool, Class)
#endif
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, BuildReversedIndexBuffer, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, GenerateLightmapUVs, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, GenerateDistanceFieldAsIfTwoSided, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, SupportFaceRemap, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, MinLightmapResolution, int32, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, SrcLightmapIndex, int32, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, DstLightmapIndex, int32, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, BuildScale3D, FVector, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, DistanceFieldResolutionScale, float, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, DistanceFieldReplacementMesh, FSoftObjectPath, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, MaxLumenMeshCards, int32, Class)

		// Nanite build settings
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteExplicitTangents, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteLerpUVs, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteSeparable, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteVoxelNDF, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteVoxelOpacity, bool, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteShapePreservation, int32, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NanitePositionPrecision, int32, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteNormalPrecision, int32, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteTangentPrecision, int32, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteBoneWeightPrecision, int32, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteTargetMinimumResidencyInKB, int32, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteKeepPercentTriangles, float, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteTrimRelativeError, float, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteGenerateFallback, int32, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteFallbackTarget, int32, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteFallbackPercentTriangles, float, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteFallbackRelativeError, float, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteMaxEdgeLengthFactor, float, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteNumRays, int32, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteVoxelLevel, int32, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteRayBackUp, float, Class)
		COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(StaticMeshFactoryNode, UInterchangeStaticMeshFactoryNode, NaniteDisplacementUVChannel, int32, Class)
	}
}

#if WITH_EDITOR

FString UInterchangeStaticMeshFactoryNode::GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	FString KeyDisplayName = NodeAttributeKey.ToString();
	const FString OriginalKeyName = KeyDisplayName;
	if (NodeAttributeKey == UE::Interchange::FStaticMeshNodeStaticData::GetSocketUidsBaseKey())
	{
		KeyDisplayName = TEXT("Socket Count");
	}
	else if (NodeAttributeKey == UE::Interchange::FStaticMeshNodeStaticData::GetSocketUidsBaseKey())
	{
		KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Socket Index "), OriginalKeyName);
	}
	else
	{
		KeyDisplayName = Super::GetKeyDisplayName(NodeAttributeKey);
	}

	return KeyDisplayName;
}

FString UInterchangeStaticMeshFactoryNode::GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	if (NodeAttributeKey.ToString().StartsWith(UE::Interchange::FStaticMeshNodeStaticData::GetSocketUidsBaseKey().ToString()))
	{
		return TEXT("Sockets");
	}
	else
	{
		return Super::GetAttributeCategory(NodeAttributeKey);
	}
}

#endif //WITH_EDITOR

bool UInterchangeStaticMeshFactoryNode::GetCustomAutoComputeLODScreenSizes(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AutoComputeLODScreenSizes, bool)
}

bool UInterchangeStaticMeshFactoryNode::SetCustomAutoComputeLODScreenSizes(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AutoComputeLODScreenSizes, bool)
}

int32 UInterchangeStaticMeshFactoryNode::GetLODScreenSizeCount() const
{
	return LODScreenSizes.GetCount();
}

void UInterchangeStaticMeshFactoryNode::GetLODScreenSizes(TArray<float>& OutLODScreenSizes) const
{
	LODScreenSizes.GetItems(OutLODScreenSizes);
}

bool UInterchangeStaticMeshFactoryNode::SetLODScreenSizes(const TArray<float>& InLODScreenSizes)
{
	LODScreenSizes.RemoveAllItems();
	for (const float& ScreenSize: InLODScreenSizes)
	{
		LODScreenSizes.AddItem(ScreenSize);
	}

	return true;
}

bool UInterchangeStaticMeshFactoryNode::GetCustomBuildNanite(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(BuildNanite, bool)
}

bool UInterchangeStaticMeshFactoryNode::SetCustomBuildNanite(const bool& AttributeValue, bool bAddApplyDelegate /*= true*/)
{
#if WITH_EDITORONLY_DATA
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, BuildNanite, bool);
#else
	return false;
#endif
}

int32 UInterchangeStaticMeshFactoryNode::GetSocketUidCount() const
{
	return SocketUids.GetCount();
}

void UInterchangeStaticMeshFactoryNode::GetSocketUids(TArray<FString>& OutSocketUids) const
{
	SocketUids.GetItems(OutSocketUids);
}

bool UInterchangeStaticMeshFactoryNode::AddSocketUid(const FString& SocketUid)
{
	return SocketUids.AddItem(SocketUid);
}

bool UInterchangeStaticMeshFactoryNode::AddSocketUids(const TArray<FString>& InSocketUids)
{
	for (const FString& SocketUid : InSocketUids)
	{
		if (!SocketUids.AddItem(SocketUid))
		{
			return false;
		}
	}

	return true;
}

bool UInterchangeStaticMeshFactoryNode::RemoveSocketUd(const FString& SocketUid)
{
	return SocketUids.RemoveItem(SocketUid);
}

void UInterchangeStaticMeshFactoryNode::FillAssetClassFromAttribute()
{
#if WITH_ENGINE
	FString OperationName = GetTypeName() + TEXT(".GetAssetClassName");
	FString ClassName;
	InterchangePrivateNodeBase::GetCustomAttribute<FString>(*Attributes, ClassNameAttributeKey, OperationName, ClassName);
	if (ClassName.Equals(UStaticMesh::StaticClass()->GetName()))
	{
		AssetClass = UStaticMesh::StaticClass();
		bIsNodeClassInitialized = true;
	}
#endif
}

bool UInterchangeStaticMeshFactoryNode::SetNodeClassFromClassAttribute()
{
	if (!bIsNodeClassInitialized)
	{
		FillAssetClassFromAttribute();
	}
	return bIsNodeClassInitialized;
}

bool UInterchangeStaticMeshFactoryNode::GetCustomBuildReversedIndexBuffer(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(BuildReversedIndexBuffer, bool)
}
bool UInterchangeStaticMeshFactoryNode::SetCustomBuildReversedIndexBuffer(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, BuildReversedIndexBuffer, bool);
}
bool UInterchangeStaticMeshFactoryNode::ApplyCustomBuildReversedIndexBufferToAsset(UObject* Asset) const
{
	IMPLEMENT_STATICMESH_BUILD_VALUE_TO_ASSET(BuildReversedIndexBuffer, bool, bBuildReversedIndexBuffer);
}
bool UInterchangeStaticMeshFactoryNode::FillCustomBuildReversedIndexBufferFromAsset(UObject* Asset)
{
	IMPLEMENT_STATICMESH_BUILD_ASSET_TO_VALUE(BuildReversedIndexBuffer, bool, bBuildReversedIndexBuffer);
}

bool UInterchangeStaticMeshFactoryNode::GetCustomGenerateLightmapUVs(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GenerateLightmapUVs, bool)
}
bool UInterchangeStaticMeshFactoryNode::SetCustomGenerateLightmapUVs(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, GenerateLightmapUVs, bool);
}
bool UInterchangeStaticMeshFactoryNode::ApplyCustomGenerateLightmapUVsToAsset(UObject* Asset) const
{
	IMPLEMENT_STATICMESH_BUILD_VALUE_TO_ASSET(GenerateLightmapUVs, bool, bGenerateLightmapUVs);
}
bool UInterchangeStaticMeshFactoryNode::FillCustomGenerateLightmapUVsFromAsset(UObject* Asset)
{
	IMPLEMENT_STATICMESH_BUILD_ASSET_TO_VALUE(GenerateLightmapUVs, bool, bGenerateLightmapUVs);
}

bool UInterchangeStaticMeshFactoryNode::GetCustomGenerateDistanceFieldAsIfTwoSided(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GenerateDistanceFieldAsIfTwoSided, bool)
}
bool UInterchangeStaticMeshFactoryNode::SetCustomGenerateDistanceFieldAsIfTwoSided(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, GenerateDistanceFieldAsIfTwoSided, bool);
}
bool UInterchangeStaticMeshFactoryNode::ApplyCustomGenerateDistanceFieldAsIfTwoSidedToAsset(UObject* Asset) const
{
	IMPLEMENT_STATICMESH_BUILD_VALUE_TO_ASSET(GenerateDistanceFieldAsIfTwoSided, bool, bGenerateDistanceFieldAsIfTwoSided);

}
bool UInterchangeStaticMeshFactoryNode::FillCustomGenerateDistanceFieldAsIfTwoSidedFromAsset(UObject* Asset)
{
	IMPLEMENT_STATICMESH_BUILD_ASSET_TO_VALUE(GenerateDistanceFieldAsIfTwoSided, bool, bGenerateDistanceFieldAsIfTwoSided);
}

bool UInterchangeStaticMeshFactoryNode::GetCustomSupportFaceRemap(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SupportFaceRemap, bool)
}
bool UInterchangeStaticMeshFactoryNode::SetCustomSupportFaceRemap(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, SupportFaceRemap, bool);
}
bool UInterchangeStaticMeshFactoryNode::ApplyCustomSupportFaceRemapToAsset(UObject* Asset) const
{
	IMPLEMENT_STATICMESH_BUILD_VALUE_TO_ASSET(SupportFaceRemap, bool, bSupportFaceRemap);
}
bool UInterchangeStaticMeshFactoryNode::FillCustomSupportFaceRemapFromAsset(UObject* Asset)
{
	IMPLEMENT_STATICMESH_BUILD_ASSET_TO_VALUE(SupportFaceRemap, bool, bSupportFaceRemap);
}

bool UInterchangeStaticMeshFactoryNode::GetCustomMinLightmapResolution(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(MinLightmapResolution, int32)
}
bool UInterchangeStaticMeshFactoryNode::SetCustomMinLightmapResolution(const int32& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, MinLightmapResolution, int32);
}
bool UInterchangeStaticMeshFactoryNode::ApplyCustomMinLightmapResolutionToAsset(UObject* Asset) const
{
	IMPLEMENT_STATICMESH_BUILD_VALUE_TO_ASSET(MinLightmapResolution, int32, MinLightmapResolution);
}
bool UInterchangeStaticMeshFactoryNode::FillCustomMinLightmapResolutionFromAsset(UObject* Asset)
{
	IMPLEMENT_STATICMESH_BUILD_ASSET_TO_VALUE(MinLightmapResolution, int32, MinLightmapResolution);
}

bool UInterchangeStaticMeshFactoryNode::GetCustomSrcLightmapIndex(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SrcLightmapIndex, int32)
}
bool UInterchangeStaticMeshFactoryNode::SetCustomSrcLightmapIndex(const int32& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, SrcLightmapIndex, int32);
}
bool UInterchangeStaticMeshFactoryNode::ApplyCustomSrcLightmapIndexToAsset(UObject* Asset) const
{
	IMPLEMENT_STATICMESH_BUILD_VALUE_TO_ASSET(SrcLightmapIndex, int32, SrcLightmapIndex);
}
bool UInterchangeStaticMeshFactoryNode::FillCustomSrcLightmapIndexFromAsset(UObject* Asset)
{
	IMPLEMENT_STATICMESH_BUILD_ASSET_TO_VALUE(SrcLightmapIndex, int32, SrcLightmapIndex);
}

bool UInterchangeStaticMeshFactoryNode::GetCustomDstLightmapIndex(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DstLightmapIndex, int32)
}
bool UInterchangeStaticMeshFactoryNode::SetCustomDstLightmapIndex(const int32& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, DstLightmapIndex, int32);
}
bool UInterchangeStaticMeshFactoryNode::ApplyCustomDstLightmapIndexToAsset(UObject* Asset) const
{
	IMPLEMENT_STATICMESH_BUILD_VALUE_TO_ASSET(DstLightmapIndex, int32, DstLightmapIndex);
}
bool UInterchangeStaticMeshFactoryNode::FillCustomDstLightmapIndexFromAsset(UObject* Asset)
{
	IMPLEMENT_STATICMESH_BUILD_ASSET_TO_VALUE(DstLightmapIndex, int32, DstLightmapIndex);
}

bool UInterchangeStaticMeshFactoryNode::GetCustomBuildScale3D(FVector& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(BuildScale3D, FVector)
}
bool UInterchangeStaticMeshFactoryNode::SetCustomBuildScale3D(const FVector& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, BuildScale3D, FVector);
}
bool UInterchangeStaticMeshFactoryNode::ApplyCustomBuildScale3DToAsset(UObject* Asset) const
{
	IMPLEMENT_STATICMESH_BUILD_VALUE_TO_ASSET(BuildScale3D, FVector, BuildScale3D);
}
bool UInterchangeStaticMeshFactoryNode::FillCustomBuildScale3DFromAsset(UObject* Asset)
{
	IMPLEMENT_STATICMESH_BUILD_ASSET_TO_VALUE(BuildScale3D, FVector, BuildScale3D);
}

bool UInterchangeStaticMeshFactoryNode::GetCustomDistanceFieldResolutionScale(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DistanceFieldResolutionScale, float)
}
bool UInterchangeStaticMeshFactoryNode::SetCustomDistanceFieldResolutionScale(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, DistanceFieldResolutionScale, float);
}
bool UInterchangeStaticMeshFactoryNode::ApplyCustomDistanceFieldResolutionScaleToAsset(UObject* Asset) const
{
	IMPLEMENT_STATICMESH_BUILD_VALUE_TO_ASSET(DistanceFieldResolutionScale, float, DistanceFieldResolutionScale);
}
bool UInterchangeStaticMeshFactoryNode::FillCustomDistanceFieldResolutionScaleFromAsset(UObject* Asset)
{
	IMPLEMENT_STATICMESH_BUILD_ASSET_TO_VALUE(DistanceFieldResolutionScale, float, DistanceFieldResolutionScale);
}

bool UInterchangeStaticMeshFactoryNode::GetCustomDistanceFieldReplacementMesh(FSoftObjectPath& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DistanceFieldReplacementMesh, FSoftObjectPath)
}
bool UInterchangeStaticMeshFactoryNode::SetCustomDistanceFieldReplacementMesh(const FSoftObjectPath& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, DistanceFieldReplacementMesh, FSoftObjectPath);
}
bool UInterchangeStaticMeshFactoryNode::ApplyCustomDistanceFieldReplacementMeshToAsset(UObject* Asset) const
{
#if WITH_EDITOR
	FSoftObjectPath ValueData;
	if (GetAttribute<FSoftObjectPath>(Macro_CustomDistanceFieldReplacementMeshKey.Key, ValueData))
	{
		if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))
		{
			if (StaticMesh->GetNumSourceModels() > 0)
			{
				StaticMesh->GetSourceModel(0).BuildSettings.DistanceFieldReplacementMesh = Cast<UStaticMesh>(ValueData.TryLoad());
				return true;
			}
		}
	}
#endif //WITH_EDITOR
	return false;
}
bool UInterchangeStaticMeshFactoryNode::FillCustomDistanceFieldReplacementMeshFromAsset(UObject* Asset)
{
#if WITH_EDITOR
	if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))
	{
		if (StaticMesh->GetNumSourceModels() > 0)
		{
			FSoftObjectPath SoftObjectPath(StaticMesh->GetSourceModel(0).BuildSettings.DistanceFieldReplacementMesh);
			return SetAttribute(Macro_CustomDistanceFieldReplacementMeshKey.Key, SoftObjectPath);
		}
	}
#endif //WITH_EDITOR
	return false;
}

bool UInterchangeStaticMeshFactoryNode::GetCustomMaxLumenMeshCards(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(MaxLumenMeshCards, int32)
}
bool UInterchangeStaticMeshFactoryNode::SetCustomMaxLumenMeshCards(const int32& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, MaxLumenMeshCards, int32);
}
bool UInterchangeStaticMeshFactoryNode::ApplyCustomMaxLumenMeshCardsToAsset(UObject* Asset) const
{
	IMPLEMENT_STATICMESH_BUILD_VALUE_TO_ASSET(MaxLumenMeshCards, int32, MaxLumenMeshCards);
}
bool UInterchangeStaticMeshFactoryNode::FillCustomMaxLumenMeshCardsFromAsset(UObject* Asset)
{
	IMPLEMENT_STATICMESH_BUILD_ASSET_TO_VALUE(MaxLumenMeshCards, int32, MaxLumenMeshCards);
}

bool UInterchangeStaticMeshFactoryNode::SetCustomSubstituteUID(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SubstituteUID, FString)
}

bool UInterchangeStaticMeshFactoryNode::GetCustomSubstituteUID(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SubstituteUID, FString)
}

// Nanite Build Settings - bool properties
#define IMPLEMENT_NANITE_BOOL_ACCESSORS(AttributeName, NaniteProperty) \
bool UInterchangeStaticMeshFactoryNode::GetCustom##AttributeName(bool& AttributeValue) const \
{ \
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttributeName, bool) \
} \
bool UInterchangeStaticMeshFactoryNode::SetCustom##AttributeName(const bool& AttributeValue, bool bAddApplyDelegate) \
{ \
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, AttributeName, bool); \
} \
bool UInterchangeStaticMeshFactoryNode::ApplyCustom##AttributeName##ToAsset(UObject* Asset) const \
{ \
	IMPLEMENT_NANITE_VALUE_TO_ASSET(AttributeName, bool, NaniteProperty); \
} \
bool UInterchangeStaticMeshFactoryNode::FillCustom##AttributeName##FromAsset(UObject* Asset) \
{ \
	IMPLEMENT_NANITE_ASSET_TO_VALUE(AttributeName, bool, NaniteProperty); \
}

IMPLEMENT_NANITE_BOOL_ACCESSORS(NaniteExplicitTangents, bExplicitTangents)
IMPLEMENT_NANITE_BOOL_ACCESSORS(NaniteLerpUVs, bLerpUVs)
IMPLEMENT_NANITE_BOOL_ACCESSORS(NaniteSeparable, bSeparable)
IMPLEMENT_NANITE_BOOL_ACCESSORS(NaniteVoxelNDF, bVoxelNDF)
IMPLEMENT_NANITE_BOOL_ACCESSORS(NaniteVoxelOpacity, bVoxelOpacity)

#undef IMPLEMENT_NANITE_BOOL_ACCESSORS

// Nanite Build Settings - int32 properties
#define IMPLEMENT_NANITE_INT32_ACCESSORS(AttributeName, NaniteProperty) \
bool UInterchangeStaticMeshFactoryNode::GetCustom##AttributeName(int32& AttributeValue) const \
{ \
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttributeName, int32) \
} \
bool UInterchangeStaticMeshFactoryNode::SetCustom##AttributeName(const int32& AttributeValue, bool bAddApplyDelegate) \
{ \
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, AttributeName, int32); \
} \
bool UInterchangeStaticMeshFactoryNode::ApplyCustom##AttributeName##ToAsset(UObject* Asset) const \
{ \
	IMPLEMENT_NANITE_VALUE_TO_ASSET(AttributeName, int32, NaniteProperty); \
} \
bool UInterchangeStaticMeshFactoryNode::FillCustom##AttributeName##FromAsset(UObject* Asset) \
{ \
	IMPLEMENT_NANITE_ASSET_TO_VALUE(AttributeName, int32, NaniteProperty); \
}

IMPLEMENT_NANITE_INT32_ACCESSORS(NanitePositionPrecision, PositionPrecision)
IMPLEMENT_NANITE_INT32_ACCESSORS(NaniteNormalPrecision, NormalPrecision)
IMPLEMENT_NANITE_INT32_ACCESSORS(NaniteTangentPrecision, TangentPrecision)
IMPLEMENT_NANITE_INT32_ACCESSORS(NaniteBoneWeightPrecision, BoneWeightPrecision)
IMPLEMENT_NANITE_INT32_ACCESSORS(NaniteNumRays, NumRays)
IMPLEMENT_NANITE_INT32_ACCESSORS(NaniteVoxelLevel, VoxelLevel)
IMPLEMENT_NANITE_INT32_ACCESSORS(NaniteDisplacementUVChannel, DisplacementUVChannel)

#undef IMPLEMENT_NANITE_INT32_ACCESSORS

// Nanite Build Settings - TargetMinimumResidencyInKB (stored as int32, applied as uint32)
bool UInterchangeStaticMeshFactoryNode::GetCustomNaniteTargetMinimumResidencyInKB(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(NaniteTargetMinimumResidencyInKB, int32)
}
bool UInterchangeStaticMeshFactoryNode::SetCustomNaniteTargetMinimumResidencyInKB(const int32& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, NaniteTargetMinimumResidencyInKB, int32);
}
bool UInterchangeStaticMeshFactoryNode::ApplyCustomNaniteTargetMinimumResidencyInKBToAsset(UObject* Asset) const
{
#if WITH_EDITOR
	int32 ValueData;
	if (GetAttribute<int32>(Macro_CustomNaniteTargetMinimumResidencyInKBKey.Key, ValueData))
	{
		if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))
		{
			StaticMesh->GetNaniteSettings().TargetMinimumResidencyInKB = static_cast<uint32>(ValueData);
			return true;
		}
	}
#endif
	return false;
}
bool UInterchangeStaticMeshFactoryNode::FillCustomNaniteTargetMinimumResidencyInKBFromAsset(UObject* Asset)
{
#if WITH_EDITOR
	if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))
	{
		return SetAttribute<int32>(Macro_CustomNaniteTargetMinimumResidencyInKBKey.Key, static_cast<int32>(StaticMesh->GetNaniteSettings().TargetMinimumResidencyInKB));
	}
#endif
	return false;
}

// Nanite Build Settings - float properties
#define IMPLEMENT_NANITE_FLOAT_ACCESSORS(AttributeName, NaniteProperty) \
bool UInterchangeStaticMeshFactoryNode::GetCustom##AttributeName(float& AttributeValue) const \
{ \
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttributeName, float) \
} \
bool UInterchangeStaticMeshFactoryNode::SetCustom##AttributeName(const float& AttributeValue, bool bAddApplyDelegate) \
{ \
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, AttributeName, float); \
} \
bool UInterchangeStaticMeshFactoryNode::ApplyCustom##AttributeName##ToAsset(UObject* Asset) const \
{ \
	IMPLEMENT_NANITE_VALUE_TO_ASSET(AttributeName, float, NaniteProperty); \
} \
bool UInterchangeStaticMeshFactoryNode::FillCustom##AttributeName##FromAsset(UObject* Asset) \
{ \
	IMPLEMENT_NANITE_ASSET_TO_VALUE(AttributeName, float, NaniteProperty); \
}

IMPLEMENT_NANITE_FLOAT_ACCESSORS(NaniteKeepPercentTriangles, KeepPercentTriangles)
IMPLEMENT_NANITE_FLOAT_ACCESSORS(NaniteTrimRelativeError, TrimRelativeError)
IMPLEMENT_NANITE_FLOAT_ACCESSORS(NaniteFallbackPercentTriangles, FallbackPercentTriangles)
IMPLEMENT_NANITE_FLOAT_ACCESSORS(NaniteFallbackRelativeError, FallbackRelativeError)
IMPLEMENT_NANITE_FLOAT_ACCESSORS(NaniteMaxEdgeLengthFactor, MaxEdgeLengthFactor)
IMPLEMENT_NANITE_FLOAT_ACCESSORS(NaniteRayBackUp, RayBackUp)

#undef IMPLEMENT_NANITE_FLOAT_ACCESSORS

// Nanite Build Settings - enum properties (stored as int32, cast to/from engine enums)
#define IMPLEMENT_NANITE_ENUM_ACCESSORS(AttributeName, InterchangeEnumType, EngineEnumType, NaniteProperty) \
bool UInterchangeStaticMeshFactoryNode::GetCustom##AttributeName(int32& AttributeValue) const \
{ \
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttributeName, int32) \
} \
bool UInterchangeStaticMeshFactoryNode::SetCustom##AttributeName(const int32& AttributeValue, bool bAddApplyDelegate) \
{ \
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(UInterchangeStaticMeshFactoryNode, AttributeName, int32); \
} \
bool UInterchangeStaticMeshFactoryNode::ApplyCustom##AttributeName##ToAsset(UObject* Asset) const \
{ \
	IMPLEMENT_NANITE_ENUM_VALUE_TO_ASSET(AttributeName, InterchangeEnumType, EngineEnumType, NaniteProperty); \
} \
bool UInterchangeStaticMeshFactoryNode::FillCustom##AttributeName##FromAsset(UObject* Asset) \
{ \
	IMPLEMENT_NANITE_ENUM_ASSET_TO_VALUE(AttributeName, InterchangeEnumType, EngineEnumType, NaniteProperty); \
}

IMPLEMENT_NANITE_ENUM_ACCESSORS(NaniteShapePreservation, EInterchangeNaniteShapePreservation, ENaniteShapePreservation, ShapePreservation)
IMPLEMENT_NANITE_ENUM_ACCESSORS(NaniteGenerateFallback, EInterchangeNaniteGenerateFallback, ENaniteGenerateFallback, GenerateFallback)
IMPLEMENT_NANITE_ENUM_ACCESSORS(NaniteFallbackTarget, EInterchangeNaniteFallbackTarget, ENaniteFallbackTarget, FallbackTarget)

#undef IMPLEMENT_NANITE_ENUM_ACCESSORS