// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeStaticMeshLodDataNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeStaticMeshLodDataNode)

namespace UE
{
	namespace Interchange
	{
		const FAttributeKey& FStaticMeshNodeLodDataStaticData::GetLODMeshDataArrayBaseKey()
		{
			static FAttributeKey MeshUids_BaseKey(TEXT("__LODMeshDataArray__Key"));
			return MeshUids_BaseKey;
		}

		const FAttributeKey& FStaticMeshNodeLodDataStaticData::GetBoxCollisionMeshDataArrayBaseKey()
		{
			static FAttributeKey MeshUids_BaseKey(TEXT("__BoxCollisionMeshDataArray__Key"));
			return MeshUids_BaseKey;
		}

		const FAttributeKey& FStaticMeshNodeLodDataStaticData::GetCapsuleCollisionMeshDataArrayBaseKey()
		{
			static FAttributeKey MeshUids_BaseKey(TEXT("__CapsuleCollisionMeshDataArray__Key"));
			return MeshUids_BaseKey;
		}

		const FAttributeKey& FStaticMeshNodeLodDataStaticData::GetSphereCollisionMeshDataArrayBaseKey()
		{
			static FAttributeKey MeshUids_BaseKey(TEXT("__SphereCollisionMeshDataArray__Key"));
			return MeshUids_BaseKey;
		}

		const FAttributeKey& FStaticMeshNodeLodDataStaticData::GetConvexCollisionMeshDataArrayBaseKey()
		{
			static FAttributeKey MeshUids_BaseKey(TEXT("__ConvexCollisionMeshDataArray__Key"));
			return MeshUids_BaseKey;
		}


		/*
		* Deprecated functions:
		*/
		const FAttributeKey& FStaticMeshNodeLodDataStaticData::GetMeshUidsBaseKey()
		{
			static FAttributeKey MeshUids_BaseKey(TEXT("__MeshUids__Key"));
			return MeshUids_BaseKey;
		}

		const FAttributeKey& FStaticMeshNodeLodDataStaticData::GetBoxCollisionMeshUidsBaseKey()
		{
			static FAttributeKey CollisionMeshUids_BaseKey(TEXT("__BoxCollisionMeshUids__Key"));
			return CollisionMeshUids_BaseKey;
		}

		const FAttributeKey& FStaticMeshNodeLodDataStaticData::GetCapsuleCollisionMeshUidsBaseKey()
		{
			static FAttributeKey CollisionMeshUids_BaseKey(TEXT("__CapsuleCollisionMeshUids__Key"));
			return CollisionMeshUids_BaseKey;
		}

		const FAttributeKey& FStaticMeshNodeLodDataStaticData::GetSphereCollisionMeshUidsBaseKey()
		{
			static FAttributeKey CollisionMeshUids_BaseKey(TEXT("__SphereCollisionMeshUids__Key"));
			return CollisionMeshUids_BaseKey;
		}

		const FAttributeKey& FStaticMeshNodeLodDataStaticData::GetConvexCollisionMeshUidsBaseKey()
		{
			static FAttributeKey CollisionMeshUids_BaseKey(TEXT("__ConvexCollisionMeshUids__Key"));
			return CollisionMeshUids_BaseKey;
		}

	} // namespace Interchange
} // namespace UE


UInterchangeStaticMeshLodDataNode::UInterchangeStaticMeshLodDataNode()
{
	using namespace UE::Interchange;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MeshUids.Initialize(Attributes, FStaticMeshNodeLodDataStaticData::GetMeshUidsBaseKey().ToString());
	BoxCollisionMeshUids.Initialize(Attributes.ToSharedRef(), FStaticMeshNodeLodDataStaticData::GetBoxCollisionMeshUidsBaseKey().ToString());
	CapsuleCollisionMeshUids.Initialize(Attributes.ToSharedRef(), FStaticMeshNodeLodDataStaticData::GetCapsuleCollisionMeshUidsBaseKey().ToString());
	SphereCollisionMeshUids.Initialize(Attributes.ToSharedRef(), FStaticMeshNodeLodDataStaticData::GetSphereCollisionMeshUidsBaseKey().ToString());
	ConvexCollisionMeshUids.Initialize(Attributes.ToSharedRef(), FStaticMeshNodeLodDataStaticData::GetConvexCollisionMeshUidsBaseKey().ToString());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	LODMeshDataArray.Initialize(Attributes, FStaticMeshNodeLodDataStaticData::GetLODMeshDataArrayBaseKey().ToString());
	BoxCollisionMeshDataArray.Initialize(Attributes, FStaticMeshNodeLodDataStaticData::GetBoxCollisionMeshDataArrayBaseKey().ToString());
	CapsuleCollisionMeshDataArray.Initialize(Attributes, FStaticMeshNodeLodDataStaticData::GetCapsuleCollisionMeshDataArrayBaseKey().ToString());
	SphereCollisionMeshDataArray.Initialize(Attributes, FStaticMeshNodeLodDataStaticData::GetSphereCollisionMeshDataArrayBaseKey().ToString());
	ConvexCollisionMeshDataArray.Initialize(Attributes, FStaticMeshNodeLodDataStaticData::GetConvexCollisionMeshDataArrayBaseKey().ToString());
}

/**
 * Return the node type name of the class. This is used when reporting errors.
 */
FString UInterchangeStaticMeshLodDataNode::GetTypeName() const
{
	const FString TypeName = TEXT("StaticMeshLodDataNode");
	return TypeName;
}

#if WITH_EDITOR

FString UInterchangeStaticMeshLodDataNode::GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	FString KeyDisplayName = NodeAttributeKey.ToString();
	const FString NodeAttributeKeyString = KeyDisplayName;
	if (NodeAttributeKey == UE::Interchange::FStaticMeshNodeLodDataStaticData::GetLODMeshDataArrayBaseKey())
	{
		return KeyDisplayName = TEXT("Mesh count");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FStaticMeshNodeLodDataStaticData::GetLODMeshDataArrayBaseKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Mesh index "), NodeAttributeKeyString);
	}
	else if (NodeAttributeKey == UE::Interchange::FStaticMeshNodeLodDataStaticData::GetBoxCollisionMeshDataArrayBaseKey())
	{
		return KeyDisplayName = TEXT("Box collision count");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FStaticMeshNodeLodDataStaticData::GetBoxCollisionMeshDataArrayBaseKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Box collision index "), NodeAttributeKeyString);
	}
	else if (NodeAttributeKey == UE::Interchange::FStaticMeshNodeLodDataStaticData::GetCapsuleCollisionMeshDataArrayBaseKey())
	{
		return KeyDisplayName = TEXT("Capsule collision count");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FStaticMeshNodeLodDataStaticData::GetCapsuleCollisionMeshDataArrayBaseKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Capsule collision index "), NodeAttributeKeyString);
	}
	else if (NodeAttributeKey == UE::Interchange::FStaticMeshNodeLodDataStaticData::GetSphereCollisionMeshDataArrayBaseKey())
	{
		return KeyDisplayName = TEXT("Sphere collision count");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FStaticMeshNodeLodDataStaticData::GetSphereCollisionMeshDataArrayBaseKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Sphere collision index "), NodeAttributeKeyString);
	}
	else if (NodeAttributeKey == UE::Interchange::FStaticMeshNodeLodDataStaticData::GetConvexCollisionMeshDataArrayBaseKey())
	{
		return KeyDisplayName = TEXT("Convex collision count");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FStaticMeshNodeLodDataStaticData::GetConvexCollisionMeshDataArrayBaseKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Convex collision index "), NodeAttributeKeyString);
	}
	else if (NodeAttributeKey == Macro_CustomOneConvexHullPerUCXKey)
	{
		return KeyDisplayName = TEXT("One Convex Hull Per UCX");
	}
	else if (NodeAttributeKey == Macro_CustomImportCollisionKey)
	{
		return KeyDisplayName = TEXT("Import Collision");
	}

	return Super::GetKeyDisplayName(NodeAttributeKey);
}

FString UInterchangeStaticMeshLodDataNode::GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	if (NodeAttributeKey.ToString().StartsWith(UE::Interchange::FStaticMeshNodeLodDataStaticData::GetLODMeshDataArrayBaseKey().ToString()))
	{
		return FString(TEXT("Meshes"));
	}
	else if (NodeAttributeKey.ToString().StartsWith(UE::Interchange::FStaticMeshNodeLodDataStaticData::GetBoxCollisionMeshDataArrayBaseKey().ToString()))
	{
		return FString(TEXT("Box Collisions"));
	}
	else if (NodeAttributeKey.ToString().StartsWith(UE::Interchange::FStaticMeshNodeLodDataStaticData::GetCapsuleCollisionMeshDataArrayBaseKey().ToString()))
	{
		return FString(TEXT("Capsule Collisions"));
	}
	else if (NodeAttributeKey.ToString().StartsWith(UE::Interchange::FStaticMeshNodeLodDataStaticData::GetSphereCollisionMeshDataArrayBaseKey().ToString()))
	{
		return FString(TEXT("Sphere Collisions"));
	}
	else if (NodeAttributeKey.ToString().StartsWith(UE::Interchange::FStaticMeshNodeLodDataStaticData::GetConvexCollisionMeshDataArrayBaseKey().ToString()))
	{
		return FString(TEXT("Convex Collisions"));
	}
	return Super::GetAttributeCategory(NodeAttributeKey);
}

#endif //WITH_EDITOR

int32 UInterchangeStaticMeshLodDataNode::GetLODMeshDataArrayCount() const
{
	return LODMeshDataArray.GetCount();
}

void UInterchangeStaticMeshLodDataNode::GetLODMeshDataArray(TArray<FInterchangeLODMeshData>& OutLODMeshDataArray) const
{
	LODMeshDataArray.GetItems(OutLODMeshDataArray);
}

bool UInterchangeStaticMeshLodDataNode::AddLODMeshData(const FInterchangeLODMeshData& LODMeshData)
{
	return LODMeshDataArray.AddItem(LODMeshData);
}

void UInterchangeStaticMeshLodDataNode::RemoveMeshUid(const FString& MeshUid)
{
	TArray<FInterchangeLODMeshData> LODMeshDataArrayArray;
	LODMeshDataArray.GetItems(LODMeshDataArrayArray);
	LODMeshDataArray.RemoveAllItems();

	for (const FInterchangeLODMeshData& LODMeshData : LODMeshDataArrayArray)
	{
		if (LODMeshData.MeshUid != MeshUid)
		{
			LODMeshDataArray.AddItem(LODMeshData);
		}
	}
}

bool UInterchangeStaticMeshLodDataNode::RemoveAllLODMeshData()
{
	return LODMeshDataArray.RemoveAllItems();
}


/*
* Deprecated functions:
*/

int32 UInterchangeStaticMeshLodDataNode::GetMeshUidsCount() const
{
	return LODMeshDataArray.GetCount();
}
void UInterchangeStaticMeshLodDataNode::GetMeshUids(TArray<FString>& OutMeshUids) const
{
	TArray<FInterchangeLODMeshData> LODMeshDataArrayArray;
	LODMeshDataArray.GetItems(LODMeshDataArrayArray);

	OutMeshUids.Empty();
	OutMeshUids.Reserve(LODMeshDataArrayArray.Num());
	for (const FInterchangeLODMeshData& Pair : LODMeshDataArrayArray)
	{
		OutMeshUids.Add(Pair.MeshUid);
	}
}
bool UInterchangeStaticMeshLodDataNode::AddMeshUid(const FString& MeshUid)
{
	return AddLODMeshData(FInterchangeLODMeshData(MeshUid, FTransform::Identity));
}
bool UInterchangeStaticMeshLodDataNode::RemoveAllMeshes()
{
	return LODMeshDataArray.RemoveAllItems();
}

int32 UInterchangeStaticMeshLodDataNode::GetBoxCollisionMeshUidsCount() const
{
	return BoxCollisionMeshUids.GetCount();
}

TMap<FString, FString> UInterchangeStaticMeshLodDataNode::GetBoxCollisionMeshMap() const
{
	return BoxCollisionMeshUids.ToMap();
}

// Deprecated
void UInterchangeStaticMeshLodDataNode::GetBoxCollisionMeshUids(TArray<FString>& OutColliderUids) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TMap<FString, FString> Map = GetBoxCollisionMeshMap();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Map.GetKeys(OutColliderUids);
}

void UInterchangeStaticMeshLodDataNode::GetBoxColliderRenderMeshUid(const FString& InColliderMeshUid, FString& OutRenderMeshUid) const
{
	BoxCollisionMeshUids.GetValue(InColliderMeshUid, OutRenderMeshUid);
}

bool UInterchangeStaticMeshLodDataNode::AddBoxCollisionMeshUid(const FString& ColliderMeshUid)
{
	return false;
}

bool UInterchangeStaticMeshLodDataNode::AddBoxCollisionMeshUids(const FString& ColliderMeshUid, const FString& RenderMeshUid)
{
	return BoxCollisionMeshUids.SetKeyValue(ColliderMeshUid, RenderMeshUid);
}

bool UInterchangeStaticMeshLodDataNode::RemoveBoxCollisionMeshUid(const FString& ColliderMeshUid)
{
	return BoxCollisionMeshUids.RemoveKey(ColliderMeshUid);
}

bool UInterchangeStaticMeshLodDataNode::RemoveAllBoxCollisionMeshes()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool bHadItems = GetBoxCollisionMeshUidsCount() > 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	BoxCollisionMeshUids.Empty();
	return bHadItems;
}

int32 UInterchangeStaticMeshLodDataNode::GetCapsuleCollisionMeshUidsCount() const
{
	return CapsuleCollisionMeshUids.GetCount();
}

TMap<FString, FString> UInterchangeStaticMeshLodDataNode::GetCapsuleCollisionMeshMap() const
{
	return CapsuleCollisionMeshUids.ToMap();
}

// Deprecated
void UInterchangeStaticMeshLodDataNode::GetCapsuleCollisionMeshUids(TArray<FString>& OutColliderUids) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TMap<FString, FString> Map = GetCapsuleCollisionMeshMap();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Map.GetKeys(OutColliderUids);
}

void UInterchangeStaticMeshLodDataNode::GetCapsuleColliderRenderMeshUid(const FString& InColliderMeshUid, FString& OutRenderMeshUid) const
{
	CapsuleCollisionMeshUids.GetValue(InColliderMeshUid, OutRenderMeshUid);
}

bool UInterchangeStaticMeshLodDataNode::AddCapsuleCollisionMeshUid(const FString& ColliderMeshUid)
{
	return false;
}

bool UInterchangeStaticMeshLodDataNode::AddCapsuleCollisionMeshUids(const FString& ColliderMeshUid, const FString& RenderMeshUid)
{
	return CapsuleCollisionMeshUids.SetKeyValue(ColliderMeshUid, RenderMeshUid);
}

bool UInterchangeStaticMeshLodDataNode::RemoveCapsuleCollisionMeshUid(const FString& ColliderMeshUid)
{
	return CapsuleCollisionMeshUids.RemoveKey(ColliderMeshUid);
}

bool UInterchangeStaticMeshLodDataNode::RemoveAllCapsuleCollisionMeshes()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool bHadItems = GetCapsuleCollisionMeshUidsCount() > 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	CapsuleCollisionMeshUids.Empty();
	return bHadItems;
}

int32 UInterchangeStaticMeshLodDataNode::GetSphereCollisionMeshUidsCount() const
{
	return SphereCollisionMeshUids.GetCount();
}

TMap<FString, FString> UInterchangeStaticMeshLodDataNode::GetSphereCollisionMeshMap() const
{
	return SphereCollisionMeshUids.ToMap();
}

// Deprecated
void UInterchangeStaticMeshLodDataNode::GetSphereCollisionMeshUids(TArray<FString>& OutColliderUids) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TMap<FString, FString> Map = GetSphereCollisionMeshMap();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Map.GetKeys(OutColliderUids);
}

void UInterchangeStaticMeshLodDataNode::GetSphereColliderRenderMeshUid(const FString& InColliderMeshUid, FString& OutRenderMeshUid) const
{
	SphereCollisionMeshUids.GetValue(InColliderMeshUid, OutRenderMeshUid);
}

bool UInterchangeStaticMeshLodDataNode::AddSphereCollisionMeshUid(const FString& ColliderMeshUid)
{
	return false;
}

bool UInterchangeStaticMeshLodDataNode::AddSphereCollisionMeshUids(const FString& ColliderMeshUid, const FString& RenderMeshUid)
{
	return SphereCollisionMeshUids.SetKeyValue(ColliderMeshUid, RenderMeshUid);
}

bool UInterchangeStaticMeshLodDataNode::RemoveSphereCollisionMeshUid(const FString& ColliderMeshUid)
{
	return SphereCollisionMeshUids.RemoveKey(ColliderMeshUid);
}

bool UInterchangeStaticMeshLodDataNode::RemoveAllSphereCollisionMeshes()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool bHadItems = GetSphereCollisionMeshUidsCount() > 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	SphereCollisionMeshUids.Empty();
	return bHadItems;
}

int32 UInterchangeStaticMeshLodDataNode::GetConvexCollisionMeshUidsCount() const
{
	return ConvexCollisionMeshUids.GetCount();
}

TMap<FString, FString> UInterchangeStaticMeshLodDataNode::GetConvexCollisionMeshMap() const
{
	return ConvexCollisionMeshUids.ToMap();
}

// Deprecated
void UInterchangeStaticMeshLodDataNode::GetConvexCollisionMeshUids(TArray<FString>& OutColliderUids) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TMap<FString, FString> Map = GetConvexCollisionMeshMap();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Map.GetKeys(OutColliderUids);
}

void UInterchangeStaticMeshLodDataNode::GetConvexColliderRenderMeshUid(const FString& InColliderMeshUid, FString& OutRenderMeshUid) const
{
	ConvexCollisionMeshUids.GetValue(InColliderMeshUid, OutRenderMeshUid);
}

bool UInterchangeStaticMeshLodDataNode::AddConvexCollisionMeshUid(const FString& ColliderMeshUid)
{
	return false;
}

bool UInterchangeStaticMeshLodDataNode::AddConvexCollisionMeshUids(const FString& ColliderMeshUid, const FString& RenderMeshUid)
{
	return ConvexCollisionMeshUids.SetKeyValue(ColliderMeshUid, RenderMeshUid);
}

bool UInterchangeStaticMeshLodDataNode::RemoveConvexCollisionMeshUid(const FString& ColliderMeshUid)
{
	return ConvexCollisionMeshUids.RemoveKey(ColliderMeshUid);
}

bool UInterchangeStaticMeshLodDataNode::RemoveAllConvexCollisionMeshes()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool bHadItems = GetConvexCollisionMeshUidsCount() > 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	ConvexCollisionMeshUids.Empty();
	return bHadItems;
}

/*
* Deprecated functions until here.
*/

bool UInterchangeStaticMeshLodDataNode::GetOneConvexHullPerUCX(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(OneConvexHullPerUCX, bool);
}

bool UInterchangeStaticMeshLodDataNode::SetOneConvexHullPerUCX(bool AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(OneConvexHullPerUCX, bool)
}

bool UInterchangeStaticMeshLodDataNode::GetImportCollisionType(EInterchangeMeshCollision& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ImportCollisionType, EInterchangeMeshCollision);
}

bool UInterchangeStaticMeshLodDataNode::SetImportCollisionType(EInterchangeMeshCollision AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ImportCollisionType, EInterchangeMeshCollision)
}

bool UInterchangeStaticMeshLodDataNode::GetForceCollisionPrimitiveGeneration(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ForceCollisionPrimitiveGeneration, bool);
}

bool UInterchangeStaticMeshLodDataNode::SetForceCollisionPrimitiveGeneration(bool AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ForceCollisionPrimitiveGeneration, bool);
}

bool UInterchangeStaticMeshLodDataNode::GetImportCollision(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ImportCollision, bool);
}

bool UInterchangeStaticMeshLodDataNode::SetImportCollision(bool AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ImportCollision, bool);
}

bool UInterchangeStaticMeshLodDataNode::IsEditorOnlyDataDefined()
{
#if WITH_EDITORONLY_DATA
	return true;
#else
	return false;
#endif
}


bool UInterchangeStaticMeshLodDataNode::AddBoxCollisionMeshData(const FInterchangeLODMeshData& MeshData)
{
	return BoxCollisionMeshDataArray.AddItem(MeshData);
}

bool UInterchangeStaticMeshLodDataNode::AddSphereCollisionMeshData(const FInterchangeLODMeshData& MeshData)
{
	return SphereCollisionMeshDataArray.AddItem(MeshData);
}

bool UInterchangeStaticMeshLodDataNode::AddCapsuleCollisionMeshData(const FInterchangeLODMeshData& MeshData)
{
	return CapsuleCollisionMeshDataArray.AddItem(MeshData);
}

bool UInterchangeStaticMeshLodDataNode::AddConvexCollisionMeshData(const FInterchangeLODMeshData& MeshData)
{
	return ConvexCollisionMeshDataArray.AddItem(MeshData);
}

void UInterchangeStaticMeshLodDataNode::GetBoxCollisionsMeshDataArray(TArray<FInterchangeLODMeshData>& OutMeshDataArray) const
{
	BoxCollisionMeshDataArray.GetItems(OutMeshDataArray);
}

void UInterchangeStaticMeshLodDataNode::GetSphereCollisionsMeshDataArray(TArray<FInterchangeLODMeshData>& OutMeshDataArray) const
{
	SphereCollisionMeshDataArray.GetItems(OutMeshDataArray);
}

void UInterchangeStaticMeshLodDataNode::GetCapsuleCollisionsMeshDataArray(TArray<FInterchangeLODMeshData>& OutMeshDataArray) const
{
	CapsuleCollisionMeshDataArray.GetItems(OutMeshDataArray);
}

void UInterchangeStaticMeshLodDataNode::GetConvexCollisionsMeshDataArray(TArray<FInterchangeLODMeshData>& OutMeshDataArray) const
{
	ConvexCollisionMeshDataArray.GetItems(OutMeshDataArray);
}

bool RemoveCollisionMeshData(UE::Interchange::TArrayAttributeHelper<FInterchangeLODMeshData>& CollisionDataArray, const FString& MeshUid)
{
	TArray<FInterchangeLODMeshData> ContentArray;
	CollisionDataArray.GetItems(ContentArray);

	CollisionDataArray.RemoveAllItems();

	bool bSuccess = true;

	for (const FInterchangeLODMeshData& Entry : ContentArray)
	{
		if (Entry.MeshUid != MeshUid)
		{
			bSuccess = CollisionDataArray.AddItem(Entry) && bSuccess;
		}
	}

	return bSuccess;
}

bool UInterchangeStaticMeshLodDataNode::RemoveBoxCollisionMeshData(const FString& MeshUid)
{
	return RemoveCollisionMeshData(BoxCollisionMeshDataArray, MeshUid);
}

bool UInterchangeStaticMeshLodDataNode::RemoveSphereCollisionMeshData(const FString& MeshUid)
{
	return RemoveCollisionMeshData(SphereCollisionMeshDataArray, MeshUid);
}

bool UInterchangeStaticMeshLodDataNode::RemoveCapsuleCollisionMeshData(const FString& MeshUid)
{
	return RemoveCollisionMeshData(CapsuleCollisionMeshDataArray, MeshUid);
}

bool UInterchangeStaticMeshLodDataNode::RemoveConvexCollisionMeshData(const FString& MeshUid)
{
	return RemoveCollisionMeshData(ConvexCollisionMeshDataArray, MeshUid);
}


bool UInterchangeStaticMeshLodDataNode::RemoveAllBoxCollisionMeshData()
{
	return BoxCollisionMeshDataArray.RemoveAllItems();
}

bool UInterchangeStaticMeshLodDataNode::RemoveAllSphereCollisionMeshData()
{
	return SphereCollisionMeshDataArray.RemoveAllItems();
}

bool UInterchangeStaticMeshLodDataNode::RemoveAllCapsuleCollisionMeshData()
{
	return CapsuleCollisionMeshDataArray.RemoveAllItems();
}

bool UInterchangeStaticMeshLodDataNode::RemoveAllConvexCollisionMeshData()
{
	return ConvexCollisionMeshDataArray.RemoveAllItems();
}