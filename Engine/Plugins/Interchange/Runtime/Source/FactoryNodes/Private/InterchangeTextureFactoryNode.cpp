// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeTextureFactoryNode.h"

#include "ColorManagement/ColorSpace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeTextureFactoryNode)


bool UInterchangeTextureFactoryNode::GetCustomColorSpace(ETextureColorSpace& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ColorSpace, ETextureColorSpace);
}

bool UInterchangeTextureFactoryNode::SetCustomColorSpace(ETextureColorSpace AttributeValue, bool bAddApplyDelegate)
{
#if WITH_EDITORONLY_DATA
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeTextureFactoryNode, ColorSpace, ETextureColorSpace, UTexture)
#else
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ColorSpace, ETextureColorSpace)
#endif
}

#if WITH_EDITORONLY_DATA
bool UInterchangeTextureFactoryNode::ApplyCustomColorSpaceToAsset(UObject* Asset) const
{
	if(UTexture* Texture = Cast<UTexture>(Asset))
	{
		if(ETextureColorSpace ValueData; GetCustomColorSpace(ValueData))
		{
			Texture->SourceColorSettings.ColorSpace = ValueData;
			Texture->SourceColorSettings.UpdateColorSpaceChromaticities();

			return true;
		}
	}
	return false;
}

bool UInterchangeTextureFactoryNode::FillCustomColorSpaceFromAsset(UObject* Asset)
{
	if(UTexture* TypedObject = Cast<UTexture>(Asset))
	{
		if(SetCustomColorSpace(TypedObject->SourceColorSettings.ColorSpace, false))
		{
			return true;
		}
	}
	return false;
}
#endif // WITH_EDITORONLY_DATA