// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeFactoryBaseNode.h"

#include "InterchangeActorFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeActorFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	UE_API UInterchangeActorFactoryNode();

	UE_API virtual UClass* GetObjectClass() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | ActorFactory")
	UE_API bool GetCustomGlobalTransform(FTransform& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | ActorFactory")
	UE_API bool SetCustomGlobalTransform(const FTransform& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | ActorFactory")
	UE_API bool GetCustomLocalTransform(FTransform& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | ActorFactory")
	UE_API bool SetCustomLocalTransform(const FTransform& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | ActorFactory")
	UE_API bool GetCustomComponentVisibility(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | ActorFactory")
	UE_API bool SetCustomComponentVisibility(bool AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | ActorFactory")
	UE_API bool GetCustomActorVisibility(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | ActorFactory")
	UE_API bool SetCustomActorVisibility(bool AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | ActorFactory")
	UE_API bool GetCustomActorClassName(FString& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | ActorFactory")
	UE_API bool SetCustomActorClassName(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | ActorFactory")
	UE_API bool GetCustomMobility(uint8& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | ActorFactory")
	UE_API bool SetCustomMobility(const uint8& AttributeValue, bool bAddApplyDelegate = true);

	UE_API virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override;


	/** Gets the LayerNames that this Actor is supposed to be part of. */
	UE_API void GetLayerNames(TArray<FString>& OutLayerNames) const;

	/** Add LayerName that this Actor is supposed to be part of. */
	UE_API bool AddLayerName(const FString& InLayerName);

	/** Add LayerNames that this Actor is supposed to be part of. */
	UE_API bool AddLayerNames(const TArray<FString>& InLayerNames);

	/** Remove LayerName that this Actor is supposed to be part of. */
	UE_API bool RemoveLayerName(const FString& InLayerName);


	/** Gets the Tags that this Actor is supposed to have. */
	UE_API void GetTags(TArray<FString>& OutTags) const;

	/** Add Tag that this Actor is supposed to have. */
	UE_API bool AddTag(const FString& Tag);

	/** Add Tags that this Actor is supposed to have. */
	UE_API bool AddTags(const TArray<FString>& InTags);

	/** Remove Tag that this Actor is supposed to have. */
	UE_API bool RemoveTag(const FString& Tag);


	UE_API virtual void OnRestoreAllCustomAttributeDelegates() override;


	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component")
	UE_API bool AddComponentUid(const FString& ComponentUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component")
	UE_API void GetComponentUids(TArray<FString>& OutComponentUids) const;

private:

	bool ApplyCustomGlobalTransformToAsset(UObject* Asset) const;
	bool FillCustomGlobalTransformFromAsset(UObject* Asset);

	bool ApplyCustomComponentVisibilityToAsset(UObject* Asset) const;
	bool FillCustomComponentVisibilityFromAsset(UObject* Asset);

	bool ApplyCustomLocalTransformToAsset(UObject* Asset) const;
	bool FillCustomLocalTransformFromAsset(UObject* Asset);

	bool ApplyCustomActorVisibilityToAsset(UObject* Asset) const;
	bool FillCustomActorVisibilityFromAsset(UObject* Asset);

	IMPLEMENT_NODE_ATTRIBUTE_KEY(GlobalTransform);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(LocalTransform);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ComponentVisibility);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ActorVisibility);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ActorClassName);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Mobility);

	//A scene node can be part of multiple Layers.
	UE::Interchange::TArrayAttributeHelper<FString> LayerNames;
	UE::Interchange::TArrayAttributeHelper<FString> Tags;

	//A scene node can represent many special types
	UE::Interchange::TArrayAttributeHelper<FString> ComponentUids;
};

#undef UE_API
