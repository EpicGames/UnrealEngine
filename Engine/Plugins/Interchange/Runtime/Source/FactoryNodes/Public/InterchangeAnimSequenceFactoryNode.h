// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Animation/AnimSequence.h"
#include "Misc/FrameRate.h"
#include "InterchangeAnimSequenceFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

namespace UE::Interchange::Animation
{
	UE_DEPRECATED(5.8, "Use ConvertSampleRateToFrameRate in InterchangeAnimationTrackSetNode.h instead")
	INTERCHANGEFACTORYNODES_API FFrameRate ConvertSampleRatetoFrameRate(double SampleRate);
}

namespace UE::Interchange
{
	struct FAnimSequenceNodeStaticData : public FBaseNodeStaticData
	{
		static UE_API const FAttributeKey& GetAnimatedMorphTargetDependenciesKey();
		static UE_API const FAttributeKey& GetAnimatedAttributeCurveNamesKey();
		static UE_API const FAttributeKey& GetAnimatedAttributeStepCurveNamesKey();
		static UE_API const FAttributeKey& GetAnimatedMaterialCurveSuffixesKey();
		static UE_API const FAttributeKey& GetSceneNodeAnimationPayloadKeyUidMapKey();
		static UE_API const FAttributeKey& GetSceneNodeAnimationPayloadKeyTypeMapKey();
		static UE_API const FAttributeKey& GetMorphTargetNodePayloadKeyUidMapKey();
		static UE_API const FAttributeKey& GetMorphTargetNodePayloadKeyTypeMapKey();
	};
}//ns UE::Interchange

struct FInterchangeAnimationPayLoadKey;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeAnimSequenceFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	UE_API UInterchangeAnimSequenceFactoryNode();

	/**
	 * Initialize node data.
	 * @param UniqueID - The unique ID for this node.
	 * @param DisplayLabel - The name of the node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API void InitializeAnimSequenceNode(const FString& UniqueID, const FString& DisplayLabel, UInterchangeBaseNodeContainer* NodeContainer);

	/**
	 * Override Serialize() to restore SlotMaterialDependencies on load.
	 */
	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);

		if (Ar.IsLoading() && bIsInitialized)
		{
			SceneNodeAnimationPayLoadKeyUidMap.RebuildCache();
			SceneNodeAnimationPayLoadKeyTypeMap.RebuildCache();

			MorphTargetNodePayloadKeyUidMap.RebuildCache();
			MorphTargetNodePayloadKeyTypeMap.RebuildCache();
		}
	}

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	UE_API virtual FString GetTypeName() const override;

#if WITH_EDITOR
	UE_API virtual FString GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;
	UE_API virtual FString GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;
	UE_API virtual bool ShouldHideAttribute(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;
#endif //WITH_EDITOR

	/** Get the class this node creates. */
	UE_API virtual class UClass* GetObjectClass() const override;

public:
	/** Get the unique ID of the skeleton factory node. Return false if the attribute is not set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool GetCustomSkeletonFactoryNodeUid(FString& AttributeValue) const;

	/** Set the unique ID of the skeleton factory node. Return false if the attribute cannot be set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetCustomSkeletonFactoryNodeUid(const FString& AttributeValue);


	/**********************************************************************************************
	 * Import bone tracks API begin
	 */

	/**
	 * Get the import bone tracks state. If the attribute is true, bone tracks are imported. If the attribute 
	 * is false, bone tracks are not imported.
	 * 
	 * Return false if the attribute is not set. Return true if the attribute exists and can be queried.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool GetCustomImportBoneTracks(bool& AttributeValue) const;

	/** Set the import bone tracks state. Pass true to import bone tracks, or false to not import bone tracks. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetCustomImportBoneTracks(const bool& AttributeValue);

	/** Get the import bone tracks sample rate. Return false if the attribute is not set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool GetCustomImportBoneTracksSampleRate(double& AttributeValue) const;

	/** Set the import bone tracks sample rate. Return false if the attribute could not be set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetCustomImportBoneTracksSampleRate(const double& AttributeValue);

	/** Get the import bone tracks start time in seconds. Return false if the attribute is not set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool GetCustomImportBoneTracksRangeStart(double& AttributeValue) const;

	/** Set the import bone tracks start time in seconds. Return false if the attribute could not be set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetCustomImportBoneTracksRangeStart(const double& AttributeValue);

	/** Get the import bone tracks end time in seconds. Return false if the attribute is not set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool GetCustomImportBoneTracksRangeStop(double& AttributeValue) const;

	/** Set the import bone tracks end time in seconds. Return false if the attribute could not be set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetCustomImportBoneTracksRangeStop(const double& AttributeValue);

	/*
	 * Import bone tracks API end
	 **********************************************************************************************/


	/**********************************************************************************************
	 * Curves API begin
	 */


	 /**
	  * Get the import attribute curves state. If true, all user custom attributes on nodes are imported.
	  * 
	  * Return false if the attribute is not set.
	  */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool GetCustomImportAttributeCurves(bool& AttributeValue) const;

	/** Set the import attribute curves state. Return false if the attribute could not be set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetCustomImportAttributeCurves(const bool& AttributeValue);

	/**
	 * Get the custom attribute DoNotImportCurveWithZero. Return false if the attribute is not set.
	 * 
	 * Note - If this attribute is enabled, only curves that have a value other than zero will be imported. This is to avoid adding extra curves to evaluate.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool GetCustomDoNotImportCurveWithZero(bool& AttributeValue) const;

	/**
	 * Set the custom attribute DoNotImportCurveWithZero. Return false if the attribute could not be set.
	 * 
	 * Note - If this attribute is enabled, only curves that have a value other than zero will be imported. This is to avoid adding extra curves to evaluate.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetCustomDoNotImportCurveWithZero(const bool& AttributeValue);

	/**
	 * Get the custom attribute AddCurveMetadataToSkeleton. Return false if the attribute is not set.
	 * 
	 * Note - If this setting is disabled, curve metadata will be added to skeletal meshes for morph targets, but no metadata entry will be created for general curves.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool GetCustomAddCurveMetadataToSkeleton(bool& AttributeValue) const;

	/**
	 * Set the custom attribute AddCurveMetadataToSkeleton. Return false if the attribute could not be set.
	 * 
	 * Note - If this setting is disabled, curve metadata will be added to skeletal meshes for morph targets, but no metadata entry will be created for general curves.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetCustomAddCurveMetadataToSkeleton(const bool& AttributeValue);
	
	/**
	 * Get the custom attribute RemoveCurveRedundantKeys. Return false if the attribute is not set.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool GetCustomRemoveCurveRedundantKeys(bool& AttributeValue) const;

	/**
	 * Set the custom attribute RemoveCurveRedundantKeys. Return false if the attribute could not be set.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetCustomRemoveCurveRedundantKeys(const bool& AttributeValue);

	


	/*********************************************************************************************************
	 * Morph target curve API Begin
	 *
	 * Note: Morph target curve payload is FRichCurve.
	 */

	/**
	 * Get the custom attribute DeleteExistingMorphTargetCurves. Return false if the attribute is not set.
	 * 
	 * Note: If true, all previous morph target curves are deleted if you reimport.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool GetCustomDeleteExistingMorphTargetCurves(bool& AttributeValue) const;

	/**
	 * Set the custom attribute DeleteExistingMorphTargetCurves. Return false if the attribute could not be set.
	 * 
	 * Note: If true, all previous morph target curves are deleted if you reimport.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetCustomDeleteExistingMorphTargetCurves(const bool& AttributeValue);

	/**
	 * Morph target curve API End
	 *********************************************************************************************************/


	
	/*********************************************************************************************************
	 * Attribute curve API Begin
	 * 
	 * Note - Attribute curve payload information can be retrieve via the UInterchangeUserDefinedAttributesAPI.
	 *        Attribute curves are import has float FRichCurve
	 */

	 /** Return the number of animated attribute curve names this anim sequence drives. Curves are FRichCurve of type float. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API int32 GetAnimatedAttributeCurveNamesCount() const;

	/** Get all animated attribute curve names. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API void GetAnimatedAttributeCurveNames(TArray<FString>& OutAttributeCurveNames) const;

	/** Get the animated attribute curve name at the specified index. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API void GetAnimatedAttributeCurveName(const int32 Index, FString& OutAttributeCurveName) const;

	/** Add an animated attribute curve name. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetAnimatedAttributeCurveName(const FString& AttributeCurveName);

	/** Remove the specified animated attribute curve name. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool RemoveAnimatedAttributeCurveName(const FString& AttributeCurveName);

	/**
	 * Attribute curve API End
	 *********************************************************************************************************/



	 /*********************************************************************************************************
	  * Material curve API Begin
	  *
	  * Note - Material curves are attribute curves that can animate a material parameter.
	  */

	/**
	 * Get the custom attribute MaterialDriveParameterOnCustomAttribute. Return false if the attribute is not set.
	 *
	 * Note: If true, sets Material Curve Type for all custom attributes.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool GetCustomMaterialDriveParameterOnCustomAttribute(bool& AttributeValue) const;

	/**
	 * Set the custom attribute MaterialDriveParameterOnCustomAttribute. Return false if the attribute could not be set.
	 *
	 * Note: If true, sets Material Curve Type for all custom attributes.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetCustomMaterialDriveParameterOnCustomAttribute(const bool& AttributeValue);

	/** Return the number of animated material curve suffixes this anim sequence drives. Curves are FRichCurve of type float. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API int32 GetAnimatedMaterialCurveSuffixesCount() const;

	/** Get all animated material curve suffixes. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API void GetAnimatedMaterialCurveSuffixes(TArray<FString>& OutMaterialCurveSuffixes) const;

	/** Get the animated material curve suffix with the specified index. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API void GetAnimatedMaterialCurveSuffixe(const int32 Index, FString& OutMaterialCurveSuffixe) const;

	/** Add an animated material curve suffix. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetAnimatedMaterialCurveSuffixe(const FString& MaterialCurveSuffixe);

	/** Remove the specified animated material curve suffix. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool RemoveAnimatedMaterialCurveSuffixe(const FString& MaterialCurveSuffixe);

	/**
	 * Material curve API End
	 *********************************************************************************************************/



	/*********************************************************************************************************
	 * Attribute step curve API Begin
	 * 
	 * Note - Attribute step curve payload information can be retrieved via the UInterchangeUserDefinedAttributesAPI.
	 *        Imported attribute step curves have a TArray<float> for the key times and a TArray<ValueType> for the values.
	 *        Supported value type are: int32, float, FString.
	 */

	 /** Return the number of animated attribute step curve names this anim sequence drives. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API int32 GetAnimatedAttributeStepCurveNamesCount() const;

	/** Get all animated attribute step curve names. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API void GetAnimatedAttributeStepCurveNames(TArray<FString>& OutAttributeStepCurveNames) const;

	/** Get the animated attribute step curve name at the specified index. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API void GetAnimatedAttributeStepCurveName(const int32 Index, FString& OutAttributeStepCurveName) const;

	/** Add an animated attribute step curve name. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetAnimatedAttributeStepCurveName(const FString& AttributeStepCurveName);

	/** Remove the specified animated attribute step curve name. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool RemoveAnimatedAttributeStepCurveName(const FString& AttributeStepCurveName);

	/**
	 * Attribute step curve API End
	 *********************************************************************************************************/



	/**
	 * Get the custom attribute DeleteExistingCustomAttributeCurves. Return false if the attribute is not set.
	 * 
	 * Note - If true, all previous custom attribute curves are deleted if you reimport.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool GetCustomDeleteExistingCustomAttributeCurves(bool& AttributeValue) const;

	/**
	 * Set the custom attribute DeleteExistingCustomAttributeCurves. Return false if the attribute could not be set.
	 * 
	 * Note - If true, all previous custom attribute curves are deleted if you reimport.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetCustomDeleteExistingCustomAttributeCurves(const bool& AttributeValue);
	
	/*
	 * Curves API end
	 **********************************************************************************************/


	/**
	 * Get the custom attribute DeleteExistingNonCurveCustomAttributes. Return false if the attribute is not set.
	 * 
	 * Note - If true, all previous non-curve custom attributes are deleted if you reimport.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool GetCustomDeleteExistingNonCurveCustomAttributes(bool& AttributeValue) const;

	/**
	 * Set the custom attribute DeleteExistingNonCurveCustomAttributes. Return false if the attribute could not be set.
	 * 
	 * Note - If true, all previous non-curve custom attributes are deleted if you reimport.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetCustomDeleteExistingNonCurveCustomAttributes(const bool& AttributeValue);

	/**
	 * Query the optional existing USkeleton this animation must use. If this attribute is set and the skeleton is valid, 
	 * the AnimSequence factory uses this skeleton instead of the one imported from GetCustomSkeletonFactoryNodeUid.
	 * Pipelines set this attribute when the user wants to specify an existing skeleton.
	 * Return false if the attribute was not set.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool GetCustomSkeletonSoftObjectPath(FSoftObjectPath& AttributeValue) const;

	/**
	 * Set the optional existing USkeleton this animation must use. If this attribute is set and the skeleton is valid, 
	 * the AnimSequence factory uses this skeleton instead of the one imported from GetCustomSkeletonFactoryNodeUid.
	 * Pipelines set this attribute when the user wants to specify an existing skeleton.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	UE_API bool SetCustomSkeletonSoftObjectPath(const FSoftObjectPath& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API void GetSceneNodeAnimationPayloadKeys(TMap<FString, FInterchangeAnimationPayLoadKey>& OutSceneNodeAnimationPayloadKeys) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API void SetAnimationPayloadKeysForSceneNodeUids(const TMap<FString, FString>& SceneNodeAnimationPayloadKeyUids, const TMap<FString, uint8>& SceneNodeAnimationPayloadKeyTypes);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API void GetMorphTargetNodeAnimationPayloadKeys(TMap<FString, FInterchangeAnimationPayLoadKey>& OutMorphTargetNodeAnimationPayloads) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API void SetAnimationPayloadKeysForMorphTargetNodeUids(const TMap<FString, FString>& MorphTargetAnimationPayloadKeyUids, const TMap<FString, uint8>& MorphTargetAnimationPayloadKeyTypes);

	/** Return if the import of the class is allowed at runtime.*/
	virtual bool IsRuntimeImportAllowed() const override
	{
		return true;
	}

private:
	const UE::Interchange::FAttributeKey Macro_CustomSkeletonFactoryNodeUidKey = UE::Interchange::FAttributeKey(TEXT("SkeletonFactoryNodeUid"));

	const UE::Interchange::FAttributeKey Macro_CustomImportBoneTracksKey = UE::Interchange::FAttributeKey(TEXT("ImportBoneTracks"));
	const UE::Interchange::FAttributeKey Macro_CustomImportBoneTracksSampleRateKey = UE::Interchange::FAttributeKey(TEXT("ImportBoneTracksSampleRate"));
	const UE::Interchange::FAttributeKey Macro_CustomImportBoneTracksRangeStartKey = UE::Interchange::FAttributeKey(TEXT("ImportBoneTracksRangeStart"));
	const UE::Interchange::FAttributeKey Macro_CustomImportBoneTracksRangeStopKey = UE::Interchange::FAttributeKey(TEXT("ImportBoneTracksRangeStop"));

	const UE::Interchange::FAttributeKey Macro_CustomImportAttributeCurvesKey = UE::Interchange::FAttributeKey(TEXT("ImportAttributeCurves"));
	const UE::Interchange::FAttributeKey Macro_CustomDoNotImportCurveWithZeroKey = UE::Interchange::FAttributeKey(TEXT("DoNotImportCurveWithZero"));
	const UE::Interchange::FAttributeKey Macro_CustomAddCurveMetadataToSkeletonKey = UE::Interchange::FAttributeKey(TEXT("AddCurveMetadataToSkeleton"));
	const UE::Interchange::FAttributeKey Macro_CustomRemoveCurveRedundantKeysKey = UE::Interchange::FAttributeKey(TEXT("RemoveCurveRedundantKeys"));
	const UE::Interchange::FAttributeKey Macro_CustomMaterialDriveParameterOnCustomAttributeKey = UE::Interchange::FAttributeKey(TEXT("MaterialDriveParameterOnCustomAttribute"));
	const UE::Interchange::FAttributeKey Macro_CustomDeleteExistingMorphTargetCurvesKey = UE::Interchange::FAttributeKey(TEXT("DeleteExistingMorphTargetCurves"));
	const UE::Interchange::FAttributeKey Macro_CustomDeleteExistingCustomAttributeCurvesKey = UE::Interchange::FAttributeKey(TEXT("DeleteExistingCustomAttributeCurves"));

	const UE::Interchange::FAttributeKey Macro_CustomDeleteExistingNonCurveCustomAttributesKey = UE::Interchange::FAttributeKey(TEXT("DeleteExistingNonCurveCustomAttributes"));

	const UE::Interchange::FAttributeKey Macro_CustomSkeletonSoftObjectPathKey = UE::Interchange::FAttributeKey(TEXT("SkeletonSoftObjectPath"));

	UE::Interchange::TArrayAttributeHelper<FString> AnimatedMorphTargetDependencies;
	UE::Interchange::TArrayAttributeHelper<FString> AnimatedAttributeCurveNames;
	UE::Interchange::TArrayAttributeHelper<FString> AnimatedAttributeStepCurveNames;
	UE::Interchange::TArrayAttributeHelper<FString> AnimatedMaterialCurveSuffixes;

	UE::Interchange::TMapAttributeHelper<FString, FString>	SceneNodeAnimationPayLoadKeyUidMap;
	UE::Interchange::TMapAttributeHelper<FString, uint8>	SceneNodeAnimationPayLoadKeyTypeMap;

	UE::Interchange::TMapAttributeHelper<FString, FString>	MorphTargetNodePayloadKeyUidMap;
	UE::Interchange::TMapAttributeHelper<FString, uint8>	MorphTargetNodePayloadKeyTypeMap;
};

#undef UE_API
