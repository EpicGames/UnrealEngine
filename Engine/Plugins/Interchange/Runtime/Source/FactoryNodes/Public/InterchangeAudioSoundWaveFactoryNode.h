// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Sound/SoundWave.h"

#include "InterchangeAudioSoundWaveFactoryNode.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeAudioSoundWaveFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	virtual FString GetTypeName() const override
	{
		return TEXT("SoundWaveFactoryNode");
	}

	virtual UClass* GetObjectClass() const override
	{
		return USoundWave::StaticClass();
	}
};
