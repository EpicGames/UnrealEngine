// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeActorFactoryNode.h"
#include "InterchangeCameraNode.h"

#include "CoreMinimal.h"
#if WITH_ENGINE
	#include "CineCameraComponent.h"
#endif

#include "InterchangeCameraFactoryNode.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeBaseCameraFactoryNode : public UInterchangeActorFactoryNode
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomNearClipPlane(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(NearClipPlane, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomNearClipPlane(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeBaseCameraFactoryNode, NearClipPlane, float, UCameraComponent);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomAutoExposureMethod(EInterchangeAutoExposureMethod& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AutoExposureMethod, EInterchangeAutoExposureMethod);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomAutoExposureMethod(const EInterchangeAutoExposureMethod& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeBaseCameraFactoryNode, AutoExposureMethod, EInterchangeAutoExposureMethod, UCameraComponent);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomExposureCompensation(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(ExposureCompensation, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomExposureCompensation(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeBaseCameraFactoryNode, ExposureCompensation, float, UCameraComponent);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomExposureFstop(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(ExposureFstop, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomExposureFstop(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeBaseCameraFactoryNode, ExposureFstop, float, UCameraComponent);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomExposureISO(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(ExposureISO, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomExposureISO(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeBaseCameraFactoryNode, ExposureISO, float, UCameraComponent);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomExposureShutterSpeed(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(ExposureShutterSpeed, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomExposureShutterSpeed(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeBaseCameraFactoryNode, ExposureShutterSpeed, float, UCameraComponent);
	}

	virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override
	{
		Super::CopyWithObject(SourceNode, Object);

		if (const UInterchangeBaseCameraFactoryNode* ActorFactoryNode = Cast<UInterchangeBaseCameraFactoryNode>(SourceNode))
		{
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangeBaseCameraFactoryNode, NearClipPlane, float, UCameraComponent::StaticClass())
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangeBaseCameraFactoryNode, AutoExposureMethod, EInterchangeAutoExposureMethod, UCameraComponent::StaticClass())
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangeBaseCameraFactoryNode, ExposureCompensation, float, UCameraComponent::StaticClass())
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangeBaseCameraFactoryNode, ExposureFstop, float, UCameraComponent::StaticClass())
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangeBaseCameraFactoryNode, ExposureISO, float, UCameraComponent::StaticClass())
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangeBaseCameraFactoryNode, ExposureShutterSpeed, float, UCameraComponent::StaticClass())
		}
	}

protected:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(NearClipPlane);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AutoExposureMethod);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ExposureCompensation);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ExposureFstop);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ExposureISO);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ExposureShutterSpeed);

	// We need to defer these to the derived classes as the implementation is slightly different for each
	virtual bool ApplyCustomNearClipPlaneToAsset(UObject* Asset) const
	{
		// These must always be implemented by derived classes and we should never be calling this base version.
		// We can't make these pure virtual though because some of these macros will need to instantiate this base class directly
		return ensure(false);
	}
	virtual bool FillCustomNearClipPlaneFromAsset(UObject* Asset)
	{
		return ensure(false);
	}

	// We must implement our other PostProcessSettings delegates because we need to also set the bOverride properties
	// alongside the property values themselves, as otherwise they will have no effect
	bool ApplyCustomAutoExposureMethodToAsset(UObject* Asset) const
	{
		const bool bValue = true;
		return ApplyPropertyValueToObject(bValue, Asset, TEXT("PostProcessSettings.bOverride_AutoExposureMethod")) &&
		       ApplyAttributeToObject<EInterchangeAutoExposureMethod>(Macro_CustomAutoExposureMethodKey.Key, Asset, TEXT("PostProcessSettings.AutoExposureMethod"));
	}
	bool FillCustomAutoExposureMethodFromAsset(UObject* Asset)
	{
		return FillAttributeFromObject<EInterchangeAutoExposureMethod>(Macro_CustomAutoExposureMethodKey.Key, Asset, TEXT("PostProcessSettings.AutoExposureMethod"));
	}

	bool ApplyCustomExposureCompensationToAsset(UObject* Asset) const
	{
		const bool bValue = true;
		return ApplyPropertyValueToObject(bValue, Asset, TEXT("PostProcessSettings.bOverride_AutoExposureBias")) &&
		       ApplyAttributeToObject<float>(Macro_CustomExposureCompensationKey.Key, Asset, TEXT("PostProcessSettings.AutoExposureBias"));
	}
	bool FillCustomExposureCompensationFromAsset(UObject* Asset)
	{
		return FillAttributeFromObject<float>(Macro_CustomExposureCompensationKey.Key, Asset, TEXT("PostProcessSettings.AutoExposureBias"));
	}

	virtual bool ApplyCustomExposureFstopToAsset(UObject* Asset) const
	{
		return ensure(false);
	}
	virtual bool FillCustomExposureFstopFromAsset(UObject* Asset)
	{
		return ensure(false);
	}

	bool ApplyCustomExposureISOToAsset(UObject* Asset) const
	{
		const bool bValue = true;
		return ApplyPropertyValueToObject(bValue, Asset, TEXT("PostProcessSettings.bOverride_CameraISO")) &&
			   ApplyAttributeToObject<float>(Macro_CustomExposureISOKey.Key, Asset, TEXT("PostProcessSettings.CameraISO"));
	}
	bool FillCustomExposureISOFromAsset(UObject* Asset)
	{
		return FillAttributeFromObject<float>(Macro_CustomExposureISOKey.Key, Asset, TEXT("PostProcessSettings.CameraISO"));
	}

	bool ApplyCustomExposureShutterSpeedToAsset(UObject* Asset) const
	{
		const bool bValue = true;
		return ApplyPropertyValueToObject(bValue, Asset, TEXT("PostProcessSettings.bOverride_CameraShutterSpeed")) &&
			   ApplyAttributeToObject<float>(Macro_CustomExposureShutterSpeedKey.Key, Asset, TEXT("PostProcessSettings.CameraShutterSpeed"));
	}
	bool FillCustomExposureShutterSpeedFromAsset(UObject* Asset)
	{
		return FillAttributeFromObject<float>(Macro_CustomExposureShutterSpeedKey.Key, Asset, TEXT("PostProcessSettings.CameraShutterSpeed"));
	}
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangePhysicalCameraFactoryNode : public UInterchangeBaseCameraFactoryNode
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomFocalLength(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(FocalLength, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomFocalLength(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangePhysicalCameraFactoryNode, FocalLength, float, UCineCameraComponent);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomFocusDistance(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(FocusDistance, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomFocusDistance(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangePhysicalCameraFactoryNode, FocusDistance, float, UCineCameraComponent);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomSensorWidth(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(SensorWidth, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomSensorWidth(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangePhysicalCameraFactoryNode, SensorWidth, float, UCineCameraComponent);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomSensorHeight(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(SensorHeight, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomSensorHeight(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangePhysicalCameraFactoryNode, SensorHeight, float, UCineCameraComponent);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomSensorHorizontalOffset(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(SensorHorizontalOffset, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomSensorHorizontalOffset(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangePhysicalCameraFactoryNode, SensorHorizontalOffset, float, UCineCameraComponent);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomSensorVerticalOffset(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(SensorVerticalOffset, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomSensorVerticalOffset(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangePhysicalCameraFactoryNode, SensorVerticalOffset, float, UCineCameraComponent);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomFocusMethod(ECameraFocusMethod& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(FocusMethod, ECameraFocusMethod);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomFocusMethod(const ECameraFocusMethod& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangePhysicalCameraFactoryNode, FocusMethod, ECameraFocusMethod, UCineCameraComponent);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomDepthOfFieldFstop(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(DepthOfFieldFstop, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomDepthOfFieldFstop(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangePhysicalCameraFactoryNode, DepthOfFieldFstop, float, UCineCameraComponent);
	}

	virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override
	{
		Super::CopyWithObject(SourceNode, Object);

		if (const UInterchangePhysicalCameraFactoryNode* ActorFactoryNode = Cast<UInterchangePhysicalCameraFactoryNode>(SourceNode))
		{
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangePhysicalCameraFactoryNode, FocalLength, float, UCineCameraComponent::StaticClass())
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangePhysicalCameraFactoryNode, FocusDistance, float, UCineCameraComponent::StaticClass())
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangePhysicalCameraFactoryNode, SensorWidth, float, UCineCameraComponent::StaticClass())
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangePhysicalCameraFactoryNode, SensorHeight, float, UCineCameraComponent::StaticClass())
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangePhysicalCameraFactoryNode, SensorHorizontalOffset, float, UCineCameraComponent::StaticClass())
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangePhysicalCameraFactoryNode, SensorVerticalOffset, float, UCineCameraComponent::StaticClass())
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangePhysicalCameraFactoryNode, FocusMethod, ECameraFocusMethod, UCineCameraComponent::StaticClass())
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangePhysicalCameraFactoryNode, DepthOfFieldFstop, float, UCineCameraComponent::StaticClass())
		}
	}

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(FocalLength);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(FocusDistance);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SensorWidth);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SensorHeight);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SensorHorizontalOffset);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SensorVerticalOffset);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(FocusMethod);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(DepthOfFieldFstop);

	IMPLEMENT_NODE_ATTRIBUTE_DELEGATE_BY_PROPERTYNAME(FocalLength, float, UCineCameraComponent, TEXT("CurrentFocalLength"));
	IMPLEMENT_NODE_ATTRIBUTE_DELEGATE_BY_PROPERTYNAME(SensorWidth, float, UCineCameraComponent, TEXT("Filmback.SensorWidth"));
	IMPLEMENT_NODE_ATTRIBUTE_DELEGATE_BY_PROPERTYNAME(SensorHeight, float, UCineCameraComponent, TEXT("Filmback.SensorHeight"));
	IMPLEMENT_NODE_ATTRIBUTE_DELEGATE_BY_PROPERTYNAME(SensorHorizontalOffset, float, UCineCameraComponent, TEXT("Filmback.SensorHorizontalOffset"));
	IMPLEMENT_NODE_ATTRIBUTE_DELEGATE_BY_PROPERTYNAME(SensorVerticalOffset, float, UCineCameraComponent, TEXT("Filmback.SensorVerticalOffset"));
	IMPLEMENT_NODE_ATTRIBUTE_DELEGATE_BY_PROPERTYNAME(FocusMethod, ECameraFocusMethod, UCineCameraComponent, TEXT("FocusSettings.FocusMethod"));
	IMPLEMENT_NODE_ATTRIBUTE_DELEGATE_BY_PROPERTYNAME(FocusDistance, float, UCineCameraComponent, TEXT("FocusSettings.ManualFocusDistance"));

	virtual bool ApplyCustomNearClipPlaneToAsset(UObject* Asset) const override
	{
		const bool bValue = true;
		return ApplyPropertyValueToObject(bValue, Asset, TEXT("bOverride_CustomNearClippingPlane")) &&
			   ApplyAttributeToObject<float>(Macro_CustomNearClipPlaneKey.Key, Asset, TEXT("CustomNearClippingPlane"));
	}
	virtual bool FillCustomNearClipPlaneFromAsset(UObject* Asset) override
	{
		return FillAttributeFromObject<float>(Macro_CustomNearClipPlaneKey.Key, Asset, TEXT("CustomNearClippingPlane"));
	}

	virtual bool ApplyCustomExposureFstopToAsset(UObject* Asset) const override
	{
		// On the cine camera component, CurrentAperture is used for exposure, and PostProcessSettings.DepthOfFieldFstop for DoF
		return ApplyAttributeToObject<float>(Macro_CustomExposureFstopKey.Key, Asset, TEXT("CurrentAperture"));
	}
	virtual bool FillCustomExposureFstopFromAsset(UObject* Asset) override
	{
		return FillAttributeFromObject<float>(Macro_CustomExposureFstopKey.Key, Asset, TEXT("CurrentAperture"));
	}

	bool ApplyCustomDepthOfFieldFstopToAsset(UObject* Asset) const
	{
		const bool bValue = true;
		return ApplyPropertyValueToObject(bValue, Asset, TEXT("PostProcessSettings.bOverride_DepthOfFieldFstop")) &&
		ApplyAttributeToObject<float>(Macro_CustomDepthOfFieldFstopKey.Key, Asset, TEXT("PostProcessSettings.DepthOfFieldFstop"));
	}
	bool FillCustomDepthOfFieldFstopFromAsset(UObject* Asset)
	{
		return FillAttributeFromObject<float>(Macro_CustomDepthOfFieldFstopKey.Key, Asset, TEXT("PostProcessSettings.DepthOfFieldFstop"));
	}
};

/*
* Used for common non-physical camera with orthographic or perspective projection.
*/
UCLASS(MinimalAPI, BlueprintType)
class UInterchangeStandardCameraFactoryNode : public UInterchangeBaseCameraFactoryNode
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomProjectionMode(TEnumAsByte<ECameraProjectionMode::Type>& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(ProjectionMode, TEnumAsByte<ECameraProjectionMode::Type>);
	}
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomProjectionMode(const TEnumAsByte<ECameraProjectionMode::Type>& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeStandardCameraFactoryNode, ProjectionMode, TEnumAsByte<ECameraProjectionMode::Type>, UCameraComponent);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomWidth(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(Width, float);
	}
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomWidth(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeStandardCameraFactoryNode, Width, float, UCameraComponent);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomFarClipPlane(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(FarClipPlane, float);
	}
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomFarClipPlane(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeStandardCameraFactoryNode, FarClipPlane, float, UCameraComponent);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomAspectRatio(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AspectRatio, float);
	}
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomAspectRatio(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeStandardCameraFactoryNode, AspectRatio, float, UCameraComponent);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool GetCustomFieldOfView(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(FieldOfView, float);
	}
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | CameraFactory")
	bool SetCustomFieldOfView(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeStandardCameraFactoryNode, FieldOfView, float, UCameraComponent);
	}

	virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override
	{
		Super::CopyWithObject(SourceNode, Object);

		if (const UInterchangeStandardCameraFactoryNode* ActorFactoryNode = Cast<UInterchangeStandardCameraFactoryNode>(SourceNode))
		{
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangeStandardCameraFactoryNode, ProjectionMode, TEnumAsByte<ECameraProjectionMode::Type>, UCameraComponent::StaticClass())
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangeStandardCameraFactoryNode, Width, float, UCameraComponent::StaticClass())
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangeStandardCameraFactoryNode, FarClipPlane, float, UCameraComponent::StaticClass())
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangeStandardCameraFactoryNode, AspectRatio, float, UCameraComponent::StaticClass())
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(ActorFactoryNode, UInterchangeStandardCameraFactoryNode, FieldOfView, float, UCameraComponent::StaticClass())
		}
	}

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ProjectionMode);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Width);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(FarClipPlane);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AspectRatio);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(FieldOfView);

	IMPLEMENT_NODE_ATTRIBUTE_DELEGATE_BY_PROPERTYNAME(ProjectionMode, TEnumAsByte<ECameraProjectionMode::Type>, UCameraComponent, TEXT("ProjectionMode"));
	IMPLEMENT_NODE_ATTRIBUTE_DELEGATE_BY_PROPERTYNAME(Width, float, UCameraComponent, TEXT("OrthoWidth"));
	IMPLEMENT_NODE_ATTRIBUTE_DELEGATE_BY_PROPERTYNAME(AspectRatio, float, UCameraComponent, TEXT("AspectRatio"));
	IMPLEMENT_NODE_ATTRIBUTE_DELEGATE_BY_PROPERTYNAME(FieldOfView, float, UCameraComponent, TEXT("FieldOfView"));

	virtual bool ApplyCustomNearClipPlaneToAsset(UObject* Asset) const override
	{
		const bool bValue = false;
		return ApplyPropertyValueToObject(bValue, Asset, TEXT("bAutoCalculateOrthoPlanes")) &&
			   ApplyAttributeToObject<float>(Macro_CustomNearClipPlaneKey.Key, Asset, TEXT("OrthoNearClipPlane"));
	}
	virtual bool FillCustomNearClipPlaneFromAsset(UObject* Asset) override
	{
		return FillAttributeFromObject<float>(Macro_CustomNearClipPlaneKey.Key, Asset, TEXT("OrthoNearClipPlane"));
	}

	virtual bool ApplyCustomExposureFstopToAsset(UObject* Asset) const override
	{
		// On the regular camera component, PostProcessSettings.DepthOfFieldFstop is used for exposure, and CurrentAperture doesn't exist
		const bool bValue = true;
		return ApplyPropertyValueToObject(bValue, Asset, TEXT("PostProcessSettings.bOverride_DepthOfFieldFstop")) &&
			   ApplyAttributeToObject<float>(Macro_CustomExposureFstopKey.Key, Asset, TEXT("PostProcessSettings.DepthOfFieldFstop"));
	}
	virtual bool FillCustomExposureFstopFromAsset(UObject* Asset) override
	{
		return FillAttributeFromObject<float>(Macro_CustomExposureFstopKey.Key, Asset, TEXT("PostProcessSettings.DepthOfFieldFstop"));
	}

	bool ApplyCustomFarClipPlaneToAsset(UObject* Asset) const
	{
		const bool bValue = false;
		return ApplyPropertyValueToObject(bValue, Asset, TEXT("bAutoCalculateOrthoPlanes")) &&
			   ApplyAttributeToObject<float>(Macro_CustomFarClipPlaneKey.Key, Asset, TEXT("OrthoFarClipPlane"));
	}
	bool FillCustomFarClipPlaneFromAsset(UObject* Asset)
	{
		return FillAttributeFromObject<float>(Macro_CustomFarClipPlaneKey.Key, Asset, TEXT("OrthoFarClipPlane"));
	}
};
