// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeFactoryBaseNode.h"

#include "InterchangeCommonPipelineDataFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

class UInterchangeBaseNodeContainer;


/* This factory node is where pipelines can set global data that can be used by factories. */
UCLASS(MinimalAPI, BlueprintType)
class UInterchangeCommonPipelineDataFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	/* Pipelines that want to modify the common data should ensure they create the unique common pipeline node. */
	static UE_API UInterchangeCommonPipelineDataFactoryNode* FindOrCreateUniqueInstance(UInterchangeBaseNodeContainer* NodeContainer);
	
	/* If the unique instance doesn't exist, this will return nullptr. This function should be use by the factories, to avoid creating a node. */
	static UE_API UInterchangeCommonPipelineDataFactoryNode* GetUniqueInstance(const UInterchangeBaseNodeContainer* NodeContainer);

	/** Return the global offset transform set by the pipelines. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Common Pipeline Data")
	UE_API bool GetCustomGlobalOffsetTransform(FTransform& AttributeValue) const;

	/** Pipelines can set a global transform. Factories will use this global offset when importing assets. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Common Pipeline Data")
	UE_API bool SetCustomGlobalOffsetTransform(const UInterchangeBaseNodeContainer* NodeContainer, const FTransform& AttributeValue);

	/** Return the value of the Bake Meshes setting set by the pipelines. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Common Pipeline Data")
	UE_API bool GetBakeMeshes(bool& AttributeValue) const;

	/** Pipelines can set this Bake Meshes setting. Factories use this to identify whether they should apply global transforms to static meshes and skeletal meshes. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Common Pipeline Data")
	UE_API bool SetBakeMeshes(const bool& AttributeValue);

	/** Return the value of the Bake Meshes setting set by the pipelines. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Common Pipeline Data")
	UE_API bool GetBakePivotMeshes(bool& AttributeValue) const;

	/** Pipelines can set this Bake Meshes setting. Factories use this to identify whether they should apply global transforms to static meshes and skeletal meshes. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Common Pipeline Data")
	UE_API bool SetBakePivotMeshes(const bool& AttributeValue);

private:
	UInterchangeCommonPipelineDataFactoryNode() {};

	const UE::Interchange::FAttributeKey Macro_CustomGlobalOffsetTransformKey = UE::Interchange::FAttributeKey(TEXT("GlobalOffsetTransform"));
	const UE::Interchange::FAttributeKey Macro_CustomBakeMeshesKey = UE::Interchange::FAttributeKey(TEXT("BakeMeshes"));
	const UE::Interchange::FAttributeKey Macro_CustomBakePivotMeshesKey = UE::Interchange::FAttributeKey(TEXT("BakePivotMeshes"));
};

#undef UE_API
