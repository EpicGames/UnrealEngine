// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeActorFactoryNode.h"

#include "InterchangeDecalActorFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeDecalActorFactoryNode : public UInterchangeActorFactoryNode
{
	GENERATED_BODY()

public:
	UE_API UClass* GetObjectClass()const override;

	UE_API virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
	UE_API bool GetCustomSortOrder(int32& AttributeValue) const;
	
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
	UE_API bool SetCustomSortOrder(const int32& AttributeValue, bool bAddApplyDelegate = true);
	
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
	UE_API bool GetCustomDecalSize(FVector& AttributeValue) const;
	
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
	UE_API bool SetCustomDecalSize(const FVector& AttributeValue, bool bAddApplyDelegate = true);
	
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
	UE_API bool GetCustomDecalMaterialPathName(FString& AttributeValue) const;
	

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
	UE_API bool SetCustomDecalMaterialPathName(const FString& AttributeValue);
	
private:
	IMPLEMENT_NODE_ATTRIBUTE_DELEGATE_BY_PROPERTYNAME(SortOrder, int32, UDecalComponent, TEXT("SortOrder"));
	IMPLEMENT_NODE_ATTRIBUTE_DELEGATE_BY_PROPERTYNAME(DecalSize, FVector, UDecalComponent, TEXT("DecalSize"));

private:
	const UE::Interchange::FAttributeKey Macro_CustomSortOrderKey = UE::Interchange::FAttributeKey(TEXT("SortOrder"));
	const UE::Interchange::FAttributeKey Macro_CustomDecalSizeKey = UE::Interchange::FAttributeKey(TEXT("DecalSize"));
	const UE::Interchange::FAttributeKey Macro_CustomDecalMaterialPathNameKey = UE::Interchange::FAttributeKey(TEXT("DecalMaterialPathName"));
};

#undef UE_API
