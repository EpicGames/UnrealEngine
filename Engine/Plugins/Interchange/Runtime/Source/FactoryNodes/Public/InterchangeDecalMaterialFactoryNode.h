// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeMaterialFactoryNode.h"

#include "InterchangeDecalMaterialFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeDecalMaterialFactoryNode : public UInterchangeBaseMaterialFactoryNode
{
	GENERATED_BODY()

public:
	UE_API UClass* GetObjectClass()const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
	UE_API bool GetCustomDiffuseTexturePath(FString& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
	UE_API bool SetCustomDiffuseTexturePath(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
	UE_API bool GetCustomNormalTexturePath(FString& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
	UE_API bool SetCustomNormalTexturePath(const FString& AttributeValue);

private:
	const UE::Interchange::FAttributeKey Macro_CustomDiffuseTexturePathKey = UE::Interchange::FAttributeKey(TEXT("DiffuseTexturePath"));
	const UE::Interchange::FAttributeKey Macro_CustomNormalTexturePathKey = UE::Interchange::FAttributeKey(TEXT("NormalTexturePath"));
};

#undef UE_API
