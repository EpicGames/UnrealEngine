// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeMeshDefinitions.h"
#include "InterchangeMeshFactoryNode.h"
#include "Nodes/InterchangeFactoryBaseNode.h"

#include "Templates/SubclassOf.h"

#include "InterchangeGeometryCacheFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

namespace UE::Interchange
{
	struct FGeometryCacheNodeStaticData : public FBaseNodeStaticData
	{
		static UE_API const FAttributeKey& GetSceneNodeAnimationPayloadKeyUidMapKey();
	};
}

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeGeometryCacheFactoryNode : public UInterchangeMeshFactoryNode
{
	GENERATED_BODY()

public:
	UE_API UInterchangeGeometryCacheFactoryNode();

	/**
	 * Initialize node data. Also adds it to NodeContainer.
	 * @param UniqueID - The unique ID for this node.
	 * @param DisplayLabel - The name of the node.
	 * @param InAssetClass - The class the GeometryCache factory will create for this node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API void InitializeGeometryCacheNode(const FString& UniqueID, const FString& DisplayLabel, const FString& InAssetClass, UInterchangeBaseNodeContainer* NodeContainer);

	/** For the following, see GEOMETRY_CACHES_CATEGORY Properties in InterchangeGenericMeshPipeline for more details */

	/** Get whether to merge all geometries into a single mesh */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool GetCustomFlattenTracks(bool& AttributeValue) const;

	/** Set whether to merge all geometries into a single mesh */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool SetCustomFlattenTracks(const bool& AttributeValue);

	/** Get the precision used for compressing vertex positions */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool GetCustomPositionPrecision(float& AttributeValue) const;

	/** Set the precision used for compressing vertex positions */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool SetCustomPositionPrecision(const float& AttributeValue);

	/** Get the number of bits for compressing the UV into */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool GetCustomNumBitsForUVs(int32& AttributeValue) const;

	/** Set the number of bits for compressing the UV into */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool SetCustomNumBitsForUVs(const int32& AttributeValue);

	/** Get the start frame index of the animation */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool GetCustomStartFrame(int32& AttributeValue) const;

	/** Set the start frame index of the animation */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool SetCustomStartFrame(const int32& AttributeValue);

	/** Get the end frame index of the animation */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool GetCustomEndFrame(int32& AttributeValue) const;

	/** Set the end frame index of the animation */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool SetCustomEndFrame(const int32& AttributeValue);

	/** Get how the motion vectors are managed */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool GetCustomMotionVectorsImport(EInterchangeMotionVectorsHandling& AttributeValue) const;

	/** Set how the motion vectors are managed */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool SetCustomMotionVectorsImport(EInterchangeMotionVectorsHandling AttributeValue);

	/** Get whether constant topology optimization is applied */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool GetCustomApplyConstantTopologyOptimization(bool& AttributeValue) const;

	/** Get whether constant topology optimization is applied */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool SetCustomApplyConstantTopologyOptimization(const bool& AttributeValue);

	/** Get whether vertex numbers from DCC are stored in the geometry cache */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool GetCustomStoreImportedVertexNumbers(bool& AttributeValue) const;

	/** Set whether vertex numbers from DCC are stored in the geometry cache */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool SetCustomStoreImportedVertexNumbers(const bool& AttributeValue);

	/** Get whether to optimize the index buffers when building the geometry cache */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool GetCustomOptimizeIndexBuffers(bool& AttributeValue) const;

	/** Set whether to optimize the index buffers when building the geometry cache */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool SetCustomOptimizeIndexBuffers(const bool& AttributeValue);

	/** Get the animation payload keys that are relevant to building the geometry cache */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API void GetSceneNodeAnimationPayloadKeys(TMap<FString, FString>& OutSceneNodeAnimationPayloadKeyUids) const;

	/** Set the animation payload keys that are relevant to building the geometry cache */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool SetAnimationPayloadKeyForSceneNodeUid(const FString& SceneNodeUid, const FString& InUniqueId);

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	UE_API virtual FString GetTypeName() const override;

	/** Get the class this node creates. */
	UE_API virtual class UClass* GetObjectClass() const override;

private:
	UE_API virtual void FillAssetClassFromAttribute() override;
	UE_API virtual bool SetNodeClassFromClassAttribute() override;

protected:
	TSubclassOf<class UGeometryCache> AssetClass;

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(FlattenTracks);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(PositionPrecision);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(NumBitsForUVs);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(OverrideTimeRange);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(StartFrame);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(EndFrame);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(MotionVectors);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ApplyConstantTopologyOptimization);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(StoreImportedVertexNumbers);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(OptimizeIndexBuffers);

	UE::Interchange::TMapAttributeHelper<FString, FString> SceneNodeAnimationPayloadKeyUidMap;
};

#undef UE_API
