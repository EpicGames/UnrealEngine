// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeFactoryBaseNode.h"

#include "InterchangeGroomBindingFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeGroomBindingFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	UE_API virtual FString GetTypeName() const override;
	UE_API virtual class UClass* GetObjectClass() const override;

	/** Get the number of points used for the rbf interpolation */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Binding")
	UE_API bool GetNumInterpolationPoints(int32& AttributeValue) const;

	/** Set the number of points used for the rbf interpolation */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Binding")
	UE_API bool SetNumInterpolationPoints(int32 AttributeValue);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(NumInterpolationPoints);
};

#undef UE_API