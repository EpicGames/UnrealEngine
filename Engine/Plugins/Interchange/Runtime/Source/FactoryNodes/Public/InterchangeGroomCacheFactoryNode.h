// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeFactoryBaseNode.h"

#include "InterchangeGroomCacheFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

UENUM(BlueprintType)
enum class EInterchangeGroomCacheImportType : uint8
{
	None = 0x00 UMETA(Hidden),
	Strands = 0x01,
	Guides = 0x02,
	All = Strands | Guides
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeGroomCacheFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	UE_API virtual FString GetTypeName() const override;
	UE_API virtual class UClass* GetObjectClass() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool GetCustomStartFrame(int32& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool SetCustomStartFrame(const int32& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool GetCustomEndFrame(int32& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool SetCustomEndFrame(const int32& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool GetCustomNumFrames(int32& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool SetCustomNumFrames(const int32& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool GetCustomFrameRate(double& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool SetCustomFrameRate(const double& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool GetCustomGroomCacheAttributes(EInterchangeGroomCacheAttributes& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool SetCustomGroomCacheAttributes(const EInterchangeGroomCacheAttributes& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool GetCustomGroomCacheImportType(EInterchangeGroomCacheImportType& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool SetCustomGroomCacheImportType(const EInterchangeGroomCacheImportType& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool GetCustomGroomAssetPath(FSoftObjectPath& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool SetCustomGroomAssetPath(const FSoftObjectPath& AttributeValue);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(StartFrame);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(EndFrame);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(NumFrames);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(FrameRate);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(GroomCacheAttributes);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ImportType);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(GroomAssetPath);
};

#undef UE_API