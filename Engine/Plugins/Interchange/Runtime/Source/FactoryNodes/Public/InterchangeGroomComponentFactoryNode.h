// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeSceneComponentFactoryNodes.h"
//#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeGroomComponentFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeGroomComponentFactoryNode : public UInterchangeSceneComponentFactoryNode
{
	GENERATED_BODY()

public:

	UE_API virtual FString GetTypeName() const override;

	UE_API virtual UClass* GetObjectClass() const override;

	/** Get the groom asset to apply on the groom component */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | Groom")
	UE_API bool GetGroomDependencyUid(FString& AttributeValue) const;

	/** Set the groom asset to apply on the groom component */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | Groom")
	UE_API bool SetGroomDependencyUid(const FString& AttributeValue);

	/** Get the groom binding asset to apply on the groom component */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | Groom")
	UE_API bool GetGroomBindingDependencyUid(FString& AttributeValue) const;

	/** Set the groom binding asset to apply on the groom component */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | Groom")
	UE_API bool SetGroomBindingDependencyUid(const FString& AttributeValue);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(GroomUid);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(GroomBindingUid);
};

#undef UE_API
