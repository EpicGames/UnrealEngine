// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeFactoryBaseNode.h"

#include "InterchangeGroomFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

struct FHairGroupsInterpolation;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeGroomFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	UE_API virtual FString GetTypeName() const override;
	UE_API virtual class UClass* GetObjectClass() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool GetCustomGroupInterpolationSettings(FHairGroupsInterpolation& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool SetCustomGroupInterpolationSettings(const FHairGroupsInterpolation& AttributeValue);

private:
	// DecimationSettings
	IMPLEMENT_NODE_ATTRIBUTE_KEY(CurveDecimation);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(VertexDecimation);

	// InterpolationSettings
	IMPLEMENT_NODE_ATTRIBUTE_KEY(GuideType);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(HairToGuideDensity);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(RiggedGuideNumCurves);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(RiggedGuideNumPoints);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(InterpolationQuality);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(InterpolationDistance);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(bRandomizeGuide);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(bUseUniqueGuide);
};

#undef UE_API