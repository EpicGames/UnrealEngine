// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeActorFactoryNode.h"

#include "InterchangeHeterogeneousVolumeActorFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeHeterogeneousVolumeActorFactoryNode : public UInterchangeActorFactoryNode
{
	GENERATED_BODY()

public:
	UE_API UClass* GetObjectClass() const override;

	/** Gets the Uid of the material that should be assigned to the spawned HeterogeneousVolumeActor at its single material slot */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | HeterogeneousVolumeActor")
	UE_API bool GetCustomVolumetricMaterialUid(FString& MaterialFactoryNodeUid) const;

	/** Sets the Uid of the material that should be assigned to the spawned HeterogeneousVolumeActor at its single material slot */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | HeterogeneousVolumeActor")
	UE_API bool SetCustomVolumetricMaterialUid(const FString& MaterialFactoryNodeUid);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(MaterialDependency)
};

#undef UE_API
