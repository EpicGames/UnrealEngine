// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeFactoryBaseNode.h"

#include "InterchangeLevelFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeLevelFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

	UE_API UInterchangeLevelFactoryNode();

public:
	static FStringView StaticAssetTypeName()
	{
		return TEXT("LevelFactory");
	}

	/**
	 * Return the node type name of the class, we use this when reporting errors
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("LevelFactoryNode");
		return TypeName;
	}

	/** Get the class this node want to create */
	UE_API virtual class UClass* GetObjectClass() const override;

	/**
	 * This function allow to retrieve the number of track dependencies for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelFactory")
	UE_API int32 GetCustomActorFactoryNodeUidCount() const;

	/**
	 * This function allow to retrieve all actor factory node unique id for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelFactory")
	UE_API void GetCustomActorFactoryNodeUids(TArray<FString>& OutActorFactoryNodeUids) const;

	/**
	 * This function allow to retrieve one actor factory node unique id for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelFactory")
	UE_API void GetCustomActorFactoryNodeUid(const int32 Index, FString& OutActorFactoryNodeUid) const;

	/**
	 * Add one actor factory node unique id to this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelFactory")
	UE_API bool AddCustomActorFactoryNodeUid(const FString& ActorFactoryNodeUid);

	/**
	 * Remove one actor factory node unique id from this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelFactory")
	UE_API bool RemoveCustomActorFactoryNodeUid(const FString& ActorFactoryNodeUid);

	/**
	 * Get the actor factory node unique id that hold the re-import data.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelFactory")
	UE_API bool GetCustomSceneImportAssetFactoryNodeUid(FString& AttributeValue) const;

	/**
	 * Set the actor factory node unique id that hold the re-import data.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelFactory")
	UE_API bool SetCustomSceneImportAssetFactoryNodeUid(const FString& AttributeValue);

	/**
	 * Get actors bounding box.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelFactory")
	UE_API bool GetCustomShouldCreateLevel(bool& AttributeValue) const;

	/**
	 * Set actors bounding box.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelFactory")
	UE_API bool SetCustomShouldCreateLevel(const bool& AttributeValue);

	/**
	 * If true, created world partition level.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelFactory")
	UE_API bool GetCustomCreateWorldPartitionLevel(bool& AttributeValue) const;

	/**
	 * If true, created world partition level.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelFactory")
	UE_API bool SetCustomCreateWorldPartitionLevel(const bool& AttributeValue);

private:
	
	const UE::Interchange::FAttributeKey Macro_CustomShouldCreateLevelKey = UE::Interchange::FAttributeKey(TEXT("ShouldCreateLevel"));
	const UE::Interchange::FAttributeKey Macro_CustomSceneImportAssetFactoryNodeUidKey = UE::Interchange::FAttributeKey(TEXT("SceneImportAssetFactoryNodeUid"));
	const UE::Interchange::FAttributeKey Macro_CustomCreateWorldPartitionLevelKey = UE::Interchange::FAttributeKey(TEXT("CreateWorldPartitionLevel"));
	const UE::Interchange::FAttributeKey Macro_CustomParentLevelReferenceKey = UE::Interchange::FAttributeKey(TEXT("ParentLevelReference"));

	UE::Interchange::TArrayAttributeHelper<FString> CustomActorFactoryNodeUids;
};

#undef UE_API
