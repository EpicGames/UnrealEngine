// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeActorFactoryNode.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"


#include "InterchangeLevelInstanceActorFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeLevelInstanceActorFactoryNode : public UInterchangeActorFactoryNode
{
	GENERATED_BODY()
public:

	/** Get the Level this Level instance actor refer to. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelInstanceActor")
	UE_API bool GetCustomLevelReference(FString& AttributeValue) const;
	/** Set the Level this Level instance actor refer to. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelInstanceActor")
	UE_API bool SetCustomLevelReference(const FString& AttributeValue);

private:
	const UE::Interchange::FAttributeKey Macro_CustomParentLevelKey = UE::Interchange::FAttributeKey(TEXT("ParentLevel"));
	const UE::Interchange::FAttributeKey Macro_CustomLevelReferenceKey = UE::Interchange::FAttributeKey(TEXT("LevelReference"));
};

#undef UE_API
