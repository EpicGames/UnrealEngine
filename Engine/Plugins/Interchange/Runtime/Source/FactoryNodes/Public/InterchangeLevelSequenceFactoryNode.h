// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeFactoryBaseNode.h"

#include "InterchangeLevelSequenceFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeLevelSequenceFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

	UE_API UInterchangeLevelSequenceFactoryNode();

public:
	static FStringView StaticAssetTypeName()
	{
		return TEXT("LevelSequenceFactory");
	}

	/**
	 * Return the node type name of the class, we use this when reporting errors
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("LevelSequenceFactoryNode");
		return TypeName;
	}

	/** Get the class this node want to create */
	UE_API virtual class UClass* GetObjectClass() const override;

	/**
	 * This function allow to retrieve the number of track dependencies for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelSequenceFactory")
	UE_API int32 GetCustomAnimationTrackUidCount() const;

	/**
	 * This function allow to retrieve the track dependency for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelSequenceFactory")
	UE_API void GetCustomAnimationTrackUids(TArray<FString>& OutAnimationTrackUids) const;

	/**
	 * This function allow to retrieve one track dependency for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelSequenceFactory")
	UE_API void GetCustomAnimationTrackUid(const int32 Index, FString& OutAnimationTrackUid) const;

	/**
	 * Add one track dependency to this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelSequenceFactory")
	UE_API bool AddCustomAnimationTrackUid(const FString& AnimationTrackUid);

	/**
	 * Remove one track dependency from this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelSequenceFactory")
	UE_API bool RemoveCustomAnimationTrackUid(const FString& AnimationTrackUid);

	/**
	 * Set the frame rate for the animations in the level sequence.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelSequenceFactory")
	UE_API bool SetCustomFrameRate(const float& AttributeValue);

	/**
	 * Get the frame rate for the animations in the level sequence.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LevelSequenceFactory")
	UE_API bool GetCustomFrameRate(float& AttributeValue) const;

private:
	const UE::Interchange::FAttributeKey Macro_CustomFrameRateKey = UE::Interchange::FAttributeKey(TEXT("FrameRate"));

	UE::Interchange::TArrayAttributeHelper<FString> CustomAnimationTrackUids;
};

#undef UE_API
