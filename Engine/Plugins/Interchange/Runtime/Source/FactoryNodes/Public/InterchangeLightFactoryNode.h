// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeActorFactoryNode.h"
#include "InterchangeLightNode.h"

#include "InterchangeLightFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeBaseLightFactoryNode : public UInterchangeActorFactoryNode
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool GetCustomLightColor(FColor& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool SetCustomLightColor(const FColor& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool GetCustomIntensity(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool SetCustomIntensity(float AttributeValue, bool bAddApplyDelegate = true);
	
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool GetCustomTemperature(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool SetCustomTemperature(float AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool GetCustomUseTemperature(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool SetCustomUseTemperature(bool AttributeValue, bool bAddApplyDelegate = true);

	UE_API virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override;

	virtual void OnRestoreAllCustomAttributeDelegates() override;
private:

	IMPLEMENT_NODE_ATTRIBUTE_KEY(LightColor)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Intensity)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Temperature)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(bUseTemperature)
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeDirectionalLightFactoryNode : public UInterchangeBaseLightFactoryNode
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | DirectionalLightFactory")
	UE_API bool GetCustomLightSourceAngle(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | DirectionalLightFactory")
	UE_API bool SetCustomLightSourceAngle(float AttributeValue, bool bAddApplyDelegate = true);

	UE_API virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override;

	virtual void OnRestoreAllCustomAttributeDelegates() override;

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(LightSourceAngle)
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeLightFactoryNode : public UInterchangeBaseLightFactoryNode
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool GetCustomIntensityUnits(ELightUnits& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool SetCustomIntensityUnits(ELightUnits AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool GetCustomAttenuationRadius(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool SetCustomAttenuationRadius(float AttributeValue, bool bAddApplyDelegate = true);
	
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool GetCustomIESTexture(FString& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool SetCustomIESTexture(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool GetCustomUseIESBrightness(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool SetCustomUseIESBrightness(const bool& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool GetCustomIESBrightnessScale(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool SetCustomIESBrightnessScale(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool GetCustomRotation(FRotator& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | LightFactory")
	UE_API bool SetCustomRotation(const FRotator& AttributeValue, bool bAddApplyDelegate = true);

	UE_API virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override;

	virtual void OnRestoreAllCustomAttributeDelegates() override;
private:

	IMPLEMENT_NODE_ATTRIBUTE_KEY(IntensityUnits)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AttenuationRadius)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(IESTexture)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(UseIESBrightness)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(IESBrightnessScale)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Rotation)
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeRectLightFactoryNode : public UInterchangeLightFactoryNode
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | RectLightFactory")
	UE_API bool GetCustomSourceWidth(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | RectLightFactory")
	UE_API bool SetCustomSourceWidth(float AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | RectLightFactory")
	UE_API bool GetCustomSourceHeight(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | RectLightFactory")
	UE_API bool SetCustomSourceHeight(float AttributeValue, bool bAddApplyDelegate = true);

	UE_API virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override;

	virtual void OnRestoreAllCustomAttributeDelegates() override;
private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SourceWidth)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SourceHeight)
};


UCLASS(MinimalAPI, BlueprintType)
class UInterchangePointLightFactoryNode : public UInterchangeLightFactoryNode
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | PointLightFactory")
	UE_API bool GetCustomUseInverseSquaredFalloff(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | PointLightFactory")
	UE_API bool SetCustomUseInverseSquaredFalloff(bool AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | PointLightFactory")
	UE_API bool GetCustomLightFalloffExponent(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | PointLightFactory")
	UE_API bool SetCustomLightFalloffExponent(float AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | PointLightFactory")
	UE_API bool GetCustomSourceRadius(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | PointLightFactory")
	UE_API bool SetCustomSourceRadius(float AttributeValue, bool bAddApplyDelegate = true);

	UE_API virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override;

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(bUseInverseSquaredFalloff)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(LightFalloffExponent)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SourceRadius)
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSpotLightFactoryNode : public UInterchangePointLightFactoryNode
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SpotLightFactory")
	UE_API bool GetCustomInnerConeAngle(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SpotLightFactory")
	UE_API bool SetCustomInnerConeAngle(float AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SpotLightFactory")
	UE_API bool GetCustomOuterConeAngle(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SpotLightFactory")
	UE_API bool SetCustomOuterConeAngle(float AttributeValue, bool bAddApplyDelegate = true);

	UE_API virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override;

private:

	IMPLEMENT_NODE_ATTRIBUTE_KEY(InnerConeAngle)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(OuterConeAngle)
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSkyLightFactoryNode : public UInterchangeBaseLightFactoryNode
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkyLightFactory")
	UE_API bool GetCustomCubemapDependency(FString& TextureCubeFactoryNodeUid) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkyLightFactory")
	UE_API bool SetCustomCubemapDependency(FString TextureCubeFactoryNodeUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkyLightFactory")
	UE_API bool GetCustomSourceType(EInterchangeSkyLightSourceType& SourceType) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkyLightFactory")
	UE_API bool SetCustomSourceType(EInterchangeSkyLightSourceType SourceType);

private:

	IMPLEMENT_NODE_ATTRIBUTE_KEY(CubemapDependency)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SourceType)
};

#undef UE_API
