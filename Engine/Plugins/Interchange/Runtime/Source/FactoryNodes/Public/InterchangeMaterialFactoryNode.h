// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "InterchangeMaterialDefinitions.h"
#include "InterchangeShaderGraphNode.h"

#if WITH_ENGINE
#include "Materials/MaterialInterface.h"
#endif

#include "InterchangeMaterialFactoryNode.generated.h"

UCLASS(MinimalAPI, Abstract)
class UInterchangeBaseMaterialFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	static INTERCHANGEFACTORYNODES_API FString GetMaterialFactoryNodeUidFromMaterialNodeUid(const FString& TranslatedNodeUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool GetCustomIsMaterialImportEnabled(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool SetCustomIsMaterialImportEnabled(const bool& AttributeValue);

private:
	const UE::Interchange::FAttributeKey Macro_CustomIsMaterialImportEnabledKey = UE::Interchange::FAttributeKey(TEXT("IsMaterialImportEnabled"));
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeMaterialFactoryNode : public UInterchangeBaseMaterialFactoryNode
{
	GENERATED_BODY()

public:

	INTERCHANGEFACTORYNODES_API virtual FString GetTypeName() const override;

	INTERCHANGEFACTORYNODES_API virtual class UClass* GetObjectClass() const override;

// Material Inputs
public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetBaseColorConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToBaseColor(const FString& AttributeValue);
	
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToBaseColor(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetMetallicConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToMetallic(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToMetallic(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetSpecularConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToSpecular(const FString& ExpressionNodeUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToSpecular(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetRoughnessConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToRoughness(const FString& ExpressionNodeUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToRoughness(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetAnisotropyConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToAnisotropy(const FString& ExpressionNodeUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToAnisotropy(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetEmissiveColorConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToEmissiveColor(const FString& ExpressionNodeUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToEmissiveColor(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetNormalConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToNormal(const FString& ExpressionNodeUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToNormal(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetTangentConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToTangent(const FString& ExpressionNodeUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToTangent(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetSubsurfaceConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToSubsurface(const FString& ExpressionNodeUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToSubsurface(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetOpacityConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToOpacity(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToOpacity(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetOcclusionConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToOcclusion(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToOcclusion(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetRefractionConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToRefraction(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToRefraction(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetClearCoatConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToClearCoat(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToClearCoat(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetClearCoatRoughnessConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToClearCoatRoughness(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToClearCoatRoughness(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetClearCoatNormalConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToClearCoatNormal(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToClearCoatNormal(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetTransmissionColorConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToTransmissionColor(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToTransmissionColor(const FString& ExpressionNodeUid, const FString& OutputName);
	
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetSurfaceCoverageConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToSurfaceCoverage(const FString& ExpressionUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToSurfaceCoverage(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetFuzzColorConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToFuzzColor(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToFuzzColor(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetClothConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToCloth(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToCloth(const FString& ExpressionNodeUid, const FString& OutputName);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetDisplacementConnection(FString& ExpressionNodeUid, FString& OutputName) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectToDisplacement(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool ConnectOutputToDisplacement(const FString& ExpressionNodeUid, const FString& OutputName);

// Material parameters
public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool GetCustomShadingModel(TEnumAsByte<EMaterialShadingModel>& AttributeValue) const;
 
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool SetCustomShadingModel(const TEnumAsByte<EMaterialShadingModel>& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool GetCustomTranslucencyLightingMode(TEnumAsByte<ETranslucencyLightingMode>& AttributeValue) const;
 
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool SetCustomTranslucencyLightingMode(const TEnumAsByte<ETranslucencyLightingMode>& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool GetCustomBlendMode(TEnumAsByte<EBlendMode>& AttributeValue) const;
 
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool SetCustomBlendMode(const TEnumAsByte<EBlendMode>& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool GetCustomTwoSided(bool& AttributeValue) const;
 
	/** Sets if this shader graph should be rendered two sided or not. Defaults to off. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool SetCustomTwoSided(const bool& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool GetCustomOpacityMaskClipValue(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool SetCustomOpacityMaskClipValue(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool GetCustomRefractionMethod(TEnumAsByte<ERefractionMode>& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool SetCustomRefractionMethod(const TEnumAsByte<ERefractionMode>& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool GetCustomScreenSpaceReflections(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool SetCustomScreenSpaceReflections(const bool& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool GetCustomDisplacementCenter(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool SetCustomDisplacementCenter(float AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool GetCustomDisplacementMagnitude(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool SetCustomDisplacementMagnitude(float AttributeValue);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ShadingModel)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(TranslucencyLightingMode)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(BlendMode)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(TwoSided)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(OpacityMaskClipValue);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(RefractionMethod)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ScreenSpaceReflections)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(DisplacementCenter)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(DisplacementMagnitude)
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeMaterialExpressionFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	INTERCHANGEFACTORYNODES_API virtual FString GetTypeName() const override;

public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool GetCustomExpressionClassName(FString& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool SetCustomExpressionClassName(const FString& AttributeValue);

private:
	const UE::Interchange::FAttributeKey Macro_CustomExpressionClassNameKey = UE::Interchange::FAttributeKey(TEXT("ExpressionClassName"));

};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeMaterialInstanceFactoryNode : public UInterchangeBaseMaterialFactoryNode
{
	GENERATED_BODY()

public:
	INTERCHANGEFACTORYNODES_API virtual FString GetTypeName() const override;
	INTERCHANGEFACTORYNODES_API virtual UClass* GetObjectClass() const override;

public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialInstanceFactory")
	INTERCHANGEFACTORYNODES_API bool GetCustomInstanceClassName(FString& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialInstanceFactory")
	INTERCHANGEFACTORYNODES_API bool SetCustomInstanceClassName(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialInstanceFactory")
	INTERCHANGEFACTORYNODES_API bool GetCustomParent(FString& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialInstanceFactory")
	INTERCHANGEFACTORYNODES_API bool SetCustomParent(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialInstanceFactory")
	INTERCHANGEFACTORYNODES_API bool GetCustomBlendMode(TEnumAsByte<EBlendMode>& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialInstanceFactory")
	INTERCHANGEFACTORYNODES_API bool SetCustomBlendMode(TEnumAsByte<EBlendMode> AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialInstanceFactory")
	INTERCHANGEFACTORYNODES_API bool GetCustomTwoSided(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialInstanceFactory")
	INTERCHANGEFACTORYNODES_API bool SetCustomTwoSided(bool AttributeValue);

	/** Used to enable displacement*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialInstanceFactory")
	INTERCHANGEFACTORYNODES_API bool GetCustomEnableTessellation(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialInstanceFactory")
	INTERCHANGEFACTORYNODES_API bool SetCustomEnableTessellation(bool AttributeValue);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(InstanceClassName)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Parent)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(BlendMode)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(TwoSided)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(EnableTessellation)
};

/**
 * Describes a reference to an existing (as in, not imported) material. Note that the material is referenced
 * via the UInterchangeFactoryBaseNode::CustomReferenceObject member.
 *
 * The idea is that mesh / actor factory nodes can reference one of these nodes as a slot dependency, and
 * Interchange will assign that existing material to the corresponding slot during import
 */
UCLASS(MinimalAPI, BlueprintType)
class UInterchangeMaterialReferenceFactoryNode : public UInterchangeBaseMaterialFactoryNode
{
	GENERATED_BODY()

public:
	INTERCHANGEFACTORYNODES_API virtual FString GetTypeName() const override;
	INTERCHANGEFACTORYNODES_API virtual UClass* GetObjectClass() const override;
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeMaterialFunctionCallExpressionFactoryNode : public UInterchangeMaterialExpressionFactoryNode
{
	GENERATED_BODY()

public:
	INTERCHANGEFACTORYNODES_API virtual FString GetTypeName() const override;

public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool GetCustomMaterialFunctionDependency(FString& AttributeValue) const;

	/**
	 * Set the unique ID of the material function that the function call expression
	 * is referring to.
	 * Note that a call to AddFactoryDependencyUid is made to guarantee that
	 * the material function is created before the function call expression
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	INTERCHANGEFACTORYNODES_API bool SetCustomMaterialFunctionDependency(const FString& AttributeValue);

private:
	const UE::Interchange::FAttributeKey Macro_CustomMaterialFunctionDependencyKey = UE::Interchange::FAttributeKey(TEXT("MaterialFunctionDependency"));

};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeMaterialFunctionFactoryNode : public UInterchangeBaseMaterialFactoryNode
{
	GENERATED_BODY()

public:

	INTERCHANGEFACTORYNODES_API virtual FString GetTypeName() const override;

	INTERCHANGEFACTORYNODES_API virtual class UClass* GetObjectClass() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MaterialFactory")
	INTERCHANGEFACTORYNODES_API bool GetInputConnection(const FString& InputName, FString& ExpressionNodeUid, FString& OutputName) const;

private:
};
