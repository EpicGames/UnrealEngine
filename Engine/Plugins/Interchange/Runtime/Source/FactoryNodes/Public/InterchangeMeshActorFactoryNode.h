// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeActorFactoryNode.h"

#include "InterchangeMeshActorFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeMeshActorFactoryNode : public UInterchangeActorFactoryNode
{
	GENERATED_BODY()

public:
	UE_API UInterchangeMeshActorFactoryNode();

	/**
	 * Override Serialize() to restore SlotMaterialDependencies on load.
	 */
	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);

		if (Ar.IsLoading() && bIsInitialized)
		{
			SlotMaterialDependencies.RebuildCache();
		}
	}

	/** Set the resolved asset factory node. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetCustomInstancedAssetFactoryNodeUid(const FString& AttributeValue);

	/** Get the resolved asset factory node. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetCustomInstancedAssetFactoryNodeUid(FString& AttributeValue) const;

	/**
	 * Retrieve the correspondence table between slot names and assigned materials for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API void GetSlotMaterialDependencies(TMap<FString, FString>& OutMaterialDependencies) const;

	/**
	 * Retrieve the Material dependency for the specified slot of this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetSlotMaterialDependencyUid(const FString& SlotName, FString& OutMaterialDependency) const;

	/**
	 * Add a Material dependency to the specified slot of this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetSlotMaterialDependencyUid(const FString& SlotName, const FString& MaterialDependencyUid);

	/**
	 * Remove the Material dependency associated with the specified slot name from this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool RemoveSlotMaterialDependencyUid(const FString& SlotName);

	/** Set the animation asset for this scene node to play. (This is only relevant for SkeletalMeshActors: scene nodes that are instantiating skeletal meshes.) */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	UE_API bool SetCustomAnimationAssetUidToPlay(const FString& AttributeValue);
	/** Get the animation asset set for this scene node to play. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	UE_API bool GetCustomAnimationAssetUidToPlay(FString& AttributeValue) const;

	/** Get the geometric offset. Any mesh attached to this scene node will be offset using this transform. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomGeometricTransform(FTransform& AttributeValue) const;

	/** Set the geometric offset. Any mesh attached to this scene node will be offset using this transform. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomGeometricTransform(const FTransform& AttributeValue);

	bool GetCustomKeepSectionsSeparate(bool& AttributeValue) const
	{
		//Scene import do not support this options so we set the value to false and return false
		AttributeValue = false;
		return false;
	}

private:
	UE::Interchange::TMapAttributeHelper<FString, FString> SlotMaterialDependencies;

	//A factory node can reference an asset on top of base asset:
	const UE::Interchange::FAttributeKey Macro_CustomInstancedAssetFactoryNodeUidKey = UE::Interchange::FAttributeKey(TEXT("InstancedAssetFactoryNodeUid"));

	//A scene node can reference an animation asset on top of base asset:
	const UE::Interchange::FAttributeKey Macro_CustomAnimationAssetUidToPlayKey = UE::Interchange::FAttributeKey(TEXT("AnimationAssetUidToPlay"));

	//A scene node can have a transform apply to the mesh it reference.
	const UE::Interchange::FAttributeKey Macro_CustomGeometricTransformKey = UE::Interchange::FAttributeKey(TEXT("GeometricTransform"));
};

#undef UE_API
