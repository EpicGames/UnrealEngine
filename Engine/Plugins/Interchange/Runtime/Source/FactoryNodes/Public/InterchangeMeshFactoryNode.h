// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeFactoryBaseNode.h"

#include "InterchangeMeshFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API


namespace UE::Interchange
{
	struct FMeshFactoryNodeStaticData : public FBaseNodeStaticData
	{
		static UE_API const FAttributeKey& GetLodDependenciesBaseKey();
		static UE_API const FAttributeKey& GetSlotMaterialDependencyBaseKey();
		static UE_API const FAttributeKey& GetAssemblyPartDependencyBaseKey();
	};
} // namespace Interchange


UCLASS(MinimalAPI, BlueprintType, Abstract)
class UInterchangeMeshFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	UE_API UInterchangeMeshFactoryNode();

	/**
	 * Override Serialize() to restore SlotMaterialDependencies and AssemblyPartDependencies on load.
	 */
	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);

		if (Ar.IsLoading() && bIsInitialized)
		{
			SlotMaterialDependencies.RebuildCache();
			AssemblyPartDependencies.RebuildCache();
#if WITH_ENGINE
			// Make sure the class is properly set when we compile with engine, this will set the bIsNodeClassInitialized to true.
			SetNodeClassFromClassAttribute();
#endif
		}
	}

#if WITH_EDITOR
	UE_API virtual FString GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;
	UE_API virtual FString GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;
	UE_API virtual bool ShouldHideAttribute(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;
#endif //WITH_EDITOR

public:

	static UE_API const FString& GetMeshSocketPrefix();

	/** Return the number of LODs this static mesh has.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API int32 GetLodDataCount() const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API void GetLodDataUniqueIds(TArray<FString>& OutLodDataUniqueIds) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool AddLodDataUniqueId(const FString& LodDataUniqueId);

	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool RemoveLodDataUniqueId(const FString& LodDataUniqueId);

	/** Query whether the static mesh factory should replace the vertex color. Return false if the attribute was not set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool GetCustomVertexColorReplace(bool& AttributeValue) const;

	/** Set whether the static mesh factory should replace the vertex color. Return false if the attribute could not be set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool SetCustomVertexColorReplace(const bool& AttributeValue);

	/** Query whether the static mesh factory should ignore the vertex color. Return false if the attribute was not set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool GetCustomVertexColorIgnore(bool& AttributeValue) const;

	/** Set whether the static mesh factory should ignore the vertex color. Return false if the attribute could not be set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool SetCustomVertexColorIgnore(const bool& AttributeValue);

	/** Query whether the static mesh factory should override the vertex color. Return false if the attribute was not set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool GetCustomVertexColorOverride(FColor& AttributeValue) const;

	/** Set whether the static mesh factory should override the vertex color. Return false if the attribute could not be set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool SetCustomVertexColorOverride(const FColor& AttributeValue);

	/** Query whether sections with matching materials are kept separate and will not get combined. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool GetCustomKeepSectionsSeparate(bool& AttributeValue) const;

	/** Set whether sections with matching materials are kept separate and will not get combined. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool SetCustomKeepSectionsSeparate(const bool& AttributeValue);

	/** Query whether the mesh factory should create sockets.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetCustomImportSockets(bool& AttributeValue) const;

	/** Set whether the mesh factory should create sockets.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetCustomImportSockets(const bool& AttributeValue);

	/** Retrieve the correspondence table between slot names and assigned materials for this object. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API void GetSlotMaterialDependencies(TMap<FString, FString>& OutMaterialDependencies) const;

	/** Retrieve the Material dependency for the specified slot of this object. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool GetSlotMaterialDependencyUid(const FString& SlotName, FString& OutMaterialDependency) const;

	/** Add a Material dependency to the specified slot of this object. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool SetSlotMaterialDependencyUid(const FString& SlotName, const FString& MaterialDependencyUid);

	/** Remove the Material dependency associated with the specified slot name of this object. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool RemoveSlotMaterialDependencyUid(const FString& SlotName);

	/** Reset all the material dependencies. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool ResetSlotMaterialDependencies();
	
	/** Retrieve the number of Nanite assembly part dependencies for this object. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API int32 GetAssemblyPartDependenciesCount() const;

	/** Retrieve the Nanite assembly part dependencies for this object. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API void GetAssemblyPartDependencies(TMap<FString, FString>& OutAssemblyPartDependencies) const;

	/** Add a Nanite assembly part dependency to this object. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool SetAssemblyPartDependencyUid(const FString& MeshUid, const FString& AssemblyPartDependencyUid);

	/** Remove a Nanite assembly part dependency associated with this object */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool RemoveAssemblyPartDependencyUid(const FString& MeshUid);

	/** Reset all the Nanite assembly dependencies. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool ResetAssemblyDependencies();

	/** Query whether a custom LOD group is set for the mesh. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool GetCustomLODGroup(FName& AttributeValue) const;

	/** Set a custom LOD group for the mesh. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool SetCustomLODGroup(const FName& AttributeValue, bool bAddApplyDelegate = true);

	/** Query whether normals in the imported mesh are ignored and recomputed. When normals are recomputed, the tangents are also recomputed. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool GetCustomRecomputeNormals(bool& AttributeValue) const;

	/** Set whether normals in the imported mesh are ignored and recomputed. When normals are recomputed, the tangents are also recomputed. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool SetCustomRecomputeNormals(const bool& AttributeValue, bool bAddApplyDelegate = true);
	
	/** Query whether tangents in the imported mesh are ignored and recomputed. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool GetCustomRecomputeTangents(bool& AttributeValue) const;

	/** Set whether tangents in the imported mesh are ignored and recomputed. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool SetCustomRecomputeTangents(const bool& AttributeValue, bool bAddApplyDelegate = true);

	/** Query whether tangents are recomputed using MikkTSpace when they need to be recomputed. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool GetCustomUseMikkTSpace(bool& AttributeValue) const;

	/** Set whether tangents are recomputed using MikkTSpace when they need to be recomputed. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool SetCustomUseMikkTSpace(const bool& AttributeValue, bool bAddApplyDelegate = true);

	/** Query whether normals are recomputed by weighting the surface area and the corner angle of the triangle as a ratio. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool GetCustomComputeWeightedNormals(bool& AttributeValue) const;

	/** Set whether normals are recomputed by weighting the surface area and the corner angle of the triangle as a ratio. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool SetCustomComputeWeightedNormals(const bool& AttributeValue, bool bAddApplyDelegate = true);

	/** Query whether tangents are stored at 16-bit precision instead of the default 8-bit precision. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool GetCustomUseHighPrecisionTangentBasis(bool& AttributeValue) const;

	/** Set whether tangents are stored at 16-bit precision instead of the default 8-bit precision. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool SetCustomUseHighPrecisionTangentBasis(const bool& AttributeValue, bool bAddApplyDelegate = true);

	/** Query whether UVs are stored at full floating point precision. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool GetCustomUseFullPrecisionUVs(bool& AttributeValue) const;

	/** Set whether UVs are stored at full floating point precision. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool SetCustomUseFullPrecisionUVs(const bool& AttributeValue, bool bAddApplyDelegate = true);

	/** Query whether UVs are converted to 16-bit by a legacy truncation process instead of the default rounding process. This may avoid differences when reimporting older content. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool GetCustomUseBackwardsCompatibleF16TruncUVs(bool& AttributeValue) const;

	/** Set whether UVs are converted to 16-bit by a legacy truncation process instead of the default rounding process. This may avoid differences when reimporting older content. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool SetCustomUseBackwardsCompatibleF16TruncUVs(const bool& AttributeValue, bool bAddApplyDelegate = true);

	/** Query whether degenerate triangles are removed. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool GetCustomRemoveDegenerates(bool& AttributeValue) const;

	/** Set whether degenerate triangles are removed. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | Mesh")
	UE_API bool SetCustomRemoveDegenerates(const bool& AttributeValue, bool bAddApplyDelegate = true);

	UE_API virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override;

	/** Get a payload key string attribute from this node. Returns false if the attribute does not exist. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetPayloadKeyStringAttribute(const FString& PayloadAttributeKey, FString& Value);

	/** Add a string attribute for the payload. Returns false if the attribute does not exist or if it cannot be added. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool AddPayloadKeyStringAttribute(const FString& PayloadAttributeKey, const FString& Value);

	/** Get a payload key float attribute from this node. Returns false if the attribute does not exist. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetPayloadKeyFloatAttribute(const FString& PayloadAttributeKey, float& Value);

	/** Add a float attribute for the payload. Returns false if the attribute does not exist or if it cannot be added. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool AddPayloadKeyFloatAttribute(const FString& PayloadAttributeKey, float Value);
	
	/** Get a payload key int32 attribute from this node. Returns false if the attribute does not exist. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetPayloadKeyInt32Attribute(const FString& PayloadAttributeKey, int32& Value);

	/** Add an int attribute for the payload. Returns false if the attribute does not exist or if it cannot be added. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool AddPayloadKeyInt32Attribute(const FString& PayloadAttributeKey, int32 Value);
	
	/** Get a payload key boolean attribute from this node. Returns false if the attribute does not exist. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetPayloadKeyBooleanAttribute(const FString& PayloadAttributeKey, bool& Value);

	/** Add a boolean attribute for the payload. Returns false if the attribute does not exist or if it cannot be added. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool AddPayloadKeyBooleanAttribute(const FString& PayloadAttributeKey, bool Value);

	/** Get a payload key double attribute from this node. Returns false if the attribute does not exist. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetPayloadKeyDoubleAttribute(const FString& PayloadAttributeKey, double& Value);

	/** Add a double attribute for the payload. Returns false if the attribute does not exist or if it cannot be added. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool AddPayloadKeyDoubleAttribute(const FString& PayloadAttributeKey, double Value);

	static UE_API void CopyPayloadKeyStorageAttributes(const UInterchangeBaseNode* SourceNode, UE::Interchange::FAttributeStorage& DestinationStorage);

private:

	IMPLEMENT_NODE_ATTRIBUTE_KEY(VertexColorReplace)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(VertexColorIgnore)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(VertexColorOverride)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(KeepSectionsSeparate)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(LODGroup)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(RecomputeNormals)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(RecomputeTangents)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(UseMikkTSpace)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ComputeWeightedNormals)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(UseHighPrecisionTangentBasis)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(UseFullPrecisionUVs)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(UseBackwardsCompatibleF16TruncUVs)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(RemoveDegenerates)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ImportSockets)

	UE::Interchange::TArrayAttributeHelper<FString> LodDependencies;
	UE::Interchange::TMapAttributeHelper<FString, FString> SlotMaterialDependencies;
	UE::Interchange::TMapAttributeHelper<FString, FString> AssemblyPartDependencies;

protected:
	virtual void FillAssetClassFromAttribute() PURE_VIRTUAL("FillAssetClassFromAttribute");
	virtual bool SetNodeClassFromClassAttribute() PURE_VIRTUAL("SetNodeClassFromClassAttribute", return false;);

	const UE::Interchange::FAttributeKey ClassNameAttributeKey = UE::Interchange::FBaseNodeStaticData::ClassTypeAttributeKey();

	UE_API bool ApplyCustomRecomputeNormalsToAsset(UObject * Asset) const;
	UE_API bool FillCustomRecomputeNormalsFromAsset(UObject * Asset);
	UE_API bool ApplyCustomRecomputeTangentsToAsset(UObject * Asset) const;
	UE_API bool FillCustomRecomputeTangentsFromAsset(UObject * Asset);
	UE_API bool ApplyCustomUseMikkTSpaceToAsset(UObject * Asset) const;
	UE_API bool FillCustomUseMikkTSpaceFromAsset(UObject * Asset);
	UE_API bool ApplyCustomComputeWeightedNormalsToAsset(UObject * Asset) const;
	UE_API bool FillCustomComputeWeightedNormalsFromAsset(UObject * Asset);
	UE_API bool ApplyCustomUseHighPrecisionTangentBasisToAsset(UObject * Asset) const;
	UE_API bool FillCustomUseHighPrecisionTangentBasisFromAsset(UObject * Asset);
	UE_API bool ApplyCustomUseFullPrecisionUVsToAsset(UObject * Asset) const;
	UE_API bool FillCustomUseFullPrecisionUVsFromAsset(UObject * Asset);
	UE_API bool ApplyCustomUseBackwardsCompatibleF16TruncUVsToAsset(UObject * Asset) const;
	UE_API bool FillCustomUseBackwardsCompatibleF16TruncUVsFromAsset(UObject * Asset);
	UE_API bool ApplyCustomRemoveDegeneratesToAsset(UObject * Asset) const;
	UE_API bool FillCustomRemoveDegeneratesFromAsset(UObject * Asset);

	bool bIsNodeClassInitialized = false;
};

#undef UE_API
