// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Templates/SubclassOf.h"

#if WITH_ENGINE
#include "Animation/Skeleton.h"
#include "PhysicsEngine/PhysicsAsset.h"
#endif

#include "InterchangePhysicsAssetFactoryNode.generated.h"

UCLASS(BlueprintType, MinimalAPI)
class UInterchangePhysicsAssetFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	/**
	 * Initialize node data.
	 * @param: UniqueID - The unique ID for this node.
	 * @param DisplayLabel - The name of the node.
	 * @param InAssetClass - The class the Skeleton factory will create for this node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Skeleton")
	void InitializePhysicsAssetNode(const FString& UniqueID, const FString& DisplayLabel, const FString& InAssetClass, UInterchangeBaseNodeContainer* NodeContainer)
	{
		bIsNodeClassInitialized = false;
		NodeContainer->SetupNode(this, UniqueID, DisplayLabel, EInterchangeNodeContainerType::FactoryData);
		FString OperationName = GetTypeName() + TEXT(".SetAssetClassName");
		InterchangePrivateNodeBase::SetCustomAttribute<FString>(*Attributes, ClassNameAttributeKey, OperationName, InAssetClass);
		FillAssetClassFromAttribute();
	}

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
#if WITH_ENGINE
		if(Ar.IsLoading())
		{
			//Make sure the class is properly set when we compile with engine, this will set the
			//bIsNodeClassInitialized to true.
			SetNodeClassFromClassAttribute();
		}
#endif //#if WITH_ENGINE
	}

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("SkeletonNode");
		return TypeName;
	}

	/** Get the class this node creates. */
	virtual class UClass* GetObjectClass() const override
	{
		ensure(bIsNodeClassInitialized);
#if WITH_ENGINE
		return AssetClass.Get() != nullptr ? AssetClass.Get() : UPhysicsAsset::StaticClass();
#else
		return nullptr;
#endif
	}

#if WITH_EDITOR

	virtual bool ShouldHideAttribute(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override
	{
		if (NodeAttributeKey == Macro_CustomSkeletalMeshUidKey)
		{
			return true;
		}

		return Super::ShouldHideAttribute(NodeAttributeKey);
	}
#endif

	virtual FGuid GetHash() const override
	{
		return Attributes->GetStorageHash();
	}

public:
	/** Get the Skeletal Mesh asset UID used to create the data in the post-pipeline step. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Skeleton")
	bool GetCustomSkeletalMeshUid(FString& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(SkeletalMeshUid, FString);
	}
	/** Set the Skeletal Mesh asset UID used to create the data in the post-pipeline step. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Skeleton")
	bool SetCustomSkeletalMeshUid(const FString& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SkeletalMeshUid, FString)
	}

	/** Return if the import of the class is allowed at runtime.*/
	virtual bool IsRuntimeImportAllowed() const override
	{
		return false;
	}

private:

	void FillAssetClassFromAttribute()
	{
#if WITH_ENGINE
		FString OperationName = GetTypeName() + TEXT(".GetAssetClassName");
		FString ClassName;
		InterchangePrivateNodeBase::GetCustomAttribute<FString>(*Attributes, ClassNameAttributeKey, OperationName, ClassName);
		if (ClassName.Equals(UPhysicsAsset::StaticClass()->GetName()))
		{
			AssetClass = UPhysicsAsset::StaticClass();
			bIsNodeClassInitialized = true;
		}
#endif
	}

	bool SetNodeClassFromClassAttribute()
	{
		if (!bIsNodeClassInitialized)
		{
			FillAssetClassFromAttribute();
		}
		return bIsNodeClassInitialized;
	}

	bool IsEditorOnlyDataDefined()
	{
#if WITH_EDITORONLY_DATA
		return true;
#else
		return false;
#endif
	}

	const UE::Interchange::FAttributeKey ClassNameAttributeKey = UE::Interchange::FBaseNodeStaticData::ClassTypeAttributeKey();
	//SkeletalMesh asset Uid use to create the data in the post pipeline step
	const UE::Interchange::FAttributeKey Macro_CustomSkeletalMeshUidKey = UE::Interchange::FAttributeKey(TEXT("SkeletalMeshUid"));
	
protected:
#if WITH_ENGINE
	TSubclassOf<UPhysicsAsset> AssetClass;
#endif
	bool bIsNodeClassInitialized = false;
};
