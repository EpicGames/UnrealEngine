// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeFactoryBaseNode.h"

#include "InterchangeSceneComponentFactoryNodes.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

//TODO: move to its own file.
UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSceneComponentFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	UE_API UInterchangeSceneComponentFactoryNode();

	UE_API virtual FString GetTypeName() const override;

	UE_API virtual UClass* GetObjectClass() const override;


	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component")
	UE_API bool AddComponentUid(const FString& ComponentUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component")
	UE_API void GetComponentUids(TArray<FString>& OutComponentUids) const;


	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Component")
	UE_API bool GetCustomLocalTransform(FTransform& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Component")
	UE_API bool SetCustomLocalTransform(const FTransform& AttributeValue);


	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Component")
	UE_API bool GetCustomComponentVisibility(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Component")
	UE_API bool SetCustomComponentVisibility(const bool& AttributeValue);

private:
	//Component's Local Transform.
	const UE::Interchange::FAttributeKey Macro_CustomLocalTransformKey = UE::Interchange::FAttributeKey(TEXT("LocalTransform"));

	//Component's Visibility
	const UE::Interchange::FAttributeKey Macro_CustomComponentVisibilityKey = UE::Interchange::FAttributeKey(TEXT("ComponentVisibility"));

	//Childnre Component Uids.
	UE::Interchange::TArrayAttributeHelper<FString> ComponentUids;
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeInstancedStaticMeshComponentFactoryNode  : public UInterchangeSceneComponentFactoryNode
{
	GENERATED_BODY()
public:
	UE_API UInterchangeInstancedStaticMeshComponentFactoryNode ();

	UE_API virtual FString GetTypeName() const override;

	UE_API virtual UClass* GetObjectClass() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | ComponentFactory | Instanced")
	UE_API void AddInstanceTransform(const FTransform& InstanceTransform);
	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | ComponentFactory | Instanced")
	UE_API void AddInstanceTransforms(const TArray<FTransform>& InInstanceTransforms);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | ComponentFactory | Instanced")
	UE_API void GetInstanceTransforms(TArray<FTransform>& OutInstanceTransforms) const;

	/** Get which asset, if any, a scene node is instantiating. Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | ComponentFactory | Instanced")
	UE_API bool GetCustomInstancedAssetUid(FString& AttributeValue) const;

	/** Add an asset for this scene node to instantiate. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | ComponentFactory | Instanced")
	UE_API bool SetCustomInstancedAssetUid(const FString& AttributeValue);

	
	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | ComponentFactory | Instanced")
	UE_API bool GetCustomHierarchicalISM(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | ComponentFactory | Instanced")
	UE_API bool SetCustomHierarchicalISM(const bool& AttributeValue);

private:
	//Instanced StaticMesh Component's Instances.
	UE::Interchange::TArrayAttributeHelper<FTransform> InstanceTransforms;

	//To track/connect StaticMesh.
	const UE::Interchange::FAttributeKey Macro_CustomAssetInstanceUidKey = UE::Interchange::FAttributeKey(TEXT("AssetInstanceUid"));

	//To track if it should be Hierarchical ISM
	const UE::Interchange::FAttributeKey Macro_CustomHierarchicalISMKey = UE::Interchange::FAttributeKey(TEXT("Hierarchical"));
};

#undef UE_API
