// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeActorFactoryNode.h"

#include "InterchangeSceneImportAssetFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSceneImportAssetFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	// UInterchangeFactoryBaseNode Begin
	UE_API virtual class UClass* GetObjectClass() const override;
	// UInterchangeFactoryBaseNode End

private:
};

#undef UE_API
