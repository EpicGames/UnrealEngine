// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeFactoryBaseNode.h"

#include "InterchangeSceneVariantSetsFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSceneVariantSetsFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

	UE_API UInterchangeSceneVariantSetsFactoryNode();

public:
	static FStringView StaticAssetTypeName()
	{
		return TEXT("SceneVariantSetFactory");
	}

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("SceneVariantSetFactoryNode");
		return TypeName;
	}

	/** Get the class this node creates. */
	UE_API virtual class UClass* GetObjectClass() const override;

	/**
	 * Retrieve the number of unique IDs of all translated VariantSets for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SceneVariantSetFactory")
	UE_API int32 GetCustomVariantSetUidCount() const;

	/**
	 * Retrieve the unique IDs of all translated VariantSets for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SceneVariantSetFactory")
	UE_API void GetCustomVariantSetUids(TArray<FString>& OutVariantUids) const;

	/**
	 * Retrieve the UID of the VariantSet with the specified index.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SceneVariantSetFactory")
	UE_API void GetCustomVariantSetUid(const int32 Index, FString& OutVariantUid) const;

	/**
	 * Add a unique id of a translated VariantSet for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SceneVariantSetFactory")
	UE_API bool AddCustomVariantSetUid(const FString& VariantUid);

	/**
	 * Remove the specified unique ID of a translated VariantSet from this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SceneVariantSetFactory")
	UE_API bool RemoveCustomVariantSetUid(const FString& VariantUid);

	/** Return if the import of the class is allowed at runtime.*/
	virtual bool IsRuntimeImportAllowed() const override
	{
		return false;
	}

private:
	UE::Interchange::TArrayAttributeHelper<FString> CustomVariantSetUids;
};

#undef UE_API
