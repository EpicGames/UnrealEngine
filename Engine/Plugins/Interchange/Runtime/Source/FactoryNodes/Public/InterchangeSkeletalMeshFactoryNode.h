// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeMeshFactoryNode.h"
#include "Nodes/InterchangeFactoryBaseNode.h"

#if WITH_ENGINE
#include "Engine/SkeletalMesh.h"
#endif

#include "InterchangeSkeletalMeshFactoryNode.generated.h"

UENUM(BlueprintType)
enum class EInterchangeSkeletalMeshContentType : uint8
{
	All UMETA(DisplayName = "Geometry and Skin Weights", ToolTip = "Imports all skeletal mesh content: geometry and skin weights."),
	Geometry UMETA(DisplayName = "Geometry Only", ToolTip = "Imports the skeletal mesh geometry only. This creates a default skeleton, or maps the geometry to the existing one. You can import morph targets and LODs with the mesh."),
	SkinningWeights UMETA(DisplayName = "Skin Weights Only", ToolTip = "Imports the skeletal mesh skin weights only. No geometry, morph targets, or LODs are imported."),
	MAX,
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSkeletalMeshFactoryNode : public UInterchangeMeshFactoryNode
{
	GENERATED_BODY()

public:
	INTERCHANGEFACTORYNODES_API UInterchangeSkeletalMeshFactoryNode();

	/**
	 * Initialize node data. Also adds it to NodeContainer.
	 * @param: UniqueID - The unique ID for this node.
	 * @param DisplayLabel - The name of the node.
	 * @param InAssetClass - The class the SkeletalMesh factory will create for this node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API void InitializeSkeletalMeshNode(const FString& UniqueID, const FString& DisplayLabel, const FString& InAssetClass, UInterchangeBaseNodeContainer* NodeContainer);

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	INTERCHANGEFACTORYNODES_API virtual FString GetTypeName() const override;

	/** Get the class this node creates. */
	INTERCHANGEFACTORYNODES_API virtual class UClass* GetObjectClass() const override;

public:
	/** Query the skeletal mesh factory skeleton UObject. Return false if the attribute was not set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool GetCustomSkeletonSoftObjectPath(FSoftObjectPath& AttributeValue) const;

	/** Set the skeletal mesh factory skeleton UObject. Return false if the attribute could not be set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool SetCustomSkeletonSoftObjectPath(const FSoftObjectPath& AttributeValue);

	/** Query whether the skeletal mesh factory should create morph targets. Return false if the attribute was not set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool GetCustomImportMorphTarget(bool& AttributeValue) const;

	/** Set whether the skeletal mesh factory should create morph targets. Return false if the attribute could not be set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool SetCustomImportMorphTarget(const bool& AttributeValue);

	/**
	 * Get the custom attribute AddCurveMetadataToSkeleton. Return false if the attribute is not set.
	 * 
	 * Note - If this setting is disabled, curve metadata will be added to skeletal meshes for morph targets, but no metadata entry will be created for general curves.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	INTERCHANGEFACTORYNODES_API bool GetCustomAddCurveMetadataToSkeleton(bool& AttributeValue) const;

	/**
	 * Set the custom attribute AddCurveMetadataToSkeleton. Return false if the attribute could not be set.
	 * 
	 * Note - If this setting is disabled, curve metadata will be added to skeletal meshes for morph targets, but no metadata entry will be created for general curves.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimSequence")
	INTERCHANGEFACTORYNODES_API bool SetCustomAddCurveMetadataToSkeleton(const bool& AttributeValue);

	/** Query whether the skeletal mesh factory should import vertex attributes. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool GetCustomImportVertexAttributes(bool& AttributeValue) const;

	/** Set whether the skeletal mesh factory should import vertex attributes. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool SetCustomImportVertexAttributes(const bool& AttributeValue);

	/** Query whether the skeletal mesh factory should create a physics asset. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool GetCustomCreatePhysicsAsset(bool& AttributeValue) const;

	/** Set whether the skeletal mesh factory should create a physics asset. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool SetCustomCreatePhysicsAsset(const bool& AttributeValue);

	/** Query a physics asset the skeletal mesh factory should use. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool GetCustomPhysicAssetSoftObjectPath(FSoftObjectPath& AttributeValue) const;

	/** Set a physics asset the skeletal mesh factory should use. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool SetCustomPhysicAssetSoftObjectPath(const FSoftObjectPath& AttributeValue);

	/** Query the skeletal mesh import content type. This content type determines whether the factory imports partial or full translated content. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool GetCustomImportContentType(EInterchangeSkeletalMeshContentType& AttributeValue) const;

	/** Set the skeletal mesh import content type. This content type determines whether the factory imports partial or full translated content. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool SetCustomImportContentType(const EInterchangeSkeletalMeshContentType& AttributeValue);

	/** Query the skeletal mesh UseHighPrecisionSkinWeights setting. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool GetCustomUseHighPrecisionSkinWeights(bool& AttributeValue) const;

	/** Set the skeletal mesh UseHighPrecisionSkinWeights setting. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool SetCustomUseHighPrecisionSkinWeights(const bool& AttributeValue, bool bAddApplyDelegate = true);
		
	/** Query the skeletal mesh threshold value that is used to decide whether two vertex positions are equal. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool GetCustomThresholdPosition(float& AttributeValue) const;

	/** Set the skeletal mesh threshold value that is used to decide whether two vertex positions are equal. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool SetCustomThresholdPosition(const float& AttributeValue, bool bAddApplyDelegate = true);

	/** Query the skeletal mesh threshold value that is used to decide whether two normals, tangents, or bi-normals are equal. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool GetCustomThresholdTangentNormal(float& AttributeValue) const;

	/** Set the skeletal mesh threshold value that is used to decide whether two normals, tangents, or bi-normals are equal. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool SetCustomThresholdTangentNormal(const float& AttributeValue, bool bAddApplyDelegate = true);

	/** Query the skeletal mesh threshold value that is used to decide whether two UVs are equal. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool GetCustomThresholdUV(float& AttributeValue) const;

	/** Set the skeletal mesh threshold value that is used to decide whether two UVs are equal. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool SetCustomThresholdUV(const float& AttributeValue, bool bAddApplyDelegate = true);

	/** Query the skeletal mesh threshold value that is used to compare vertex position equality when computing morph target deltas. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool GetCustomMorphThresholdPosition(float& AttributeValue) const;

	/** Set the skeletal mesh threshold value that is used to compare vertex position equality when computing morph target deltas. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool SetCustomMorphThresholdPosition(const float& AttributeValue, bool bAddApplyDelegate = true);

	/**
	 * Query the maximum number of bone influences to allow each vertex in this mesh to use.
	 * If set higher than the limit determined by the project settings, it has no effect.
	 * If set to 0, the value is taken from the DefaultBoneInfluenceLimit project setting.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool GetCustomBoneInfluenceLimit(int32& AttributeValue) const;

	/**
	 * Set the maximum number of bone influences to allow each vertex in this mesh to use.
	 * If set higher than the limit determined by the project settings, it has no effect.
	 * If set to 0, the value is taken from the DefaultBoneInfluenceLimit project setting.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool SetCustomBoneInfluenceLimit(const int32& AttributeValue, bool bAddApplyDelegate = true);

	/** Query whether the skeletal mesh factory should merge morph target shape with the same name under one morph target. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool GetCustomMergeMorphTargetShapeWithSameName(bool& AttributeValue) const;

	/** Set whether the skeletal mesh factory should merge morph target shape with the same name under one morph target. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	INTERCHANGEFACTORYNODES_API bool SetCustomMergeMorphTargetShapeWithSameName(const bool& AttributeValue);

	/**
	 * The skeletal mesh thumbnail can have an overlay if the last reimport was geometry only. This thumbnail overlay feature uses the metadata to find out if the last import was geometry only.
	 */
	INTERCHANGEFACTORYNODES_API virtual void AppendAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	INTERCHANGEFACTORYNODES_API virtual void AppendAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;

	INTERCHANGEFACTORYNODES_API virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override;

	/** Return if the import of the class is allowed at runtime.*/
	virtual bool IsRuntimeImportAllowed () const override
	{
		return false;
	}
private:

	INTERCHANGEFACTORYNODES_API virtual void FillAssetClassFromAttribute() override;
	INTERCHANGEFACTORYNODES_API virtual bool SetNodeClassFromClassAttribute() override;

	const UE::Interchange::FAttributeKey ClassNameAttributeKey = UE::Interchange::FBaseNodeStaticData::ClassTypeAttributeKey();
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ImportMorphTarget)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AddCurveMetadataToSkeleton)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ImportVertexAttributes)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SkeletonSoftObjectPath)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(CreatePhysicsAsset)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(PhysicAssetSoftObjectPath)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ImportContentType)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(UseHighPrecisionSkinWeights)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ThresholdPosition)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ThresholdTangentNormal)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ThresholdUV)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(MorphThresholdPosition)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(BoneInfluenceLimit)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(MergeMorphTargetShapeWithSameName)
	

	INTERCHANGEFACTORYNODES_API bool ApplyCustomUseHighPrecisionSkinWeightsToAsset(UObject* Asset) const;
	INTERCHANGEFACTORYNODES_API bool FillCustomUseHighPrecisionSkinWeightsFromAsset(UObject* Asset);
	INTERCHANGEFACTORYNODES_API bool ApplyCustomThresholdPositionToAsset(UObject* Asset) const;
	INTERCHANGEFACTORYNODES_API bool FillCustomThresholdPositionFromAsset(UObject* Asset);
	INTERCHANGEFACTORYNODES_API bool ApplyCustomThresholdTangentNormalToAsset(UObject* Asset) const;
	INTERCHANGEFACTORYNODES_API bool FillCustomThresholdTangentNormalFromAsset(UObject* Asset);
	INTERCHANGEFACTORYNODES_API bool ApplyCustomThresholdUVToAsset(UObject* Asset) const;
	INTERCHANGEFACTORYNODES_API bool FillCustomThresholdUVFromAsset(UObject* Asset);
	INTERCHANGEFACTORYNODES_API bool ApplyCustomMorphThresholdPositionToAsset(UObject* Asset) const;
	INTERCHANGEFACTORYNODES_API bool FillCustomMorphThresholdPositionFromAsset(UObject* Asset);
	INTERCHANGEFACTORYNODES_API bool ApplyCustomBoneInfluenceLimitToAsset(UObject* Asset) const;
	INTERCHANGEFACTORYNODES_API bool FillCustomBoneInfluenceLimitFromAsset(UObject* Asset);
protected:
	TSubclassOf<USkeletalMesh> AssetClass;
};
