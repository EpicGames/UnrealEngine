// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeFactoryBaseNode.h"

#include "InterchangeSkeletalMeshLodDataNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

//Interchange namespace
namespace UE
{
	namespace Interchange
	{

		struct FSkeletalMeshNodeLodDataStaticData : public FBaseNodeStaticData
		{
			static UE_API const FAttributeKey& GetLODMeshDataArrayBaseKey();

			UE_DEPRECATED(5.8, "Deprecated. Use GetLODMeshDataArrayBaseKey instead.")
			static UE_API const FAttributeKey& GetMeshUidsBaseKey();
		};

	}//ns Interchange
}//ns UE

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSkeletalMeshLodDataNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	UE_API UInterchangeSkeletalMeshLodDataNode();

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	UE_API virtual FString GetTypeName() const override;

#if WITH_EDITOR
	UE_API virtual FString GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;

	UE_API virtual FString GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;
#endif //WITH_EDITOR

public:
	/** Query the LOD skeletal mesh factory skeleton reference. Return false if the attribute was not set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMeshLodData")
	UE_API bool GetCustomSkeletonUid(FString& AttributeValue) const;

	/** Set the LOD skeletal mesh factory skeleton reference. Return false if the attribute could not be set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMeshLodData")
	UE_API bool SetCustomSkeletonUid(const FString& AttributeValue);

	/* Return the number of mesh geometries this LOD will be made from. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMeshLodData")
	UE_API int32 GetMeshUidsCount() const;

	/* Query all mesh geometry this LOD will be made from. */
	UE_DEPRECATED(5.8, "Deprecated. Use GetLODMeshDataArray instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMeshLodData")
	UE_API void GetMeshUids(TArray<FString>& OutMeshUids) const;

	/* Add a mesh geometry used to create this LOD geometry. */
	UE_DEPRECATED(5.8, "Deprecated. Use AddLODMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMeshLodData")
	UE_API bool AddMeshUid(const FString& MeshUid);

	/* Gets all MeshUid+Transform pairs. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMeshLodData")
	UE_API void GetLODMeshDataArray(TArray<FInterchangeLODMeshData>& OutLODMeshDataArray) const;

	/* Adds a MeshUid + Transform pair to the node. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMeshLodData")
	UE_API bool AddLODMeshData(const FInterchangeLODMeshData& LODMeshData);

	/* Remove all mesh geometry used to create this LOD geometry.  */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMeshLodData")
	UE_API bool RemoveAllMeshes();

	/* Removes all occurances of a specific MeshUid from the entries. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMeshLodData")
	UE_API void RemoveMeshUid(const FString& MeshUid);

	/** Return if the import of the class is allowed at runtime.*/
	virtual bool IsRuntimeImportAllowed() const override
	{
		return false;
	}

private:

	UE_API bool IsEditorOnlyDataDefined();

	const UE::Interchange::FAttributeKey Macro_CustomSkeletonUidKey = UE::Interchange::FAttributeKey(TEXT("__SkeletonUid__Key"));

	UE::Interchange::TArrayAttributeHelper<FInterchangeLODMeshData> LODMeshDataArray;
protected:
};

#undef UE_API
