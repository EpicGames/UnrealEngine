// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Templates/SubclassOf.h"

#if WITH_ENGINE
#include "Animation/Skeleton.h"
#endif

#include "InterchangeSkeletonFactoryNode.generated.h"

UCLASS(BlueprintType, MinimalAPI)
class UInterchangeSkeletonFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:

	UInterchangeSkeletonFactoryNode()
	{
		SkeletalMeshFactoryNodeUids.Initialize(Attributes.ToSharedRef(), "__SkeletalMeshFactoryNodeUids__");
	}

	/**
	 * Initialize node data.
	 * @param: UniqueID - The unique ID for this node.
	 * @param DisplayLabel - The name of the node.
	 * @param InAssetClass - The class the Skeleton factory will create for this node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Skeleton")
	void InitializeSkeletonNode(const FString& UniqueID, const FString& DisplayLabel, const FString& InAssetClass, UInterchangeBaseNodeContainer* NodeContainer)
	{
		bIsNodeClassInitialized = false;
		NodeContainer->SetupNode(this, UniqueID, DisplayLabel, EInterchangeNodeContainerType::FactoryData);
		FString OperationName = GetTypeName() + TEXT(".SetAssetClassName");
		InterchangePrivateNodeBase::SetCustomAttribute<FString>(*Attributes, ClassNameAttributeKey, OperationName, InAssetClass);
		FillAssetClassFromAttribute();
	}

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
#if WITH_ENGINE
		if(Ar.IsLoading())
		{
			//Make sure the class is properly set when we compile with engine, this will set the
			//bIsNodeClassInitialized to true.
			SetNodeClassFromClassAttribute();
		}
#endif //#if WITH_ENGINE
	}

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("SkeletonNode");
		return TypeName;
	}

	/** Get the class this node creates. */
	virtual class UClass* GetObjectClass() const override
	{
		ensure(bIsNodeClassInitialized);
#if WITH_ENGINE
		return AssetClass.Get() != nullptr ? AssetClass.Get() : USkeleton::StaticClass();
#else
		return nullptr;
#endif
	}

	virtual FGuid GetHash() const override
	{
		return Attributes->GetStorageHash();
	}

public:
	/** Return false if the attribute was not set previously. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Skeleton")
	bool GetCustomRootJointUid(FString& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(RootJointID, FString);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Skeleton")
	bool SetCustomRootJointUid(const FString& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(RootJointID, FString)
	}

	/** Query whether this skeleton should replace joint transforms with time-zero evaluation instead of bind pose. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Skeleton")
	bool GetCustomUseTimeZeroForBindPose(bool& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(UseTimeZeroForBindPose, bool);
	}

	/** If AttributeValue is true, force this skeleton to use time-zero evaluation instead of its bind pose. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Skeleton")
	bool SetCustomUseTimeZeroForBindPose(const bool& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(UseTimeZeroForBindPose, bool)
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Skeleton",
		meta = (DeprecatedFunction, DeprecationMessage = "Use GetCustomSkeletalMeshFactoryNodeUidAtIndex (or GetCustomSkeletalMeshFactoryNodeUids) instead."))
	bool GetCustomSkeletalMeshFactoryNodeUid(FString& AttributeValue) const
	{
		if (GetCustomSkeletalMeshFactoryNodeCount() > 0)
		{
			GetCustomSkeletalMeshFactoryNodeUidAtIndex(0, AttributeValue);
			return true;
		}

		return false;
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Skeleton")
	int32 GetCustomSkeletalMeshFactoryNodeCount() const
	{
		return SkeletalMeshFactoryNodeUids.GetCount();
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Skeleton")
	void GetCustomSkeletalMeshFactoryNodeUids(TArray<FString>& OutDependencies) const
	{
		SkeletalMeshFactoryNodeUids.GetItems(OutDependencies);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Skeleton")
	void GetCustomSkeletalMeshFactoryNodeUidAtIndex(const int32 Index, FString& OutDependency) const
	{
		SkeletalMeshFactoryNodeUids.GetItem(Index, OutDependency);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Skeleton")
	bool AddCustomSkeletalMeshFactoryNodeUid(const FString& InDependency)
	{
		return SkeletalMeshFactoryNodeUids.AddItem(InDependency);
	}

private:

	void FillAssetClassFromAttribute()
	{
#if WITH_ENGINE
		FString OperationName = GetTypeName() + TEXT(".GetAssetClassName");
		FString ClassName;
		InterchangePrivateNodeBase::GetCustomAttribute<FString>(*Attributes, ClassNameAttributeKey, OperationName, ClassName);
		if (ClassName.Equals(USkeleton::StaticClass()->GetName()))
		{
			AssetClass = USkeleton::StaticClass();
			bIsNodeClassInitialized = true;
		}
#endif
	}

	bool SetNodeClassFromClassAttribute()
	{
		if (!bIsNodeClassInitialized)
		{
			FillAssetClassFromAttribute();
		}
		return bIsNodeClassInitialized;
	}

	bool IsEditorOnlyDataDefined()
	{
#if WITH_EDITORONLY_DATA
		return true;
#else
		return false;
#endif
	}

	const UE::Interchange::FAttributeKey ClassNameAttributeKey = UE::Interchange::FBaseNodeStaticData::ClassTypeAttributeKey();

	//Skeleton
	const UE::Interchange::FAttributeKey Macro_CustomRootJointIDKey = UE::Interchange::FAttributeKey(TEXT("RootJointID"));
	const UE::Interchange::FAttributeKey Macro_CustomUseTimeZeroForBindPoseKey = UE::Interchange::FAttributeKey(TEXT("UseTimeZeroForBindPose"));
	//const UE::Interchange::FAttributeKey Macro_CustomSkeletalMeshFactoryNodeUidKey = UE::Interchange::FAttributeKey(TEXT("SkeletalMeshFactoryNodeUid"));
	const UE::Interchange::FAttributeKey Macro_CustomSkeletalMeshFactoryNodeUidKey = UE::Interchange::FAttributeKey(TEXT("SkeletalMeshFactoryNodeUids"));
	
	UE::Interchange::TArrayAttributeHelper<FString> SkeletalMeshFactoryNodeUids;
	
protected:
#if WITH_ENGINE
	TSubclassOf<USkeleton> AssetClass;
#endif
	bool bIsNodeClassInitialized = false;
};
