// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Volume/InterchangeVolumeDefinitions.h"

#include "InterchangeSparseVolumeTextureFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSparseVolumeTextureFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	UE_API virtual FString GetTypeName() const override;
	UE_API virtual class UClass* GetObjectClass() const override;

public:
	/** Gets the data type of the AttributesA texture of the SparseVolumeTexture we'll create */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomAttributesAFormat(EInterchangeSparseVolumeTextureFormat& Format) const;

	/** Sets the data type of the AttributesA texture of the SparseVolumeTexture we'll create */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomAttributesAFormat(EInterchangeSparseVolumeTextureFormat Format);

	/** Gets the data type of the AttributesB texture of the SparseVolumeTexture we'll create */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomAttributesBFormat(EInterchangeSparseVolumeTextureFormat& Format) const;

	/** Sets the data type of the AttributesB texture of the SparseVolumeTexture we'll create */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomAttributesBFormat(EInterchangeSparseVolumeTextureFormat Format);

	/** Gets the grid name and component index that will be assigned to the AttributesA texture, channel X (e.g. "density_0" or "temperature_2") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomAttributesAChannelX(FString& GridNameAndComponentIndex) const;

	/** Sets the grid name and component index that will be assigned to the AttributesA texture, channel X (e.g. "density_0" or "temperature_2") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomAttributesAChannelX(FString GridNameAndComponentIndex);

	/** Gets the grid name and component index that will be assigned to the AttributesA texture, channel Y (e.g. "density_0" or "temperature_2") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomAttributesAChannelY(FString& GridNameAndComponentIndex) const;

	/** Sets the grid name and component index that will be assigned to the AttributesA texture, channel Y (e.g. "density_0" or "temperature_2") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomAttributesAChannelY(FString GridNameAndComponentIndex);

	/** Gets the grid name and component index that will be assigned to the AttributesA texture, channel Z (e.g. "density_0" or "temperature_2") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomAttributesAChannelZ(FString& GridNameAndComponentIndex) const;

	/** Sets the grid name and component index that will be assigned to the AttributesA texture, channel Z (e.g. "density_0" or "temperature_2") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomAttributesAChannelZ(FString GridNameAndComponentIndex);

	/** Gets the grid name and component index that will be assigned to the AttributesA texture, channel W (e.g. "density_0" or "temperature_2") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomAttributesAChannelW(FString& GridNameAndComponentIndex) const;

	/** Sets the grid name and component index that will be assigned to the AttributesA texture, channel W (e.g. "density_0" or "temperature_2") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomAttributesAChannelW(FString GridNameAndComponentIndex);

	/** Gets the grid name and component index that will be assigned to the AttributesB texture, channel X (e.g. "density_0" or "temperature_2") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomAttributesBChannelX(FString& GridNameAndComponentIndex) const;

	/** Sets the grid name and component index that will be assigned to the AttributesB texture, channel X (e.g. "density_0" or "temperature_2") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomAttributesBChannelX(FString GridNameAndComponentIndex);

	/** Gets the grid name and component index that will be assigned to the AttributesB texture, channel Y (e.g. "density_0" or "temperature_2") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomAttributesBChannelY(FString& GridNameAndComponentIndex) const;

	/** Sets the grid name and component index that will be assigned to the AttributesB texture, channel Y (e.g. "density_0" or "temperature_2") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomAttributesBChannelY(FString GridNameAndComponentIndex);

	/** Gets the grid name and component index that will be assigned to the AttributesB texture, channel Z (e.g. "density_0" or "temperature_2") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomAttributesBChannelZ(FString& GridNameAndComponentIndex) const;

	/** Sets the grid name and component index that will be assigned to the AttributesB texture, channel Z (e.g. "density_0" or "temperature_2") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomAttributesBChannelZ(FString GridNameAndComponentIndex);

	/** Gets the grid name and component index that will be assigned to the AttributesB texture, channel W (e.g. "density_0" or "temperature_2") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomAttributesBChannelW(FString& GridNameAndComponentIndex) const;

	/** Sets the grid name and component index that will be assigned to the AttributesB texture, channel W (e.g. "density_0" or "temperature_2") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomAttributesBChannelW(FString GridNameAndComponentIndex);

	/** Gets the animation ID of the volume nodes that were grouped together to create this animated SparseVolumeTextureFactoryNode, if any */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomAnimationID(FString& OutAnimationID) const;

	/** Sets the animation ID of the volume nodes that were grouped together to create this animated SparseVolumeTextureFactoryNode, if any */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomAnimationID(const FString& InAnimationID);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AttributesADataType)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AttributesBDataType)

	IMPLEMENT_NODE_ATTRIBUTE_KEY(AttributesAChannelX)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AttributesAChannelY)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AttributesAChannelZ)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AttributesAChannelW)

	IMPLEMENT_NODE_ATTRIBUTE_KEY(AttributesBChannelX)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AttributesBChannelY)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AttributesBChannelZ)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AttributesBChannelW)

	IMPLEMENT_NODE_ATTRIBUTE_KEY(AnimationID)
};

#undef UE_API
