// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeFactoryBaseNode.h"

#include "InterchangeSpecularProfileFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

enum class ESpecularProfileFormat : uint8;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSpecularProfileFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	UE_API virtual FString GetTypeName() const override;

	UE_API virtual class UClass* GetObjectClass() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | SpecularProfile")
	UE_API bool SetCustomFormat(ESpecularProfileFormat Format);

	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | SpecularProfile")
	UE_API bool GetCustomFormat(ESpecularProfileFormat& Format) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | SpecularProfile")
	UE_API bool GetCustomTexture(FString& TextureUid) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | FactoryNode | SpecularProfile")
	UE_API bool SetCustomTexture(const FString& TextureUid);

private:

	IMPLEMENT_NODE_ATTRIBUTE_KEY(Format)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Texture)
};

#undef UE_API
