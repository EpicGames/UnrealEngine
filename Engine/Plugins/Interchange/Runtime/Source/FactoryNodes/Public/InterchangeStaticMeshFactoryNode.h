// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeMeshFactoryNode.h"
#include "Nodes/InterchangeFactoryBaseNode.h"

#if WITH_ENGINE
#include "Engine/StaticMesh.h"
#endif

#include "InterchangeStaticMeshFactoryNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API

/** Mirrors ENaniteShapePreservation from Engine/EngineTypes.h */
UENUM()
enum class EInterchangeNaniteShapePreservation : int32
{
	None,
	PreserveArea,
	Voxelize
};

/** Mirrors ENaniteGenerateFallback from Engine/EngineTypes.h */
UENUM()
enum class EInterchangeNaniteGenerateFallback : int32
{
	PlatformDefault,
	Enabled
};

/** Mirrors ENaniteFallbackTarget from Engine/EngineTypes.h */
UENUM()
enum class EInterchangeNaniteFallbackTarget : int32
{
	Auto,
	PercentTriangles,
	RelativeError
};

namespace UE
{
	namespace Interchange
	{
		struct FStaticMeshNodeStaticData : public FBaseNodeStaticData
		{
			static UE_API const FAttributeKey& GetLODScreenSizeBaseKey();
			static UE_API const FAttributeKey& GetSocketUidsBaseKey();
		};
	} // namespace Interchange
} // namespace UE


UCLASS(MinimalAPI, BlueprintType)
class UInterchangeStaticMeshFactoryNode : public UInterchangeMeshFactoryNode
{
	GENERATED_BODY()

public:
	UE_API UInterchangeStaticMeshFactoryNode();

	/**
	 * Initialize node data. Also adds it to NodeContainer.
	 * @param UniqueID - The unique ID for this node.
	 * @param DisplayLabel - The name of the node.
	 * @param InAssetClass - The class the StaticMesh factory will create for this node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API void InitializeStaticMeshNode(const FString& UniqueID, const FString& DisplayLabel, const FString& InAssetClass, UInterchangeBaseNodeContainer* NodeContainer);

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	UE_API virtual FString GetTypeName() const override;

	/** Get the class this node creates. */
	UE_API virtual class UClass* GetObjectClass() const override;

	UE_API virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override;

#if WITH_EDITOR

	UE_API virtual FString GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;
	UE_API virtual FString GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;

#endif //WITH_EDITOR

public:
	/** Get whether the static mesh factory should auto compute LOD Screen Sizes. Return false if the attribute was not set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomAutoComputeLODScreenSizes(bool& AttributeValue) const;

	/** Set whether the static mesh factory should auto compute LOD Screen Sizes. Return false if the attribute was not set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomAutoComputeLODScreenSizes(const bool& AttributeValue);

	/** Returns the number of LOD Screen Sizes the static mesh has.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API int32 GetLODScreenSizeCount() const;

	/** Returns All the LOD Screen Sizes set for the static mesh.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API void GetLODScreenSizes(TArray<float>& OutLODScreenSizes) const;

	/** Sets the LOD Screen Sizes for the static mesh.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetLODScreenSizes(const TArray<float>& InLODScreenSizes);

	/** Get whether the static mesh factory should set the Nanite build setting. Return false if the attribute was not set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomBuildNanite(bool& AttributeValue) const;

	/** Set whether the static mesh factory should set the Nanite build setting. Return false if the attribute was not set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomBuildNanite(const bool& AttributeValue, bool bAddApplyDelegate = true);

	/** Return the number of socket UIDs this static mesh has. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API int32 GetSocketUidCount() const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API void GetSocketUids(TArray<FString>& OutSocketUids) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool AddSocketUid(const FString& SocketUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool AddSocketUids(const TArray<FString>& InSocketUids);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool RemoveSocketUd(const FString& SocketUid);

	/** Get whether the static mesh should build a reversed index buffer. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomBuildReversedIndexBuffer(bool& AttributeValue) const;

	/** Set whether the static mesh should build a reversed index buffer. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomBuildReversedIndexBuffer(const bool& AttributeValue, bool bAddApplyDelegate = true);

	/** Get whether the static mesh should generate lightmap UVs. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomGenerateLightmapUVs(bool& AttributeValue) const;

	/** Set whether the static mesh should generate lightmap UVs. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomGenerateLightmapUVs(const bool& AttributeValue, bool bAddApplyDelegate = true);

	/**
	 * Get whether to generate the distance field by treating every triangle hit as a front face.  
	 * This prevents the distance field from being discarded due to the mesh being open, but also lowers distance field ambient occlusion quality.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomGenerateDistanceFieldAsIfTwoSided(bool& AttributeValue) const;

	/**
	 * Set whether to generate the distance field by treating every triangle hit as a front face.
	 * This prevents the distance field from being discarded due to the mesh being open, but also lowers distance field ambient occlusion quality.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomGenerateDistanceFieldAsIfTwoSided(const bool& AttributeValue, bool bAddApplyDelegate = true);

	/** Get whether the static mesh is set up for use with physical material masks. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomSupportFaceRemap(bool& AttributeValue) const;

	/** Set whether the static mesh is set up for use with physical material masks. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomSupportFaceRemap(const bool& AttributeValue, bool bAddApplyDelegate = true);

	/** Get the amount of padding used to pack UVs for the static mesh. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomMinLightmapResolution(int32& AttributeValue) const;

	/** Set the amount of padding used to pack UVs for the static mesh. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomMinLightmapResolution(const int32& AttributeValue, bool bAddApplyDelegate = true);

	/** Get the index of the UV that is used as the source for generating lightmaps for the static mesh.  */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomSrcLightmapIndex(int32& AttributeValue) const;

	/** Set the index of the UV that is used as the source for generating lightmaps for the static mesh. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomSrcLightmapIndex(const int32& AttributeValue, bool bAddApplyDelegate = true);

	/** Get the index of the UV that is used to store generated lightmaps for the static mesh. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomDstLightmapIndex(int32& AttributeValue) const;

	/** Set the index of the UV that is used to store generated lightmaps for the static mesh. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomDstLightmapIndex(const int32& AttributeValue, bool bAddApplyDelegate = true);

	/** Get the local scale that is applied when building the static mesh. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomBuildScale3D(FVector& AttributeValue) const;

	/** Set the local scale that is applied when building the static mesh. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomBuildScale3D(const FVector& AttributeValue, bool bAddApplyDelegate = true);

	/**
	 * Get the scale to apply to the mesh when allocating the distance field volume texture.
	 * The default scale is 1, which assumes that the mesh will be placed unscaled in the world.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomDistanceFieldResolutionScale(float& AttributeValue) const;

	/**
	 * Set the scale to apply to the mesh when allocating the distance field volume texture.
	 * The default scale is 1, which assumes that the mesh will be placed unscaled in the world.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomDistanceFieldResolutionScale(const float& AttributeValue, bool bAddApplyDelegate = true);

	/** Get the static mesh asset whose distance field will be used as the distance field for the imported mesh. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomDistanceFieldReplacementMesh(FSoftObjectPath& AttributeValue) const;

	/** Set the static mesh asset whose distance field will be used as the distance field for the imported mesh. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomDistanceFieldReplacementMesh(const FSoftObjectPath& AttributeValue, bool bAddApplyDelegate = true);

	/**
	 * Get the maximum number of Lumen mesh cards to generate for this mesh.
	 * More cards means that the surface will have better coverage, but will result in increased runtime overhead.
	 * Set this to 0 to disable mesh card generation for this mesh.
	 * The default is 12.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomMaxLumenMeshCards(int32& AttributeValue) const;

	/**
	 * Set the maximum number of Lumen mesh cards to generate for this mesh.
	 * More cards means that the surface will have better coverage, but will result in increased runtime overhead.
	 * Set this to 0 to disable mesh card generation for this mesh.
	 * The default is 12.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomMaxLumenMeshCards(const int32& AttributeValue, bool bAddApplyDelegate = true);


	/**
	* Currently specifically used for LOD group nodes.
	* If an LOD Group is identified as identical to another one (when bakeMesh is turned off),
	*	then said LOD Group's asset won't be created and the substitute UID will be set to the FactoryNode that's identical to the one at hand.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomSubstituteUID(const FString& AttributeValue);

	/**
	* Gets the Substitute UID, said UID can be used to acquire an identical FactoryNode.
	* (for more see SetCustomSubstituteUID)
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomSubstituteUID(FString& AttributeValue) const;

	// Nanite Build Settings

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteExplicitTangents(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteExplicitTangents(const bool& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteLerpUVs(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteLerpUVs(const bool& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteSeparable(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteSeparable(const bool& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteVoxelNDF(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteVoxelNDF(const bool& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteVoxelOpacity(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteVoxelOpacity(const bool& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteShapePreservation(int32& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteShapePreservation(const int32& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNanitePositionPrecision(int32& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNanitePositionPrecision(const int32& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteNormalPrecision(int32& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteNormalPrecision(const int32& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteTangentPrecision(int32& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteTangentPrecision(const int32& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteBoneWeightPrecision(int32& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteBoneWeightPrecision(const int32& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteTargetMinimumResidencyInKB(int32& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteTargetMinimumResidencyInKB(const int32& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteKeepPercentTriangles(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteKeepPercentTriangles(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteTrimRelativeError(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteTrimRelativeError(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteGenerateFallback(int32& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteGenerateFallback(const int32& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteFallbackTarget(int32& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteFallbackTarget(const int32& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteFallbackPercentTriangles(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteFallbackPercentTriangles(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteFallbackRelativeError(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteFallbackRelativeError(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteMaxEdgeLengthFactor(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteMaxEdgeLengthFactor(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteNumRays(int32& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteNumRays(const int32& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteVoxelLevel(int32& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteVoxelLevel(const int32& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteRayBackUp(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteRayBackUp(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetCustomNaniteDisplacementUVChannel(int32& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetCustomNaniteDisplacementUVChannel(const int32& AttributeValue, bool bAddApplyDelegate = true);

private:
	UE_API virtual void FillAssetClassFromAttribute() override;
	UE_API virtual bool SetNodeClassFromClassAttribute() override;

	const UE::Interchange::FAttributeKey Macro_CustomBuildReversedIndexBufferKey = UE::Interchange::FAttributeKey(TEXT("BuildReversedIndexBuffer"));
	const UE::Interchange::FAttributeKey Macro_CustomGenerateLightmapUVsKey = UE::Interchange::FAttributeKey(TEXT("GenerateLightmapUVs"));
	const UE::Interchange::FAttributeKey Macro_CustomGenerateDistanceFieldAsIfTwoSidedKey = UE::Interchange::FAttributeKey(TEXT("GenerateDistanceFieldAsIfTwoSided"));
	const UE::Interchange::FAttributeKey Macro_CustomSupportFaceRemapKey = UE::Interchange::FAttributeKey(TEXT("SupportFaceRemap"));
	const UE::Interchange::FAttributeKey Macro_CustomMinLightmapResolutionKey = UE::Interchange::FAttributeKey(TEXT("MinLightmapResolution"));
	const UE::Interchange::FAttributeKey Macro_CustomSrcLightmapIndexKey = UE::Interchange::FAttributeKey(TEXT("SrcLightmapIndex"));
	const UE::Interchange::FAttributeKey Macro_CustomDstLightmapIndexKey = UE::Interchange::FAttributeKey(TEXT("DstLightmapIndex"));
	const UE::Interchange::FAttributeKey Macro_CustomBuildScale3DKey = UE::Interchange::FAttributeKey(TEXT("BuildScale3D"));
	const UE::Interchange::FAttributeKey Macro_CustomDistanceFieldResolutionScaleKey = UE::Interchange::FAttributeKey(TEXT("DistanceFieldResolutionScale"));
	const UE::Interchange::FAttributeKey Macro_CustomDistanceFieldReplacementMeshKey = UE::Interchange::FAttributeKey(TEXT("DistanceFieldReplacementMesh"));
	const UE::Interchange::FAttributeKey Macro_CustomMaxLumenMeshCardsKey = UE::Interchange::FAttributeKey(TEXT("MaxLumenMeshCards"));
	const UE::Interchange::FAttributeKey Macro_CustomBuildNaniteKey = UE::Interchange::FAttributeKey(TEXT("BuildNanite"));
	const UE::Interchange::FAttributeKey Macro_CustomAutoComputeLODScreenSizesKey = UE::Interchange::FAttributeKey(TEXT("AutoComputeLODScreenSizes"));
	const UE::Interchange::FAttributeKey Macro_CustomSubstituteUIDKey = UE::Interchange::FAttributeKey(TEXT("SubstituteUID"));

	// Nanite build settings attribute keys
	const UE::Interchange::FAttributeKey Macro_CustomNaniteExplicitTangentsKey = UE::Interchange::FAttributeKey(TEXT("NaniteExplicitTangents"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteLerpUVsKey = UE::Interchange::FAttributeKey(TEXT("NaniteLerpUVs"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteSeparableKey = UE::Interchange::FAttributeKey(TEXT("NaniteSeparable"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteVoxelNDFKey = UE::Interchange::FAttributeKey(TEXT("NaniteVoxelNDF"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteVoxelOpacityKey = UE::Interchange::FAttributeKey(TEXT("NaniteVoxelOpacity"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteShapePreservationKey = UE::Interchange::FAttributeKey(TEXT("NaniteShapePreservation"));
	const UE::Interchange::FAttributeKey Macro_CustomNanitePositionPrecisionKey = UE::Interchange::FAttributeKey(TEXT("NanitePositionPrecision"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteNormalPrecisionKey = UE::Interchange::FAttributeKey(TEXT("NaniteNormalPrecision"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteTangentPrecisionKey = UE::Interchange::FAttributeKey(TEXT("NaniteTangentPrecision"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteBoneWeightPrecisionKey = UE::Interchange::FAttributeKey(TEXT("NaniteBoneWeightPrecision"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteTargetMinimumResidencyInKBKey = UE::Interchange::FAttributeKey(TEXT("NaniteTargetMinimumResidencyInKB"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteKeepPercentTrianglesKey = UE::Interchange::FAttributeKey(TEXT("NaniteKeepPercentTriangles"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteTrimRelativeErrorKey = UE::Interchange::FAttributeKey(TEXT("NaniteTrimRelativeError"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteGenerateFallbackKey = UE::Interchange::FAttributeKey(TEXT("NaniteGenerateFallback"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteFallbackTargetKey = UE::Interchange::FAttributeKey(TEXT("NaniteFallbackTarget"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteFallbackPercentTrianglesKey = UE::Interchange::FAttributeKey(TEXT("NaniteFallbackPercentTriangles"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteFallbackRelativeErrorKey = UE::Interchange::FAttributeKey(TEXT("NaniteFallbackRelativeError"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteMaxEdgeLengthFactorKey = UE::Interchange::FAttributeKey(TEXT("NaniteMaxEdgeLengthFactor"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteNumRaysKey = UE::Interchange::FAttributeKey(TEXT("NaniteNumRays"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteVoxelLevelKey = UE::Interchange::FAttributeKey(TEXT("NaniteVoxelLevel"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteRayBackUpKey = UE::Interchange::FAttributeKey(TEXT("NaniteRayBackUp"));
	const UE::Interchange::FAttributeKey Macro_CustomNaniteDisplacementUVChannelKey = UE::Interchange::FAttributeKey(TEXT("NaniteDisplacementUVChannel"));

	UE::Interchange::TArrayAttributeHelper<float> LODScreenSizes;
	UE::Interchange::TArrayAttributeHelper<FString> SocketUids;

protected:
	
#if WITH_EDITORONLY_DATA
	IMPLEMENT_NODE_ATTRIBUTE_DELEGATE_BY_PROPERTYNAME(BuildNanite, bool, UStaticMesh, TEXT("NaniteSettings.bEnabled"));
#endif

	UE_API bool ApplyCustomBuildReversedIndexBufferToAsset(UObject* Asset) const;
	UE_API bool FillCustomBuildReversedIndexBufferFromAsset(UObject* Asset);
	UE_API bool ApplyCustomGenerateLightmapUVsToAsset(UObject* Asset) const;
	UE_API bool FillCustomGenerateLightmapUVsFromAsset(UObject* Asset);
	UE_API bool ApplyCustomGenerateDistanceFieldAsIfTwoSidedToAsset(UObject* Asset) const;
	UE_API bool FillCustomGenerateDistanceFieldAsIfTwoSidedFromAsset(UObject* Asset);
	UE_API bool ApplyCustomSupportFaceRemapToAsset(UObject* Asset) const;
	UE_API bool FillCustomSupportFaceRemapFromAsset(UObject* Asset);
	UE_API bool ApplyCustomMinLightmapResolutionToAsset(UObject* Asset) const;
	UE_API bool FillCustomMinLightmapResolutionFromAsset(UObject* Asset);
	UE_API bool ApplyCustomSrcLightmapIndexToAsset(UObject* Asset) const;
	UE_API bool FillCustomSrcLightmapIndexFromAsset(UObject* Asset);
	UE_API bool ApplyCustomDstLightmapIndexToAsset(UObject* Asset) const;
	UE_API bool FillCustomDstLightmapIndexFromAsset(UObject* Asset);
	UE_API bool ApplyCustomBuildScale3DToAsset(UObject* Asset) const;
	UE_API bool FillCustomBuildScale3DFromAsset(UObject* Asset);
	UE_API bool ApplyCustomDistanceFieldResolutionScaleToAsset(UObject* Asset) const;
	UE_API bool FillCustomDistanceFieldResolutionScaleFromAsset(UObject* Asset);
	UE_API bool ApplyCustomDistanceFieldReplacementMeshToAsset(UObject* Asset) const;
	UE_API bool FillCustomDistanceFieldReplacementMeshFromAsset(UObject* Asset);
	UE_API bool ApplyCustomMaxLumenMeshCardsToAsset(UObject* Asset) const;
	UE_API bool FillCustomMaxLumenMeshCardsFromAsset(UObject* Asset);

	// Nanite build settings Apply/Fill
	UE_API bool ApplyCustomNaniteExplicitTangentsToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteExplicitTangentsFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteLerpUVsToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteLerpUVsFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteSeparableToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteSeparableFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteVoxelNDFToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteVoxelNDFFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteVoxelOpacityToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteVoxelOpacityFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteShapePreservationToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteShapePreservationFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNanitePositionPrecisionToAsset(UObject* Asset) const;
	UE_API bool FillCustomNanitePositionPrecisionFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteNormalPrecisionToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteNormalPrecisionFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteTangentPrecisionToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteTangentPrecisionFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteBoneWeightPrecisionToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteBoneWeightPrecisionFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteTargetMinimumResidencyInKBToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteTargetMinimumResidencyInKBFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteKeepPercentTrianglesToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteKeepPercentTrianglesFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteTrimRelativeErrorToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteTrimRelativeErrorFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteGenerateFallbackToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteGenerateFallbackFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteFallbackTargetToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteFallbackTargetFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteFallbackPercentTrianglesToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteFallbackPercentTrianglesFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteFallbackRelativeErrorToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteFallbackRelativeErrorFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteMaxEdgeLengthFactorToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteMaxEdgeLengthFactorFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteNumRaysToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteNumRaysFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteVoxelLevelToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteVoxelLevelFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteRayBackUpToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteRayBackUpFromAsset(UObject* Asset);
	UE_API bool ApplyCustomNaniteDisplacementUVChannelToAsset(UObject* Asset) const;
	UE_API bool FillCustomNaniteDisplacementUVChannelFromAsset(UObject* Asset);

#if WITH_ENGINE
	TSubclassOf<UStaticMesh> AssetClass;
#endif
};

#undef UE_API