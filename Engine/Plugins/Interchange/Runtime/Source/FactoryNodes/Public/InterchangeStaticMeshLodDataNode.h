// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeMeshDefinitions.h"
#include "UObject/ObjectMacros.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Types/InterchangeLODMeshData.h"

#include "InterchangeStaticMeshLodDataNode.generated.h"

#define UE_API INTERCHANGEFACTORYNODES_API


namespace UE
{
	namespace Interchange
	{
		struct FStaticMeshNodeLodDataStaticData : public FBaseNodeStaticData
		{
			static const FAttributeKey& GetLODMeshDataArrayBaseKey();
			static const FAttributeKey& GetBoxCollisionMeshDataArrayBaseKey();
			static const FAttributeKey& GetCapsuleCollisionMeshDataArrayBaseKey();
			static const FAttributeKey& GetSphereCollisionMeshDataArrayBaseKey();
			static const FAttributeKey& GetConvexCollisionMeshDataArrayBaseKey();

			UE_DEPRECATED(5.8, "Use GetLODMeshDataArrayBaseKey instead")
			static const FAttributeKey& GetMeshUidsBaseKey();

			UE_DEPRECATED(5.8, "Use GetBoxCollisionMeshDataArrayBaseKey instead")
			static const FAttributeKey& GetBoxCollisionMeshUidsBaseKey();

			UE_DEPRECATED(5.8, "Use GetCapsuleCollisionMeshDataArrayBaseKey instead")
			static const FAttributeKey& GetCapsuleCollisionMeshUidsBaseKey();

			UE_DEPRECATED(5.8, "Use GetSphereCollisionMeshDataArrayBaseKey instead")
			static const FAttributeKey& GetSphereCollisionMeshUidsBaseKey();

			UE_DEPRECATED(5.8, "Use GetConvexCollisionMeshDataArrayBaseKey instead")
			static const FAttributeKey& GetConvexCollisionMeshUidsBaseKey();
		};
	} // namespace Interchange
} // namespace UE


UCLASS(MinimalAPI, BlueprintType)
class UInterchangeStaticMeshLodDataNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	UE_API UInterchangeStaticMeshLodDataNode();

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	UE_API virtual FString GetTypeName() const override;

#if WITH_EDITOR
	UE_API virtual FString GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;
	UE_API virtual FString GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;
#endif //WITH_EDITOR

public:

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API int32 GetLODMeshDataArrayCount() const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API void GetLODMeshDataArray(TArray<FInterchangeLODMeshData>& OutLODMeshDataArray) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool AddLODMeshData(const FInterchangeLODMeshData& LODMeshData);

	/**
	* Removes all entries where FInterchangeLODMeshData.MeshUid == MeshUid
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API void RemoveMeshUid(const FString& MeshUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveAllLODMeshData();

	/*
	* New collision-handling functions
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool AddBoxCollisionMeshData(const FInterchangeLODMeshData& MeshData);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool AddSphereCollisionMeshData(const FInterchangeLODMeshData& MeshData);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool AddCapsuleCollisionMeshData(const FInterchangeLODMeshData& MeshData);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool AddConvexCollisionMeshData(const FInterchangeLODMeshData& MeshData);


	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API void GetBoxCollisionsMeshDataArray(TArray<FInterchangeLODMeshData>& OutMeshDataArray) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API void GetSphereCollisionsMeshDataArray(TArray<FInterchangeLODMeshData>& OutMeshDataArray) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API void GetCapsuleCollisionsMeshDataArray(TArray<FInterchangeLODMeshData>& OutMeshDataArray) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API void GetConvexCollisionsMeshDataArray(TArray<FInterchangeLODMeshData>& OutMeshDataArray) const;


	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveBoxCollisionMeshData(const FString& MeshUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveSphereCollisionMeshData(const FString& MeshUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveCapsuleCollisionMeshData(const FString& MeshUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveConvexCollisionMeshData(const FString& MeshUid);


	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveAllBoxCollisionMeshData();

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveAllSphereCollisionMeshData();

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveAllCapsuleCollisionMeshData();

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveAllConvexCollisionMeshData();
	/*
	* End of new collision-handling functions
	*/


	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool GetOneConvexHullPerUCX(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool SetOneConvexHullPerUCX(bool AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool GetImportCollisionType(EInterchangeMeshCollision& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool SetImportCollisionType(EInterchangeMeshCollision AttributeValue);

	/** 
	 * Gets whether we're generating collision primitive shapes even if the mesh data 
	 * doesn't match the desired shape very well
	 */
	 UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	 UE_API bool GetForceCollisionPrimitiveGeneration(bool& bGenerate) const;
	 
	 /** 
	  * Sets whether we're generating collision primitive shapes even if the mesh data 
	  * doesn't match the desired shape very well
	  */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool SetForceCollisionPrimitiveGeneration(bool bGenerate);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool GetImportCollision(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool SetImportCollision(bool AttributeValue);


	/*
	* Deprecated functions from here:
	*/
	UE_DEPRECATED(5.8, "Use GetLODMeshDataArrayCount instead")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API int32 GetMeshUidsCount() const;

	UE_DEPRECATED(5.8, "Use GetLODMeshDataArray instead")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API void GetMeshUids(TArray<FString>& OutMeshNames) const;

	UE_DEPRECATED(5.8, "Use AddLODMeshData instead")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool AddMeshUid(const FString& MeshName);

	UE_DEPRECATED(5.8, "Use RemoveAllLODMeshData instead")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveAllMeshes();


	UE_DEPRECATED(5.8, "Use GetBoxCollisionsMeshDataArray instead")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API int32 GetBoxCollisionMeshUidsCount() const;

	UE_DEPRECATED(5.8, "Use GetBoxCollisionsMeshDataArray instead")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API TMap<FString, FString> GetBoxCollisionMeshMap() const;

	UE_DEPRECATED(5.8, "No longer used: Collect the keys from GetBoxCollisionsMeshDataArray() instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API void GetBoxCollisionMeshUids(TArray<FString>& OutMeshNames) const;

	UE_DEPRECATED(5.8, "Deprecated. No longer used.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API void GetBoxColliderRenderMeshUid(const FString& InColliderMeshUid, FString& OutRenderMeshUid) const;

	UE_DEPRECATED(5.8, "Use AddBoxCollisionMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool AddBoxCollisionMeshUid(const FString& ColliderMeshUid);

	UE_DEPRECATED(5.8, "Use AddBoxCollisionMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool AddBoxCollisionMeshUids(const FString& ColliderMeshUid, const FString& RenderMeshUid);

	UE_DEPRECATED(5.8, "Use RemoveBoxCollisionMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveBoxCollisionMeshUid(const FString& ColliderMeshUid);

	UE_DEPRECATED(5.8, "Use RemoveAllBoxCollisionMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveAllBoxCollisionMeshes();


	UE_DEPRECATED(5.8, "Use GetCapsuleCollisionsMeshDataArray instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API int32 GetCapsuleCollisionMeshUidsCount() const;

	UE_DEPRECATED(5.8, "Use GetCapsuleCollisionsMeshDataArray instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API TMap<FString, FString> GetCapsuleCollisionMeshMap() const;

	UE_DEPRECATED(5.8, "No longer used: Collect the keys from GetCapsuleCollisionsMeshDataArray() instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API void GetCapsuleCollisionMeshUids(TArray<FString>& OutMeshNames) const;

	UE_DEPRECATED(5.8, "Deprecated. No longer used.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API void GetCapsuleColliderRenderMeshUid(const FString& InColliderMeshUid, FString& OutRenderMeshUid) const;

	UE_DEPRECATED(5.8, "Use AddCapsuleCollisionMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool AddCapsuleCollisionMeshUid(const FString& ColliderMeshUid);

	UE_DEPRECATED(5.8, "Use AddCapsuleCollisionMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool AddCapsuleCollisionMeshUids(const FString& ColliderMeshUid, const FString& RenderMeshUid);

	UE_DEPRECATED(5.8, "Use RemoveCapsuleCollisionMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveCapsuleCollisionMeshUid(const FString& ColliderMeshUid);

	UE_DEPRECATED(5.8, "Use RemoveAllCapsuleCollisionMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveAllCapsuleCollisionMeshes();


	UE_DEPRECATED(5.8, "Use GetSphereCollisionsMeshDataArray instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API int32 GetSphereCollisionMeshUidsCount() const;

	UE_DEPRECATED(5.8, "Use GetSphereCollisionsMeshDataArray instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API TMap<FString, FString> GetSphereCollisionMeshMap() const;

	UE_DEPRECATED(5.8, "No longer used: Collect the keys from GetSphereCollisionsMeshDataArray() instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API void GetSphereCollisionMeshUids(TArray<FString>& OutMeshNames) const;

	UE_DEPRECATED(5.8, "Deprecated. No longer used.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API void GetSphereColliderRenderMeshUid(const FString& InColliderMeshUid, FString& OutRenderMeshUid) const;

	UE_DEPRECATED(5.8, "Use AddSphereCollisionMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool AddSphereCollisionMeshUid(const FString& ColliderMeshUid);

	UE_DEPRECATED(5.8, "Use AddSphereCollisionMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool AddSphereCollisionMeshUids(const FString& ColliderMeshUid, const FString& RenderMeshUid);

	UE_DEPRECATED(5.8, "Use RemoveSphereCollisionMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveSphereCollisionMeshUid(const FString& ColliderMeshUid);

	UE_DEPRECATED(5.8, "Use RemoveAllSphereCollisionMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveAllSphereCollisionMeshes();


	UE_DEPRECATED(5.8, "Use GetConvexCollisionsMeshDataArray instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API int32 GetConvexCollisionMeshUidsCount() const;

	UE_DEPRECATED(5.8, "Use GetConvexCollisionsMeshDataArray instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API TMap<FString, FString> GetConvexCollisionMeshMap() const;

	UE_DEPRECATED(5.8, "Use GetConvexCollisionsMeshDataArray instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API void GetConvexCollisionMeshUids(TArray<FString>& OutMeshNames) const;

	UE_DEPRECATED(5.8, "Deprecated. No longer used.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API void GetConvexColliderRenderMeshUid(const FString& InColliderMeshUid, FString& OutRenderMeshUid) const;

	UE_DEPRECATED(5.8, "Use AddConvexCollisionMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool AddConvexCollisionMeshUid(const FString& ColliderMeshUid);

	UE_DEPRECATED(5.8, "Use AddConvexCollisionMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool AddConvexCollisionMeshUids(const FString& ColliderMeshUid, const FString& RenderMeshUid);

	UE_DEPRECATED(5.8, "Use RemoveConvexCollisionMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveConvexCollisionMeshUid(const FString& MeshName);

	UE_DEPRECATED(5.8, "Use RemoveConvexCollisionMeshData instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMeshLodData")
	UE_API bool RemoveAllConvexCollisionMeshes();

	/*
	* Deprecated function until here.
	*/

private:

	bool IsEditorOnlyDataDefined();

	UE::Interchange::TArrayAttributeHelper<FInterchangeLODMeshData> LODMeshDataArray;
	UE::Interchange::TArrayAttributeHelper<FInterchangeLODMeshData> BoxCollisionMeshDataArray;
	UE::Interchange::TArrayAttributeHelper<FInterchangeLODMeshData> CapsuleCollisionMeshDataArray;
	UE::Interchange::TArrayAttributeHelper<FInterchangeLODMeshData> SphereCollisionMeshDataArray;
	UE::Interchange::TArrayAttributeHelper<FInterchangeLODMeshData> ConvexCollisionMeshDataArray;

	//Deprecated:
	UE::Interchange::TArrayAttributeHelper<FString> MeshUids;
	UE::Interchange::TMapAttributeHelper<FString, FString> BoxCollisionMeshUids;
	UE::Interchange::TMapAttributeHelper<FString, FString> CapsuleCollisionMeshUids;
	UE::Interchange::TMapAttributeHelper<FString, FString> SphereCollisionMeshUids;
	UE::Interchange::TMapAttributeHelper<FString, FString> ConvexCollisionMeshUids;

	IMPLEMENT_NODE_ATTRIBUTE_KEY(OneConvexHullPerUCX)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ForceCollisionPrimitiveGeneration)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ImportCollision)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ImportCollisionType)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(MeshTransform)
};

#undef UE_API
