// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeTextureFactoryNode.h"
#include "Nodes/InterchangeBaseNode.h"

#if WITH_ENGINE
#include "Engine/Texture.h"
#include "Engine/Texture2DArray.h"
#endif


#include "InterchangeTexture2DArrayFactoryNode.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeTexture2DArrayFactoryNode : public UInterchangeTextureFactoryNode
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomAddressX(uint8& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AddressX, uint8);
	}

	bool SetCustomAddressX(const uint8 AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTexture2DArrayFactoryNode, AddressX, uint8, UTexture2DArray)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AddressX, uint8)
#endif
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomAddressY(uint8& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AddressY, uint8);
	}

	bool SetCustomAddressY(const uint8 AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTexture2DArrayFactoryNode, AddressY, uint8, UTexture2DArray)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AddressY, uint8)
#endif
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomAddressZ(uint8& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AddressZ, uint8);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomAddressZ(const uint8 AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTexture2DArrayFactoryNode, AddressZ, uint8, UTexture2DArray)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AddressZ, uint8)
#endif
	}

	virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override
	{
		Super::CopyWithObject(SourceNode, Object);

#if WITH_EDITORONLY_DATA
		if (const UInterchangeTexture2DArrayFactoryNode* TextureFactoryNode = Cast<UInterchangeTexture2DArrayFactoryNode>(SourceNode))
		{
			COPY_NODE_DELEGATES(TextureFactoryNode, AddressX, uint8, UTexture2DArray)
			COPY_NODE_DELEGATES(TextureFactoryNode, AddressY, uint8, UTexture2DArray)
			COPY_NODE_DELEGATES(TextureFactoryNode, AddressZ, uint8, UTexture2DArray)
		}
#endif
	}

private:
	virtual UClass* GetObjectClass() const override
	{
		return UTexture2DArray::StaticClass();
	}

	// Addressing
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AddressX);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AddressY);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AddressZ);
};
