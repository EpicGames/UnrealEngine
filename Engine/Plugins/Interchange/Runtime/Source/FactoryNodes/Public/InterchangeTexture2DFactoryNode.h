// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeTextureFactoryNode.h"

#include "CoreMinimal.h"
#include "Engine/Texture2D.h"
#include "UDIMUtilities.h"

#include "InterchangeTexture2DFactoryNode.generated.h"

namespace UE::Interchange
{ 
	struct FTexture2DFactoryNodeStaticData : public FBaseNodeStaticData
	{
		static const FAttributeKey& GetBaseSourceBlocksKey()
		{
			static FAttributeKey AttributeKey(TEXT("SourceBlocks"));
			return AttributeKey;
		}
	};
}//ns UE::Interchange

UCLASS(BlueprintType, MinimalAPI)
class UInterchangeTexture2DFactoryNode : public UInterchangeTextureFactoryNode
{
	GENERATED_BODY()

public:


	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomAddressX(TEnumAsByte<TextureAddress>& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AddressX, TEnumAsByte<TextureAddress>);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomAddressX(const TEnumAsByte<TextureAddress> AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTexture2DFactoryNode, AddressX, TEnumAsByte<TextureAddress>, UTexture2D)
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomAddressY(TEnumAsByte<TextureAddress>& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AddressY, TEnumAsByte<TextureAddress>);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomAddressY(const TEnumAsByte<TextureAddress> AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTexture2DFactoryNode, AddressY, TEnumAsByte<TextureAddress>, UTexture2D)
	}

	/**
	 * Override serialize to restore SourceBlocks on load.
	 */
	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);

		if (Ar.IsLoading() && bIsInitialized)
		{
			SourceBlocks.RebuildCache();
		}
	}

	//////////////////////////////////////////////////////////////////////////
	// UDIMs begin here
	// UDIM base textures use a different model for the source data.

	/**
	 * Get the source blocks for the texture.
	 * If the map is empty, the texture is imported as a normal texture using the payload key.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture | UDIMs")
	TMap<int32, FString> GetSourceBlocks() const
	{
		return SourceBlocks.ToMap();
	}

	/**
	 * Set the source blocks for the texture.
	 * Using this suggests to the pipeline to consider this texture as a UDIM. The pipeline can choose whether to pass these blocks to the texture factory node.
	 * @param InSourceBlocks The blocks and their source image that compose the whole texture.
	 * The textures must be of the same format and use the same pixel format.
	 * The first block in the map is used to determine the accepted texture format and pixel format.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture | UDIMs")
	void SetSourceBlocks(const TMap<int32, FString>& InSourceBlocks)
	{
		SourceBlocks = InSourceBlocks;
	}

	
	/**
	 * Set the source blocks for the texture.
	 * Using this suggests to the pipeline to consider this texture as a UDIM. The pipeline can choose whether to pass these blocks to the texture factory node.
	 * @param InSourceBlocks The blocks and their source image that compose the whole texture.
	 * The textures must be of the same format and use the same pixel format.
	 * The first block in the map is used to determine the accepted texture format and pixel format.
	 */
	void SetSourceBlocks(TMap<int32, FString>&& InSourceBlocks)
	{
		SourceBlocks = InSourceBlocks;
	}


	/**
	 * Get a source block from the texture.
	 *
	 * @param X The X coordinate of the block.
	 * @param Y The Y coordinate of the block.
	 * @param OutSourceFile The source file for that block if found.
	 * @return True if the source file for the block was found.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture | UDIMs")
	bool GetSourceBlockByCoordinates(int32 X, int32 Y, FString& OutSourceFile) const
	{
		return GetSourceBlock(UE::TextureUtilitiesCommon::GetUDIMIndex(X, Y), OutSourceFile);
	}

	/**
	 * Get a source block from the texture.
	 *
	 * @param BlockIndex The UDIM Index of the block.
	 * @param OutSourceFile The source file for that block if found.
	 * @return True if the source file for the block was found.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture | UDIMs")
	bool GetSourceBlock(int32 BlockIndex, FString& OutSourceFile) const
	{
		return SourceBlocks.GetValue(BlockIndex, OutSourceFile);
	}


	/**
	 * Set a source block for the texture.
	 *
	 * @param X The X coordinate of the block.
	 * @param Y The Y coordinate of the block.
	 * @param InSourceFile The source file for that block.
	 * 
	 * The textures must be of the same format and use the same pixel format.
	 * The first block in the map is used to determine the accepted texture format and pixel format.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture | UDIMs")
	void SetSourceBlockByCoordinates(int32 X, int32 Y,  const FString& InSourceFile)
	{
		SetSourceBlock(UE::TextureUtilitiesCommon::GetUDIMIndex(X, Y), InSourceFile);
	}

	/**
	 * Set a source block for the texture.
	 *
	 * @param BlockIndex The UDIM index of the block.
	 * @param InSourceFile The source file for that block.
	 * 
	 * The textures must be of the same format and use the same pixel format.
	 * The first block in the map is used to determine the accepted texture format and pixel format.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture | UDIMs")
	void SetSourceBlock(int32 BlockIndex, const FString& InSourceFile)
	{
		SourceBlocks.SetKeyValue(BlockIndex, InSourceFile);
	}

	// UDIMs ends here
	//////////////////////////////////////////////////////////////////////////

	virtual UClass* GetObjectClass() const override
	{
		return UTexture2D::StaticClass();
	}

	virtual void PostInitProperties()
	{
		Super::PostInitProperties();
		SourceBlocks.Initialize(Attributes.ToSharedRef(), UE::Interchange::FTexture2DFactoryNodeStaticData::GetBaseSourceBlocksKey().ToString());
	}

	virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override
	{
		Super::CopyWithObject(SourceNode, Object);

		if (const UInterchangeTexture2DFactoryNode* TextureFactoryNode = Cast<UInterchangeTexture2DFactoryNode>(SourceNode))
		{
			COPY_NODE_DELEGATES(TextureFactoryNode, AddressX, TEnumAsByte<TextureAddress>, UTexture2D)
			COPY_NODE_DELEGATES(TextureFactoryNode, AddressY, TEnumAsByte<TextureAddress>, UTexture2D)
		}
	}

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AddressX);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AddressY);

	UE::Interchange::TMapAttributeHelper<int32, FString> SourceBlocks;
};