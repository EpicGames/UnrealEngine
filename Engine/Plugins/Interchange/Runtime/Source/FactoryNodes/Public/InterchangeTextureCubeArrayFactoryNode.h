// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeTextureFactoryNode.h"

#if WITH_ENGINE
#include "Engine/TextureCubeArray.h"
#endif


#include "InterchangeTextureCubeArrayFactoryNode.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeTextureCubeArrayFactoryNode : public UInterchangeTextureFactoryNode
{
	GENERATED_BODY()

private:
	virtual UClass* GetObjectClass() const override
	{
		return UTextureCubeArray::StaticClass();
	}
};
