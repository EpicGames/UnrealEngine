// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeTextureFactoryNode.h"

#if WITH_ENGINE
#include "Engine/TextureCube.h"
#endif


#include "InterchangeTextureCubeFactoryNode.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeTextureCubeFactoryNode : public UInterchangeTextureFactoryNode
{
	GENERATED_BODY()

private:
	virtual UClass* GetObjectClass() const override
	{
		return UTextureCube::StaticClass();
	}
};
