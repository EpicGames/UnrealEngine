// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"

#include "Engine/Texture.h"

#include "InterchangeTextureFactoryNode.generated.h"


UCLASS(BlueprintType, Abstract, MinimalAPI)
class UInterchangeTextureFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	/**
	 * Initialize node data.
	 * @param: UniqueID - The unique ID for this node.
	 * @param DisplayLabel - The name of the node.
	 * @param InAssetClass - The class the texture factory will create for this node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	void InitializeTextureNode(const FString& UniqueID, const FString& DisplayLabel, const FString& InAssetName, UInterchangeBaseNodeContainer* NodeContainer)
	{
		//Initialize
		NodeContainer->SetupNode(this, UniqueID, DisplayLabel, EInterchangeNodeContainerType::FactoryData);

		//Set the asset name
		FString OperationName = GetTypeName() + TEXT(".SetAssetName");
		InterchangePrivateNodeBase::SetCustomAttribute<FString>(*Attributes, AssetNameKey, OperationName, InAssetName);
	}

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("TextureFactoryNode");
		return TypeName;
	}

	/** Get the class this node creates. */
	virtual UClass* GetObjectClass() const override
	{
		// This function should always be overridden by the sub texture factory node
		ensure(false);
		return nullptr;
	}

	virtual FGuid GetHash() const override
	{
		return Attributes->GetStorageHash();
	}

	static FString GetTextureFactoryNodeUidFromTextureNodeUid(const FString& TranslatedNodeUid)
	{
		FString NewUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TranslatedNodeUid);
		return NewUid;
	}

	/**
	 * Get the unique ID of the translated texture node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomTranslatedTextureNodeUid(FString& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(TranslatedTextureNodeUid, FString);
	}

	/**
	 * Set the unique ID of the translated texture node. This is a reference to the node that was created by the translator. It is needed to get the texture payload.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomTranslatedTextureNodeUid(const FString& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(TranslatedTextureNodeUid, FString);
	}

public:
	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomAdjustBrightness(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AdjustBrightness, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomAdjustBrightness(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, AdjustBrightness, float, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AdjustBrightness, float)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomAdjustBrightnessCurve(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AdjustBrightnessCurve, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomAdjustBrightnessCurve(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, AdjustBrightnessCurve, float, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AdjustBrightnessCurve, float)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomAdjustVibrance(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AdjustVibrance, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomAdjustVibrance(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, AdjustVibrance, float, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AdjustVibrance, float)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomAdjustSaturation(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AdjustSaturation, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomAdjustSaturation(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, AdjustSaturation, float, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AdjustSaturation, float)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomAdjustRGBCurve(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AdjustRGBCurve, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomAdjustRGBCurve(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, AdjustRGBCurve, float, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AdjustRGBCurve, float)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomAdjustHue(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AdjustHue, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomAdjustHue(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, AdjustHue, float, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AdjustHue, float)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomAdjustMinAlpha(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AdjustMinAlpha, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomAdjustMinAlpha(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, AdjustMinAlpha, float, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AdjustMinAlpha, float)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomAdjustMaxAlpha(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AdjustMaxAlpha, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomAdjustMaxAlpha(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, AdjustMaxAlpha, float, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AdjustMaxAlpha, float)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustombChromaKeyTexture(bool& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(bChromaKeyTexture, bool);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustombChromaKeyTexture(const bool& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, bChromaKeyTexture, bool, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(bChromaKeyTexture, bool)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomChromaKeyThreshold(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(ChromaKeyThreshold, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomChromaKeyThreshold(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, ChromaKeyThreshold, float, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ChromaKeyThreshold, float)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomChromaKeyColor(FColor& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(ChromaKeyColor, FColor);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomChromaKeyColor(const FColor& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, ChromaKeyColor, FColor, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ChromaKeyColor, FColor)
#endif
	}

	//////////////////////////////////////////////////////////////////////////
	//Texture Compression Begin

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomCompressionNoAlpha(bool& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(CompressionNoAlpha, bool);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomCompressionNoAlpha(const bool& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, CompressionNoAlpha, bool, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(CompressionNoAlpha, bool)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomDeferCompression(bool& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(DeferCompression, bool);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomDeferCompression(const bool& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, DeferCompression, bool, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DeferCompression, bool)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomLossyCompressionAmount(uint8& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(LossyCompressionAmount, uint8);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomLossyCompressionAmount(const uint8& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, LossyCompressionAmount, uint8, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(LossyCompressionAmount, uint8)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomMaxTextureSize(int32& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(MaxTextureSize, int32);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomMaxTextureSize(const int32& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, MaxTextureSize, int32, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(MaxTextureSize, int32)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomCompressionQuality(uint8& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(CompressionQuality, uint8);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomCompressionQuality(const uint8& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, CompressionQuality, uint8, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(CompressionQuality, uint8)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomCompressionSettings(uint8& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(CompressionSettings, uint8);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomCompressionSettings(const uint8& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, CompressionSettings, uint8, UTexture)
	}

	//////////////////////////////////////////////////////////////////////////
	//Texture general

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomAlphaCoverageThresholds(FVector4& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AlphaCoverageThresholds, FVector4);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomAlphaCoverageThresholds(const FVector4& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, AlphaCoverageThresholds, FVector4, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AlphaCoverageThresholds, FVector4)
#endif
	}

	
	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustombDoScaleMipsForAlphaCoverage(bool& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(bDoScaleMipsForAlphaCoverage, bool);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustombDoScaleMipsForAlphaCoverage(const bool& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, bDoScaleMipsForAlphaCoverage, bool, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(bDoScaleMipsForAlphaCoverage, bool)
#endif
	}


	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustombFlipGreenChannel(bool& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(bFlipGreenChannel, bool);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustombFlipGreenChannel(const bool& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, bFlipGreenChannel, bool, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(bFlipGreenChannel, bool)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomPowerOfTwoMode(uint8& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(PowerOfTwoMode, uint8);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomPowerOfTwoMode(const uint8& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, PowerOfTwoMode, uint8, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(PowerOfTwoMode, uint8)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomPaddingColor(FColor& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(PaddingColor, FColor);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomPaddingColor(const FColor& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, PaddingColor, FColor, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(PaddingColor, FColor)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomFilter(uint8& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(Filter, uint8);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomFilter(const uint8& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, Filter, uint8, UTexture)
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomMipLoadOptions(uint8& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(MipLoadOptions, uint8);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomMipLoadOptions(const uint8& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, MipLoadOptions, uint8, UTexture)
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomSRGB(bool& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(SRGB, bool);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomSRGB(const bool& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, SRGB, bool, UTexture)
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustombUseLegacyGamma(bool& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(bUseLegacyGamma, bool);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustombUseLegacyGamma(const bool& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, bUseLegacyGamma, bool, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(bUseLegacyGamma, bool)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomVirtualTextureStreaming(bool& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(VirtualTextureStreaming, bool);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomVirtualTextureStreaming(const bool& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, VirtualTextureStreaming, bool, UTexture)
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	INTERCHANGEFACTORYNODES_API bool GetCustomColorSpace(ETextureColorSpace& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	INTERCHANGEFACTORYNODES_API bool SetCustomColorSpace(ETextureColorSpace AttributeValue, bool bAddApplyDelegate = true);

	//////////////////////////////////////////////////////////////////////////
	//Level of Detail

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustombPreserveBorder(bool& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(bPreserveBorder, bool);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustombPreserveBorder(const bool& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, bPreserveBorder, bool, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(bPreserveBorder, bool)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomMipGenSettings(uint8& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(MipGenSettings, uint8);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomMipGenSettings(const uint8& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, MipGenSettings, uint8, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(MipGenSettings, uint8)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomLODBias(int32& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(LODBias, int32);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomLODBias(const int32& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, LODBias, int32, UTexture)
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomLODGroup(uint8& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(LODGroup, uint8);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomLODGroup(const uint8& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, LODGroup, uint8, UTexture)
	}

	//TODO support per platform data in the FAttributeStorage, so we can set different value per platform at the pipeline stage, We set only the default value for now
	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomDownscale(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(Downscale, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomDownscale(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(UInterchangeTextureFactoryNode, Downscale, float, UTexture)
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomDownscaleOptions(uint8& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(DownscaleOptions, uint8);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomDownscaleOptions(const uint8& AttributeValue, bool bAddApplyDelegate = true)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, DownscaleOptions, uint8, UTexture)
	}

	//////////////////////////////////////////////////////////////////////////
	//Compositing

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomCompositeTextureMode(uint8& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(CompositeTextureMode, uint8);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomCompositeTextureMode(const uint8& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, CompositeTextureMode, uint8, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(CompositeTextureMode, uint8)
#endif
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomCompositePower(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(CompositePower, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomCompositePower(const float& AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureFactoryNode, CompositePower, float, UTexture)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(CompositePower, float)
#endif
	}

	//////////////////////////////////////////////////////////////////////////
	//Factory Import settings

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture | Import Settings")
	bool GetCustomAllowNonPowerOfTwo(bool& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AllowNonPowerOfTwo, bool);
	}

	/*
	 * Should the factory allow the import of texture that would have a resolution that is not a power of two
	 * By default this is not allowed
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture | Import Settings")
	bool SetCustomAllowNonPowerOfTwo(const bool& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AllowNonPowerOfTwo, bool)
	}

	/** Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture | Import Settings")
	bool GetCustomPreferCompressedSourceData(bool& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(PreferCompressedSourceData, bool);
	}

	/*
	 * Determines whether the factory should tell the translator to provide a compressed source data payload when available.
	 * This will generally result in smaller assets. However, some operations like the texture build might be slower, because the source data first needs to be decompressed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture | Import Settings")
	bool SetCustomPreferCompressedSourceData(const bool& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(PreferCompressedSourceData, bool)
	}

	virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override
	{
		Super::CopyWithObject(SourceNode, Object);

		if (const UInterchangeTextureFactoryNode* TextureFactoryNode = Cast<UInterchangeTextureFactoryNode>(SourceNode))
		{
#if WITH_EDITORONLY_DATA
			COPY_NODE_DELEGATES(TextureFactoryNode, AdjustBrightness, float, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, AdjustBrightnessCurve, float, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, AdjustVibrance, float, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, AdjustSaturation, float, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, AdjustRGBCurve, float, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, AdjustHue, float, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, AdjustMinAlpha, float, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, AdjustMaxAlpha, float, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, bChromaKeyTexture, bool, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, ChromaKeyThreshold, float, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, ChromaKeyColor, FColor, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, CompressionNoAlpha, bool, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, DeferCompression, bool, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, LossyCompressionAmount, uint8, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, MaxTextureSize, int32, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, CompressionQuality, uint8, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, AlphaCoverageThresholds, FVector4, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, bDoScaleMipsForAlphaCoverage, bool, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, bFlipGreenChannel, bool, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, PowerOfTwoMode, uint8, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, PaddingColor, FColor, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, bUseLegacyGamma, bool, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, bPreserveBorder, bool, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, MipGenSettings, uint8, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, CompositeTextureMode, uint8, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, CompositePower, float, UTexture)
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(TextureFactoryNode, UInterchangeTextureFactoryNode, ColorSpace, ETextureColorSpace, UTexture::StaticClass())
#endif
			COPY_NODE_DELEGATES(TextureFactoryNode, VirtualTextureStreaming, bool, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, CompressionSettings, uint8, UTexture)
			COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(TextureFactoryNode, UInterchangeTextureFactoryNode, Downscale, float, UTexture::StaticClass())
			COPY_NODE_DELEGATES(TextureFactoryNode, DownscaleOptions, uint8, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, Filter, uint8, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, LODBias, int32, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, LODGroup, uint8, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, MipLoadOptions, uint8, UTexture)
			COPY_NODE_DELEGATES(TextureFactoryNode, SRGB, bool, UTexture)
		}
	}

private:

	const UE::Interchange::FAttributeKey ClassNameAttributeKey = UE::Interchange::FBaseNodeStaticData::ClassTypeAttributeKey();
	const UE::Interchange::FAttributeKey AssetNameKey = UE::Interchange::FBaseNodeStaticData::AssetNameKey();

	// Translated texture node unique id
	IMPLEMENT_NODE_ATTRIBUTE_KEY(TranslatedTextureNodeUid)
	
	// Texture Adjustments
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AdjustBrightness)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AdjustBrightnessCurve)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AdjustVibrance)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AdjustSaturation)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AdjustRGBCurve)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AdjustHue)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AdjustMinAlpha)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AdjustMaxAlpha)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(bChromaKeyTexture)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ChromaKeyThreshold)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ChromaKeyColor)

	// Texture Compression
	IMPLEMENT_NODE_ATTRIBUTE_KEY(CompressionNoAlpha)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(DeferCompression)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(LossyCompressionAmount)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(MaxTextureSize)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(CompressionQuality)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(CompressionSettings)

	// Texture general
	IMPLEMENT_NODE_ATTRIBUTE_KEY(bDoScaleMipsForAlphaCoverage)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AlphaCoverageThresholds)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(bFlipGreenChannel)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(bForcePVRTC4)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(PowerOfTwoMode)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(PaddingColor)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Filter)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(MipLoadOptions)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SRGB)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(bUseLegacyGamma)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(VirtualTextureStreaming)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ColorSpace)

	// Level of Detail
	IMPLEMENT_NODE_ATTRIBUTE_KEY(bPreserveBorder)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(MipGenSettings)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(LODBias)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(LODGroup)
	//TODO support per platform data in the FAttributeStorage, so we can set different value per platform at the pipeline stage, We set only the default value for now
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Downscale)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(DownscaleOptions)

	// Compositing
	IMPLEMENT_NODE_ATTRIBUTE_KEY(CompositeTextureMode)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(CompositePower)

	// Texture Factory Settings
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AllowNonPowerOfTwo)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(PreferCompressedSourceData)


	// no IMPLEMENT_NODE_ATTRIBUTE_APPLY_UOBJECT here ?

	//TODO support per platform data in the FAttributeStorage, so we can set different value per platform at the pipeline stage, We set only the default value for now
	//IMPLEMENT_NODE_ATTRIBUTE_APPLY_UOBJECT(Downscale, float, UTexture, );


	bool ApplyCustomDownscaleToAsset(UObject* Asset) const
	{
		if (!Asset)
		{
			return false;
		}
		UTexture* TypedObject = Cast<UTexture>(Asset);
		if (!TypedObject)
		{
			return false;
		}
		float ValueData;
		if (GetCustomDownscale(ValueData))
		{
			TypedObject->Downscale.Default = ValueData;
			return true;
		}
		return false;
	}

	bool FillCustomDownscaleFromAsset(UObject* Asset)
	{
		if (!Asset)
		{
			return false;
		}
		UTexture* TypedObject = Cast<UTexture>(Asset);
		if (!TypedObject)
		{
			return false;
		}
		if (SetCustomDownscale(TypedObject->Downscale.Default, false))
		{
			return true;
		}
		return false;
	}

#if WITH_EDITORONLY_DATA
	bool ApplyCustomColorSpaceToAsset(UObject* Asset) const;

	bool FillCustomColorSpaceFromAsset(UObject* Asset);
#endif // WITH_EDITORONLY_DATA
};
