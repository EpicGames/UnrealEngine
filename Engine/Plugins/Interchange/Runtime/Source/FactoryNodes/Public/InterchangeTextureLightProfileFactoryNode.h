// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeTexture2DFactoryNode.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Misc/Base64.h"

#if WITH_ENGINE
#include "Engine/Texture.h"
#include "Engine/TextureLightProfile.h"
#endif


#include "InterchangeTextureLightProfileFactoryNode.generated.h"

UCLASS(BlueprintType, MinimalAPI)
class UInterchangeTextureLightProfileFactoryNode : public UInterchangeTexture2DFactoryNode
{
	GENERATED_BODY()

public:
	UInterchangeTextureLightProfileFactoryNode()
	{
		AttributeHelperIESSourceFileContentChunks.Initialize(Attributes, TEXT("IESSourceFileContentChunks"));
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomBrightness(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(Brightness, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomBrightness(const float AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureLightProfileFactoryNode, Brightness, float, UTextureLightProfile)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Brightness, float)
#endif
	}


	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomTextureMultiplier(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(TextureMultiplier, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomTextureMultiplier(const float AttributeValue, bool bAddApplyDelegate = true)
	{
#if WITH_EDITORONLY_DATA
		IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeTextureLightProfileFactoryNode, TextureMultiplier, float, UTextureLightProfile)
#else
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(TextureMultiplier, float)
#endif
	}

	void StoreIESSourceFileContents(const TArray64<uint8>& IESSourceFileContents) 
	{
		int64 NumberOfElements = IESSourceFileContents.Num();
		constexpr int64 SubPartSize = 65536;// 2 ^ 16;
		int64 ElementIndex = 0;
		while (ElementIndex < NumberOfElements)
		{
			int64 RawSubPartSize = ((ElementIndex + SubPartSize) < NumberOfElements) ? SubPartSize : (NumberOfElements - ElementIndex);
			TArray<uint8> RawSubPart;
			RawSubPart.SetNum(RawSubPartSize);

			FPlatformMemory::Memcpy(RawSubPart.GetData(), IESSourceFileContents.GetData() + ElementIndex * sizeof(uint8), RawSubPartSize * sizeof(uint8));

			AttributeHelperIESSourceFileContentChunks.AddItem(FBase64::Encode(RawSubPart));

			ElementIndex += SubPartSize;
		}
	}

	void GetIESSourceFileContents(TArray64<uint8>& IESSourceFileContent) const
	{
		TArray<FString> IESSourceFileContentChunks;
		AttributeHelperIESSourceFileContentChunks.GetItems(IESSourceFileContentChunks);

		int64 NumberOfElements = 0;
		for (const FString& Chunk : IESSourceFileContentChunks)
		{
			NumberOfElements += Chunk.Len();
		}

		IESSourceFileContent.Reserve(NumberOfElements);

		int64 ElementIndex = 0;
		for (const FString& Chunk : IESSourceFileContentChunks)
		{
			TArray<uint8> ChunkDecoded;
			FBase64::Decode(Chunk, ChunkDecoded);

			IESSourceFileContent.SetNum(ElementIndex + ChunkDecoded.Num());

			FPlatformMemory::Memcpy(IESSourceFileContent.GetData() + ElementIndex * sizeof(uint8), ChunkDecoded.GetData(), ChunkDecoded.Num() * sizeof(uint8));
		}
	}

	virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override
	{
		Super::CopyWithObject(SourceNode, Object);

#if WITH_EDITORONLY_DATA
		if (const UInterchangeTextureLightProfileFactoryNode* TextureFactoryNode = Cast<UInterchangeTextureLightProfileFactoryNode>(SourceNode))
		{
			COPY_NODE_DELEGATES(TextureFactoryNode, Brightness, float, UTextureLightProfile)
			COPY_NODE_DELEGATES(TextureFactoryNode, TextureMultiplier, float, UTextureLightProfile)
		}
#endif
	}

	

private:
	virtual UClass* GetObjectClass() const override
	{
		return UTextureLightProfile::StaticClass();
	}

	// Addressing
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Brightness);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(TextureMultiplier);

	//Note: Base64 encoded chunks of 2^16 sized strings.
	UE::Interchange::TArrayAttributeHelper<FString> AttributeHelperIESSourceFileContentChunks; 
};