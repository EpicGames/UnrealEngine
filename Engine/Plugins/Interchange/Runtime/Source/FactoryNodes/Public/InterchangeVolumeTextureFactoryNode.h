// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeTextureFactoryNode.h"

#if WITH_ENGINE
#include "Engine/VolumeTexture.h"
#endif


#include "InterchangeVolumeTextureFactoryNode.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeVolumeTextureFactoryNode : public UInterchangeTextureFactoryNode
{
	GENERATED_BODY()

private:
	virtual UClass* GetObjectClass() const override
	{
		return UVolumeTexture::StaticClass();
	}
};
