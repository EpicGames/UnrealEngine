// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class InterchangeImport : ModuleRules
	{
		public InterchangeImport(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"ClothingSystemRuntimeCommon",
					"Core",
					"CoreUObject",
					"Engine",
					"HairStrandsCore",
					"InterchangeCore",
					"InterchangeCommon",
					"InterchangeDispatcher",
					"InterchangeEngine",
					"InterchangeFactoryNodes",
					"InterchangeNodes",
					"LevelSequence",
					"MeshDescription",
					"MovieScene",
					"MovieSceneTracks",
					"StaticMeshDescription",
					"SkeletalMeshDescription",
				}
			);
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
					"AudioMixer",
					"CinematicCamera",
					"ClothingSystemRuntimeCommon",
					"GeometryCache",
					"GLTFCore",
					"IESFile",
					"ImageCore",
					"ImageWrapper",
					"InterchangeCommonParser",
					"InterchangeMessages",
					"Json",
					"RenderCore",
					"RHI",
					"TextureUtilitiesCommon",
					"VariantManagerContent",
				}
			);

			if (Target.bBuildEditor)
			{
				bEnableExceptions = true;
				bDisableAutoRTFMInstrumentation = true;

				// Public because MaterialX headers are #include'd in
				// public InterchangeImport headers.
				PublicDependencyModuleNames.AddRange(
					new string[]
					{
						"MaterialX"
					}
				);

				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"BSPUtils",
						"HairStrandsEditor",
						"InterchangeFbxParser",
						"MaterialEditor",
						"SkeletalMeshUtilitiesCommon",
						"UnrealEd",
						"VariantManager",
					}
				);
			}
		}
	}
}
