// Copyright Epic Games, Inc. All Rights Reserved. 
#include "Animation/InterchangeAnimSequenceFactory.h"

#include "Animation/AnimSequence.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "Animation/InterchangeAnimationPayloadInterface.h"
#include "InterchangeAnimSequenceFactoryNode.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeCommonAnimationPayload.h"
#include "InterchangeCommonPipelineDataFactoryNode.h"
#include "InterchangeHelper.h"
#include "InterchangeImportCommon.h"
#include "InterchangeImportLog.h"
#include "InterchangeMeshNode.h"
#include "InterchangeJointNode.h"
#include "InterchangeSkeletonFactoryNode.h"
#include "InterchangeSourceData.h"
#include "InterchangeTranslatorBase.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeSourceNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"
#include "InterchangeAnimationTrackSetNode.h"
#include "Async/Async.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeAnimSequenceFactory)

#if WITH_EDITORONLY_DATA

#include "EditorFramework/AssetImportData.h"

#endif //WITH_EDITORONLY_DATA

#if WITH_EDITOR
namespace UE::Interchange::Private
{
	void GetSkeletonSceneNodeFlatListRecursive(const UInterchangeBaseNodeContainer* NodeContainer, const FString& NodeUid, TArray<FString>& SkeletonSceneNodeUids)
	{
		SkeletonSceneNodeUids.Add(NodeUid);
		TArray<FString> Children = NodeContainer->GetNodeChildrenUids(NodeUid);
		for (const FString& ChildUid : Children)
		{
			GetSkeletonSceneNodeFlatListRecursive(NodeContainer, ChildUid, SkeletonSceneNodeUids);
		}
	}
	
	template<typename ValueType>
	bool AreAllValuesZero(const TArray<ValueType>& Values, TFunctionRef<bool(const ValueType& Value)> CompareValueToZeroFunction)
	{
		const int32 KeyCount = Values.Num();
		for (int32 KeyIndex = 0; KeyIndex < KeyCount; ++KeyIndex)
		{
			//Only add the not equal keys
			if (!CompareValueToZeroFunction(Values[KeyIndex]))
			{
				return false;
			}
		}
		return true;
	}

	template <class T>
	bool ConvertToRichCurve(const TOptional<TArray<T>>& OptionalValues, const TArray<float>& Keys, FRichCurve& OutRichCurve)
	{
		if (!OptionalValues.IsSet())
		{
			return false;
		}

		const TArray<T>& Values = OptionalValues.GetValue();
		if (Values.Num() != Keys.Num())
		{
			return false;
		}

		OutRichCurve.Keys.Reserve(Keys.Num());

		for (size_t CurveEntryIndex = 0; CurveEntryIndex < Keys.Num(); CurveEntryIndex++)
		{
			FKeyHandle RichCurveKeyHandle = OutRichCurve.AddKey(Keys[CurveEntryIndex], Values[CurveEntryIndex]);
			FRichCurveKey& RichCurveKey = OutRichCurve.GetKey(RichCurveKeyHandle);
			RichCurveKey.InterpMode = ERichCurveInterpMode::RCIM_Constant;
		}

		OutRichCurve.AutoSetTangents();

		return true;
	}

	bool InternalCreateCurve(UAnimSequence* TargetSequence
		, TArray<FRichCurve>& Curves
		, const TArray<FString>& CurveNames
		, const int32 CurveFlags
		, const bool bDoNotImportCurveWithZero
		, const bool bAddCurveMetadataToSkeleton
		, const bool bMorphTargetCurve
		, const bool bMaterialCurve
		, const bool bShouldTransact)
	{
		bool bResult = false;
		if (!TargetSequence || CurveNames.Num() <= 0 || CurveNames.Num() != Curves.Num())
		{
			return bResult;
		}


		for (int32 CurveIndex = 0; CurveIndex < Curves.Num(); ++CurveIndex)
		{
			FRichCurve& Curve = Curves[CurveIndex];
			if (Curve.IsEmpty())
			{
				continue;
			}

			FName Name(UE::Interchange::MakeSanitizedName(CurveNames[CurveIndex]));

			if (bDoNotImportCurveWithZero)
			{
				bool bAllCurveValueAreZero = true;
				FKeyHandle KeyHandle = Curve.GetFirstKeyHandle();
				while (KeyHandle != FKeyHandle::Invalid())
				{
					if (!FMath::IsNearlyZero(Curve.GetKeyValue(KeyHandle)))
					{
						bAllCurveValueAreZero = false;
						break;
					}
					KeyHandle = Curve.GetNextKey(KeyHandle);
				}
				if (bAllCurveValueAreZero)
				{
					continue;
				}
			}
			
			FAnimationCurveIdentifier FloatCurveId(Name, ERawCurveTrackTypes::RCT_Float);

			IAnimationDataModel* DataModel = TargetSequence->GetDataModel();
			IAnimationDataController& Controller = TargetSequence->GetController();

			const FFloatCurve* TargetCurve = DataModel->FindFloatCurve(FloatCurveId);
			if (TargetCurve == nullptr)
			{
				// Need to add the curve first
				Controller.AddCurve(FloatCurveId, AACF_DefaultCurve | CurveFlags, bShouldTransact);
				TargetCurve = DataModel->FindFloatCurve(FloatCurveId);
			}
			else
			{
				// Need to update any of the flags
				Controller.SetCurveFlags(FloatCurveId, CurveFlags | TargetCurve->GetCurveTypeFlags(), bShouldTransact);
			}

			// Should be valid at this point
			ensure(TargetCurve);

			//MorphTarget curves are shift to 0 second
			const float CurveStartTime = Curve.GetFirstKey().Time;
			if (CurveStartTime > UE_SMALL_NUMBER)
			{
				Curve.ShiftCurve(-CurveStartTime);
			}
			// Set actual keys on curve within the model
			Controller.SetCurveKeys(FloatCurveId, Curve.GetConstRefOfKeys(), bShouldTransact);

			if (bMaterialCurve || bMorphTargetCurve)
			{
				if (bAddCurveMetadataToSkeleton)
				{
					USkeleton* Skeleton = TargetSequence->GetSkeleton();
					Skeleton->AccumulateCurveMetaData(Name, bMaterialCurve, bMorphTargetCurve);
				}
			}
			bResult = true;
		}
		return bResult;
	}

	bool CreateAttributeStepCurve(UAnimSequence* TargetSequence
		, TArray<FInterchangeStepCurve>& StepCurves
		, const FString& CurveName
		, const FString& BoneName
		, const int32 CurveFlags
		, const bool bDoNotImportCurveWithZero
		, const bool bAddCurveMetadataToSkeleton
		, const bool bIsMorphTargetCurve
		, const bool bIsMaterialCurve
		, const bool bShouldTransact)
	{
		//For bone attribute we support only single curve type (structured type like vector are not allowed)
		if (!TargetSequence || StepCurves.Num() != 1 || CurveName.IsEmpty())
		{
			return false;
		}

		if (bDoNotImportCurveWithZero)
		{
			bool bAllCurveValuesZero = true;
			for (const FInterchangeStepCurve& StepCurve : StepCurves)
			{
				if (StepCurve.FloatKeyValues.IsSet())
				{
					if (!AreAllValuesZero<float>(StepCurve.FloatKeyValues.GetValue(), [](const float& Value)
						{
							return FMath::IsNearlyZero(Value);
						}))
					{
						bAllCurveValuesZero = false;
						break;
					}
				}
				else if (StepCurve.IntegerKeyValues.IsSet())
				{
					if (!AreAllValuesZero<int32>(StepCurve.IntegerKeyValues.GetValue(), [](const int32& Value)
						{
							return (Value == 0);
						}))
					{
						bAllCurveValuesZero = false;
						break;
					}
				}
				else if (StepCurve.StringKeyValues.IsSet())
				{
					if (!AreAllValuesZero<FString>(StepCurve.StringKeyValues.GetValue(), [](const FString& Value)
						{
							return Value.IsEmpty();
						}))
					{
						bAllCurveValuesZero = false;
						break;
					}
				}
			}
			if (bAllCurveValuesZero)
			{
				return false;
			}
		}

		TArray<FRichCurve> RichCurves;
		TArray<FString> RichCurveNames;
		RichCurves.Reserve(StepCurves.Num());

		bool bResult = false;
		for (const FInterchangeStepCurve& StepCurve : StepCurves)
		{
			if (StepCurve.FloatKeyValues.IsSet())
			{
				bResult |= UE::Anim::AddTypedCustomAttribute<FFloatAnimationAttribute, float>(FName(CurveName)
					, FName(BoneName)
					, TargetSequence
					, MakeArrayView(StepCurve.KeyTimes)
					, MakeArrayView(StepCurve.FloatKeyValues.GetValue()));
			}
			else if (StepCurve.IntegerKeyValues.IsSet())
			{
				bResult |= UE::Anim::AddTypedCustomAttribute<FIntegerAnimationAttribute, int32>(FName(CurveName)
					, FName(BoneName)
					, TargetSequence
					, MakeArrayView(StepCurve.KeyTimes)
					, MakeArrayView(StepCurve.IntegerKeyValues.GetValue()));
			}
			else if (StepCurve.StringKeyValues.IsSet())
			{
				bResult |= UE::Anim::AddTypedCustomAttribute<FStringAnimationAttribute, FString>(FName(CurveName)
					, FName(BoneName)
					, TargetSequence
					, MakeArrayView(StepCurve.KeyTimes)
					, MakeArrayView(StepCurve.StringKeyValues.GetValue()));
			}

			FRichCurve RichCurve;
			bool bRichCurveConverted = ConvertToRichCurve(StepCurve.FloatKeyValues, StepCurve.KeyTimes, RichCurve)
				|| ConvertToRichCurve(StepCurve.IntegerKeyValues, StepCurve.KeyTimes, RichCurve)
				|| ConvertToRichCurve(StepCurve.ByteKeyValues, StepCurve.KeyTimes, RichCurve)
				|| ConvertToRichCurve(StepCurve.BooleanKeyValues, StepCurve.KeyTimes, RichCurve);

			if (bRichCurveConverted)
			{
				RichCurves.Add(RichCurve);
				if (RichCurveNames.Num() > 0)
				{
					//As CurveNames are used as uids:
					FString CurveNamePerCurve = CurveName + TEXT("_") + FString::FromInt(RichCurveNames.Num());
					RichCurveNames.Add(CurveNamePerCurve);
				}
				else
				{
					RichCurveNames.Add(CurveName);
				}
			}
		}

		if (RichCurves.Num() > 0)
		{
			bResult = InternalCreateCurve(TargetSequence, RichCurves, RichCurveNames, CurveFlags, bDoNotImportCurveWithZero, bAddCurveMetadataToSkeleton, bIsMorphTargetCurve, bIsMaterialCurve, bShouldTransact);
		}

		return bResult;
	}

	void ResolveWeightsForBlendShape(const TArray<float>& InbetweenFullWeights, float InWeight, float& OutMainWeight, TArray<float>& OutInbetweenWeights)
	{
		int32 NumInbetweens = InbetweenFullWeights.Num();
		if (NumInbetweens == 0)
		{
			OutMainWeight = InWeight;
			return;
		}

		OutInbetweenWeights.SetNumUninitialized(NumInbetweens);
		for (float& OutInbetweenWeight : OutInbetweenWeights)
		{
			OutInbetweenWeight = 0.0f;
		}

		if (FMath::IsNearlyEqual(InWeight, 0.0f))
		{
			OutMainWeight = 0.0f;
			return;
		}
		else if (FMath::IsNearlyEqual(InWeight, 1.0f))
		{
			OutMainWeight = 1.0f;
			return;
		}

		// Note how we don't care if UpperIndex/LowerIndex are beyond the bounds of the array here,
		// as that signals when we're above/below all inbetweens
		int32 UpperIndex = Algo::UpperBoundBy(InbetweenFullWeights, InWeight, [](const double& InbetweenWeight)
			{
				return InbetweenWeight;
			});
		int32 LowerIndex = UpperIndex - 1;

		float UpperWeight = 1.0f;
		if (UpperIndex <= NumInbetweens - 1)
		{
			UpperWeight = InbetweenFullWeights[UpperIndex];
		}

		float LowerWeight = 0.0f;
		if (LowerIndex >= 0)
		{
			LowerWeight = InbetweenFullWeights[LowerIndex];
		}

		UpperWeight = (InWeight - LowerWeight) / (UpperWeight - LowerWeight);
		LowerWeight = (1.0f - UpperWeight);

		// We're between upper inbetween and the 1.0 weight
		if (UpperIndex > NumInbetweens - 1)
		{
			OutMainWeight = UpperWeight;
			OutInbetweenWeights[NumInbetweens - 1] = LowerWeight;
		}
		// We're between 0.0 and the first inbetween weight
		else if (LowerIndex < 0)
		{
			OutMainWeight = 0;
			OutInbetweenWeights[0] = UpperWeight;
		}
		// We're between two inbetweens
		else
		{
			OutInbetweenWeights[UpperIndex] = UpperWeight;
			OutInbetweenWeights[LowerIndex] = LowerWeight;
		}
	}

	TArray<FRichCurve> ResolveWeightsForBlendShapeCurve(FRichCurve& ChannelWeightCurve, const TArray<float>& InbetweenFullWeights)
	{
		int32 NumInbetweens = InbetweenFullWeights.Num();
		if (NumInbetweens == 0)
		{
			return { ChannelWeightCurve };
		}

		TArray<FRichCurve> Result;
		Result.SetNum(NumInbetweens + 1);

		TArray<float> ResolvedInbetweenWeightsSample;
		ResolvedInbetweenWeightsSample.SetNum(NumInbetweens);

		for (const FRichCurveKey& SourceKey : ChannelWeightCurve.Keys)
		{
			const float SourceTime = SourceKey.Time;
			const float SourceValue = SourceKey.Value;

			float ResolvedPrimarySample = 0.0f;

			ResolveWeightsForBlendShape(InbetweenFullWeights, SourceValue, ResolvedPrimarySample, ResolvedInbetweenWeightsSample);

			FRichCurve& PrimaryCurve = Result[0];
			FKeyHandle PrimaryHandle = PrimaryCurve.AddKey(SourceTime, ResolvedPrimarySample);
			PrimaryCurve.SetKeyInterpMode(PrimaryHandle, SourceKey.InterpMode);

			for (int32 InbetweenIndex = 0; InbetweenIndex < NumInbetweens; ++InbetweenIndex)
			{
				FRichCurve& InbetweenCurve = Result[InbetweenIndex + 1];
				FKeyHandle InbetweenHandle = InbetweenCurve.AddKey(SourceTime, ResolvedInbetweenWeightsSample[InbetweenIndex]);
				InbetweenCurve.SetKeyInterpMode(InbetweenHandle, SourceKey.InterpMode);
			}
		}

		return Result;
	}

	bool CreateMorphTargetCurve(UAnimSequence* TargetSequence
		, TArray<FRichCurve>& Curves
		, const FString& CurveName
		, TArray<FString>& InbetweenCurveNames
		, TArray<float>& InbetweenFullWeights
		, bool bRemoveCurveRedundantKeys
		, int32 CurveFlags
		, bool bDoNotImportCurveWithZero
		, bool bAddCurveMetadataToSkeleton
		, bool bShouldTransact)
	{
		constexpr bool bIsMorphTargetCurve = true;
		constexpr bool bIsMaterialCurve = false;
		if (Curves.Num() == 1 && InbetweenCurveNames.Num() == InbetweenFullWeights.Num()+1)
		{
			//We must create inbetween shape curve to simulate the result
			//First bake the channel weight curves
			FRichCurve& ChannelWeightCurve = Curves[0];
#if WITH_EDITORONLY_DATA
			//Cannot bake a curve with only one frame
			if (ChannelWeightCurve.GetNumKeys() > 1)
			{
				ChannelWeightCurve.BakeCurve(1.0f / TargetSequence->ImportResampleFramerate);
			}
#endif

			// use the primary curve to generate inbetween shape curves + a modified primary curve
			TArray<FRichCurve> Results = ResolveWeightsForBlendShapeCurve(ChannelWeightCurve, InbetweenFullWeights);

			for (FRichCurve& Result : Results)
			{
				if (bRemoveCurveRedundantKeys)
				{
					Result.RemoveRedundantAutoTangentKeys(SMALL_NUMBER);
				}
			}
			return InternalCreateCurve(TargetSequence, Results, InbetweenCurveNames, CurveFlags, bDoNotImportCurveWithZero, bAddCurveMetadataToSkeleton, bIsMorphTargetCurve, bIsMaterialCurve, bShouldTransact);
		}
		return InternalCreateCurve(TargetSequence, Curves, { CurveName }, CurveFlags, bDoNotImportCurveWithZero, bAddCurveMetadataToSkeleton, bIsMorphTargetCurve, bIsMaterialCurve, bShouldTransact);
	}

	bool CreateMaterialCurve(UAnimSequence* TargetSequence, TArray<FRichCurve>& Curves, const FString& CurveName, int32 CurveFlags, bool bDoNotImportCurveWithZero, bool bAddCurveMetadataToSkeleton, bool bShouldTransact)
	{
		constexpr bool bIsMorphTargetCurve = false;
		constexpr bool bIsMaterialCurve = true;
		return InternalCreateCurve(TargetSequence, Curves, { CurveName }, CurveFlags, bDoNotImportCurveWithZero, bAddCurveMetadataToSkeleton, bIsMorphTargetCurve, bIsMaterialCurve, bShouldTransact);
	}

	bool CreateAttributeCurve(UAnimSequence* TargetSequence, TArray<FRichCurve>& Curves, const FString& CurveName, int32 CurveFlags, bool bDoNotImportCurveWithZero, bool bAddCurveMetadataToSkeleton, bool bShouldTransact)
	{
		//This curve don't animate morph target or material parameter.
		constexpr bool bIsMorphTargetCurve = false;
		constexpr bool bIsMaterialCurve = false;
		return InternalCreateCurve(TargetSequence, Curves, { CurveName }, CurveFlags, bDoNotImportCurveWithZero, bAddCurveMetadataToSkeleton, bIsMorphTargetCurve, bIsMaterialCurve, bShouldTransact);
	}

	void RetrieveAnimationPayloads(UAnimSequence* AnimSequence
		, UInterchangeAnimSequenceFactory::FBoneTrackData& BoneTrackData
		, UInterchangeAnimSequenceFactory::FMorphTargetData& MorphTargetData
		, const UInterchangeAnimSequenceFactoryNode* AnimSequenceFactoryNode
		, const UInterchangeBaseNodeContainer* NodeContainer
		, const UInterchangeSkeletonFactoryNode* SkeletonFactoryNode
		, const IInterchangeAnimationPayloadInterface* AnimSequenceTranslatorPayloadInterface
		, const FString& AssetName
		, const bool bIsReimporting
		, TArray<FString>& OutCurvesNotFound
		, UInterchangeAnimSequenceFactory* Factory)
	{
		TMap<const UInterchangeSceneNode*, TFuture<TOptional<UE::Interchange::FAnimationPayloadData>>> AnimationPayloads;

		FString SkeletonRootUid;
		if (!SkeletonFactoryNode->GetCustomRootJointUid(SkeletonRootUid))
		{
			//Cannot import animation without a skeleton
			return;
		}
		
		IAnimationDataController& Controller = AnimSequence->GetController();
		USkeleton* Skeleton = AnimSequence->GetSkeleton();
		check(Skeleton);

		TArray<FString> SkeletonNodes;
		GetSkeletonSceneNodeFlatListRecursive(NodeContainer, SkeletonRootUid, SkeletonNodes);
		TArray<FString> NonAnimatedSkeletonNodes = SkeletonNodes;

		TMap<FString, FInterchangeAnimationPayLoadKey> PayloadKeys;
		AnimSequenceFactoryNode->GetSceneNodeAnimationPayloadKeys(PayloadKeys);
		
		const bool bShouldTransact = bIsReimporting;
		float PreviousSequenceLength = bIsReimporting ? AnimSequence->GetPlayLength() : 0;

		bool bImportBoneTracks = false;
		AnimSequenceFactoryNode->GetCustomImportBoneTracks(bImportBoneTracks);
		if (bImportBoneTracks)
		{
			//Get the sample rate, default to 30Hz in case the attribute is missing
			double SampleRate = 30.0;
			AnimSequenceFactoryNode->GetCustomImportBoneTracksSampleRate(SampleRate);

			double RangeStart = 0.0;
			AnimSequenceFactoryNode->GetCustomImportBoneTracksRangeStart(RangeStart);

			double RangeEnd = 1.0 / SampleRate; //One frame duration per default
			AnimSequenceFactoryNode->GetCustomImportBoneTracksRangeStop(RangeEnd);

			const double BakeInterval = 1.0 / SampleRate;

			//This destroy all previously imported animation raw data
			Controller.RemoveAllBoneTracks(bShouldTransact);

			FTransform GlobalOffsetTransform;
			bool bBakeMeshes = false;
			{
				GlobalOffsetTransform = FTransform::Identity;
				if (UInterchangeCommonPipelineDataFactoryNode* CommonPipelineDataFactoryNode = UInterchangeCommonPipelineDataFactoryNode::GetUniqueInstance(NodeContainer))
				{
					CommonPipelineDataFactoryNode->GetCustomGlobalOffsetTransform(GlobalOffsetTransform);
					CommonPipelineDataFactoryNode->GetBakeMeshes(bBakeMeshes);
				}
			}

			const double SequenceLength = FMath::Max<double>(BoneTrackData.MergedRangeEnd - BoneTrackData.MergedRangeStart, MINIMUM_ANIMATION_LENGTH);
			int32 FrameCount = FMath::RoundToInt32(SequenceLength * SampleRate);
			int32 BakeKeyCount = FrameCount + 1;
			const FFrameRate ResampleFrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(SampleRate);
			Controller.SetFrameRate(ResampleFrameRate, bShouldTransact);
			Controller.SetNumberOfFrames(FrameCount, bShouldTransact);

			if (bIsReimporting && 
				PreviousSequenceLength > MINIMUM_ANIMATION_LENGTH && 
				AnimSequence->GetDataModel()->GetNumberOfFloatCurves() > 0)
			{
				// The sequence already existed when we began the import. We need to scale the key times for all curves to match the new 
				// duration before importing over them. This is to catch any user-added curves
				float ScaleFactor = AnimSequence->GetPlayLength() / PreviousSequenceLength;
				if (!FMath::IsNearlyEqual(ScaleFactor, 1.f))
				{
					for (const FFloatCurve& Curve : AnimSequence->GetDataModel()->GetFloatCurves())
					{
						const FAnimationCurveIdentifier CurveId(Curve.GetName(), ERawCurveTrackTypes::RCT_Float);
						Controller.ScaleCurve(CurveId, 0.f, ScaleFactor, bShouldTransact);
					}
				}
			}

			for (TTuple< const UInterchangeSceneNode*, UE::Interchange::FAnimationPayloadData>& AnimationPayload : BoneTrackData.PreProcessedAnimationPayloads)
			{
				NonAnimatedSkeletonNodes.Remove(AnimationPayload.Key->GetUniqueID());

				const FName BoneName = FName(*(AnimationPayload.Key->GetDisplayLabel()));
				UE::Interchange::FAnimationPayloadData& AnimationTransformPayload = AnimationPayload.Value;
				
				//If we are getting the root 
				bool bApplyGlobalOffset = AnimationPayload.Key->GetUniqueID().Equals(SkeletonRootUid);

				if (AnimationTransformPayload.Transforms.Num() == 0)
				{
					//We need at least one transform
					AnimationTransformPayload.Transforms.Add(FTransform::Identity);
				}

				const double SequenceLengthForAnimationPayload = FMath::Max<double>(AnimationTransformPayload.RangeEndTime - AnimationTransformPayload.RangeStartTime, MINIMUM_ANIMATION_LENGTH);
				int32 BakeKeyCountForAnimationPayload = FMath::RoundToInt32(SequenceLengthForAnimationPayload * AnimationTransformPayload.BakeFrequency) + 1;

				FRawAnimSequenceTrack RawTrack;
				RawTrack.PosKeys.Reserve(BakeKeyCountForAnimationPayload);
				RawTrack.RotKeys.Reserve(BakeKeyCountForAnimationPayload);
				RawTrack.ScaleKeys.Reserve(BakeKeyCountForAnimationPayload);
				
				if (!ensure(AnimationTransformPayload.Transforms.Num() == BakeKeyCountForAnimationPayload))
				{
					FString PayloadKey = PayloadKeys[AnimationPayload.Key->GetUniqueID()].UniqueId;
					UInterchangeResultWarning_Generic* Message = Factory->AddMessage<UInterchangeResultWarning_Generic>();
					Message->DestinationAssetName = AssetName;
					Message->AssetType = UAnimSequence::StaticClass();
					Message->Text = FText::Format(NSLOCTEXT("UInterchangeAnimSequenceFactory", "AnimationPayloadUnexpectedTransformNumber", "Animation Payload [{0}] has unexpected number of Baked Transforms."), FText::FromString(PayloadKey));
					break;
				}

				if (AnimationTransformPayload.PayloadKey.Type == EInterchangeAnimationPayLoadType::BAKED)
				{
					//Everything should match Key count, sample rate and range
					if (!(ensure(FMath::IsNearlyEqual(AnimationTransformPayload.BakeFrequency, SampleRate, UE_DOUBLE_KINDA_SMALL_NUMBER)) &&
						  ensure(FMath::IsNearlyEqual(AnimationTransformPayload.RangeStartTime, RangeStart, UE_DOUBLE_KINDA_SMALL_NUMBER)) &&
						  ensure(FMath::IsNearlyEqual(AnimationTransformPayload.RangeEndTime, RangeEnd, UE_DOUBLE_KINDA_SMALL_NUMBER))))
					{
						FString PayloadKey = PayloadKeys[AnimationPayload.Key->GetUniqueID()].UniqueId;
						UInterchangeResultWarning_Generic* Message = Factory->AddMessage<UInterchangeResultWarning_Generic>();
						Message->DestinationAssetName = AssetName;
						Message->AssetType = UAnimSequence::StaticClass();
						Message->Text = FText::Format(NSLOCTEXT("UInterchangeAnimSequenceFactory", "AnimationPayloadBakeFrequencyNotTheExpected", "The BakeFrequency, RangeStartTime and RangeEndTime of Animation Payload [{0}] are not the same as the values provided."), FText::FromString(PayloadKey));
					}
				}

				double CurrentTime = 0;
				for (int32 BakeIndex = 0; BakeIndex < BakeKeyCountForAnimationPayload; BakeIndex++, CurrentTime += BakeInterval)
				{
					FTransform3f AnimKeyTransform = FTransform3f(AnimationTransformPayload.Transforms[BakeIndex]);
					if (bApplyGlobalOffset && bBakeMeshes)
					{
 						if (const UInterchangeSceneNode* RootJointNode = Cast<UInterchangeSceneNode>(NodeContainer->GetNode(SkeletonRootUid)))
						{
							FString RootJointParentNodeUid = RootJointNode->GetParentUid();
							if (const UInterchangeSceneNode* RootJointParentNode = Cast<UInterchangeSceneNode>(NodeContainer->GetNode(RootJointParentNodeUid)))
							{
								FTransform GlobalTransform;
								RootJointParentNode->GetCustomGlobalTransform(NodeContainer, GlobalOffsetTransform, GlobalTransform);
								AnimKeyTransform = AnimKeyTransform * FTransform3f(GlobalTransform);
							}
						}
					}
					//Default value to identity
					FVector3f Position(0.0f);
					FQuat4f Quaternion(0.0f);
					FVector3f Scale(1.0f);

					Position = AnimKeyTransform.GetLocation();
					Quaternion = AnimKeyTransform.GetRotation();
					Scale = AnimKeyTransform.GetScale3D();
					RawTrack.ScaleKeys.Add(Scale);
					RawTrack.PosKeys.Add(Position);
					RawTrack.RotKeys.Add(Quaternion);
				}

				//Make sure we create the correct amount of keys
				if (!ensure(RawTrack.ScaleKeys.Num() == BakeKeyCountForAnimationPayload
					&& RawTrack.PosKeys.Num() == BakeKeyCountForAnimationPayload
					&& RawTrack.RotKeys.Num() == BakeKeyCountForAnimationPayload))
				{
					FString PayloadKey = PayloadKeys[AnimationPayload.Key->GetUniqueID()].UniqueId;
					UInterchangeResultWarning_Generic* Message = Factory->AddMessage<UInterchangeResultWarning_Generic>();
					Message->DestinationAssetName = AssetName;
					Message->AssetType = UAnimSequence::StaticClass();
					Message->Text = FText::Format(NSLOCTEXT("UInterchangeAnimSequenceFactory", "AnimationPayloadBadKeyNumber", "Animation Payload [{0}] has unexpected number of animation keys. Animation will be incorrect."), FText::FromString(PayloadKey));
					continue;
				}

				//add new track
				if (BoneName.GetStringLength() > 92)
				{
					//The bone name exceed the maximum length supported by the animation system
					//The animation system is adding _CONTROL to the bone name to name the animation controller and
					//the maximum total length is cap at 100, so user should not import bone name longer then 92 characters
					UInterchangeResultWarning_Generic* Message = Factory->AddMessage<UInterchangeResultWarning_Generic>();
					Message->DestinationAssetName = AssetName;
					Message->AssetType = UAnimSequence::StaticClass();
					Message->Text = FText::Format(NSLOCTEXT("UInterchangeAnimSequenceFactory", "BoneNameExceed92Characters", "Bone with animation cannot have a name exceeding 92 characters: {0}"), FText::FromName(BoneName));
					continue;
				}
				Controller.AddBoneCurve(BoneName, bShouldTransact);
				Controller.SetBoneTrackKeys(BoneName, RawTrack.PosKeys, RawTrack.RotKeys, RawTrack.ScaleKeys, bShouldTransact);
			}

			//For joint with no animation, verify if the bind pose equal the local time 0 pose.
			// If not, Add one animation track with only one transform key at time 0 with the Local transform.
			for (const FString& NonAnimatedSkeletonNodeUID : NonAnimatedSkeletonNodes)
			{
				if (const UInterchangeSceneNode* SkeletonNode = Cast<UInterchangeSceneNode>(NodeContainer->GetNode(NonAnimatedSkeletonNodeUID)))
				{
					TOptional<FTransform> ReferenceTransform;
					if (Skeleton)
					{
						const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
						int32 BoneIndex = RefSkeleton.FindBoneIndex(*SkeletonNode->GetDisplayLabel());
						if (BoneIndex != INDEX_NONE && RefSkeleton.GetRefBonePose().IsValidIndex(BoneIndex))
						{
							ReferenceTransform = RefSkeleton.GetRefBonePose()[BoneIndex];
						}
					}

					//check if BindPose exists and if it does, does it equal to LocalTransform/ReferenceTransform
					FTransform LocalBindPoseTransform;
					FTransform LocalTransform;

					if (const UInterchangeJointNode* JointNode = Cast<UInterchangeJointNode>(SkeletonNode))
					{
						if (JointNode->GetBindPoseLocalTransform(LocalBindPoseTransform)
							&& JointNode->GetCustomLocalTransform(LocalTransform)
							&& (!LocalBindPoseTransform.Equals(LocalTransform)
								|| (ReferenceTransform.IsSet() && (!ReferenceTransform.GetValue().Equals(LocalBindPoseTransform))))
							)
						{
							//If we bake the mesh and the current non animated node is the root joint, get the global transform instead of the local
							if (bBakeMeshes && JointNode->GetUniqueID().Equals(SkeletonRootUid))
							{
								if (const UInterchangeSceneNode* RootJointNode = Cast<UInterchangeSceneNode>(NodeContainer->GetNode(SkeletonRootUid)))
								{
									RootJointNode->GetCustomGlobalTransform(NodeContainer, GlobalOffsetTransform, LocalTransform);
								}
							}

							FTransform3f AnimKeyTransform = static_cast<FTransform3f>(LocalTransform);
							const FName BoneName = FName(*(JointNode->GetDisplayLabel()));
							//Add only one transform key at time 0 since this node is not animated
							FRawAnimSequenceTrack RawTrack;
							RawTrack.ScaleKeys.Add(AnimKeyTransform.GetScale3D());
							RawTrack.PosKeys.Add(AnimKeyTransform.GetLocation());
							RawTrack.RotKeys.Add(AnimKeyTransform.GetRotation());
							Controller.AddBoneCurve(BoneName, bShouldTransact);
							Controller.SetBoneTrackKeys(BoneName, RawTrack.PosKeys, RawTrack.RotKeys, RawTrack.ScaleKeys, bShouldTransact);
						}
					}
				}
			}
		}

		bool bDeleteExistingMorphTargetCurves = false;
		AnimSequenceFactoryNode->GetCustomDeleteExistingMorphTargetCurves(bDeleteExistingMorphTargetCurves);
		bool bDeleteExistingCustomAttributeCurves = false;
		AnimSequenceFactoryNode->GetCustomDeleteExistingCustomAttributeCurves(bDeleteExistingCustomAttributeCurves);
		bool bDeleteExistingNonCurveCustomAttributes = false;
		AnimSequenceFactoryNode->GetCustomDeleteExistingNonCurveCustomAttributes(bDeleteExistingNonCurveCustomAttributes);
		if (bDeleteExistingMorphTargetCurves || bDeleteExistingCustomAttributeCurves)
		{
			TArray<FName> CurveNamesToRemove;
			for (const FFloatCurve& Curve : AnimSequence->GetDataModel()->GetFloatCurves())
			{
				const FCurveMetaData* MetaData = Skeleton->GetCurveMetaData(Curve.GetName());
				if (MetaData)
				{
					bool bDeleteCurve = MetaData->Type.bMorphtarget ? bDeleteExistingMorphTargetCurves : bDeleteExistingCustomAttributeCurves;
					if (bDeleteCurve)
					{
						CurveNamesToRemove.Add(Curve.GetName());
					}
				}
			}

			for (const FName& CurveName : CurveNamesToRemove)
			{
				const FAnimationCurveIdentifier CurveId(CurveName, ERawCurveTrackTypes::RCT_Float);
				Controller.RemoveCurve(CurveId, bShouldTransact);
			}
		}

		if (bDeleteExistingNonCurveCustomAttributes)
		{
			Controller.RemoveAllAttributes(bShouldTransact);
		}

		
		bool bImportAttributeCurves = false;
		AnimSequenceFactoryNode->GetCustomImportAttributeCurves(bImportAttributeCurves);
		if (bImportAttributeCurves)
		{
			const IAnimationDataModel* DataModel = AnimSequence->GetDataModel();
			const int32 NumFloatCurves = DataModel->GetNumberOfFloatCurves();
			const FAnimationCurveData& CurveData = DataModel->GetCurveData();

			OutCurvesNotFound.Reset(NumFloatCurves);

			for (const FFloatCurve& FloatCurve : CurveData.FloatCurves)
			{
				const FCurveMetaData* MetaData = Skeleton->GetCurveMetaData(FloatCurve.GetName());

				if (MetaData && !MetaData->Type.bMorphtarget)
				{
					OutCurvesNotFound.Add(FloatCurve.GetName().ToString());
				}
			}

			bool bMaterialDriveParameterOnCustomAttribute = false;
			AnimSequenceFactoryNode->GetCustomMaterialDriveParameterOnCustomAttribute(bMaterialDriveParameterOnCustomAttribute);
			TArray<FString> MaterialSuffixes;
			AnimSequenceFactoryNode->GetAnimatedMaterialCurveSuffixes(MaterialSuffixes);
			bool bDoNotImportCurveWithZero = false;
			AnimSequenceFactoryNode->GetCustomDoNotImportCurveWithZero(bDoNotImportCurveWithZero);
			bool bAddCurveMetadataToSkeleton = false;
			AnimSequenceFactoryNode->GetCustomAddCurveMetadataToSkeleton(bAddCurveMetadataToSkeleton);
			bool bRemoveCurveRedundantKeys = false;
			AnimSequenceFactoryNode->GetCustomRemoveCurveRedundantKeys(bRemoveCurveRedundantKeys);

			auto IsCurveHookToMaterial = [&MaterialSuffixes](const FString& CurveName)
			{
				for (const FString& MaterialSuffixe : MaterialSuffixes)
				{
					if (CurveName.EndsWith(MaterialSuffixe))
					{
						return true;
					}
				}
				return false;
			};

			//Import morph target curves
			{
				for (TPair<FString, UE::Interchange::FAnimationPayloadData>& CurveNameAndPayload : MorphTargetData.CurvesPayloads)
				{
					UE::Interchange::FAnimationPayloadData& AnimationCurvePayload = CurveNameAndPayload.Value;
					if (bRemoveCurveRedundantKeys)
					{
						for (FRichCurve& RichCurve : AnimationCurvePayload.Curves)
						{
							RichCurve.RemoveRedundantAutoTangentKeys(SMALL_NUMBER);
						}
					}
					constexpr int32 CurveFlags = 0;
					CreateMorphTargetCurve(AnimSequence
						, AnimationCurvePayload.Curves
						, MorphTargetData.CurveNodeNamePerPayloadKey.FindChecked(CurveNameAndPayload.Key)
						, AnimationCurvePayload.InbetweenCurveNames
						, AnimationCurvePayload.InbetweenFullWeights
						, bRemoveCurveRedundantKeys
						, CurveFlags
						, bDoNotImportCurveWithZero
						, bAddCurveMetadataToSkeleton
						, bShouldTransact);
				}
			}

			//Import Attribute curves
			{
				//Utility to make sure the curve is compatible with FRichCurve
				auto IsDecimalType = [](UE::Interchange::EAttributeTypes Type)
				{
					switch (Type)
					{
					case UE::Interchange::EAttributeTypes::Double:
					case UE::Interchange::EAttributeTypes::Float:
					case UE::Interchange::EAttributeTypes::Float16:
					case UE::Interchange::EAttributeTypes::Vector2d:
					case UE::Interchange::EAttributeTypes::Vector2f:
					case UE::Interchange::EAttributeTypes::Vector3d:
					case UE::Interchange::EAttributeTypes::Vector3f:
					case UE::Interchange::EAttributeTypes::Vector4d:
					case UE::Interchange::EAttributeTypes::Vector4f:
						return true;
					}
					return false;
				};

				//Get if the SkeletonSceneNode has a _AnimationPayloadType attribute set, and if so, does it equal to input
				auto DoesSourceAnimationAllowCurve = [](const FString& AttibuteName, const UInterchangeSceneNode* SkeletonSceneNode, EInterchangeAnimationPayLoadType AnimationPayloadTypeToCheck) -> bool
					{
						EInterchangeAnimationPayLoadType AnimationPayloadType = EInterchangeAnimationPayLoadType::NONE;
						return SkeletonSceneNode->GetAnimationCurveTypeForCurveName(AttibuteName, AnimationPayloadType) && (AnimationPayloadType == AnimationPayloadTypeToCheck);
					};

				//Import Attribute curves (FRichCurve)
				{
					TArray<FString> AttributeCurveNames;
					AnimSequenceFactoryNode->GetAnimatedAttributeCurveNames(AttributeCurveNames);
					TArray<FAnimationPayloadQuery> PayloadQueries;
					TMap<TPair<FString, FString>, FString> CurveNames; // <<NodeUid,PayloadUID>, Name>
					for (const FString& NodeUid : SkeletonNodes)
					{
						if (const UInterchangeSceneNode* SkeletonSceneNode = Cast<UInterchangeSceneNode>(NodeContainer->GetNode(NodeUid)))
						{
							//Import material parameter curves (FRichCurve)
							TMap<FString, FString> CurveNamePayloads;
							TArray<FInterchangeUserDefinedAttributeInfo> AttributeInfos = UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttributeInfos(SkeletonSceneNode);
							for (const FInterchangeUserDefinedAttributeInfo& AttributeInfo : AttributeInfos)
							{
								//Material curve must be convertible to float since we need a FRichCurve
								if (AttributeInfo.PayloadKey.IsSet() && (IsDecimalType(AttributeInfo.Type) || DoesSourceAnimationAllowCurve(AttributeInfo.Name, SkeletonSceneNode, EInterchangeAnimationPayLoadType::CURVE)) && AttributeCurveNames.Contains(AttributeInfo.Name))
								{
									CurveNamePayloads.Add(AttributeInfo.Name, AttributeInfo.PayloadKey.GetValue());
								}
							}
							for (const TPair<FString, FString>& CurveNamePayload : CurveNamePayloads)
							{
								//This goes slightly against the intent of the Type / FInterchangeAnimationPayLoadKey usage as we set the Type here, outside of the Translator
								// this is due to the nature of the Attribute curves.
								// Could be potentially reworked so that with the AttributeInfos we store the Type as well.
								PayloadQueries.Add(FAnimationPayloadQuery(NodeUid, FInterchangeAnimationPayLoadKey(CurveNamePayload.Value, EInterchangeAnimationPayLoadType::CURVE)));
								CurveNames.Add(TPair<FString, FString>(NodeUid, CurveNamePayload.Value), CurveNamePayload.Key);
							}
						}
					}
					TArray<FAnimationPayloadData> Payloads = AnimSequenceTranslatorPayloadInterface->GetAnimationPayloadData(PayloadQueries);

					for (FAnimationPayloadData& CurvePayloadData : Payloads)
					{
						FString* CurveNamePtr = CurveNames.Find(TPair<FString, FString>(CurvePayloadData.SceneNodeUniqueID, CurvePayloadData.PayloadKey.UniqueId));
						FString CurveName = CurveNamePtr ? *CurveNamePtr : TEXT("Unknown");

						if (bRemoveCurveRedundantKeys)
						{
							for (FRichCurve& RichCurve : CurvePayloadData.Curves)
							{
								RichCurve.RemoveRedundantAutoTangentKeys(SMALL_NUMBER);
							}
						}
						OutCurvesNotFound.Remove(CurveName);
						constexpr int32 CurveFlags = 0;
						if (bMaterialDriveParameterOnCustomAttribute || IsCurveHookToMaterial(CurveName))
						{
							CreateMaterialCurve(AnimSequence, CurvePayloadData.Curves, CurveName, CurveFlags, bDoNotImportCurveWithZero, bAddCurveMetadataToSkeleton, bShouldTransact);
						}
						else
						{
							CreateAttributeCurve(AnimSequence, CurvePayloadData.Curves, CurveName, CurveFlags, bDoNotImportCurveWithZero, bAddCurveMetadataToSkeleton, bShouldTransact);
						}
					}
				}

				//Import attribute step curves (step curves, only time and value array, no FRichCurve. The anim API expect the value array is of the type of the translated attribute)
				{
					TArray<FString> AttributeStepCurveNames;
					AnimSequenceFactoryNode->GetAnimatedAttributeStepCurveNames(AttributeStepCurveNames);
					TMap<TPair<FString, FString>, FString> CurveNames; // <<NodeUid,PayloadUID>, Name>
					TMap<TPair<FString, FString>, FString> StepCurveNames; // <<NodeUid,PayloadUID>, Name>

					TArray<FAnimationPayloadQuery> CurvePayloadQueries;
					TArray<FAnimationPayloadQuery> StepCurvePayloadQueries;

					TMap<FString, FString> AnimationBoneNames;
					for (const FString& NodeUid : SkeletonNodes)
					{
						if (const UInterchangeSceneNode* SkeletonSceneNode = Cast<UInterchangeSceneNode>(NodeContainer->GetNode(NodeUid)))
						{
							FString BoneName = SkeletonSceneNode->GetDisplayLabel();
							//Import material parameter curves (FRichCurve)
							TMap<FString, FString> CurveNameFloatPayloads;
							TMap<FString, FString> CurveNameStepCurvePayloads;
							TArray<FInterchangeUserDefinedAttributeInfo> AttributeInfos = UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttributeInfos(SkeletonSceneNode);
							for (const FInterchangeUserDefinedAttributeInfo& AttributeInfo : AttributeInfos)
							{
								//Material curve must be convertible to float since we need a FRichCurve
								if (AttributeInfo.PayloadKey.IsSet() && AttributeStepCurveNames.Contains(AttributeInfo.Name))
								{
									if (IsDecimalType(AttributeInfo.Type) && !(DoesSourceAnimationAllowCurve(AttributeInfo.Name, SkeletonSceneNode, EInterchangeAnimationPayLoadType::STEPCURVE)))
									{
										CurveNameFloatPayloads.Add(AttributeInfo.Name, AttributeInfo.PayloadKey.GetValue());
									}
									else
									{
										CurveNameStepCurvePayloads.Add(AttributeInfo.Name, AttributeInfo.PayloadKey.GetValue());
									}
								}
							}
							for (const TPair<FString, FString>& CurveNamePayload : CurveNameFloatPayloads)
							{
								//This goes slightly against the intent of the Type / FInterchangeAnimationPayLoadKey usage as we set the Type here, outside of the Translator
								// this is due to the nature of the Attribute curves.
								// Could be potentially reworked so that with the AttributeInfos we store the Type as well.
								CurvePayloadQueries.Add(FAnimationPayloadQuery(NodeUid, FInterchangeAnimationPayLoadKey(CurveNamePayload.Value, EInterchangeAnimationPayLoadType::CURVE)));
								AnimationBoneNames.Add(CurveNamePayload.Key, BoneName);
								CurveNames.Add(TPair<FString, FString>(NodeUid, CurveNamePayload.Value), CurveNamePayload.Key);
							}
							for (const TPair<FString, FString>& StepCurveNamePayload : CurveNameStepCurvePayloads)
							{
								//This goes slightly against the intent of the Type / FInterchangeAnimationPayLoadKey usage as we set the Type here, outside of the Translator
								// this is due to the nature of the Attribute curves.
								// Could be potentially reworked so that with the AttributeInfos we store the Type as well.
								StepCurvePayloadQueries.Add(FAnimationPayloadQuery(NodeUid, FInterchangeAnimationPayLoadKey(StepCurveNamePayload.Value, EInterchangeAnimationPayLoadType::STEPCURVE)));
								AnimationBoneNames.Add(StepCurveNamePayload.Key, BoneName);
								StepCurveNames.Add(TPair<FString, FString>(NodeUid, StepCurveNamePayload.Value), StepCurveNamePayload.Key);
							}
						}
					}

					auto AddAttributeCurveToAnimSequence = [bRemoveCurveRedundantKeys, bDoNotImportCurveWithZero, bAddCurveMetadataToSkeleton, bShouldTransact, &AnimSequence](FAnimationPayloadData& StepCurvePayload, const FString& CurveName, const FString& BoneName)
					{
						if (bRemoveCurveRedundantKeys)
						{
							for (FInterchangeStepCurve& StepCurve : StepCurvePayload.StepCurves)
							{
								StepCurve.RemoveRedundantKeys(SMALL_NUMBER);
							}
						}

						constexpr bool bIsMorphTargetCurve = false;
						constexpr bool bIsMaterialCurve = false;

						constexpr int32 CurveFlags = 0;
						CreateAttributeStepCurve(AnimSequence, StepCurvePayload.StepCurves, CurveName, BoneName, CurveFlags, bDoNotImportCurveWithZero, bAddCurveMetadataToSkeleton, bIsMorphTargetCurve, bIsMaterialCurve, bShouldTransact);
					};

					TArray<FAnimationPayloadData> CurvePayloads = AnimSequenceTranslatorPayloadInterface->GetAnimationPayloadData(CurvePayloadQueries);
					for (FAnimationPayloadData& CurvePayloadData : CurvePayloads)
					{
						CurvePayloadData.CalculateDataFor(EInterchangeAnimationPayLoadType::STEPCURVE);

						FString* CurveNamePtr = CurveNames.Find(TPair<FString, FString>(CurvePayloadData.SceneNodeUniqueID, CurvePayloadData.PayloadKey.UniqueId));
						FString CurveName = CurveNamePtr ? *CurveNamePtr : TEXT("Unknown");

						AddAttributeCurveToAnimSequence(CurvePayloadData, CurveName, AnimationBoneNames.FindChecked(CurveName));
					}

					TArray<FAnimationPayloadData> StepCurvePayloads = AnimSequenceTranslatorPayloadInterface->GetAnimationPayloadData(StepCurvePayloadQueries);
					for (FAnimationPayloadData& StepCurvePayloadData : StepCurvePayloads)
					{
						FString* CurveNamePtr = StepCurveNames.Find(TPair<FString, FString>(StepCurvePayloadData.SceneNodeUniqueID, StepCurvePayloadData.PayloadKey.UniqueId));
						FString CurveName = CurveNamePtr ? *CurveNamePtr : TEXT("Unknown");

						AddAttributeCurveToAnimSequence(StepCurvePayloadData, CurveName, AnimationBoneNames.FindChecked(CurveName));
					}
				}
			}
		}
		else
		{
			// Store float curve tracks which use to exist on the animation
			for (const FFloatCurve& Curve : AnimSequence->GetDataModel()->GetFloatCurves())
			{
				const FCurveMetaData* MetaData = Skeleton->GetCurveMetaData(Curve.GetName());
				if (MetaData && !MetaData->Type.bMorphtarget)
				{
					OutCurvesNotFound.Add(Curve.GetName().ToString());
				}
			}
		}
	}
} //namespace UE::Interchange::Private
#endif //WITH_EDITOR

UClass* UInterchangeAnimSequenceFactory::GetFactoryClass() const
{
	return UAnimSequence::StaticClass();
}

void UInterchangeAnimSequenceFactory::CreatePayloadTasks(const FImportAssetObjectParams& Arguments, bool bAsync, TArray<TSharedPtr<UE::Interchange::FInterchangeTaskBase>>& PayloadTasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeAnimSequenceFactory::CreatePayloadTasks);
	const IInterchangeAnimationPayloadInterface* AnimSequenceTranslatorPayloadInterface = Cast<IInterchangeAnimationPayloadInterface>(Arguments.Translator);
	if (!AnimSequenceTranslatorPayloadInterface)
	{
		return;
	}

	UInterchangeAnimSequenceFactoryNode* AnimSequenceFactoryNode = Cast<UInterchangeAnimSequenceFactoryNode>(Arguments.AssetNode);
	if (AnimSequenceFactoryNode == nullptr)
	{
		return;
	}

	bool bImportBoneTracks = false;
	AnimSequenceFactoryNode->GetCustomImportBoneTracks(bImportBoneTracks);
	if (bImportBoneTracks)
	{
		//Get the sample rate, default to 30Hz in case the attribute is missing
		double SampleRate = 30.0;
		AnimSequenceFactoryNode->GetCustomImportBoneTracksSampleRate(SampleRate);

		double RangeStart = 0.0;
		AnimSequenceFactoryNode->GetCustomImportBoneTracksRangeStart(RangeStart);

		double RangeEnd = 1.0 / SampleRate; //One frame duration per default
		AnimSequenceFactoryNode->GetCustomImportBoneTracksRangeStop(RangeEnd);


		TMap<FString, FInterchangeAnimationPayLoadKey> PayloadKeys;
		AnimSequenceFactoryNode->GetSceneNodeAnimationPayloadKeys(PayloadKeys);
		BoneAnimationPayloadQueries.Empty(PayloadKeys.Num());
		BoneAnimationPayloadResults.Empty(PayloadKeys.Num());

		if (AnimSequenceTranslatorPayloadInterface->PreferGroupingBoneAnimationQueriesTogether())
		{
			//Create one task for all queries
			for (const TTuple<FString, FInterchangeAnimationPayLoadKey>& SceneNodeUidAndPayloadKey : PayloadKeys)
			{
				UE::Interchange::FAnimationPayloadQuery AnimationPayloadQuery(SceneNodeUidAndPayloadKey.Key, SceneNodeUidAndPayloadKey.Value, SampleRate, RangeStart, RangeEnd);
				BoneAnimationPayloadQueries.Add(SceneNodeUidAndPayloadKey, AnimationPayloadQuery);
				//Allocate the results so we do not need any mutex
				BoneAnimationPayloadResults.FindOrAdd(SceneNodeUidAndPayloadKey, UE::Interchange::FAnimationPayloadData(SceneNodeUidAndPayloadKey.Key, SceneNodeUidAndPayloadKey.Value));
			}
			
			TSharedPtr<UE::Interchange::FInterchangeTaskLambda, ESPMode::ThreadSafe> TaskGetAnimationPayload = MakeShared<UE::Interchange::FInterchangeTaskLambda, ESPMode::ThreadSafe>(bAsync ? UE::Interchange::EInterchangeTaskThread::AsyncThread : UE::Interchange::EInterchangeTaskThread::GameThread
				, [this, AnimSequenceTranslatorPayloadInterface]()
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeAnimSequenceFactory::GetAnimationPayloadMultipleQueries);
					TArray<UE::Interchange::FAnimationPayloadQuery> TmpPayloadQueries;
					BoneAnimationPayloadQueries.GenerateValueArray(TmpPayloadQueries);
					TArray<UE::Interchange::FAnimationPayloadData> AnimationPayloads = AnimSequenceTranslatorPayloadInterface->GetAnimationPayloadData(TmpPayloadQueries);
					if (ensure(AnimationPayloads.Num() == BoneAnimationPayloadQueries.Num()))
					{
						TArray<TTuple<FString, FInterchangeAnimationPayLoadKey>> TmpPayloadKeys;
						BoneAnimationPayloadQueries.GenerateKeyArray(TmpPayloadKeys);
						for (int32 AnimationIndex = 0; AnimationIndex < AnimationPayloads.Num(); ++AnimationIndex)
						{
							if (ensure(BoneAnimationPayloadResults.Contains(TmpPayloadKeys[AnimationIndex])))
							{
								BoneAnimationPayloadResults.FindChecked(TmpPayloadKeys[AnimationIndex]) = AnimationPayloads[AnimationIndex];
							}
						}
					}
				});
			PayloadTasks.Add(TaskGetAnimationPayload);
		}
		else
		{
			//Create one task per query
			for (const TTuple<FString, FInterchangeAnimationPayLoadKey>& SceneNodeUidAndPayloadKey : PayloadKeys)
			{
				UE::Interchange::FAnimationPayloadQuery AnimationPayloadQuery(SceneNodeUidAndPayloadKey.Key, SceneNodeUidAndPayloadKey.Value, SampleRate, RangeStart, RangeEnd);
				BoneAnimationPayloadQueries.Add(SceneNodeUidAndPayloadKey, AnimationPayloadQuery);
				//Allocate the results so we do not need any mutex
				BoneAnimationPayloadResults.FindOrAdd(SceneNodeUidAndPayloadKey, UE::Interchange::FAnimationPayloadData(SceneNodeUidAndPayloadKey.Key, SceneNodeUidAndPayloadKey.Value));

				TSharedPtr<UE::Interchange::FInterchangeTaskLambda, ESPMode::ThreadSafe> TaskGetAnimationPayload = MakeShared<UE::Interchange::FInterchangeTaskLambda, ESPMode::ThreadSafe>(bAsync ? UE::Interchange::EInterchangeTaskThread::AsyncThread : UE::Interchange::EInterchangeTaskThread::GameThread
					, [this, SceneNodeUidAndPayloadKey, AnimSequenceTranslatorPayloadInterface]()
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeAnimSequenceFactory::GetAnimationPayloadSingleQuery);
						UE::Interchange::FAnimationPayloadQuery& AnimationQuery = BoneAnimationPayloadQueries.FindChecked(SceneNodeUidAndPayloadKey);
						TArray<UE::Interchange::FAnimationPayloadData> AnimationPayloads = AnimSequenceTranslatorPayloadInterface->GetAnimationPayloadData({ AnimationQuery });
						if (ensure(AnimationPayloads.IsValidIndex(0) && BoneAnimationPayloadResults.Contains(SceneNodeUidAndPayloadKey)))
						{
							BoneAnimationPayloadResults.FindChecked(SceneNodeUidAndPayloadKey) = AnimationPayloads[0];
						}
					});
				PayloadTasks.Add(TaskGetAnimationPayload);
			}
		}
	}
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeAnimSequenceFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeAnimSequenceFactory::BeginImportAsset_GameThread);

	UInterchangeFactoryBase::FImportAssetResult ImportAssetResult;
#if WITH_EDITOR && WITH_EDITORONLY_DATA
	UAnimSequence* NewAnimSequence = nullptr;
	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	UInterchangeAnimSequenceFactoryNode* AnimSequenceFactoryNode = Cast<UInterchangeAnimSequenceFactoryNode>(Arguments.AssetNode);
	if (AnimSequenceFactoryNode == nullptr)
	{
		return ImportAssetResult;
	}

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (AnimSequenceFactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}
	}
	if (ExistingAsset)
	{
		//This is a reimport, we are just re-updating the source data
		NewAnimSequence = Cast<UAnimSequence>(ExistingAsset);
	}

	const IInterchangeAnimationPayloadInterface* AnimSequenceTranslatorPayloadInterface = Cast<IInterchangeAnimationPayloadInterface>(Arguments.Translator);
	if (!AnimSequenceTranslatorPayloadInterface)
	{
		UE_LOGF(LogInterchangeImport, Error, "Cannot import AnimSequence. The translator does not implement IInterchangeAnimationPayloadInterface.");
		return ImportAssetResult;
	}

	FString SkeletonUid;
	if (!AnimSequenceFactoryNode->GetCustomSkeletonFactoryNodeUid(SkeletonUid))
	{
		UE_LOGF(LogInterchangeImport, Error, "Could not create AnimSequence asset %ls, because there is no skeleton.", *Arguments.AssetName);
		return ImportAssetResult;
	}

	const UInterchangeSkeletonFactoryNode* SkeletonFactoryNode = Cast<UInterchangeSkeletonFactoryNode>(Arguments.NodeContainer->GetNode(SkeletonUid));
	if (!SkeletonFactoryNode)
	{
		UE_LOGF(LogInterchangeImport, Error, "Invalid skeleton factory node, the skeleton factory node is obligatory to import this animsequence [%ls]!", *Arguments.AssetName);
		return ImportAssetResult;
	}

	FSoftObjectPath SkeletonFactoryNodeReferenceObject;
	SkeletonFactoryNode->GetCustomReferenceObject(SkeletonFactoryNodeReferenceObject);

	USkeleton* Skeleton = nullptr;

	FSoftObjectPath SpecifiedSkeleton;
	AnimSequenceFactoryNode->GetCustomSkeletonSoftObjectPath(SpecifiedSkeleton);
	if (Skeleton == nullptr)
	{
		UObject* SkeletonObject = nullptr;

		if (SpecifiedSkeleton.IsValid())
		{
			SkeletonObject = SpecifiedSkeleton.TryLoad();
		}
		else if (SkeletonFactoryNodeReferenceObject.IsValid())
		{
			SkeletonObject = SkeletonFactoryNodeReferenceObject.TryLoad();
		}

		if (SkeletonObject)
		{
			Skeleton = Cast<USkeleton>(SkeletonObject);

		}
		else if (NewAnimSequence)
		{
			Skeleton = NewAnimSequence->GetSkeleton();
		}

		if (!ensure(Skeleton))
		{
			UE_LOGF(LogInterchangeImport, Error, "Invalid skeleton when importing animation sequence asset %ls.", *Arguments.AssetName);
			return ImportAssetResult;
		}
	}

	if (Skeleton->GetReferenceSkeleton().GetRawBoneNum() == 0)
	{
		UE_LOGF(LogInterchangeImport, Error, "Invalid empty skeleton when importing animation sequence asset %ls.", *Arguments.AssetName);
		return ImportAssetResult;
	}

	//Verify if the bone track animation is valid (sequence length versus framerate ...)
	if(!IsBoneTrackAnimationValid(AnimSequenceFactoryNode, Arguments))
	{
		return ImportAssetResult;
	}

	// create a new material or overwrite existing asset, if possible
	if (!ExistingAsset)
	{
		NewAnimSequence = NewObject<UAnimSequence>(Arguments.Parent, *Arguments.AssetName, RF_Public | RF_Standalone);
	}

	if (!NewAnimSequence)
	{
		UE_LOGF(LogInterchangeImport, Warning, "Could not create AnimSequence asset %ls", *Arguments.AssetName);
		return ImportAssetResult;
	}

	AnimSequenceFactoryNode->SetCustomReferenceObject(FSoftObjectPath(NewAnimSequence));

	NewAnimSequence->PreEditChange(nullptr);

	NewAnimSequence->SetSkeleton(Skeleton);

	AnimSequence = NewAnimSequence;
	ImportAssetResult.ImportedObject = NewAnimSequence;
#endif

	return ImportAssetResult;
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeAnimSequenceFactory::ImportAsset_Async(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeAnimSequenceFactory::ImportAsset_Async);

	UInterchangeFactoryBase::FImportAssetResult ImportAssetResult;
#if WITH_EDITOR && WITH_EDITORONLY_DATA
	//The game thread part should have verified all the data, so no need to do extra log
	if (!AnimSequence)
	{
		return ImportAssetResult;
	}
	const IInterchangeAnimationPayloadInterface* AnimSequenceTranslatorPayloadInterface = Cast<IInterchangeAnimationPayloadInterface>(Arguments.Translator);
	if (!AnimSequenceTranslatorPayloadInterface)
	{
		return ImportAssetResult;
	}

	UInterchangeAnimSequenceFactoryNode* AnimSequenceFactoryNode = Cast<UInterchangeAnimSequenceFactoryNode>(Arguments.AssetNode);
	if (AnimSequenceFactoryNode == nullptr)
	{
		return ImportAssetResult;
	}

	const bool bIsReImport = (Arguments.ReimportObject != nullptr);

	bool bImportBoneTracks = false;
	AnimSequenceFactoryNode->GetCustomImportBoneTracks(bImportBoneTracks);
	if (bImportBoneTracks)
	{
		USkeleton* Skeleton = AnimSequence->GetSkeleton();
		check(Skeleton);

		//Get the sample rate, default to 30Hz in case the attribute is missing
		double SampleRate = 30.0;
		AnimSequenceFactoryNode->GetCustomImportBoneTracksSampleRate(SampleRate);

		double RangeStart = 0.0;
		AnimSequenceFactoryNode->GetCustomImportBoneTracksRangeStart(RangeStart);

		double RangeEnd = 1.0 / SampleRate; //One frame duration per default
		AnimSequenceFactoryNode->GetCustomImportBoneTracksRangeStop(RangeEnd);

		TMap<FString, FInterchangeAnimationPayLoadKey> PayloadKeys;
		AnimSequenceFactoryNode->GetSceneNodeAnimationPayloadKeys(PayloadKeys);

		BoneTrackData.MergedRangeEnd = RangeEnd;
		BoneTrackData.MergedRangeStart = RangeStart;
		BoneTrackData.PreProcessedAnimationPayloads.Reset();
		for (const TTuple<FString, FInterchangeAnimationPayLoadKey>& SceneNodeUidAndPayloadKey : PayloadKeys)
		{
			if (!BoneAnimationPayloadResults.Contains(SceneNodeUidAndPayloadKey))
			{
				continue;
			}
			UE::Interchange::FAnimationPayloadData& AnimationPayload = BoneAnimationPayloadResults.FindChecked(SceneNodeUidAndPayloadKey);

			const UInterchangeSceneNode* SkeletonSceneNode = Cast<UInterchangeSceneNode>(Arguments.NodeContainer->GetNode(AnimationPayload.SceneNodeUniqueID));
			if (!SkeletonSceneNode)
			{
				continue;
			}

			const FName BoneName = FName(*(SkeletonSceneNode->GetDisplayLabel()));
			const int32 BoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(BoneName);
			if (BoneIndex == INDEX_NONE)
			{
				//Skip this bone, we did not found it in the skeleton
				continue;
			}

			UE::Interchange::FAnimationPayloadData& AnimationTransformPayload = AnimationPayload;

			if (AnimationTransformPayload.PayloadKey.Type != EInterchangeAnimationPayLoadType::BAKED)
			{
				//Where Curve is null the LocalTransform should be used for the Baked Transform generation.
				FTransform LocalTransform;
				SkeletonSceneNode->GetCustomLocalTransform(LocalTransform);

				//As non-baked transforms do not have BakeFrequency concept we set the BakeFrequency here, in order for the CalculateDataFor can pick it up correctly.
				AnimationTransformPayload.BakeFrequency = SampleRate;

				//Currently only Curve -> Baked conversion (for LevelSequence->AnimSequence conversion by ForceMeshType Skeletal use case)
				//and Curve -> Step Curve conversion (for custom attributes)
				AnimationTransformPayload.CalculateDataFor(EInterchangeAnimationPayLoadType::BAKED, LocalTransform);
				//Range End will be calculated as well:
				if (BoneTrackData.MergedRangeEnd < AnimationTransformPayload.RangeEndTime)
				{
					BoneTrackData.MergedRangeEnd = AnimationTransformPayload.RangeEndTime;
				}
			}

			BoneTrackData.PreProcessedAnimationPayloads.Add(SkeletonSceneNode, AnimationTransformPayload);
		}
	}

	//Import morph target curves
	{
		TMap<FString, FInterchangeAnimationPayLoadKey> MorphTargetNodeAnimationPayloads;
		AnimSequenceFactoryNode->GetMorphTargetNodeAnimationPayloadKeys(MorphTargetNodeAnimationPayloads);

		TArray<UE::Interchange::FAnimationPayloadQuery> PayloadQueries;
		TMap<FString, TOptional<UE::Interchange::FAnimationPayloadData>> MorphTargetCurveWeightInstanceAnimationPayloads;

		for (const TPair<FString, FInterchangeAnimationPayLoadKey>& MorphTargetNodeUidAnimationPayload : MorphTargetNodeAnimationPayloads)
		{
			FString PayloadKey = MorphTargetNodeUidAnimationPayload.Value.UniqueId;
			if (PayloadKey.Len() == 0)
			{
				continue;
			}
			if (const UInterchangeMeshNode* MorphTargetNode = Cast<UInterchangeMeshNode>(Arguments.NodeContainer->GetNode(MorphTargetNodeUidAnimationPayload.Key)))
			{
				if (MorphTargetNodeUidAnimationPayload.Value.Type == EInterchangeAnimationPayLoadType::MORPHTARGETCURVEWEIGHTINSTANCE)
				{
					TOptional<UE::Interchange::FAnimationPayloadData>& Result = MorphTargetCurveWeightInstanceAnimationPayloads.FindOrAdd(PayloadKey);
					UE::Interchange::FAnimationPayloadData AnimationPayLoadData(MorphTargetNodeUidAnimationPayload.Key, MorphTargetNodeUidAnimationPayload.Value);
					TArray<FString> PayLoadKeys;
					MorphTargetNodeUidAnimationPayload.Value.UniqueId.ParseIntoArray(PayLoadKeys, TEXT(":"));
					if (PayLoadKeys.Num() == 2)
					{
						float Weight;
						LexFromString(Weight, *PayLoadKeys[1]);

						AnimationPayLoadData.Curves.SetNum(1);
						FRichCurve& Curve = AnimationPayLoadData.Curves[0];
						Curve.AddKey(0, Weight);

						Result.Emplace(AnimationPayLoadData);
					}
				}
				else
				{
					PayloadQueries.Add(UE::Interchange::FAnimationPayloadQuery(MorphTargetNodeUidAnimationPayload.Key, MorphTargetNodeUidAnimationPayload.Value));
				}
				MorphTargetData.CurveNodeNamePerPayloadKey.Add(PayloadKey, MorphTargetNode->GetDisplayLabel());
			}
		}

		TArray<UE::Interchange::FAnimationPayloadData> AnimationCurvesPayloads = AnimSequenceTranslatorPayloadInterface->GetAnimationPayloadData(PayloadQueries);
		for (UE::Interchange::FAnimationPayloadData& AnimationCurvePayload : AnimationCurvesPayloads)
		{
			FString PayloadKey = AnimationCurvePayload.PayloadKey.UniqueId; //in this instance its the payload's uniqueid
			MorphTargetData.CurvesPayloads.Add(PayloadKey, MoveTemp(AnimationCurvePayload));
		}
		AnimationCurvesPayloads.Empty();

		for (TPair<FString, TOptional<UE::Interchange::FAnimationPayloadData>>& CurveNameAndPayload : MorphTargetCurveWeightInstanceAnimationPayloads)
		{
			TOptional<UE::Interchange::FAnimationPayloadData> AnimationCurvePayload = CurveNameAndPayload.Value;
			if (!AnimationCurvePayload.IsSet())
			{
				UE_LOGF(LogInterchangeImport, Warning, "Invalid animation morph target curve payload key [%ls] in AnimSequence asset %ls.", *CurveNameAndPayload.Key, *Arguments.AssetName);
				continue;
			}
			UE::Interchange::FAnimationPayloadData& CurvePayload = AnimationCurvePayload.GetValue();
			MorphTargetData.CurvesPayloads.Add(CurveNameAndPayload.Key, MoveTemp(CurvePayload));
		}
		MorphTargetCurveWeightInstanceAnimationPayloads.Empty();
	}

	//Getting the file Hash will cache it into the source data
	Arguments.SourceData->GetFileContentHash();
#endif

	return ImportAssetResult;
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeAnimSequenceFactory::EndImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeAnimSequenceFactory::ImportAsset_Async);

	UInterchangeFactoryBase::FImportAssetResult ImportAssetResult;
#if WITH_EDITOR && WITH_EDITORONLY_DATA
	if (!AnimSequence)
	{
		return ImportAssetResult;
	}

	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	UInterchangeAnimSequenceFactoryNode* AnimSequenceFactoryNode = Cast<UInterchangeAnimSequenceFactoryNode>(Arguments.AssetNode);
	if (AnimSequenceFactoryNode == nullptr)
	{
		return ImportAssetResult;
	}

	FString SkeletonUid;
	if (!AnimSequenceFactoryNode->GetCustomSkeletonFactoryNodeUid(SkeletonUid))
	{
		//Do not create a empty anim sequence, we need skeleton that contain animation
		return ImportAssetResult;
	}

	const UInterchangeSkeletonFactoryNode* SkeletonFactoryNode = Cast<UInterchangeSkeletonFactoryNode>(Arguments.NodeContainer->GetNode(SkeletonUid));
	if (!SkeletonFactoryNode)
	{
		UE_LOGF(LogInterchangeImport, Warning, "Invalid skeleton factory node. The skeleton factory node is obligatory to import AnimSequence [%ls].", *Arguments.AssetName);
		return ImportAssetResult;
	}

	FSoftObjectPath SkeletonFactoryNodeReferenceObject;
	SkeletonFactoryNode->GetCustomReferenceObject(SkeletonFactoryNodeReferenceObject);

	USkeleton* Skeleton = nullptr;

	FSoftObjectPath SpecifiedSkeleton;
	AnimSequenceFactoryNode->GetCustomSkeletonSoftObjectPath(SpecifiedSkeleton);
	if (Skeleton == nullptr)
	{
		UObject* SkeletonObject = nullptr;

		if (SpecifiedSkeleton.IsValid())
		{
			SkeletonObject = SpecifiedSkeleton.TryLoad();
		}
		else if (SkeletonFactoryNodeReferenceObject.IsValid())
		{
			SkeletonObject = SkeletonFactoryNodeReferenceObject.TryLoad();
		}

		if (SkeletonObject)
		{
			Skeleton = Cast<USkeleton>(SkeletonObject);

		}
		else if (AnimSequence)
		{
			Skeleton = AnimSequence->GetSkeleton();
		}

		if (!ensure(Skeleton))
		{
			UE_LOGF(LogInterchangeImport, Warning, "Invalid Skeleton when importing animation sequence asset %ls.", *Arguments.AssetName);
			return ImportAssetResult;
		}
	}

	const IInterchangeAnimationPayloadInterface* AnimSequenceTranslatorPayloadInterface = Cast<IInterchangeAnimationPayloadInterface>(Arguments.Translator);
	if (!AnimSequenceTranslatorPayloadInterface)
	{
		UE_LOGF(LogInterchangeImport, Error, "Cannot import AnimSequence. The translator does not implement IInterchangeAnimationPayloadInterface.");
		return ImportAssetResult;
	}

	const bool bIsReImport = (Arguments.ReimportObject != nullptr);

	//Fill the animsequence data, we need to retrieve the skeleton and then ask the payload for every joint
	{
		FFrameRate FrameRate(30, 1);
		double SampleRate = 30.0;

		bool bImportBoneTracks = false;
		if (AnimSequenceFactoryNode->GetCustomImportBoneTracks(bImportBoneTracks) && bImportBoneTracks)
		{
			if (AnimSequenceFactoryNode->GetCustomImportBoneTracksSampleRate(SampleRate))
			{
				FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(SampleRate);
			}
		}
		const bool bShouldTransact = bIsReImport;
		IAnimationDataController& Controller = AnimSequence->GetController();
		Controller.OpenBracket(NSLOCTEXT("InterchangeAnimSequenceFactory", "ImportAnimationInterchange_Bracket", "Importing Animation (Interchange)"), bShouldTransact);
		Controller.InitializeModel();
		
		IAnimationDataModel::FReimportScope ReimportScope(AnimSequence->GetDataModel());	
		AnimSequence->ImportFileFramerate = SampleRate;
		AnimSequence->ImportResampleFramerate = SampleRate;
		Controller.SetFrameRate(FrameRate, bShouldTransact);

		TArray<FString> CurvesNotFound;
		UE::Interchange::Private::RetrieveAnimationPayloads(AnimSequence
			, BoneTrackData
			, MorphTargetData
			, AnimSequenceFactoryNode
			, Arguments.NodeContainer
			, SkeletonFactoryNode
			, AnimSequenceTranslatorPayloadInterface
			, Arguments.AssetName
			, bIsReImport
			, CurvesNotFound
			, this);

		if (CurvesNotFound.Num())
		{
			for (const FString& CurveName : CurvesNotFound)
			{
				//This is only a verbose log
				UE_LOGF(LogInterchangeImport, Verbose, "Curve (%ls) was not found in the new Animation", *CurveName);
			}
		}
		Controller.NotifyPopulated();
		Controller.CloseBracket(bShouldTransact);
	}

	if (!bIsReImport)
	{
		/** Apply all AnimSequenceFactoryNode custom attributes to the anim sequence asset */
		AnimSequenceFactoryNode->ApplyAllCustomAttributeToObject(AnimSequence);
	}
	else
	{
		//Apply the re import strategy 
		UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(AnimSequence->AssetImportData);
		UInterchangeFactoryBaseNode* PreviousNode = nullptr;
		if (InterchangeAssetImportData)
		{
			PreviousNode = InterchangeAssetImportData->GetStoredFactoryNode(InterchangeAssetImportData->NodeUniqueID);
		}
		UInterchangeFactoryBaseNode* CurrentNode = UInterchangeFactoryBaseNode::DuplicateWithObject(AnimSequenceFactoryNode, AnimSequence);
		UE::Interchange::FFactoryCommon::ApplyReimportStrategyToAsset(AnimSequence, PreviousNode, CurrentNode, AnimSequenceFactoryNode);
	}
#endif

	return ImportAssetResult;
}

/* This function is call in the completion task on the main thread, use it to call main thread post creation step for your assets*/
void UInterchangeAnimSequenceFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeAnimSequenceFactory::SetupObject_GameThread);

	check(IsInGameThread());
	Super::SetupObject_GameThread(Arguments);

	// TODO: make sure this works at runtime
#if WITH_EDITORONLY_DATA
	if (ensure(Arguments.ImportedObject && Arguments.SourceData))
	{
		// We must call the Update of the asset source file in the main thread because UAssetImportData::Update execute some delegate we do not control
		check(AnimSequence == CastChecked<UAnimSequence>(Arguments.ImportedObject));

		UAssetImportData* ImportDataPtr = AnimSequence->AssetImportData;
		UE::Interchange::FFactoryCommon::FUpdateImportAssetDataParameters UpdateImportAssetDataParameters(AnimSequence, ImportDataPtr, Arguments.SourceData, Arguments.NodeUniqueID, Arguments.NodeContainer, Arguments.OriginalPipelines, Arguments.Translator);
		ImportDataPtr = UE::Interchange::FFactoryCommon::UpdateImportAssetData(UpdateImportAssetDataParameters);
		AnimSequence->AssetImportData = ImportDataPtr;
	}
#endif
}

void UInterchangeAnimSequenceFactory::BuildObject_GameThread(const FSetupObjectParams& Arguments, bool& OutPostEditchangeCalled)
{
	check(IsInGameThread());
	OutPostEditchangeCalled = false;
#if WITH_EDITOR
	if (Arguments.ImportedObject)
	{
		check(AnimSequence == CastChecked<UAnimSequence>(Arguments.ImportedObject));
		if (AnimSequence)
		{
			// @Todo fix me: This is temporary fix to make sure they always have compressed data
			if (AnimSequence->IsDataModelValid() && AnimSequence->IsCompressedDataOutOfDate())
			{
				AnimSequence->CacheDerivedDataForCurrentPlatform();
			}
		}
	}
#endif
}

bool UInterchangeAnimSequenceFactory::GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const
{
#if WITH_EDITORONLY_DATA
	if (const UAnimSequence* TempAnimSequence = Cast<UAnimSequence>(Object))
	{
		return UE::Interchange::FFactoryCommon::GetSourceFilenames(TempAnimSequence->AssetImportData.Get(), OutSourceFilenames);
	}
#endif

	return false;
}

bool UInterchangeAnimSequenceFactory::SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const
{
#if WITH_EDITORONLY_DATA
	if (const UAnimSequence* TempAnimSequence = Cast<UAnimSequence>(Object))
	{
		return UE::Interchange::FFactoryCommon::SetSourceFilename(TempAnimSequence->AssetImportData.Get(), SourceFilename, SourceIndex);
	}
#endif

	return false;
}

void UInterchangeAnimSequenceFactory::BackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UAnimSequence* TempAnimSequence = Cast<UAnimSequence>(Object))
	{
		UE::Interchange::FFactoryCommon::BackupSourceData(TempAnimSequence->AssetImportData.Get());
	}
#endif
}

void UInterchangeAnimSequenceFactory::ReinstateSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UAnimSequence* TempAnimSequence = Cast<UAnimSequence>(Object))
	{
		UE::Interchange::FFactoryCommon::ReinstateSourceData(TempAnimSequence->AssetImportData.Get());
	}
#endif
}

void UInterchangeAnimSequenceFactory::ClearBackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UAnimSequence* TempAnimSequence = Cast<UAnimSequence>(Object))
	{
		UE::Interchange::FFactoryCommon::ClearBackupSourceData(TempAnimSequence->AssetImportData.Get());
	}
#endif
}

bool UInterchangeAnimSequenceFactory::IsBoneTrackAnimationValid(const UInterchangeAnimSequenceFactoryNode* AnimSequenceFactoryNode, const FImportAssetObjectParams& Arguments)
{
	bool bResult = true;
	FFrameRate FrameRate(30, 1);
	double SampleRate = 30.0;

	bool bImportBoneTracks = false;
	if (AnimSequenceFactoryNode->GetCustomImportBoneTracks(bImportBoneTracks) && bImportBoneTracks)
	{
		if (AnimSequenceFactoryNode->GetCustomImportBoneTracksSampleRate(SampleRate))
		{
			FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(SampleRate);
		}

		double RangeStart = 0.0;
		AnimSequenceFactoryNode->GetCustomImportBoneTracksRangeStart(RangeStart);

		double RangeEnd = 1.0 / SampleRate; //One frame duration per default
		AnimSequenceFactoryNode->GetCustomImportBoneTracksRangeStop(RangeEnd);

		const double SequenceLength = FMath::Max<double>(RangeEnd - RangeStart, MINIMUM_ANIMATION_LENGTH);

		const float SubFrame = FrameRate.AsFrameTime(SequenceLength).GetSubFrame();

		if (!FMath::IsNearlyZero(SubFrame, KINDA_SMALL_NUMBER) && !FMath::IsNearlyEqual(SubFrame, 1.0f, KINDA_SMALL_NUMBER))
		{
			UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
			Message->SourceAssetName = Arguments.SourceData->GetFilename();
			Message->DestinationAssetName = Arguments.AssetName;
			Message->AssetType = UAnimSequence::StaticClass();
			Message->Text = FText::Format(NSLOCTEXT("UInterchangeAnimSequenceFactory", "WrongSequenceLength", "Animation length {0} is not compatible with import frame-rate {1} (sub frame {2}). The animation must be frame-border aligned."),
				FText::AsNumber(SequenceLength), FrameRate.ToPrettyText(), FText::AsNumber(SubFrame));
			bResult = false;
		}
	}
	return bResult;
}

