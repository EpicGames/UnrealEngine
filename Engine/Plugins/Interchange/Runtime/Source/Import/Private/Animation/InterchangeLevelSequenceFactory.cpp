// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Animation/InterchangeLevelSequenceFactory.h"

#include "Animation/InterchangeAnimationPayloadInterface.h"
#include "Animation/InterchangeLevelSequenceHelper.h"
#include "InterchangeAnimationTrackSetNode.h"
#include "InterchangeAnimationDefinitions.h"
#include "InterchangeAnimSequenceFactoryNode.h"
#include "InterchangeImportCommon.h"
#include "InterchangeLevelSequenceFactoryNode.h"
#include "InterchangeResult.h"
#include "InterchangeSourceData.h"
#include "InterchangeTranslatorBase.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneObjectPathChannel.h"
#include "LevelSequence.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieSceneSubTrack.h"

#if WITH_EDITORONLY_DATA

#include "EditorFramework/AssetImportData.h"

#endif //WITH_EDITORONLY_DATA

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeLevelSequenceFactory)

#define LOCTEXT_NAMESPACE "InterchangeLevelSequenceFactory"

#if WITH_EDITOR
namespace UE::Interchange::Private
{
	AActor* GetActor(const UInterchangeBaseNodeContainer* NodeContainer, const UInterchangeAnimationTrackNode* TrackNode)
	{
		AActor* Actor = nullptr;

		FString ActorNodeUid;
		if (TrackNode->GetCustomActorDependencyUid(ActorNodeUid))
		{
			const FString ActorFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(ActorNodeUid);
			const UInterchangeFactoryBaseNode* ActorFactoryNode = Cast<UInterchangeFactoryBaseNode>(NodeContainer->GetNode(ActorFactoryNodeUid));

			if (ActorFactoryNode)
			{
				FSoftObjectPath ReferenceObject;
				ActorFactoryNode->GetCustomReferenceObject(ReferenceObject);
				if (ReferenceObject.IsValid())
				{
					Actor = Cast<AActor>(ReferenceObject.TryLoad());
				}
			}
		}

		return Actor;
	}

	bool HasActorToUse(const UInterchangeBaseNodeContainer* NodeContainer, const UInterchangeLevelSequenceFactoryNode* FactoryNode)
	{

		TArray<FString> AnimationTrackUids;
		FactoryNode->GetCustomAnimationTrackUids(AnimationTrackUids);

		for (const FString& AnimationTrackUid : AnimationTrackUids)
		{
			if (const UInterchangeBaseNode* TranslatedNode = NodeContainer->GetNode(AnimationTrackUid))
			{
				if (const UInterchangeTransformAnimationTrackNode* TransformTrackNode = Cast<UInterchangeTransformAnimationTrackNode>(TranslatedNode))
				{
					AActor* Actor = GetActor(NodeContainer, TransformTrackNode);
					if (Actor)
					{
						return true;
					}
				}
				else if (const UInterchangeAnimationTrackSetInstanceNode* InstanceTrackNode = Cast<UInterchangeAnimationTrackSetInstanceNode>(TranslatedNode))
				{
					FString TrackSetNodeUid;
					if (!InstanceTrackNode->GetCustomTrackSetDependencyUid(TrackSetNodeUid))
					{
						continue;
					}

					const FString TrackSetFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TrackSetNodeUid);
					const UInterchangeLevelSequenceFactoryNode* InstanceFactoryNode = Cast<UInterchangeLevelSequenceFactoryNode>(NodeContainer->GetNode(TrackSetFactoryNodeUid));

					if (!InstanceFactoryNode)
					{
						continue;
					}
					FSoftObjectPath ReferenceObject;
					InstanceFactoryNode->GetCustomReferenceObject(ReferenceObject);
					if (!ReferenceObject.IsValid())
					{
						continue;
					}

					return true;
				}
				else if (const UInterchangeAnimationTrackNode* TrackNode = Cast<UInterchangeAnimationTrackNode>(TranslatedNode))
				{
					if (GetActor(NodeContainer, TrackNode))
					{
						return true;
					}
				}
			}
		}

		return false;
	}

	class FLevelSequenceHelper
	{
	public:
		FLevelSequenceHelper(ULevelSequence& InLevelSequence, UInterchangeLevelSequenceFactoryNode& InFactoryNode, const UInterchangeBaseNodeContainer& InNodeContainer, const IInterchangeAnimationPayloadInterface& InPayloadInterface)
			: LevelSequence(InLevelSequence)
			, MovieScene(InLevelSequence.MovieScene)
			, FactoryNode(InFactoryNode)
			, NodeContainer(InNodeContainer)
			, PayloadInterface(InPayloadInterface)
		{
		}

		void PopulateLevelSequence();

	private:
		void PopulateTransformTrack(const UInterchangeTransformAnimationTrackNode& TransformTrackNode);
		void PopulateSubsequenceTrack(const UInterchangeAnimationTrackSetInstanceNode& InstanceNode);
		void PopulateAnimationTrack(const UInterchangeAnimationTrackNode& AnimationTrackNode);

		AActor* GetActor(const UInterchangeAnimationTrackNode& TrackNode);
		FGuid BindActorToLevelSequence(AActor* Actor);
		FGuid BindComponentToLevelSequence(AActor* Actor);
		void UpdateTransformChannels(TArrayView<FMovieSceneDoubleChannel*>& Channels, int32 IndexOffset, const TArray<FRichCurve>& Curves, const FFrameNumber& FrameDuration);
		
		template<typename ChannelType, typename ValueType> 
		void UpdateStepChannel(ChannelType& Channel, const TArray<float>& KeyTimes, const TArray<ValueType>& Values, const FFrameNumber & FrameDuration)
		{
			const FFrameRate& FrameRate = MovieScene->GetTickResolution();

			auto Data = Channel.GetData();

			Data.Reset();

			TRange<FFrameNumber> MinMaxFrameNumber{ TRange<FFrameNumber>::Empty() };

			for (int32 KeyIndex = 0; KeyIndex < KeyTimes.Num(); ++KeyIndex)
			{
				FFrameNumber FrameNumber = FrameRate.AsFrameNumber(KeyTimes[KeyIndex]);

				if (MinMaxFrameNumber.IsEmpty())
				{
					MinMaxFrameNumber = TRange<FFrameNumber>{ FrameNumber };
				}
				else if (FrameNumber < MinMaxFrameNumber.GetLowerBoundValue())
				{
					MinMaxFrameNumber.SetLowerBoundValue(FrameNumber);
				}
				else if (FrameNumber > MinMaxFrameNumber.GetUpperBoundValue())
				{
					MinMaxFrameNumber.SetUpperBoundValue(FrameNumber);
				}

				if constexpr(std::is_same_v<ChannelType, FMovieSceneObjectPathChannel>)
				{
					Data.AddKey(FrameNumber, FSoftObjectPath{ Values[KeyIndex] }.TryLoad());
				}
				else
				{
					Data.AddKey(FrameNumber, Values[KeyIndex]);
				}
			}

			if (!MinMaxFrameNumber.IsEmpty())
			{
				MinMaxFrameNumber.SetUpperBound(TRangeBound<FFrameNumber>::Exclusive(MinMaxFrameNumber.GetUpperBoundValue() + FrameDuration));
				PlaybackRange = TRange<FFrameNumber>::Hull(PlaybackRange, MinMaxFrameNumber);
			}
		}

	private:
		ULevelSequence& LevelSequence;
		UMovieScene* MovieScene = nullptr;
		UInterchangeLevelSequenceFactoryNode& FactoryNode;
		const UInterchangeBaseNodeContainer& NodeContainer;
		const IInterchangeAnimationPayloadInterface& PayloadInterface;

		TRange<FFrameNumber> PlaybackRange = TRange<FFrameNumber>::Empty(); // [StartFrame, EndFrame[ in TickResolution
		bool ClearSubsequenceTrack = true;
	};

	void FLevelSequenceHelper::PopulateLevelSequence()
	{
		if (!MovieScene || FactoryNode.GetCustomAnimationTrackUidCount() == 0)
		{
			return;
		}

		float FrameRate;
		if (FactoryNode.GetCustomFrameRate(FrameRate))
		{
			if (FrameRate <= 0.f)
			{
				FrameRate = 30.f;
			}

			MovieScene->SetDisplayRate(UE::Interchange::Animation::ConvertSampleRateToFrameRate(FrameRate));
		}
		// Set 30 FPS as the default frame rate
		else
		{
			MovieScene->SetDisplayRate(UE::Interchange::Animation::ConvertSampleRateToFrameRate(30.f));
		}

		TArray<FString> AnimationTrackUids;
		FactoryNode.GetCustomAnimationTrackUids(AnimationTrackUids);

		for (const FString& AnimationTrackUid : AnimationTrackUids)
		{
			if (const UInterchangeBaseNode* TranslatedNode = NodeContainer.GetNode(AnimationTrackUid))
			{
				if (const UInterchangeTransformAnimationTrackNode* TransformTrackNode = Cast<UInterchangeTransformAnimationTrackNode>(TranslatedNode))
				{
					PopulateTransformTrack(*TransformTrackNode);
				}
				else if (const UInterchangeAnimationTrackSetInstanceNode* InstanceTrackNode = Cast<UInterchangeAnimationTrackSetInstanceNode>(TranslatedNode))
				{
					PopulateSubsequenceTrack(*InstanceTrackNode);
				}
				else if (const UInterchangeAnimationTrackNode* TrackNode = Cast<UInterchangeAnimationTrackNode>(TranslatedNode))
				{
					PopulateAnimationTrack(*TrackNode);
				}
			}
		}

		LevelSequence.MovieScene->SetPlaybackRange(PlaybackRange.IsEmpty() ? TRange<FFrameNumber>(0, 1) : PlaybackRange);
		LevelSequence.MovieScene->SetEvaluationType(EMovieSceneEvaluationType::FrameLocked);
	}

	void FLevelSequenceHelper::PopulateTransformTrack(const UInterchangeTransformAnimationTrackNode& TransformTrackNode)
	{
		// Get targeted actor exists
		AActor* Actor = GetActor(TransformTrackNode);
		FString SceneNodeUid;

		if (!Actor || !TransformTrackNode.GetCustomActorDependencyUid(SceneNodeUid))
		{
			UE_LOGF(LogInterchangeImport, Warning, "Cannot find actor for animation track %ls", *TransformTrackNode.GetDisplayLabel());
			return;
		}

		// Get payload
		FInterchangeAnimationPayLoadKey PayloadKey;
		if (!TransformTrackNode.GetCustomAnimationPayloadKey(PayloadKey))
		{
			UE_LOGF(LogInterchangeImport, Warning, "No payload key for animation track %ls on actor %ls", *TransformTrackNode.GetDisplayLabel(), *Actor->GetActorLabel());
			return;
		}

		TArray<UE::Interchange::FAnimationPayloadData> PayloadDataArray = PayloadInterface.GetAnimationPayloadData({ FAnimationPayloadQuery(SceneNodeUid, PayloadKey) });
		
		if (PayloadDataArray.Num() != 1 || PayloadDataArray[0].Curves.Num() != 9)
		{
			UE_LOGF(LogInterchangeImport, Warning, "No payload for animation track %ls on actor %ls", *TransformTrackNode.GetDisplayLabel(), *Actor->GetActorLabel());
			return;
		}

		UE::Interchange::FAnimationPayloadData* PayloadData = &PayloadDataArray[0];

		FGuid ObjectBinding = BindActorToLevelSequence(Actor);

		UMovieScene3DTransformTrack* TransformTrack = MovieScene->FindTrack<UMovieScene3DTransformTrack>(ObjectBinding);
		if (!TransformTrack)
		{
			TransformTrack = MovieScene->AddTrack<UMovieScene3DTransformTrack>(ObjectBinding);
		}
		else
		{
			TransformTrack->RemoveAllAnimationData();
		}

		if (!TransformTrack)
		{
			return;
		}

		bool bSectionAdded = false;
		UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(TransformTrack->FindOrAddSection(0, bSectionAdded));
		if (!TransformSection)
		{
			return;
		}

		if (bSectionAdded)
		{
			int32 CompletionMode;
			if (TransformTrackNode.GetCustomCompletionMode(CompletionMode))
			{
				// Make sure EMovieSceneCompletionMode enum value are still between 0 and 2
				static_assert((uint8)EInterchangeAimationCompletionMode::KeepState == (uint8)EMovieSceneCompletionMode::KeepState, "ENUM_VALUE_HAS_CHANGED");
				static_assert((uint8)EInterchangeAimationCompletionMode::RestoreState == (uint8)EMovieSceneCompletionMode::RestoreState, "ENUM_VALUE_HAS_CHANGED");
				static_assert((uint8)EInterchangeAimationCompletionMode::ProjectDefault == (uint8)EMovieSceneCompletionMode::ProjectDefault, "ENUM_VALUE_HAS_CHANGED");

				TransformSection->EvalOptions.CompletionMode = (EMovieSceneCompletionMode)CompletionMode;
			}
			// By default the completion mode is EMovieSceneCompletionMode::ProjectDefault
			else
			{
				TransformSection->EvalOptions.CompletionMode = EMovieSceneCompletionMode::ProjectDefault;
			}

			TransformSection->SetRange(TRange<FFrameNumber>::All());
		}

		const FFrameRate FrameRate = MovieScene->GetDisplayRate();
		const FFrameRate TickResolution = MovieScene->GetTickResolution();

		TArrayView<FMovieSceneDoubleChannel*> Channels = TransformSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();

		const FFrameNumber FrameDuration = FFrameRate::TransformTime(1, MovieScene->GetDisplayRate(), MovieScene->GetTickResolution()).RoundToFrame();
		UpdateTransformChannels(Channels, 0, PayloadData->Curves, FrameDuration); // Translation
		UpdateTransformChannels(Channels, 3, PayloadData->Curves, FrameDuration); // Rotation
		UpdateTransformChannels(Channels, 6, PayloadData->Curves, FrameDuration); // Scaling

		// Remove unnecessary keys
		FKeyDataOptimizationParams OptimParams;
		OptimParams.DisplayRate = MovieScene->GetDisplayRate();
		for (FMovieSceneDoubleChannel* Channel : Channels)
		{
			Channel->Optimize(OptimParams);
		}


		int32 EnabledTransformChannels;
		if (TransformTrackNode.GetCustomUsedChannels(EnabledTransformChannels))
		{
			TransformSection->SetMask((EMovieSceneTransformChannel)EnabledTransformChannels);
		}
		// By default all channels are enabled
		else
		{
			TransformSection->SetMask(EMovieSceneTransformChannel::AllTransform);
		}

		if (USceneComponent* SceneComp = Actor->GetRootComponent())
		{
			SceneComp->SetMobility(EComponentMobility::Movable);
		}
	}

	template <class T>
	void ConvertRichCurveKeyToFloatValue(const FRichCurveKey& RichCurveKey, T& OutMovieSceneKey, double TangentRatio /*= 1.0*/, double SecondsPerFrame /*= 1.0*/)
	{
		OutMovieSceneKey.Value = RichCurveKey.Value;

		OutMovieSceneKey.Tangent.TangentWeightMode = RichCurveKey.TangentWeightMode;
		if (OutMovieSceneKey.Tangent.TangentWeightMode != RCTWM_WeightedNone &&
			OutMovieSceneKey.Tangent.TangentWeightMode != RCTWM_WeightedArrive &&
			OutMovieSceneKey.Tangent.TangentWeightMode != RCTWM_WeightedLeave &&
			OutMovieSceneKey.Tangent.TangentWeightMode != RCTWM_WeightedBoth
			)
		{
			OutMovieSceneKey.Tangent.TangentWeightMode = RCTWM_WeightedNone;
		}

		OutMovieSceneKey.Tangent.ArriveTangentWeight = RichCurveKey.ArriveTangentWeight;
		OutMovieSceneKey.Tangent.LeaveTangentWeight = RichCurveKey.LeaveTangentWeight;

		if (OutMovieSceneKey.Tangent.TangentWeightMode == RCTWM_WeightedNone)
		{
			OutMovieSceneKey.Tangent.ArriveTangent = static_cast<float>(RichCurveKey.ArriveTangent * TangentRatio);
			OutMovieSceneKey.Tangent.LeaveTangent = static_cast<float>(RichCurveKey.LeaveTangent * TangentRatio);
		}
		else
		{
			OutMovieSceneKey.Tangent.ArriveTangent = static_cast<float>(RichCurveKey.ArriveTangent * SecondsPerFrame);
			OutMovieSceneKey.Tangent.LeaveTangent = static_cast<float>(RichCurveKey.LeaveTangent * SecondsPerFrame);
		}

		OutMovieSceneKey.TangentMode = RichCurveKey.TangentMode;
		OutMovieSceneKey.InterpMode = RichCurveKey.InterpMode;
	}

	template <class T>
	void ProcessRichCurveKeys(const FFrameRate& TargetFrameRate, const FRichCurve& Curve, FFrameNumber FrameDuration, TArray<FFrameNumber>& OutFrameNumbers, TArray<T>& OutValues, TRange<FFrameNumber>& PlaybackRange)
	{
		const TArray<FRichCurveKey>& CurveKeys = Curve.GetConstRefOfKeys();
		const int32 NumCurveKeys = CurveKeys.Num();
		TRange<FFrameNumber> MinMaxFrameNumber{ TRange<FFrameNumber>::Empty() };

		OutFrameNumbers.Reserve(NumCurveKeys);
		OutValues.Reserve(NumCurveKeys);

		for (int32 KeyIndex = 0; KeyIndex < CurveKeys.Num(); ++KeyIndex)
		{
			const FRichCurveKey* PrevKey = KeyIndex > 0 ? &CurveKeys[KeyIndex - 1] : nullptr;
			const FRichCurveKey* NextKey = KeyIndex < (NumCurveKeys - 1) ? &CurveKeys[KeyIndex + 1] : nullptr;
			const FRichCurveKey& RichCurveKey = CurveKeys[KeyIndex];

			FFrameNumber& FrameNumber = OutFrameNumbers.Add_GetRef(TargetFrameRate.AsFrameNumber(RichCurveKey.Time));

			if (MinMaxFrameNumber.IsEmpty())
			{
				MinMaxFrameNumber = TRange<FFrameNumber>{ FrameNumber };
			}
			else if (FrameNumber < MinMaxFrameNumber.GetLowerBoundValue())
			{
				MinMaxFrameNumber.SetLowerBoundValue(FrameNumber);
			}
			else if (FrameNumber > MinMaxFrameNumber.GetUpperBoundValue())
			{
				MinMaxFrameNumber.SetUpperBoundValue(FrameNumber);
			}

			const float SecondsDelta = [&]() -> float
				{
					if (PrevKey && NextKey)
					{
						return NextKey->Time - PrevKey->Time;
					}
					else if (PrevKey)
					{
						return RichCurveKey.Time - PrevKey->Time;
					}
					else if (NextKey)
					{
						return NextKey->Time - RichCurveKey.Time;
					}

					return 1.f;
				}();

			const int32 FrameNumberDelta = [&]() -> int32
				{
					if (PrevKey && NextKey)
					{
						return TargetFrameRate.AsFrameTime(NextKey->Time).RoundToFrame().Value - TargetFrameRate.AsFrameTime(PrevKey->Time).RoundToFrame().Value;
					}
					else if (PrevKey)
					{
						return TargetFrameRate.AsFrameTime(RichCurveKey.Time).RoundToFrame().Value - TargetFrameRate.AsFrameTime(PrevKey->Time).RoundToFrame().Value;
					}
					else if (NextKey)
					{
						return TargetFrameRate.AsFrameTime(NextKey->Time).RoundToFrame().Value - TargetFrameRate.AsFrameTime(RichCurveKey.Time).RoundToFrame().Value;
					}

					return 1;
				}();

			T& Value = OutValues.AddDefaulted_GetRef();

			// Ratio between rich-curve and moviescene key(s) timing, if there are any surrounding keys (otherwise default to a ratio of 1:1)
			const double KeyTimingRatio = (PrevKey || NextKey) ? SecondsDelta / FrameNumberDelta : 1.0;

			ConvertRichCurveKeyToFloatValue(RichCurveKey, Value, KeyTimingRatio, TargetFrameRate.AsInterval());
		}

		if (!MinMaxFrameNumber.IsEmpty())
		{
			MinMaxFrameNumber.SetUpperBound(TRangeBound<FFrameNumber>::Exclusive(MinMaxFrameNumber.GetUpperBoundValue() + FrameDuration));
			PlaybackRange = TRange<FFrameNumber>::Hull(PlaybackRange, MinMaxFrameNumber);
		}
	}

	void FLevelSequenceHelper::PopulateSubsequenceTrack(const UInterchangeAnimationTrackSetInstanceNode& InstanceNode)
	{
		FString TrackSetNodeUid;
		if (!InstanceNode.GetCustomTrackSetDependencyUid(TrackSetNodeUid))
		{
			UE_LOGF(LogInterchangeImport, Error, "No unique id specified for the animation track set referenced by animation track %ls.", *InstanceNode.GetDisplayLabel());
			return;
		}

		const FString TrackSetFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TrackSetNodeUid);
		const UInterchangeLevelSequenceFactoryNode* InstanceFactoryNode = Cast<UInterchangeLevelSequenceFactoryNode>(NodeContainer.GetNode(TrackSetFactoryNodeUid));
		
		FString InstanceNodeDisplayLabel = InstanceNode.GetDisplayLabel();
		auto LogMissingTrackError = [&InstanceNodeDisplayLabel]()
		{
			UE_LOGF(LogInterchangeImport, Error, "Cannot find factory of animation track set referenced by animation track %ls.", *InstanceNodeDisplayLabel);
		};

		if (!InstanceFactoryNode)
		{
			LogMissingTrackError();
			return;
		}
		FSoftObjectPath ReferenceObject;
		InstanceFactoryNode->GetCustomReferenceObject(ReferenceObject);
		if (!ReferenceObject.IsValid())
		{
			LogMissingTrackError();
			return;
		}
		UMovieSceneSequence* TargetMovieSceneSequence = CastChecked<UMovieSceneSequence>(ReferenceObject.TryLoad());

		// Create SubTrack
		UMovieSceneSubTrack* SubTrack = MovieScene->FindTrack<UMovieSceneSubTrack>();
		if (!SubTrack)
		{
			SubTrack = MovieScene->AddTrack<UMovieSceneSubTrack>();
		}
		else if (SubTrack && ClearSubsequenceTrack)
		{
			SubTrack->RemoveAllAnimationData();
		}

		if (!SubTrack)
		{
			return;
		}

		FFrameRate DisplayRate = MovieScene->GetDisplayRate();

		float SrcLowerBound = 0.f;
		float SrcUpperBound = 0.f;
		int32 StartFrame = 0;
		if (InstanceNode.GetCustomStartFrame(StartFrame))
		{
			SrcLowerBound = DisplayRate.AsSeconds(StartFrame);
		}

		int32 Duration = 0;
		if (InstanceNode.GetCustomDuration(Duration))
		{
			SrcUpperBound = DisplayRate.AsSeconds(StartFrame + Duration);
		}

		const FFrameRate TickResolution = MovieScene->GetTickResolution();
		const FFrameNumber DstLowerBound = TickResolution.AsFrameNumber(SrcLowerBound);
		const FFrameNumber DstUpperBound = TickResolution.AsFrameNumber(SrcUpperBound);

		// Internally AddSequenceOnRow will automatically bump overlapping subsequences, so we can just add where it's ideal for us
		UMovieSceneSubSection* NewSection = SubTrack->AddSequenceOnRow(TargetMovieSceneSequence, DstLowerBound, DstUpperBound.Value - DstLowerBound.Value, INDEX_NONE);

		float TimeScale = 1.f;
		InstanceNode.GetCustomTimeScale(TimeScale);
		NewSection->Parameters.TimeScale.Set(TimeScale);
		
		int32 CompletionMode;
		if (InstanceNode.GetCustomCompletionMode(CompletionMode))
		{
			NewSection->EvalOptions.CompletionMode = (EMovieSceneCompletionMode)CompletionMode;
		}
		// By default the completion mode is EMovieSceneCompletionMode::ProjectDefault
		else
		{
			NewSection->EvalOptions.CompletionMode = EMovieSceneCompletionMode::ProjectDefault;
		}

		PlaybackRange = TRange<FFrameNumber>::Hull(PlaybackRange, NewSection->GetRange());

		ClearSubsequenceTrack = false;
	}

	void FLevelSequenceHelper::PopulateAnimationTrack(const UInterchangeAnimationTrackNode& AnimationTrackNode)
	{
		EInterchangePropertyTracks PropertyTrack;
		if(!AnimationTrackNode.GetCustomPropertyTrack(PropertyTrack))
		{
			return;
		}

		// Get targeted actor exists
		AActor* Actor = GetActor(AnimationTrackNode);
		FString SceneNodeUid;

		if (!Actor || !AnimationTrackNode.GetCustomActorDependencyUid(SceneNodeUid))
		{
			UE_LOGF(LogInterchangeImport, Warning, "Cannot find actor for animation track %ls", *AnimationTrackNode.GetDisplayLabel());
			return;
		}

		// Get payload
		FInterchangeAnimationPayLoadKey PayloadKey;
		if (!AnimationTrackNode.GetCustomAnimationPayloadKey(PayloadKey))
		{
			UE_LOGF(LogInterchangeImport, Warning, "No payload key for animation track %ls on actor %ls", *AnimationTrackNode.GetDisplayLabel(), *Actor->GetActorLabel());
			return;
		}

		TArray<UE::Interchange::FAnimationPayloadData> PayloadDataArray = PayloadInterface.GetAnimationPayloadData({ FAnimationPayloadQuery(SceneNodeUid, PayloadKey) });
		
		if (PayloadDataArray.Num() != 1 || (PayloadDataArray[0].StepCurves.IsEmpty() && PayloadDataArray[0].Curves.IsEmpty()))
		{
			UE_LOGF(LogInterchangeImport, Warning, "No payload for animation track %ls on actor %ls", *AnimationTrackNode.GetDisplayLabel(), *Actor->GetActorLabel());
			return;
		}

		UE::Interchange::FAnimationPayloadData* PayloadData = &PayloadDataArray[0];

		FGuid ObjectBinding;
		if(PropertyTrack == EInterchangePropertyTracks::ActorHiddenInGame)
		{
			ObjectBinding = BindActorToLevelSequence(Actor);
		}
		else
		{
			ObjectBinding = BindComponentToLevelSequence(Actor);
		}

		UMovieSceneSection * Section = FInterchangePropertyTracksHelper::GetInstance().GetSection(MovieScene, AnimationTrackNode, ObjectBinding, PropertyTrack);

		if(!Section)
		{
			return;
		}

		const FName DoubleChannelTypeName = FMovieSceneDoubleChannel::StaticStruct()->GetFName();
		const FName FloatChannelTypeName = FMovieSceneFloatChannel::StaticStruct()->GetFName();
		const FName IntegerChannelTypeName = FMovieSceneIntegerChannel::StaticStruct()->GetFName();
		const FName BoolChannelTypeName = FMovieSceneBoolChannel::StaticStruct()->GetFName();
		const FName EnumChannelTypeName = FMovieSceneByteChannel::StaticStruct()->GetFName();
		const FName ObjectPathChannelTypeName = FMovieSceneObjectPathChannel::StaticStruct()->GetFName();

		const FFrameNumber FrameDuration = FFrameRate::TransformTime(1, MovieScene->GetDisplayRate(), MovieScene->GetTickResolution()).RoundToFrame();

		auto CopyToChannel = [this, &FrameDuration](auto Channel, const FRichCurve& Curve)
		{
			TArray<FFrameNumber> FrameNumbers;

			using FMovieSceneValue = typename std::remove_pointer_t<decltype(Channel)>::ChannelValueType;
			TArray<FMovieSceneValue> Values;

			ProcessRichCurveKeys(MovieScene->GetTickResolution(), Curve, FrameDuration, FrameNumbers, Values, PlaybackRange);

			if(!Values.IsEmpty())
			{
				Channel->Set(FrameNumbers, Values);
			}
			else
			{
				Channel->RemoveDefault();
			}
		};

		FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
		TArrayView<const FMovieSceneChannelEntry> ChannelEntries = ChannelProxy.GetAllEntries();

		for(const FMovieSceneChannelEntry& ChannelEntry : ChannelEntries)
		{
			const FName ChannelTypeName = ChannelEntry.GetChannelTypeName();
			const bool bIsBoolChannel = (ChannelTypeName == BoolChannelTypeName);
			const bool bIsEnumChannel = (ChannelTypeName == EnumChannelTypeName);
			const bool bIsIntegerChannel = (ChannelTypeName == IntegerChannelTypeName);
			const bool bIsDoubleChannel = (ChannelTypeName == DoubleChannelTypeName);
			const bool bIsFloatChannel = (ChannelTypeName == FloatChannelTypeName);
			const bool bIsObjectPathChannel = (ChannelTypeName == ObjectPathChannelTypeName);

			if(!bIsBoolChannel &&
			   !bIsEnumChannel &&
			   !bIsIntegerChannel &&
			   !bIsDoubleChannel &&
			   !bIsFloatChannel &&
			   !bIsObjectPathChannel)
			{
				continue;
			}

			TArrayView<FMovieSceneChannel* const> Channels = ChannelEntry.GetChannels();
			int32 NumChannels = (bIsBoolChannel || bIsEnumChannel || bIsIntegerChannel || bIsObjectPathChannel) ? PayloadData->StepCurves.Num() : PayloadData->Curves.Num();
			NumChannels = FMath::Min(NumChannels, Channels.Num());
			for(int32 Index = 0; Index < NumChannels; ++Index)
			{
				FMovieSceneChannelHandle Channel = ChannelProxy.MakeHandle(ChannelTypeName, Index);
				if(bIsBoolChannel)
				{
					UpdateStepChannel(*(Channel.Cast<FMovieSceneBoolChannel>().Get()), PayloadData->StepCurves[Index].KeyTimes, PayloadData->StepCurves[Index].BooleanKeyValues.GetValue(), FrameDuration);
				}
				else if(bIsEnumChannel)
				{
					UpdateStepChannel(*(Channel.Cast<FMovieSceneByteChannel>().Get()), PayloadData->StepCurves[Index].KeyTimes, PayloadData->StepCurves[Index].ByteKeyValues.GetValue(), FrameDuration);
				}
				else if(bIsIntegerChannel)
				{
					UpdateStepChannel(*(Channel.Cast<FMovieSceneIntegerChannel>().Get()), PayloadData->StepCurves[Index].KeyTimes, PayloadData->StepCurves[Index].IntegerKeyValues.GetValue(), FrameDuration);
				}
				else if(bIsObjectPathChannel)
				{
					UpdateStepChannel(*(Channel.Cast<FMovieSceneObjectPathChannel>().Get()), PayloadData->StepCurves[Index].KeyTimes, PayloadData->StepCurves[Index].StringKeyValues.GetValue(), FrameDuration);
				}
				else if(bIsFloatChannel)
				{
					CopyToChannel(Channel.Cast<FMovieSceneFloatChannel>().Get(), PayloadData->Curves[Index]);
				}
				else if(bIsDoubleChannel)
				{
					CopyToChannel(Channel.Cast<FMovieSceneDoubleChannel>().Get(), PayloadData->Curves[Index]);
				}
			}
		}

		if(USceneComponent* SceneComp = Actor->GetRootComponent())
		{
			SceneComp->SetMobility(EComponentMobility::Movable);
		}
	}

	AActor* FLevelSequenceHelper::GetActor(const UInterchangeAnimationTrackNode& TrackNode)
	{
		AActor* Actor = nullptr;

		FString ActorNodeUid;
		if (TrackNode.GetCustomActorDependencyUid(ActorNodeUid))
		{
			const FString ActorFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(ActorNodeUid);
			const UInterchangeFactoryBaseNode* ActorFactoryNode = Cast<UInterchangeFactoryBaseNode>(NodeContainer.GetNode(ActorFactoryNodeUid));

			if (ActorFactoryNode)
			{
				FSoftObjectPath ReferenceObject;
				ActorFactoryNode->GetCustomReferenceObject(ReferenceObject);
				if (ReferenceObject.IsValid())
				{
					Actor = Cast<AActor>(ReferenceObject.TryLoad());
				}
			}
		}

		return Actor;
	}

	FGuid FLevelSequenceHelper::BindActorToLevelSequence(AActor* Actor)
	{
		TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = MovieSceneHelpers::CreateTransientSharedPlaybackState(Actor, &LevelSequence);
		FGuid ActorBinding = LevelSequence.FindBindingFromObject(Actor, SharedPlaybackState);
		if(!ActorBinding.IsValid())
		{
			ActorBinding = MovieScene->AddPossessable(Actor->GetActorLabel(), Actor->GetClass());
			LevelSequence.BindPossessableObject(ActorBinding, *Actor, Actor->GetWorld());
		}

		return ActorBinding;
	}

	FGuid FLevelSequenceHelper::BindComponentToLevelSequence(AActor* Actor)
	{
		TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = MovieSceneHelpers::CreateTransientSharedPlaybackState(Actor, &LevelSequence);
		FGuid ActorBinding = BindActorToLevelSequence(Actor);
		USceneComponent* Component = Actor->GetDefaultAttachComponent();
		FGuid ComponentBinding = LevelSequence.FindBindingFromObject(Component, SharedPlaybackState);
		if(!ComponentBinding.IsValid())
		{
			ComponentBinding = MovieScene->AddPossessable(Component->GetReadableName(), Component->GetClass());

			if(ActorBinding.IsValid() && ComponentBinding.IsValid())
			{
				if(FMovieScenePossessable* ComponentPossessable = MovieScene->FindPossessable(ComponentBinding))
				{
					ComponentPossessable->SetParent(ActorBinding, MovieScene);
				}
			}

			// Bind component
			LevelSequence.BindPossessableObject(ComponentBinding, *Component, Actor);
		}

		return ComponentBinding;
	}

	void FLevelSequenceHelper::UpdateTransformChannels(TArrayView<FMovieSceneDoubleChannel*>& Channels, int32 IndexOffset, const TArray<FRichCurve>& Curves, const FFrameNumber& FrameDuration)
	{
		auto CopyToChannel = [this, &FrameDuration](FMovieSceneDoubleChannel* Channel, const FRichCurve& Curve)
		{
			TArray<FFrameNumber> FrameNumbers;
			TArray<FMovieSceneDoubleValue> Values;

			ProcessRichCurveKeys(MovieScene->GetTickResolution(), Curve, FrameDuration, FrameNumbers, Values, PlaybackRange);

			if (!Values.IsEmpty())
			{
				Channel->Set(FrameNumbers, Values);
			}
			else
			{
				Channel->RemoveDefault();
			}
		};

		CopyToChannel(Channels[IndexOffset + 0], Curves[IndexOffset + 0]);
		CopyToChannel(Channels[IndexOffset + 1], Curves[IndexOffset + 1]);
		CopyToChannel(Channels[IndexOffset + 2], Curves[IndexOffset + 2]);
	}
} //namespace UE::Interchange::Private
#endif //WITH_EDITOR

UClass* UInterchangeLevelSequenceFactory::GetFactoryClass() const
{
	return ULevelSequence::StaticClass();
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeLevelSequenceFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeLevelSequenceFactory::BeginImportAsset_GameThread);

	FImportAssetResult ImportAssetResult;
#if !WITH_EDITOR || !WITH_EDITORONLY_DATA

	UE_LOGF(LogInterchangeImport, Error, "Cannot import levelsequence asset in runtime, this is an editor only feature.");
	return ImportAssetResult;
#else

	auto CannotReimportMessage = [&Arguments, this]()
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName = Arguments.SourceData->GetFilename();
		Message->DestinationAssetName = Arguments.AssetName;
		Message->AssetType = ULevelSequence::StaticClass();
		Message->Text = LOCTEXT("CreateEmptyAssetUnsupportedReimport", "Re-import of ULevelSequence not supported yet.");
		Arguments.AssetNode->SetSkipNodeImport();
	};

	if (Arguments.ReimportObject)
	{
		CannotReimportMessage();
		return ImportAssetResult;
	}

	ULevelSequence* LevelSequence = nullptr;
	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	UInterchangeLevelSequenceFactoryNode* FactoryNode = Cast<UInterchangeLevelSequenceFactoryNode>(Arguments.AssetNode);
	if (FactoryNode == nullptr)
	{
		return ImportAssetResult;
	}

	if (!UE::Interchange::Private::HasActorToUse(Arguments.NodeContainer, FactoryNode))
	{
		UE_LOGF(LogInterchangeImport, Warning, "Level sequence asset, %ls, not imported, because all referenced actors are missing.", *FactoryNode->GetDisplayLabel());
		return ImportAssetResult;
	}

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (FactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}
	}

	// create a new material or overwrite existing asset, if possible
	if (!ExistingAsset)
	{
		LevelSequence = NewObject<ULevelSequence>(Arguments.Parent, *Arguments.AssetName, RF_Public | RF_Standalone);
	}
	else
	{
		// This is a reimport or an override, we are just re-updating the source data

		//TODO: put back the Cast when the LevelSequence will support re-import
		//LevelSequence = Cast<ULevelSequence>(ExistingAsset);
		CannotReimportMessage();
		return ImportAssetResult;
	}

	if (!LevelSequence)
	{
		UE_LOGF(LogInterchangeImport, Warning, "Could not create LevelSequence asset %ls", *Arguments.AssetName);
		return ImportAssetResult;
	}

	FactoryNode->SetCustomReferenceObject(LevelSequence);

	LevelSequence->PreEditChange(nullptr);

	ImportAssetResult.ImportedObject = ImportObjectSourceData(Arguments);
	return ImportAssetResult;
#endif //else !WITH_EDITOR || !WITH_EDITORONLY_DATA
}

UObject* UInterchangeLevelSequenceFactory::ImportObjectSourceData(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeLevelSequenceFactory::ImportObjectSourceData);

#if !WITH_EDITOR || !WITH_EDITORONLY_DATA
	// TODO: Can we import ULevelSequence at runtime
	UE_LOGF(LogInterchangeImport, Error, "Cannot import LevelSequence asset in runtime, this is an editor only feature.");
	return nullptr;

#else
	using namespace UE::Interchange;

	// Re-import is not supported yet
	// Need to talk to the Sequencer team about adding an AssetImportData to ULevelSequence
	if (Arguments.ReimportObject)
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName = Arguments.SourceData->GetFilename();
		Message->DestinationAssetName = Arguments.AssetName;
		Message->AssetType = ULevelSequence::StaticClass();
		Message->Text = LOCTEXT("CreateAssetUnsupportedReimport", "Re-import of ULevelSequence not supported yet.");

		return nullptr;
	}

	if (!Arguments.NodeContainer || !Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return nullptr;
	}

	UInterchangeLevelSequenceFactoryNode* FactoryNode = Cast<UInterchangeLevelSequenceFactoryNode>(Arguments.AssetNode);
	if (!FactoryNode)
	{
		return nullptr;
	}

	Translator = Arguments.Translator;
	const IInterchangeAnimationPayloadInterface* AnimSequenceTranslatorPayloadInterface = Cast<IInterchangeAnimationPayloadInterface>(Arguments.Translator);
	if (!AnimSequenceTranslatorPayloadInterface)
	{
		UE_LOGF(LogInterchangeImport, Error, "Cannot import LevelSequence, the translator do not implement the IInterchangeAnimationPayloadInterface.");
		return nullptr;
	}

	UObject* ExistingAsset = UE::Interchange::FFactoryCommon::AsyncFindObject(FactoryNode, GetFactoryClass(), Arguments.Parent, Arguments.AssetName);

	if (!ExistingAsset)
	{
		UE_LOGF(LogInterchangeImport, Error, "Could not import the LevelSequence asset %ls, because the asset do not exist.", *Arguments.AssetName);
		return nullptr;
	}

	ULevelSequence* LevelSequence = Cast<ULevelSequence>(ExistingAsset);

	if (!ensure(LevelSequence))
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName = Arguments.SourceData->GetFilename();
		Message->DestinationAssetName = Arguments.AssetName;
		Message->AssetType = ULevelSequence::StaticClass();
		Message->Text = FText::Format(LOCTEXT("CreateAssetFailed", "Could not create nor find LevelSequence asset {0}."), FText::FromString(Arguments.AssetName));
		return nullptr;
	}

	LevelSequence->Initialize();

	Private::FLevelSequenceHelper Helper(*LevelSequence, *FactoryNode, *Arguments.NodeContainer, *AnimSequenceTranslatorPayloadInterface);
	Helper.PopulateLevelSequence();

	/** Apply all FactoryNode custom attributes to the level sequence asset */
	FactoryNode->ApplyAllCustomAttributeToObject(LevelSequence);

	//Getting the file Hash will cache it into the source data
	Arguments.SourceData->GetFileContentHash();

	//The interchange completion task (call in the GameThread after the factories pass), will call PostEditChange which will trig another asynchronous system that will build all material in parallel
	return LevelSequence;
#endif //else !WITH_EDITOR || !WITH_EDITORONLY_DATA
}

/* This function is call in the completion task on the main thread, use it to call main thread post creation step for your assets*/
void UInterchangeLevelSequenceFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeLevelSequenceFactory::SetupObject_GameThread);

	check(IsInGameThread());
	Super::SetupObject_GameThread(Arguments);

	// TODO: Talk with sequence team about adding AssetImportData to ULevelSequence for re-import 
}

#undef LOCTEXT_NAMESPACE
