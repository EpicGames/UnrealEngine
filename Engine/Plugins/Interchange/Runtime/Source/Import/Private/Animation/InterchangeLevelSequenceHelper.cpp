// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeLevelSequenceHelper.h"
#include "InterchangeAnimationDefinitions.h"

#include "Camera/CameraComponent.h"
#include "CineCameraComponent.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/HeterogeneousVolumeComponent.h"
#include "Components/LocalLightComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/RectLightComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/SkyLightComponent.h"
#include "Components/SplineComponent.h"
#include "Components/SpotLightComponent.h"
#include "Components/StaticMeshComponent.h"

#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "Tracks/MovieSceneBoolTrack.h"
#include "Tracks/MovieSceneByteTrack.h"
#include "Tracks/MovieSceneColorTrack.h"
#include "Tracks/MovieSceneEnumTrack.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "Tracks/MovieSceneIntegerTrack.h"
#include "Tracks/MovieSceneObjectPropertyTrack.h"
#include "Tracks/MovieSceneVectorTrack.h"
#include "Tracks/MovieSceneVisibilityTrack.h"

namespace UE::Interchange::Private
{
	FInterchangePropertyTracksHelper::FInterchangePropertyTracksHelper()
		: PropertyTracks{
			//Common
			{EInterchangePropertyTracks::AffectDistanceFieldLighting, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bAffectDistanceFieldLighting) , TEXT("Affect Distance Field Lighting")}},
			{EInterchangePropertyTracks::AffectDynamicIndirectLighting, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bAffectDynamicIndirectLighting) , TEXT("Affect Dynamic Indirect Lighting")}},
			{EInterchangePropertyTracks::AffectIndirectLightingWhileHidden, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bAffectIndirectLightingWhileHidden) , TEXT("Affect Indirect Lighting While Hidden")}},
			{EInterchangePropertyTracks::AutoActivate, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UActorComponent, bAutoActivate) , TEXT("Auto Activate")}},
			{EInterchangePropertyTracks::BodyInstanceAngularDamping, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, BodyInstance) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FBodyInstance, AngularDamping), TEXT("Angular Damping (Body Instance)")}},
			{EInterchangePropertyTracks::BodyInstancebEnableGravity, {UMovieSceneByteTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, BodyInstance) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FBodyInstance, bEnableGravity), TEXT("Enable Gravity (Body Instance)")}},
			{EInterchangePropertyTracks::BodyInstancebNotifyRigidBodyCollision, {UMovieSceneByteTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, BodyInstance) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FBodyInstance, bNotifyRigidBodyCollision), TEXT("Simulation Generates Hit Events (Body Instance)")}},
			{EInterchangePropertyTracks::BodyInstancebSimulatePhysics, {UMovieSceneByteTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, BodyInstance) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FBodyInstance, bSimulatePhysics), TEXT("Simulate Physics (Body Instance)")}},
			{EInterchangePropertyTracks::BodyInstancebUpdateKinematicFromSimulation, {UMovieSceneByteTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, BodyInstance) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FBodyInstance, bUpdateKinematicFromSimulation), TEXT("Update Kinematic from Simulation (Body Instance)")}},
			{EInterchangePropertyTracks::BodyInstancebUseCCD, {UMovieSceneByteTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, BodyInstance) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FBodyInstance, bUseCCD), TEXT("Use CCD (Body Instance)")}},
			{EInterchangePropertyTracks::BodyInstanceLinearDamping, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, BodyInstance) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FBodyInstance, LinearDamping), TEXT("Linear Damping (Body Instance)")}},
			{EInterchangePropertyTracks::BodyInstanceMassScale, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, BodyInstance) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FBodyInstance, MassScale), TEXT("Mass Scale (Body Instance)")}},
			{EInterchangePropertyTracks::BoundsScale, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, BoundsScale) , TEXT("Bounds Scale")}},
			{EInterchangePropertyTracks::CastContactShadow, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bCastContactShadow) , TEXT("Contact Shadow")}},
			{EInterchangePropertyTracks::CastHiddenShadow, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bCastHiddenShadow) , TEXT("Hidden Shadow")}},
			{EInterchangePropertyTracks::CastInsetShadow, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bCastInsetShadow) , TEXT("Dynamic Inset Shadow")}},
			{EInterchangePropertyTracks::CastShadow, {UMovieSceneByteTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, CastShadow) , TEXT("Cast Shadow")}},
			{EInterchangePropertyTracks::CustomDepthStencilValue, {UMovieSceneIntegerTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, CustomDepthStencilValue) , TEXT("CustomDepth Stencil Value")}},
			{EInterchangePropertyTracks::CustomDepthStencilWriteMask, {UMovieSceneEnumTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, CustomDepthStencilWriteMask) , TEXT("Custom Depth Stencil Write Mask"), StaticEnum<ERendererStencilMask>()}},
			{EInterchangePropertyTracks::DefaultUpVector, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USplineComponent, DefaultUpVector) , TEXT("Default Up Vector"), 3}},
			{EInterchangePropertyTracks::DrawDebug, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USplineComponent, bDrawDebug) , TEXT("Draw Debug")}},
			{EInterchangePropertyTracks::EmissiveLightSource, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bEmissiveLightSource) , TEXT("Emissive Light Source")}},
			{EInterchangePropertyTracks::ExcludeFromLightAttachmentGroup, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bExcludeFromLightAttachmentGroup) , TEXT("Exclude from Light Attachment Group")}},
			{EInterchangePropertyTracks::HiddenInGame, {UMovieSceneVisibilityTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USceneComponent, bHiddenInGame) , TEXT("Hidden in Game")}},
			{EInterchangePropertyTracks::HiddenInSceneCapture, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bHiddenInSceneCapture) , TEXT("Hidden In Scene Capture")}},
			{EInterchangePropertyTracks::Holdout, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bHoldout) , TEXT("Holdout")}},
			{EInterchangePropertyTracks::LightAttachmentsAsGroup, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bLightAttachmentsAsGroup) , TEXT("Light Attachments as Group")}},
			{EInterchangePropertyTracks::Mobility, {UMovieSceneEnumTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USceneComponent, Mobility) , TEXT("Mobility"), StaticEnum<EComponentMobility::Type>()}},
			{EInterchangePropertyTracks::OnlyOwnerSee, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bOnlyOwnerSee) , TEXT("Only Owner See")}},
			{EInterchangePropertyTracks::OwnerNoSee, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bOwnerNoSee) , TEXT("Owner No See")}},
			{EInterchangePropertyTracks::ReceivesDecals, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bReceivesDecals) , TEXT("Receives Decals")}},
			{EInterchangePropertyTracks::RenderCustomDepth, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bRenderCustomDepth) , TEXT("Render CustomDepth Pass")}},
			{EInterchangePropertyTracks::RenderInDepthPass, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bRenderInDepthPass) , TEXT("Render in Depth Pass")}},
			{EInterchangePropertyTracks::RenderInMainPass, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bRenderInMainPass) , TEXT("Render in Main Pass")}},
			{EInterchangePropertyTracks::SingleSampleShadowFromStationaryLights, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bSingleSampleShadowFromStationaryLights) , TEXT("Single Sample Shadow from Stationary Lights")}},
			{EInterchangePropertyTracks::TranslucencySortDistanceOffset, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, TranslucencySortDistanceOffset) , TEXT("Translucency Sort Distance Offset")}},
			{EInterchangePropertyTracks::VisibleInRayTracing, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bVisibleInRayTracing) , TEXT("Visible in Ray Tracing")}},
			{EInterchangePropertyTracks::VisibleInReflections, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bVisibleInReflections) , TEXT("Visible in Reflections")}},
			{EInterchangePropertyTracks::VisibleInSceneCaptureOnly, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, bVisibleInSceneCaptureOnly) , TEXT("Visible In Scene Capture Only")}},
			{EInterchangePropertyTracks::ActorHiddenInGame, {UMovieSceneVisibilityTrack::StaticClass()->GetName(), AActor::GetHiddenPropertyName().ToString(), TEXT("Visibility")}},

			// Light
			{EInterchangePropertyTracks::LightAffectGlobalIllumination, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponentBase, bAffectGlobalIllumination) , TEXT("Affect Ray Tracing Global Illumination")}},
			{EInterchangePropertyTracks::LightAffectReflection, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponentBase, bAffectReflection) , TEXT("Affect Ray Tracing Reflections")}},
			{EInterchangePropertyTracks::LightAffectTranslucentLighting, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponentBase, bAffectTranslucentLighting) , TEXT("Affect Translucent Lighting")}},
			{EInterchangePropertyTracks::LightAtmosphereSunDiskColorScale, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, AtmosphereSunDiskColorScale) , TEXT("Atmosphere Sun Disk Color Scale")}},
			{EInterchangePropertyTracks::LightAtmosphereSunLight, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, bAtmosphereSunLight) , TEXT("Atmosphere Sun Light")}},
			{EInterchangePropertyTracks::LightAtmosphereSunLightIndex, {UMovieSceneIntegerTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, AtmosphereSunLightIndex) , TEXT("Atmosphere Sun Light Index")}},
			{EInterchangePropertyTracks::LightAttenuationRadius, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULocalLightComponent, AttenuationRadius) , TEXT("Attenuation Radius")}},
			{EInterchangePropertyTracks::LightBarnDoorAngle, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(URectLightComponent, BarnDoorAngle) , TEXT("Barn Door Angle")}},
			{EInterchangePropertyTracks::LightBarnDoorLength, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(URectLightComponent, BarnDoorLength) , TEXT("Barn Door Length")}},
			{EInterchangePropertyTracks::LightBloomMaxBrightness, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, BloomMaxBrightness) , TEXT("Bloom Max Brightness")}},
			{EInterchangePropertyTracks::LightBloomScale, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, BloomScale) , TEXT("Bloom Scale")}},
			{EInterchangePropertyTracks::LightBloomThreshold, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, BloomThreshold) , TEXT("Bloom Threshold")}},
			{EInterchangePropertyTracks::LightBloomTint, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, BloomTint) , TEXT("Bloom Tint")}},
			{EInterchangePropertyTracks::LightCascadeDistributionExponent, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CascadeDistributionExponent) , TEXT("Distribution Exponent")}},
			{EInterchangePropertyTracks::LightCascadeTransitionFraction, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CascadeTransitionFraction) , TEXT("Transition Fraction")}},
			{EInterchangePropertyTracks::LightCastDeepShadow, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponentBase, bCastDeepShadow) , TEXT("Cast Deep Shadow")}},
			{EInterchangePropertyTracks::LightCastShadows, {UMovieSceneByteTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponentBase, CastShadows) , TEXT("Cast Shadows")}},
			{EInterchangePropertyTracks::LightCastVolumetricShadow, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponentBase, bCastVolumetricShadow) , TEXT("Cast Volumetric Shadow")}},
			{EInterchangePropertyTracks::LightCloudAmbientOcclusionStrength, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USkyLightComponent, CloudAmbientOcclusionStrength) , TEXT("Cloud Ambient Occlusion Strength")}},
			{EInterchangePropertyTracks::LightCloudScatteredLuminanceScale, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CloudScatteredLuminanceScale) , TEXT("Cloud Scattered Luminance Scale")}},
			{EInterchangePropertyTracks::LightCloudShadowOnAtmosphereStrength, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CloudShadowOnAtmosphereStrength) , TEXT("Cloud Shadow on Atmosphere Strength")}},
			{EInterchangePropertyTracks::LightCloudShadowOnSurfaceStrength, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CloudShadowOnSurfaceStrength) , TEXT("Cloud Shadow on Surface Strength")}},
			{EInterchangePropertyTracks::LightCloudShadowStrength, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, CloudShadowStrength) , TEXT("Cloud Shadow Strength")}},
			{EInterchangePropertyTracks::LightColor, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponentBase, LightColor) , TEXT("Light Color")}},
			{EInterchangePropertyTracks::LightDynamicShadowCascades, {UMovieSceneIntegerTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, DynamicShadowCascades) , TEXT("Num Dynamic Shadow Cascades")}},
			{EInterchangePropertyTracks::LightDynamicShadowDistanceMovableLight, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, DynamicShadowDistanceMovableLight) , TEXT("Dynamic Shadow Distance MovableLight")}},
			{EInterchangePropertyTracks::LightDynamicShadowDistanceStationaryLight, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, DynamicShadowDistanceStationaryLight) , TEXT("Dynamic Shadow Distance StationaryLight")}},
			{EInterchangePropertyTracks::LightEnableLightShaftBloom, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, bEnableLightShaftBloom) , TEXT("Light Shaft Bloom")}},
			{EInterchangePropertyTracks::LightEnableLightShaftOcclusion, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, bEnableLightShaftOcclusion) , TEXT("Light Shaft Occlusion")}},
			{EInterchangePropertyTracks::LightFalloffExponent, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPointLightComponent, LightFalloffExponent) , TEXT("Light Falloff Exponent")}},
			{EInterchangePropertyTracks::LightForceCachedShadowsForMovablePrimitives, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, bForceCachedShadowsForMovablePrimitives) , TEXT("Force Cached Shadows for Movable Primitives")}},
			{EInterchangePropertyTracks::LightForwardShadingPriority, {UMovieSceneIntegerTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, ForwardShadingPriority) , TEXT("Forward Shading Priority")}},
			{EInterchangePropertyTracks::LightFunctionFadeDistance, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightFunctionFadeDistance) , TEXT("Fade Distance")}},
			{EInterchangePropertyTracks::LightFunctionScale, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightFunctionScale) , TEXT("Light Function Scale"), 3}},
			{EInterchangePropertyTracks::LightIESBrightnessScale, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, IESBrightnessScale) , TEXT("IES Intensity Scale")}},
			{EInterchangePropertyTracks::LightIndirectLightingIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponentBase, IndirectLightingIntensity) , TEXT("Indirect Lighting Intensity")}},
			{EInterchangePropertyTracks::LightInnerConeAngle, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USpotLightComponent, InnerConeAngle) , TEXT("Inner Cone Angle")}},
			{EInterchangePropertyTracks::LightIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponentBase, Intensity) , TEXT("Intensity")}},
			{EInterchangePropertyTracks::LightIntensityUnits, {UMovieSceneEnumTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULocalLightComponent, IntensityUnits) , TEXT("Intensity Units"), StaticEnum<ELightUnits>()}},
			{EInterchangePropertyTracks::LightInverseExposureBlend, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULocalLightComponent, InverseExposureBlend) , TEXT("Inverse Exposure Blend")}},
			{EInterchangePropertyTracks::LightLowerHemisphereColor, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USkyLightComponent, LowerHemisphereColor) , TEXT("Lower Hemisphere Color")}},
			{EInterchangePropertyTracks::LightmassSettingsLightSourceAngle, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, LightmassSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FLightmassDirectionalLightSettings, LightSourceAngle), TEXT("Light Source Angle (Lightmass Settings)")}},
			{EInterchangePropertyTracks::LightMinOcclusion, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USkyLightComponent, MinOcclusion) , TEXT("Min Occlusion")}},
			{EInterchangePropertyTracks::LightModulatedShadowColor, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, ModulatedShadowColor) , TEXT("Modulated Shadow Color")}},
			{EInterchangePropertyTracks::LightOcclusionDepthRange, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, OcclusionDepthRange) , TEXT("Occlusion Depth Range")}},
			{EInterchangePropertyTracks::LightOcclusionExponent, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USkyLightComponent, OcclusionExponent) , TEXT("Occlusion Exponent")}},
			{EInterchangePropertyTracks::LightOcclusionMaskDarkness, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, OcclusionMaskDarkness) , TEXT("Occlusion Mask Darkness")}},
			{EInterchangePropertyTracks::LightOcclusionTint, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USkyLightComponent, OcclusionTint) , TEXT("Occlusion Tint")}},
			{EInterchangePropertyTracks::LightOuterConeAngle, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USpotLightComponent, OuterConeAngle) , TEXT("Outer Cone Angle")}},
			{EInterchangePropertyTracks::LightSamplesPerPixel, {UMovieSceneIntegerTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponentBase, SamplesPerPixel) , TEXT("Samples Per Pixel")}},
			{EInterchangePropertyTracks::LightShadowAmount, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, ShadowAmount) , TEXT("Shadow Amount")}},
			{EInterchangePropertyTracks::LightShadowBias, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, ShadowBias) , TEXT("Shadow Bias")}},
			{EInterchangePropertyTracks::LightShadowCascadeBiasDistribution, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, ShadowCascadeBiasDistribution) , TEXT("Shadow Cascade Bias Distribution")}},
			{EInterchangePropertyTracks::LightShadowDistanceFadeoutFraction, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, ShadowDistanceFadeoutFraction) , TEXT("Distance Fadeout Fraction")}},
			{EInterchangePropertyTracks::LightShadowSlopeBias, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, ShadowSlopeBias) , TEXT("Shadow Slope Bias")}},
			{EInterchangePropertyTracks::LightShadowSourceAngleFactor, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, ShadowSourceAngleFactor) , TEXT("Shadow Source Angle Factor")}},
			{EInterchangePropertyTracks::LightShaftOverrideDirection, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, LightShaftOverrideDirection) , TEXT("Light Shaft Override Direction"), 3}},
			{EInterchangePropertyTracks::LightSoftSourceRadius, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPointLightComponent, SoftSourceRadius) , TEXT("Soft Source Radius")}},
			{EInterchangePropertyTracks::LightSourceAngle, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, LightSourceAngle) , TEXT("Source Angle")}},
			{EInterchangePropertyTracks::LightSourceCubemapAngle, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USkyLightComponent, SourceCubemapAngle) , TEXT("Source Cubemap Angle")}},
			{EInterchangePropertyTracks::LightSourceHeight, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(URectLightComponent, SourceHeight) , TEXT("Source Height")}},
			{EInterchangePropertyTracks::LightSourceLength, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPointLightComponent, SourceLength) , TEXT("Source Length")}},
			{EInterchangePropertyTracks::LightSourceRadius, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPointLightComponent, SourceRadius) , TEXT("Source Radius")}},
			{EInterchangePropertyTracks::LightSourceSoftAngle, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UDirectionalLightComponent, LightSourceSoftAngle) , TEXT("Source Soft Angle")}},
			{EInterchangePropertyTracks::LightSourceWidth, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(URectLightComponent, SourceWidth) , TEXT("Source Width")}},
			{EInterchangePropertyTracks::LightSpecularScale, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, SpecularScale) , TEXT("Specular Scale")}},
			{EInterchangePropertyTracks::LightDiffuseScale, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, DiffuseScale) , TEXT("Diffuse Scale")}},
			{EInterchangePropertyTracks::LightTemperature, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, Temperature) , TEXT("Temperature")}},
			{EInterchangePropertyTracks::LightTransmission, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponentBase, bTransmission) , TEXT("Transmission")}},
			{EInterchangePropertyTracks::LightUseIESBrightness, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, bUseIESBrightness) , TEXT("Use IES Intensity")}},
			{EInterchangePropertyTracks::LightUseInverseSquaredFalloff, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UPointLightComponent, bUseInverseSquaredFalloff) , TEXT("Use Inverse Squared Falloff")}},
			{EInterchangePropertyTracks::LightUseTemperature, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, bUseTemperature) , TEXT("Use Temperature")}},
			{EInterchangePropertyTracks::LightVolumetricScatteringIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(ULightComponentBase, VolumetricScatteringIntensity) , TEXT("Volumetric Scattering Intensity")}},

			// Camera
			{EInterchangePropertyTracks::CameraAspectRatio, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, AspectRatio) , TEXT("Aspect Ratio")}},
			{EInterchangePropertyTracks::CameraAspectRatioAxisConstraint, {UMovieSceneEnumTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, AspectRatioAxisConstraint) , TEXT("Aspect Ratio Axis Constraint"), StaticEnum<EAspectRatioAxisConstraint>()}},
			{EInterchangePropertyTracks::CameraAutoCalculateOrthoPlanes, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, bAutoCalculateOrthoPlanes) , TEXT("Auto Calculate Ortho Planes")}},
			{EInterchangePropertyTracks::CameraAutoPlaneShift, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, AutoPlaneShift) , TEXT("Auto Plane Shift")}},
			{EInterchangePropertyTracks::CameraConstrainAspectRatio, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, bConstrainAspectRatio) , TEXT("Constrain Aspect Ratio")}},
			{EInterchangePropertyTracks::CameraCurrentAperture, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCineCameraComponent, CurrentAperture) , TEXT("Current Aperture")}},
			{EInterchangePropertyTracks::CameraCurrentFocalLength, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCineCameraComponent, CurrentFocalLength) , TEXT("Current Focal Length")}},
			{EInterchangePropertyTracks::CameraCustomNearClippingPlane, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCineCameraComponent, CustomNearClippingPlane) , TEXT("Custom Near Clipping Plane")}},
			{EInterchangePropertyTracks::CameraFieldOfView, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, FieldOfView) , TEXT("Field Of View")}},
			{EInterchangePropertyTracks::CameraFilmbackSensorAspectRatio, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCineCameraComponent, Filmback) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FCameraFilmbackSettings, SensorAspectRatio), TEXT("Sensor Aspect Ratio (Filmback)")}},
			{EInterchangePropertyTracks::CameraFilmbackSensorHeight, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCineCameraComponent, Filmback) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FCameraFilmbackSettings, SensorHeight), TEXT("Sensor Height (Filmback)")}},
			{EInterchangePropertyTracks::CameraFilmbackSensorHorizontalOffset, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCineCameraComponent, Filmback) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FCameraFilmbackSettings, SensorHorizontalOffset), TEXT("Sensor Horizontal Offset (Filmback)")}},
			{EInterchangePropertyTracks::CameraFilmbackSensorVerticalOffset, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCineCameraComponent, Filmback) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FCameraFilmbackSettings, SensorVerticalOffset), TEXT("Sensor Vertical Offset (Filmback)")}},
			{EInterchangePropertyTracks::CameraFilmbackSensorWidth, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCineCameraComponent, Filmback) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FCameraFilmbackSettings, SensorWidth), TEXT("Sensor Width (Filmback)")}},
			{EInterchangePropertyTracks::CameraFocusSettingsFocusOffset, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCineCameraComponent, FocusSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FCameraFocusSettings, FocusOffset), TEXT("Focus Offset (Focus Settings)")}},
			{EInterchangePropertyTracks::CameraFocusSettingsManualFocusDistance, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCineCameraComponent, FocusSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FCameraFocusSettings, ManualFocusDistance), TEXT("Manual Focus Distance (Focus Settings)")}},
			{EInterchangePropertyTracks::CameraFocusSettingsTrackingFocusSettingsRelativeOffset, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCineCameraComponent, FocusSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FCameraFocusSettings, TrackingFocusSettings)+ FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FCameraTrackingFocusSettings, RelativeOffset), TEXT("Relative Offset (Tracking Focus Settings)"), 3}},
			{EInterchangePropertyTracks::CameraOrthoFarClipPlane, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, OrthoFarClipPlane) , TEXT("Ortho Far Clip Plane")}},
			{EInterchangePropertyTracks::CameraOrthoNearClipPlane, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, OrthoNearClipPlane) , TEXT("Ortho Near Clip Plane")}},
			{EInterchangePropertyTracks::CameraOrthoWidth, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, OrthoWidth) , TEXT("Ortho Width")}},
			{EInterchangePropertyTracks::CameraOverrideAspectRatioAxisConstraint, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, bOverrideAspectRatioAxisConstraint) , TEXT("Override Aspect Ratio Axis Constraint")}},
			{EInterchangePropertyTracks::CameraPostProcessBlendWeight, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessBlendWeight) , TEXT("Post Process Blend Weight")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAmbientCubemapIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AmbientCubemapIntensity), TEXT("Intensity (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAmbientCubemapTint, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AmbientCubemapTint), TEXT("Tint (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAmbientOcclusionBias, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AmbientOcclusionBias), TEXT("Bias (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAmbientOcclusionFadeDistance, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AmbientOcclusionFadeDistance), TEXT("Fade Out Distance (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAmbientOcclusionFadeRadius, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AmbientOcclusionFadeRadius), TEXT("Fade Out Radius (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAmbientOcclusionIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AmbientOcclusionIntensity), TEXT("Intensity (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAmbientOcclusionMipBlend, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AmbientOcclusionMipBlend), TEXT("Mip Blend (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAmbientOcclusionMipScale, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AmbientOcclusionMipScale), TEXT("Mip Scale (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAmbientOcclusionMipThreshold, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AmbientOcclusionMipThreshold), TEXT("Mip Threshold (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAmbientOcclusionPower, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AmbientOcclusionPower), TEXT("Power (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAmbientOcclusionQuality, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AmbientOcclusionQuality), TEXT("Quality (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAmbientOcclusionRadius, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AmbientOcclusionRadius), TEXT("Radius (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAmbientOcclusionStaticFraction, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AmbientOcclusionStaticFraction), TEXT("Static Fraction (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAmbientOcclusionTemporalBlendWeight, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AmbientOcclusionTemporalBlendWeight), TEXT("Temporal Blend Weight (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAutoExposureBias, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AutoExposureBias), TEXT("Exposure Compensation (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAutoExposureHighPercent, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AutoExposureHighPercent), TEXT("High Percent (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAutoExposureLowPercent, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AutoExposureLowPercent), TEXT("Low Percent (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAutoExposureMaxBrightness, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AutoExposureMaxBrightness), TEXT("Max Brightness (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAutoExposureMinBrightness, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AutoExposureMinBrightness), TEXT("Min Brightness (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAutoExposureSpeedDown, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AutoExposureSpeedDown), TEXT("Speed Down (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsAutoExposureSpeedUp, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AutoExposureSpeedUp), TEXT("Speed Up (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloom1Size, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, Bloom1Size), TEXT("#1 Size (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloom1Tint, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, Bloom1Tint), TEXT("#1 Tint (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloom2Size, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, Bloom2Size), TEXT("#2 Size (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloom2Tint, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, Bloom2Tint), TEXT("#2 Tint (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloom3Size, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, Bloom3Size), TEXT("#3 Size (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloom3Tint, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, Bloom3Tint), TEXT("#3 Tint (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloom4Size, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, Bloom4Size), TEXT("#4 Size (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloom4Tint, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, Bloom4Tint), TEXT("#4 Tint (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloom5Size, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, Bloom5Size), TEXT("#5 Size (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloom5Tint, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, Bloom5Tint), TEXT("#5 Tint (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloom6Size, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, Bloom6Size), TEXT("#6 Size (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloom6Tint, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, Bloom6Tint), TEXT("#6 Tint (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloomConvolutionBufferScale, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, BloomConvolutionBufferScale), TEXT("Convolution Buffer (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloomConvolutionCenterUV, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, BloomConvolutionCenterUV), TEXT("Convolution Center (Post Process Settings)"), 2}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloomConvolutionPreFilterMax, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, BloomConvolutionPreFilterMax), TEXT("Convolution Boost Max (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloomConvolutionPreFilterMin, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, BloomConvolutionPreFilterMin), TEXT("Convolution Boost Min (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloomConvolutionPreFilterMult, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, BloomConvolutionPreFilterMult), TEXT("Convolution Boost Mult (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloomConvolutionScatterDispersion, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, BloomConvolutionScatterDispersion), TEXT("Convolution Scatter Dispersion (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloomConvolutionSize, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, BloomConvolutionSize), TEXT("Convolution Scale (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloomDirtMaskIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, BloomDirtMaskIntensity), TEXT("Dirt Mask Intensity (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloomDirtMaskTint, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, BloomDirtMaskTint), TEXT("Dirt Mask Tint (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloomIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, BloomIntensity), TEXT("Intensity (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloomSizeScale, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, BloomSizeScale), TEXT("Size scale (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBloomThreshold, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, BloomThreshold), TEXT("Threshold (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsBlueCorrection, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, BlueCorrection), TEXT("Blue Correction (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsCameraISO, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, CameraISO), TEXT("Camera ISO (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsCameraShutterSpeed, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, CameraShutterSpeed), TEXT("Camera Shutter Speed (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsChromaticAberrationStartOffset, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ChromaticAberrationStartOffset), TEXT("Start Offset (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorContrast, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorContrast), TEXT("Contrast (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorContrastHighlights, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorContrastHighlights), TEXT("Contrast (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorContrastMidtones, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorContrastMidtones), TEXT("Contrast (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorContrastShadows, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorContrastShadows), TEXT("Contrast (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorCorrectionHighlightsMax, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorCorrectionHighlightsMax), TEXT("HighlightsMax (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorCorrectionHighlightsMin, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorCorrectionHighlightsMin), TEXT("HighlightsMin (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorCorrectionShadowsMax, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorCorrectionShadowsMax), TEXT("ShadowsMax (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorGain, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorGain), TEXT("Gain (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorGainHighlights, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorGainHighlights), TEXT("Gain (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorGainMidtones, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorGainMidtones), TEXT("Gain (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorGainShadows, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorGainShadows), TEXT("Gain (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorGamma, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorGamma), TEXT("Gamma (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorGammaHighlights, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorGammaHighlights), TEXT("Gamma (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorGammaMidtones, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorGammaMidtones), TEXT("Gamma (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorGammaShadows, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorGammaShadows), TEXT("Gamma (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorGradingIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorGradingIntensity), TEXT("Color Grading LUT Intensity (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorOffset, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorOffset), TEXT("Offset (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorOffsetHighlights, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorOffsetHighlights), TEXT("Offset (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorOffsetMidtones, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorOffsetMidtones), TEXT("Offset (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorOffsetShadows, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorOffsetShadows), TEXT("Offset (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorSaturation, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorSaturation), TEXT("Saturation (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorSaturationHighlights, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorSaturationHighlights), TEXT("Saturation (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorSaturationMidtones, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorSaturationMidtones), TEXT("Saturation (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsColorSaturationShadows, {UMovieSceneDoubleVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ColorSaturationShadows), TEXT("Saturation (Post Process Settings)"), 4}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldBladeCount, {UMovieSceneIntegerTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldBladeCount), TEXT("Number of diaphragm blades (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldDepthBlurAmount, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldDepthBlurAmount), TEXT("Depth Blur km for 50% (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldDepthBlurRadius, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldDepthBlurRadius), TEXT("Depth Blur Radius (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldFarBlurSize, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldFarBlurSize), TEXT("Far Blur Size (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldFarTransitionRegion, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldFarTransitionRegion), TEXT("Far Transition Region (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldFocalDistance, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldFocalDistance), TEXT("Focal Distance (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldFocalRegion, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldFocalRegion), TEXT("Focal Region (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldFstop, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldFstop), TEXT("Aperture (F-stop) (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldMinFstop, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldMinFstop), TEXT("Maximum Aperture (min F-stop) (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldNearBlurSize, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldNearBlurSize), TEXT("Near Blur Size (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldNearTransitionRegion, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldNearTransitionRegion), TEXT("Near Transition Region (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldOcclusion, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldOcclusion), TEXT("Occlusion (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldScale, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldScale), TEXT("Scale (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldSkyFocusDistance, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldSkyFocusDistance), TEXT("Sky Distance (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldUseHairDepth, {UMovieSceneByteTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldUseHairDepth), TEXT("Use Hair Depth (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDepthOfFieldVignetteSize, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DepthOfFieldVignetteSize), TEXT("Vignette Size (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsDynamicGlobalIlluminationMethod, {UMovieSceneEnumTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, DynamicGlobalIlluminationMethod), TEXT("Method (Post Process Settings)"), StaticEnum<EDynamicGlobalIlluminationMethod::Type>()}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsExpandGamut, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ExpandGamut), TEXT("Expand Gamut (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsFilmBlackClip, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, FilmBlackClip), TEXT("Black clip (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsFilmGrainHighlightsMax, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, FilmGrainHighlightsMax), TEXT("Film Grain Highlights Max (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsFilmGrainHighlightsMin, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, FilmGrainHighlightsMin), TEXT("Film Grain Highlights Min (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsFilmGrainIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, FilmGrainIntensity), TEXT("Film Grain Intensity (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsFilmGrainIntensityHighlights, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, FilmGrainIntensityHighlights), TEXT("Film Grain Intensity Highlights (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsFilmGrainIntensityMidtones, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, FilmGrainIntensityMidtones), TEXT("Film Grain Intensity Midtones (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsFilmGrainIntensityShadows, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, FilmGrainIntensityShadows), TEXT("Film Grain Intensity Shadows (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsFilmGrainShadowsMax, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, FilmGrainShadowsMax), TEXT("Film Grain Shadows Max (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsFilmGrainScale, {UMovieSceneFloatVectorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, FilmGrainScale), TEXT("Film Grain Scale (Post Process Settings)"), 2}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsFilmShoulder, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, FilmShoulder), TEXT("Shoulder (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsFilmSlope, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, FilmSlope), TEXT("Slope (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsFilmToe, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, FilmToe), TEXT("Toe (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsFilmWhiteClip, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, FilmWhiteClip), TEXT("White clip (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsHistogramLogMax, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, HistogramLogMax), TEXT("Histogram Log Max (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsHistogramLogMin, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, HistogramLogMin), TEXT("Histogram Log Min (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsIndirectLightingColor, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, IndirectLightingColor), TEXT("Indirect Lighting Color (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsIndirectLightingIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, IndirectLightingIntensity), TEXT("Indirect Lighting Intensity (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLensFlareBokehSize, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LensFlareBokehSize), TEXT("BokehSize (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLensFlareIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LensFlareIntensity), TEXT("Intensity (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLensFlareThreshold, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LensFlareThreshold), TEXT("Threshold (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLensFlareTint, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LensFlareTint), TEXT("Tint (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLocalExposureBlurredLuminanceBlend, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LocalExposureBlurredLuminanceBlend), TEXT("Blurred Luminance Blend (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLocalExposureBlurredLuminanceKernelSizePercent, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LocalExposureBlurredLuminanceKernelSizePercent), TEXT("Blurred Luminance Kernel Size Percent (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLocalExposureDetailStrength, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LocalExposureDetailStrength), TEXT("Detail Strength (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLocalExposureHighlightContrastScale, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LocalExposureHighlightContrastScale), TEXT("Highlight Contrast (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLocalExposureHighlightThreshold, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LocalExposureHighlightThreshold), TEXT("Highlight Threshold (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLocalExposureMiddleGreyBias, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LocalExposureMiddleGreyBias), TEXT("Middle Grey Bias (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLocalExposureShadowContrastScale, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LocalExposureShadowContrastScale), TEXT("Shadow Contrast (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLocalExposureShadowThreshold, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LocalExposureShadowThreshold), TEXT("Shadow Threshold (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenDiffuseColorBoost, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenDiffuseColorBoost), TEXT("Diffuse Color Boost (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenAmbientOcclusionIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenAmbientOcclusionIntensity), TEXT("Lumen Ambient Occlusion Intensity (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenFinalGatherLightingUpdateSpeed, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenFinalGatherLightingUpdateSpeed), TEXT("Final Gather Lighting Update Speed (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenFinalGatherQuality, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenFinalGatherQuality), TEXT("Final Gather Quality (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenFinalGatherScreenTraces, {UMovieSceneByteTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenFinalGatherScreenTraces), TEXT("Screen Traces (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenFrontLayerTranslucencyReflections, {UMovieSceneByteTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenFrontLayerTranslucencyReflections), TEXT("High Quality Translucency Reflections (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenFullSkylightLeakingDistance, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenFullSkylightLeakingDistance), TEXT("Full Skylight Leaking Distance (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenMaxReflectionBounces, {UMovieSceneIntegerTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenMaxReflectionBounces), TEXT("Max Reflection Bounces (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenMaxRefractionBounces, {UMovieSceneIntegerTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenMaxRefractionBounces), TEXT("Max Refraction Bounces (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenMaxRoughnessToTraceReflections, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenMaxRoughnessToTraceReflections), TEXT("Max Roughness To Trace (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenMaxTraceDistance, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenMaxTraceDistance), TEXT("Max Trace Distance (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenRayLightingMode, {UMovieSceneEnumTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenRayLightingMode), TEXT("Ray Lighting Mode (Post Process Settings)"), StaticEnum<ELumenRayLightingModeOverride>()}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenReflectionQuality, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenReflectionQuality), TEXT("Quality (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenReflectionsScreenTraces, {UMovieSceneByteTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenReflectionsScreenTraces), TEXT("Screen Traces (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenSceneDetail, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenSceneDetail), TEXT("Lumen Scene Detail (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenSceneLightingQuality, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenSceneLightingQuality), TEXT("Lumen Scene Lighting Quality (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenSceneLightingUpdateSpeed, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenSceneLightingUpdateSpeed), TEXT("Lumen Scene Lighting Update Speed (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenSceneViewDistance, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenSceneViewDistance), TEXT("Lumen Scene View Distance (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenSkylightLeaking, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenSkylightLeaking), TEXT("Skylight Leaking (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsLumenSkylightLeakingTint, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, LumenSkylightLeakingTint), TEXT("Skylight Leaking Tint (Post Process Settings)")} },
			{EInterchangePropertyTracks::CameraPostProcessSettingsMotionBlurAmount, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, MotionBlurAmount), TEXT("Amount (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsMotionBlurMax, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, MotionBlurMax), TEXT("Max (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsMotionBlurPerObjectSize, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, MotionBlurPerObjectSize), TEXT("Per Object Size (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsPathTracingMaxBounces, {UMovieSceneIntegerTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, PathTracingMaxBounces), TEXT("Max. Bounces (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsPathTracingMaxPathIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, PathTracingMaxPathIntensity), TEXT("Max Path Intensity (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsRayTracingAO, {UMovieSceneByteTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, RayTracingAO), TEXT("Enabled (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsRayTracingAOIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, RayTracingAOIntensity), TEXT("Intensity (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsRayTracingAORadius, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, RayTracingAORadius), TEXT("Radius (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsRayTracingAOSamplesPerPixel, {UMovieSceneIntegerTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, RayTracingAOSamplesPerPixel), TEXT("Samples Per Pixel (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsRayTracingTranslucencyMaxRoughness, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, RayTracingTranslucencyMaxRoughness), TEXT("Max Roughness (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsRayTracingTranslucencyRefraction, {UMovieSceneByteTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, RayTracingTranslucencyRefraction), TEXT("Refraction (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsRayTracingTranslucencyRefractionRays, {UMovieSceneIntegerTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, RayTracingTranslucencyRefractionRays), TEXT("Max. Refraction Rays (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsRayTracingTranslucencySamplesPerPixel, {UMovieSceneIntegerTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, RayTracingTranslucencySamplesPerPixel), TEXT("Samples Per Pixel (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsRayTracingTranslucencyShadows, {UMovieSceneEnumTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, RayTracingTranslucencyShadows), TEXT("Shadows (Post Process Settings)"), StaticEnum<EReflectedAndRefractedRayTracedShadows>()}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsReflectionMethod, {UMovieSceneEnumTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ReflectionMethod), TEXT("Method (Post Process Settings)"), StaticEnum<EReflectionMethod::Type>()}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsSceneColorTint, {UMovieSceneColorTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, SceneColorTint), TEXT("Scene Color Tint (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsSceneFringeIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, SceneFringeIntensity), TEXT("Intensity (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsScreenSpaceReflectionIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ScreenSpaceReflectionIntensity), TEXT("Intensity (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsScreenSpaceReflectionMaxRoughness, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ScreenSpaceReflectionMaxRoughness), TEXT("Max Roughness (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsScreenSpaceReflectionQuality, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ScreenSpaceReflectionQuality), TEXT("Quality (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsSharpen, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, Sharpen), TEXT("Sharpen (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsTemperatureType, {UMovieSceneEnumTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, TemperatureType), TEXT("Temperature Type (Post Process Settings)"), StaticEnum<ETemperatureMethod>()}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsToneCurveAmount, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, ToneCurveAmount), TEXT("Tone Curve Amount (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsTranslucencyType, {UMovieSceneEnumTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, TranslucencyType), TEXT("Type (Post Process Settings)"), StaticEnum<ETranslucencyType>()}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsVignetteIntensity, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, VignetteIntensity), TEXT("Vignette Intensity (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsWhiteTemp, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, WhiteTemp), TEXT("Temp (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraPostProcessSettingsWhiteTint, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, PostProcessSettings) + FString{ TEXT(".") } + GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, WhiteTint), TEXT("Tint (Post Process Settings)")}},
			{EInterchangePropertyTracks::CameraProjectionMode, {UMovieSceneEnumTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, ProjectionMode) , TEXT("Projection Mode"), StaticEnum<ECameraProjectionMode::Type>()}},
			{EInterchangePropertyTracks::CameraUpdateOrthoPlanes, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, bUpdateOrthoPlanes) , TEXT("Update Ortho Planes")}},
			{EInterchangePropertyTracks::CameraUseCameraHeightAsViewTarget, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, bUseCameraHeightAsViewTarget) , TEXT("Use Camera Height as View Target")}},
			{EInterchangePropertyTracks::CameraUseFieldOfViewForLOD, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UCameraComponent, bUseFieldOfViewForLOD) , TEXT("Use Field Of View for LOD")}},

			// Common Mesh
			{EInterchangePropertyTracks::MeshOverlayMaterialMaxDrawDistance, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UMeshComponent, OverlayMaterialMaxDrawDistance) , TEXT("Overlay Material Max Draw Distance")}},

			// Skinned Mesh
			{EInterchangePropertyTracks::SkinnedMeshCapsuleIndirectShadowMinVisibility, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USkinnedMeshComponent, CapsuleIndirectShadowMinVisibility) , TEXT("Capsule Indirect Shadow Min Visibility")}},
			{EInterchangePropertyTracks::SkinnedMeshCastCapsuleDirectShadow, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USkinnedMeshComponent, bCastCapsuleDirectShadow) , TEXT("Capsule Direct Shadow")}},
			{EInterchangePropertyTracks::SkinnedMeshCastCapsuleIndirectShadow, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USkinnedMeshComponent, bCastCapsuleIndirectShadow) , TEXT("Capsule Indirect Shadow")}},
			{EInterchangePropertyTracks::SkinnedMeshRenderStatic, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USkinnedMeshComponent, bRenderStatic) , TEXT("Render Static")}},
			{EInterchangePropertyTracks::SkinnedMeshVisibilityBasedAnimTickOption, {UMovieSceneEnumTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USkinnedMeshComponent, VisibilityBasedAnimTickOption) , TEXT("Visibility Based Anim Tick Option"), StaticEnum<EVisibilityBasedAnimTickOption>()}},

			// Skeletal Mesh
#if WITH_EDITORONLY_DATA
			{EInterchangePropertyTracks::SkeletalMesh, {UMovieSceneObjectPropertyTrack::StaticClass()->GetName(),  USkeletalMeshComponent::GetSkeletalMeshAssetPropertyNameChecked().ToString(), TEXT("Skeletal Mesh Asset"), USkeletalMesh::StaticClass()}},
#endif
			{EInterchangePropertyTracks::SkeletalMeshAllowClothActors, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USkeletalMeshComponent, bAllowClothActors) , TEXT("Allow Cloth Actors")}},
			{EInterchangePropertyTracks::SkeletalMeshAnimationMode, {UMovieSceneEnumTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USkeletalMeshComponent, USkeletalMeshComponent::GetAnimationModePropertyNameChecked()) , TEXT("Animation Mode"), StaticEnum<EAnimationMode::Type>()}},
			{EInterchangePropertyTracks::SkeletalMeshClothBlendWeight, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USkeletalMeshComponent, ClothBlendWeight) , TEXT("Cloth Blend Weight")}},
			{EInterchangePropertyTracks::SkeletalMeshClothMaxDistanceScale, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(USkeletalMeshComponent, ClothMaxDistanceScale) , TEXT("Cloth Max Distance Scale")}},

			// Static Mesh
			{EInterchangePropertyTracks::StaticMesh, {UMovieSceneObjectPropertyTrack::StaticClass()->GetName(), UStaticMeshComponent::GetMemberNameChecked_StaticMesh().ToString() , TEXT("Static Mesh"), UStaticMesh::StaticClass()}},
			{EInterchangePropertyTracks::StaticMeshDistanceFieldSelfShadowBias, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UStaticMeshComponent, DistanceFieldSelfShadowBias) , TEXT("Distance Field Self Shadow Bias")}},
			{EInterchangePropertyTracks::StaticMeshEvaluateWorldPositionOffset, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UStaticMeshComponent, bEvaluateWorldPositionOffset) , TEXT("Evaluate World Position Offset")}},
			{EInterchangePropertyTracks::StaticMeshEvaluateWorldPositionOffsetInRayTracing, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UStaticMeshComponent, bEvaluateWorldPositionOffsetInRayTracing) , TEXT("Evaluate World Position Offset in Ray Tracing")}},
			{EInterchangePropertyTracks::StaticMeshForcedLodModel, {UMovieSceneIntegerTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UStaticMeshComponent, ForcedLodModel) , TEXT("Forced Lod Model")}},
			{EInterchangePropertyTracks::StaticMeshReverseCulling, {UMovieSceneBoolTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UStaticMeshComponent, bReverseCulling) , TEXT("Reverse Culling")}},
			{EInterchangePropertyTracks::StaticMeshWorldPositionOffsetDisableDistance, {UMovieSceneIntegerTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UStaticMeshComponent, WorldPositionOffsetDisableDistance) , TEXT("World Position Offset Disable Distance")}},

			// Heterogeneous Volume Component
			{EInterchangePropertyTracks::HeterogeneousVolumeFrame, {UMovieSceneFloatTrack::StaticClass()->GetName(),  GET_MEMBER_NAME_STRING_CHECKED(UHeterogeneousVolumeComponent, Frame) , TEXT("Frame")}},
		}
	{}

	FInterchangePropertyTracksHelper& FInterchangePropertyTracksHelper::GetInstance()
	{
		static FInterchangePropertyTracksHelper Instance;
		return Instance;
	}

	UMovieSceneSection* FInterchangePropertyTracksHelper::GetSection(UMovieScene* MovieScene, const UInterchangeAnimationTrackNode& AnimationTrackNode, const FGuid& ObjectBinding, EInterchangePropertyTracks Property) const
	{
		const FInterchangeProperty* InterchangePropertyTrack = PropertyTracks.Find(Property);

		if(!InterchangePropertyTrack)
		{
			return nullptr;
		}

		TSubclassOf<UMovieSceneTrack> TrackClass = FindObjectClass<UMovieSceneTrack>(*InterchangePropertyTrack->ClassType);

		UMovieScenePropertyTrack* PropertyTrack = Cast<UMovieScenePropertyTrack>(MovieScene->FindTrack(TrackClass, ObjectBinding, InterchangePropertyTrack->Name));

		if(!PropertyTrack)
		{
			PropertyTrack = Cast<UMovieScenePropertyTrack>(MovieScene->AddTrack(TrackClass, ObjectBinding));

			if(!PropertyTrack)
			{
				return nullptr;
			}

			PropertyTrack->SetPropertyNameAndPath(InterchangePropertyTrack->Name, InterchangePropertyTrack->Path);
		}
		else
		{
			PropertyTrack->RemoveAllAnimationData();
		}

		if(UEnum* const* EnumClass = InterchangePropertyTrack->VariantProperty.TryGet<UEnum*>())
		{
			if(UMovieSceneByteTrack* ByteTrack = Cast<UMovieSceneByteTrack>(PropertyTrack))
			{
				ByteTrack->SetEnum(*EnumClass);
			}
		}
		else if(const int32* NumChannelsUsed = InterchangePropertyTrack->VariantProperty.TryGet<int32>())
		{
			if(UMovieSceneDoubleVectorTrack* DoubleVectorTrack = Cast<UMovieSceneDoubleVectorTrack>(PropertyTrack))
			{
				DoubleVectorTrack->SetNumChannelsUsed(*NumChannelsUsed);
			}
			else if(UMovieSceneFloatVectorTrack* FloatVectorTrack = Cast<UMovieSceneFloatVectorTrack>(PropertyTrack))
			{
				FloatVectorTrack->SetNumChannelsUsed(*NumChannelsUsed);
			}
		}
		else if(UClass* const* ObjectPropertyClass = InterchangePropertyTrack->VariantProperty.TryGet<UClass*>())
		{
			if(UMovieSceneObjectPropertyTrack* ObjectPropertyTrack = Cast<UMovieSceneObjectPropertyTrack>(PropertyTrack))
			{
				ObjectPropertyTrack->PropertyClass = *ObjectPropertyClass;
			}
		}

		bool bSectionAdded = false;

		UMovieSceneSection* Section = PropertyTrack->FindOrAddSection(0, bSectionAdded);

		if(!Section)
		{
			return nullptr;
		}

		if(bSectionAdded)
		{
			int32 CompletionMode;
			if(AnimationTrackNode.GetCustomCompletionMode(CompletionMode))
			{
				// Make sure EMovieSceneCompletionMode enum value are still between 0 and 2
				static_assert(0 == (uint32)EMovieSceneCompletionMode::KeepState, "ENUM_VALUE_HAS_CHANGED");
				static_assert(1 == (uint32)EMovieSceneCompletionMode::RestoreState, "ENUM_VALUE_HAS_CHANGED");
				static_assert(2 == (uint32)EMovieSceneCompletionMode::ProjectDefault, "ENUM_VALUE_HAS_CHANGED");

				Section->EvalOptions.CompletionMode = (EMovieSceneCompletionMode)CompletionMode;
			}
			// By default the completion mode is EMovieSceneCompletionMode::ProjectDefault
			else
			{
				Section->EvalOptions.CompletionMode = EMovieSceneCompletionMode::ProjectDefault;
			}

			Section->SetRange(TRange<FFrameNumber>::All());
		}

		return Section;
	}
}