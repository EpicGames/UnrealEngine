// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeAnimationTrackSetNode.h"
#include "InterchangeImportLog.h"

#include "UObject/ObjectRedirector.h"

class UMovieScene;
class UMovieSceneSection;

namespace UE::Interchange::Private
{
	/**
	 * Finds a UObject class by name.
	 * @param ClassName		The name of the class to look for (ie:UClass*->GetName()).
	 * @return				A sub class of UObject or nullptr.
	 */
	template<typename T>
	TSubclassOf<T> FindObjectClass(const TCHAR* ClassName)
	{
		if(!ensure(ClassName))
		{
			UE_LOGF(LogInterchangeImport, Error, "Classname is null to find an appropriate animation property track.")
				return nullptr;
		}

		UClass* ExpressionClass = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::EnsureIfAmbiguous);

		if(!ExpressionClass)
		{
			if(UObjectRedirector* RenamedClassRedirector = FindFirstObject<UObjectRedirector>(ClassName, EFindFirstObjectOptions::EnsureIfAmbiguous))
			{
				ExpressionClass = CastChecked<UClass>(RenamedClassRedirector->DestinationObject);
			}
		}

		if(ExpressionClass && ExpressionClass->IsChildOf<T>())
		{
			return ExpressionClass;
		}
		else
		{
			return nullptr;
		}
	}

	struct FInterchangePropertyTracksHelper
	{
		static FInterchangePropertyTracksHelper& GetInstance();

		UMovieSceneSection* GetSection(UMovieScene* MovieScene, const UInterchangeAnimationTrackNode& AnimationTrackNode, const FGuid& ObjectBinding, EInterchangePropertyTracks Property) const;

	private:
		FInterchangePropertyTracksHelper();

		struct FInterchangeProperty
		{
			FInterchangeProperty(FString&& ClassType, FString&& Path, FString&& Name)
				: ClassType{ MoveTemp(ClassType) }
				, Path{ MoveTemp(Path) }
				, Name{ MoveTemp(Name) }
			{}

			FInterchangeProperty(FString&& ClassType, FString&& Path, FString&& Name, UEnum* EnumClass)
				: ClassType{ MoveTemp(ClassType) }
				, Path{ MoveTemp(Path) }
				, Name{ MoveTemp(Name) }
				, VariantProperty{ TInPlaceType<UEnum*>{}, EnumClass}
			{}

			FInterchangeProperty(FString&& ClassType, FString&& Path, FString&& Name, int32 NumChannelsUsed)
				: ClassType{ MoveTemp(ClassType) }
				, Path{ MoveTemp(Path) }
				, Name{ MoveTemp(Name) }
				, VariantProperty{ TInPlaceType<int32>{}, NumChannelsUsed }
			{}

			FInterchangeProperty(FString && ClassType, FString && Path, FString && Name, UClass * ObjectPropertyClass)
				: ClassType{ MoveTemp(ClassType) }
				, Path{ MoveTemp(Path) }
				, Name{ MoveTemp(Name) }
				, VariantProperty{ TInPlaceType<UClass*>{}, ObjectPropertyClass }
			{}

			FString ClassType; // Float, Double, Byte, etc. Basically the class name of the UMovieSceneTrack
			FString Path;
			FName Name;
			TVariant<UEnum*, int32, UClass*> VariantProperty; // These are mutually exclusive, we can only have one of a kind (Either an Enum, or Number of channels used for a Vector, or an Object Path)
		};

		TMap<EInterchangePropertyTracks, FInterchangeProperty> PropertyTracks;
	};
}