// Copyright Epic Games, Inc. All Rights Reserved.
#include "Audio/Formats/InterchangeAudioFormatsPrivate.h"
#include "HAL/IConsoleManager.h"

namespace UE::Interchange::Audio
{
	const FString& FAudioFormatCVarNames::GetAudioFormatCVarName_AIF()
	{ 
		static const FString AudioFormatCVarName = TEXT("Interchange.FeatureFlags.Import.Audio.AIF");
		return AudioFormatCVarName;
	}

	const FString& FAudioFormatCVarNames::GetAudioFormatCVarName_AIFF()
	{ 
		static const FString AudioFormatCVarName = TEXT("Interchange.FeatureFlags.Import.Audio.AIFF");
		return AudioFormatCVarName;
	}

	const FString& FAudioFormatCVarNames::GetAudioFormatCVarName_OGG()
	{ 
		static const FString AudioFormatCVarName = TEXT("Interchange.FeatureFlags.Import.Audio.OGG");
		return AudioFormatCVarName;
	}

	const FString& FAudioFormatCVarNames::GetAudioFormatCVarName_FLAC()
	{ 
		static const FString AudioFormatCVarName = TEXT("Interchange.FeatureFlags.Import.Audio.FLAC");
		return AudioFormatCVarName;
	}

	const FString& FAudioFormatCVarNames::GetAudioFormatCVarName_OPUS()
	{ 
		static const FString AudioFormatCVarName = TEXT("Interchange.FeatureFlags.Import.Audio.OPUS");
		return AudioFormatCVarName;
	}

	const FString& FAudioFormatCVarNames::GetAudioFormatCVarName_MP3()
	{ 
		static const FString AudioFormatCVarName = TEXT("Interchange.FeatureFlags.Import.Audio.MP3");
		return AudioFormatCVarName;
	}

	const FString& FAudioFormatCVarNames::GetAudioFormatCVarName_WAV() 
	{ 
		static const FString AudioFormatCVarName = TEXT("Interchange.FeatureFlags.Import.Audio.WAV");
		return AudioFormatCVarName;
	}
}

static bool GInterchangeEnableAIFImport = false;
static FAutoConsoleVariableRef CCvarInterchangeEnableAIFImport(
	*UE::Interchange::Audio::FAudioFormatCVarNames::GetAudioFormatCVarName_AIF(),
	GInterchangeEnableAIFImport,
	TEXT("Whether AIF support is enabled."),
	ECVF_Default);

static bool GInterchangeEnableAIFFImport = false;
static FAutoConsoleVariableRef CCvarInterchangeEnableAIFFImport(
	*UE::Interchange::Audio::FAudioFormatCVarNames::GetAudioFormatCVarName_AIFF(),
	GInterchangeEnableAIFFImport,
	TEXT("Whether AIFF support is enabled."),
	ECVF_Default);

static bool GInterchangeEnableOGGImport = false;
static FAutoConsoleVariableRef CCvarInterchangeEnableOGGImport(
	*UE::Interchange::Audio::FAudioFormatCVarNames::GetAudioFormatCVarName_OGG(),
	GInterchangeEnableOGGImport,
	TEXT("Whether OGG support is enabled."),
	ECVF_Default);

static bool GInterchangeEnableFLACImport = false;
static FAutoConsoleVariableRef CCvarInterchangeEnableFLACImport(
	*UE::Interchange::Audio::FAudioFormatCVarNames::GetAudioFormatCVarName_FLAC(),
	GInterchangeEnableFLACImport,
	TEXT("Whether FLAC support is enabled."),
	ECVF_Default);

static bool GInterchangeEnableOPUSImport = false;
static FAutoConsoleVariableRef CCvarInterchangeEnableOPUSImport(
	*UE::Interchange::Audio::FAudioFormatCVarNames::GetAudioFormatCVarName_OPUS(),
	GInterchangeEnableOPUSImport,
	TEXT("Whether OPUS support is enabled."),
	ECVF_Default);

static bool GInterchangeEnableMP3Import = false;
static FAutoConsoleVariableRef CCvarInterchangeEnableMP3Import(
	*UE::Interchange::Audio::FAudioFormatCVarNames::GetAudioFormatCVarName_MP3(),
	GInterchangeEnableMP3Import,
	TEXT("Whether MP3 support is enabled."),
	ECVF_Default);

static bool GInterchangeEnableWAVImport = false;
static FAutoConsoleVariableRef CCvarInterchangeEnableWAVImport(
	*UE::Interchange::Audio::FAudioFormatCVarNames::GetAudioFormatCVarName_WAV(),
	GInterchangeEnableWAVImport,
	TEXT("Whether WAV support is enabled."),
	ECVF_Default);