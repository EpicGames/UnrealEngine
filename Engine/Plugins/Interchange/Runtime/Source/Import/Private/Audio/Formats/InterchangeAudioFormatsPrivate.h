// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Containers/UnrealString.h"

// TODO: This should be revisited when the code will 
// rely on the common Audio File Import Utils used by both the Sound Factory and Interchange.
#ifndef WITH_SNDFILE_IO
	#define WITH_SNDFILE_IO 0
#endif

namespace UE::Interchange::Audio
{
	struct FAudioFormatCVarNames
	{
	public:
		static const FString& GetAudioFormatCVarName_AIF();
		static const FString& GetAudioFormatCVarName_AIFF();
		static const FString& GetAudioFormatCVarName_OGG();
		static const FString& GetAudioFormatCVarName_FLAC();
		static const FString& GetAudioFormatCVarName_OPUS();
		static const FString& GetAudioFormatCVarName_MP3();
		static const FString& GetAudioFormatCVarName_WAV();
	};
}