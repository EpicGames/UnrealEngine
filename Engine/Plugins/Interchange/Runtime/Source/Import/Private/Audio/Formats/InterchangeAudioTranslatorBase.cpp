// Copyright Epic Games, Inc. All Rights Reserved.
#include "Audio/Formats/InterchangeAudioTranslatorBase.h"
#include "InterchangeAudioSoundWaveNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeAudioTranslatorBase)

#define LOCTEXT_NAMESPACE "InterchangeGenericAudioTranslator"

TArray<FString> UInterchangeAudioTranslatorBase::GetSupportedFormats() const
{
	return TArray<FString>();
}

EInterchangeTranslatorAssetType UInterchangeAudioTranslatorBase::GetSupportedAssetTypes() const
{
	return EInterchangeTranslatorAssetType::Sounds;
}

bool UInterchangeAudioTranslatorBase::Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const
{
	UInterchangeAudioSoundWaveNode* TranslatedNode = Translate_Internal(BaseNodeContainer);
	return TranslatedNode != nullptr;
}

UInterchangeAudioSoundWaveNode* UInterchangeAudioTranslatorBase::Translate_Internal(UInterchangeBaseNodeContainer& BaseNodeContainer) const
{
	FString Filename = SourceData->GetFilename();
	FPaths::NormalizeFilename(Filename);
	if (!FPaths::FileExists(Filename))
	{
		LogError(
			FText::Format(
				LOCTEXT("InterchangeGenericAudioTranslator_Translate_FileNotFound", "Filepath {0} doesn't exist"),
				FText::FromString(Filename)
			)
		);

		return nullptr;
	}

	UInterchangeAudioSoundWaveNode* SoundWaveNode = UInterchangeAudioSoundWaveNode::Create(BaseNodeContainer, FPaths::GetBaseFilename(Filename));
	SoundWaveNode->SetPayloadKey(Filename);

	return SoundWaveNode;
}

void UInterchangeAudioTranslatorBase::LogError(FText&& ErrorText) const
{
	if (UInterchangeResultError_Generic* Result = AddMessage<UInterchangeResultError_Generic>())
	{
		Result->AssetType = GetClass();

		if (SourceData)
		{
			const FString Filename = SourceData->GetFilename();
			Result->SourceAssetName = Filename;
			Result->InterchangeKey = FPaths::GetBaseFilename(Filename);
		}
		else
		{
			Result->InterchangeKey = TEXT("Undefined");
		}

		Result->Text = MoveTemp(ErrorText);
	}
}

void UInterchangeAudioTranslatorBase::LogWarning(FText&& WarningText) const
{
	if (UInterchangeResultWarning_Generic* Result = AddMessage<UInterchangeResultWarning_Generic>())
	{
		Result->AssetType = GetClass();

		if (SourceData)
		{
			const FString Filename = SourceData->GetFilename();
			Result->SourceAssetName = Filename;
			Result->InterchangeKey = FPaths::GetBaseFilename(Filename);
		}
		else
		{
			Result->InterchangeKey = TEXT("Undefined");
		}

		Result->Text = MoveTemp(WarningText);
	}
}

#undef LOCTEXT_NAMESPACE
