// Copyright Epic Games, Inc. All Rights Reserved.
#include "Audio/Formats/InterchangeAudioTranslator_AIF.h"

#include "Audio/Formats/InterchangeAudioFormatsPrivate.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeAudioTranslator_AIF)

TArray<FString> UInterchangeAudioTranslator_AIF::GetSupportedFormats() const
{
	using namespace UE::Interchange::Audio;

	TArray<FString> Formats;
#if WITH_SNDFILE_IO
	const IConsoleVariable* CVarFormatEnabled_AIF = IConsoleManager::Get().FindConsoleVariable(*FAudioFormatCVarNames::GetAudioFormatCVarName_AIF());
	const bool bFormatEnabled_AIF = CVarFormatEnabled_AIF && CVarFormatEnabled_AIF->GetBool();
	if(bFormatEnabled_AIF)
	{
		Formats.Add(TEXT("aif;Audio Interchange File"));
	}
#endif
	return Formats;
}
