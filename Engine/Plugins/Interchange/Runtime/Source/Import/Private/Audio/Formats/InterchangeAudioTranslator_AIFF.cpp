// Copyright Epic Games, Inc. All Rights Reserved.
#include "Audio/Formats/InterchangeAudioTranslator_AIFF.h"

#include "Audio/Formats/InterchangeAudioFormatsPrivate.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeAudioTranslator_AIFF)

TArray<FString> UInterchangeAudioTranslator_AIFF::GetSupportedFormats() const
{
	using namespace UE::Interchange::Audio;

	TArray<FString> Formats;
#if WITH_SNDFILE_IO
	const IConsoleVariable* CVarFormatEnabled_AIFF = IConsoleManager::Get().FindConsoleVariable(*FAudioFormatCVarNames::GetAudioFormatCVarName_AIFF());
	const bool bFormatEnabled_AIFF = CVarFormatEnabled_AIFF && CVarFormatEnabled_AIFF->GetBool();
	if(bFormatEnabled_AIFF)
	{
		Formats.Add(TEXT("aiff;Audio Interchange File Format"));
	}
#endif
	return Formats;
}
