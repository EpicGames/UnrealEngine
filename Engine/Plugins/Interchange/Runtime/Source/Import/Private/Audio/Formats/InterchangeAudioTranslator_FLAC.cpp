// Copyright Epic Games, Inc. All Rights Reserved.
#include "Audio/Formats/InterchangeAudioTranslator_FLAC.h"

#include "Audio/Formats/InterchangeAudioFormatsPrivate.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeAudioTranslator_FLAC)

TArray<FString> UInterchangeAudioTranslator_FLAC::GetSupportedFormats() const
{
	using namespace UE::Interchange::Audio;

	TArray<FString> Formats;
#if WITH_SNDFILE_IO
	const IConsoleVariable* CVarFormatEnabled_FLAC = IConsoleManager::Get().FindConsoleVariable(*FAudioFormatCVarNames::GetAudioFormatCVarName_FLAC());
	const bool bFormatEnabled_FLAC = CVarFormatEnabled_FLAC && CVarFormatEnabled_FLAC->GetBool();
	if(bFormatEnabled_FLAC)
	{
		Formats.Add(TEXT("flac;Free Lossless Audio Codec"));
	}
#endif
	return Formats;
}
