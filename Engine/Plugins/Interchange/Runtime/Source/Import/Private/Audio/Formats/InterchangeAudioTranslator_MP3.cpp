// Copyright Epic Games, Inc. All Rights Reserved.
#include "Audio/Formats/InterchangeAudioTranslator_MP3.h"

#include "Audio/Formats/InterchangeAudioFormatsPrivate.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeAudioTranslator_MP3)

TArray<FString> UInterchangeAudioTranslator_MP3::GetSupportedFormats() const
{
	using namespace UE::Interchange::Audio;

	TArray<FString> Formats;
#if WITH_SNDFILE_IO
	const IConsoleVariable* CVarFormatEnabled_MP3 = IConsoleManager::Get().FindConsoleVariable(*FAudioFormatCVarNames::GetAudioFormatCVarName_MP3());
	const bool bFormatEnabled_MP3 = CVarFormatEnabled_MP3 && CVarFormatEnabled_MP3->GetBool();
	if(bFormatEnabled_MP3)
	{
		Formats.Add(TEXT("mp3;MPEG Layer 3 Audio"));
	}
#endif
	return Formats;
}
