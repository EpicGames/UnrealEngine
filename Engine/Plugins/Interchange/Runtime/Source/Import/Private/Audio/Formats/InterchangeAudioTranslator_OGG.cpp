// Copyright Epic Games, Inc. All Rights Reserved.
#include "Audio/Formats/InterchangeAudioTranslator_OGG.h"

#include "Audio/Formats/InterchangeAudioFormatsPrivate.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeAudioTranslator_OGG)

TArray<FString> UInterchangeAudioTranslator_OGG::GetSupportedFormats() const
{
	using namespace UE::Interchange::Audio;

	TArray<FString> Formats;
#if WITH_SNDFILE_IO
	const IConsoleVariable* CVarFormatEnabled_OGG = IConsoleManager::Get().FindConsoleVariable(*FAudioFormatCVarNames::GetAudioFormatCVarName_OGG());
	const bool bFormatEnabled_OGG = CVarFormatEnabled_OGG && CVarFormatEnabled_OGG->GetBool();
	if(bFormatEnabled_OGG)
	{
		Formats.Add(TEXT("ogg;OGG Vorbis bitstream format "));
	}
#endif
	return Formats;
}
