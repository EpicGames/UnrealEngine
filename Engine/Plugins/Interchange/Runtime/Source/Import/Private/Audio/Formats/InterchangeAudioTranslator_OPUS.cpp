// Copyright Epic Games, Inc. All Rights Reserved.
#include "Audio/Formats/InterchangeAudioTranslator_OPUS.h"

#include "Audio/Formats/InterchangeAudioFormatsPrivate.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeAudioTranslator_OPUS)

TArray<FString> UInterchangeAudioTranslator_OPUS::GetSupportedFormats() const
{
	using namespace UE::Interchange::Audio;

	TArray<FString> Formats;
#if WITH_SNDFILE_IO
	const IConsoleVariable* CVarFormatEnabled_OPUS = IConsoleManager::Get().FindConsoleVariable(*FAudioFormatCVarNames::GetAudioFormatCVarName_OPUS());
	const bool bFormatEnabled_OPUS = CVarFormatEnabled_OPUS && CVarFormatEnabled_OPUS->GetBool();
	if(bFormatEnabled_OPUS)
	{
		Formats.Add(TEXT("opus;OGG OPUS bitstream format"));
	}
#endif
	return Formats;
}
