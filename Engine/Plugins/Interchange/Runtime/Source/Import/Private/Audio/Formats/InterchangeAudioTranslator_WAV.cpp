// Copyright Epic Games, Inc. All Rights Reserved.
#include "Audio/Formats/InterchangeAudioTranslator_WAV.h"

#include "Audio/Formats/InterchangeAudioFormatsPrivate.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeAudioTranslator_WAV)

TArray<FString> UInterchangeAudioTranslator_WAV::GetSupportedFormats() const
{
	using namespace UE::Interchange::Audio;

	TArray<FString> Formats;
	const IConsoleVariable* CVarFormatEnabled_WAV = IConsoleManager::Get().FindConsoleVariable(*FAudioFormatCVarNames::GetAudioFormatCVarName_WAV());
	const bool bFormatEnabled_WAV = CVarFormatEnabled_WAV && CVarFormatEnabled_WAV->GetBool();
	if(bFormatEnabled_WAV)
	{
		Formats.Add(TEXT("wav;Wave Audio File"));
	}
	return Formats;
}
