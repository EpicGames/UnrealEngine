// Copyright Epic Games, Inc. All Rights Reserved.
#include "Audio/InterchangeAudioPayloadData.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "SoundFileIO/SoundFileIO.h"

#include "InterchangeImportLog.h"

// TODO: This should be revisited when the code will 
// rely on the common Audio File Import Utils used by both the Sound Factory and Interchange.
#ifndef WITH_SNDFILE_IO
	#define WITH_SNDFILE_IO 0
#endif

namespace UE::Interchange::Private
{
	bool ConvertAudioFile(const TArray<uint8>& RawWaveData, FWaveModInfo& WaveInfo, TArray<uint8>& ConvertedRawWaveData, FString& ErrorMessage, bool bEnsureEqualNumSamples = true)
	{
		// This duplication will make sure that the buffer is not having
		// a mismatch in length and capacity. This is a must for Sound File IO operations
		// Otherwise a buffer overflow will be observed.
		TArray<uint8> RawAudioData;
		RawAudioData.Reserve(RawWaveData.Num());
		RawAudioData.AddUninitialized(RawWaveData.Num());
		FMemory::Memcpy(RawAudioData.GetData(), RawWaveData.GetData(), RawWaveData.Num());
		
		const uint32 OrigNumSamples = ::Audio::SoundFileUtils::GetNumSamples(RawAudioData);

		if (::Audio::SoundFileUtils::ConvertAudioToWav(RawAudioData, ConvertedRawWaveData))
		{
			WaveInfo = FWaveModInfo();
			if (!WaveInfo.ReadWaveInfo(ConvertedRawWaveData.GetData(), ConvertedRawWaveData.Num(), &ErrorMessage))
			{
				return false;
			}
		}

		if (bEnsureEqualNumSamples)
		{
			const uint32 ConvertedNumSamples = WaveInfo.GetNumSamples();
			if (!ensure(ConvertedNumSamples == OrigNumSamples))
			{
				ErrorMessage = "The converted wave file does not have the same number of samples as the original file.";
				return false;
			}
		}

		return true;
	}
}

TOptional<UE::Interchange::FInterchangeAudioPayloadData> FInterchangeAudioPayloadDataUtils::GetAudioPayloadFromSourceFileKey(const FString& PayloadSourceFileKey)
{
	using namespace UE::Interchange;

	if (!FPaths::FileExists(*PayloadSourceFileKey))
	{
		UE_LOGF(LogInterchangeImport, Error, "[Audio] File does not exist: [%ls]", *PayloadSourceFileKey);
		return TOptional<FInterchangeAudioPayloadData>();
	}

	TArray<uint8> ReadBuffer;
	if (!FFileHelper::LoadFileToArray(ReadBuffer, *PayloadSourceFileKey))
	{	
		UE_LOGF(LogInterchangeImport, Error, "[Audio] Failed to load file: [%ls]", *PayloadSourceFileKey);
		return TOptional<FInterchangeAudioPayloadData>();
	}

	const uint8* Buffer = &ReadBuffer[0];
	const uint8* BufferEnd = Buffer + ReadBuffer.Num() - 1;
	
	

	const uint64 Size = BufferEnd - Buffer;
	if (!IntFitsIn<int32>(Size))
	{
		UE_LOGF(LogInterchangeImport, Error, "[Audio] Audio file is too large: [%ls]", *PayloadSourceFileKey);
		return TOptional<FInterchangeAudioPayloadData>();
	}

	FInterchangeAudioPayloadData AudioPayloadData;
	FWaveModInfo WaveInfo;
	FString ErrorMessage;

	const bool bFileNeedsConversion = !FPaths::GetExtension(*(PayloadSourceFileKey.ToLower())).Equals(TEXT("wav"));
	if (!WaveInfo.ReadWaveInfo(ReadBuffer.GetData(), ReadBuffer.Num(), &ErrorMessage))
	{
		if (!bFileNeedsConversion)
		{
			UE_LOGF(LogInterchangeImport, Error, "[Audio] Failed to retrieve WaveFileInfo for '%ls' : %ls", *PayloadSourceFileKey, *ErrorMessage);
			return TOptional<FInterchangeAudioPayloadData>();
		}
	}

	if (bFileNeedsConversion || *WaveInfo.pBitsPerSample != 16 || !WaveInfo.IsFormatSupported())
	{
		TArray<uint8> ConvertedRawWaveData;
#if WITH_SNDFILE_IO
		const bool bEnsureEqualNumSamples = !bFileNeedsConversion;
		if (!Private::ConvertAudioFile(ReadBuffer, WaveInfo, ConvertedRawWaveData, ErrorMessage, bEnsureEqualNumSamples))
		{
			UE_LOGF(LogInterchangeImport, Error, "[Audio] Failed to convert '%ls' audio file: %ls", *PayloadSourceFileKey, *ErrorMessage);
			return TOptional<FInterchangeAudioPayloadData>();
		}
		AudioPayloadData.Buffer = MoveTemp(ConvertedRawWaveData);
#else
		WaveInfo.ReportImportFailure();
		return TOptional<FInterchangeAudioPayloadData>();
#endif
	}
	else
	{
		AudioPayloadData.Buffer = MoveTemp(ReadBuffer);
	}

	WaveInfo = FWaveModInfo();
	if (!WaveInfo.ReadWaveInfo(AudioPayloadData.Buffer.GetData(), AudioPayloadData.Buffer.Num(), &ErrorMessage))
	{
		WaveInfo.ReportImportFailure();
		return TOptional<FInterchangeAudioPayloadData>();
	}
	
	check(*WaveInfo.pBitsPerSample == 16);
	AudioPayloadData.FactoryInfo = FSoundWaveFactoryInfo();
	AudioPayloadData.FactoryInfo.ChannelCount = (int32)*WaveInfo.pChannels;
	check(AudioPayloadData.FactoryInfo.ChannelCount > 0);

	AudioPayloadData.FactoryInfo.SizeOfSample = (*WaveInfo.pBitsPerSample) / 8;
	AudioPayloadData.FactoryInfo.NumSamples = WaveInfo.SampleDataSize / AudioPayloadData.FactoryInfo.SizeOfSample;
	AudioPayloadData.FactoryInfo.SamplesPerSec = (int32)*WaveInfo.pSamplesPerSec;
	AudioPayloadData.FactoryInfo.SampleDataOffset = (int16*)WaveInfo.SampleDataStart - (int16*)AudioPayloadData.Buffer.GetData();
	ensure(AudioPayloadData.FactoryInfo.SampleDataOffset > 0);

	AudioPayloadData.FactoryInfo.NumFrames = AudioPayloadData.FactoryInfo.NumSamples / AudioPayloadData.FactoryInfo.ChannelCount;

	AudioPayloadData.WaveModInfo = WaveInfo;
	return TOptional<FInterchangeAudioPayloadData>(AudioPayloadData);
}