// Copyright Epic Games, Inc. All Rights Reserved.

#include "Audio/InterchangeAudioSoundWaveFactory.h"
#include "Engine/Engine.h"
#include "AudioDeviceManager.h"
#include "Components/AudioComponent.h"
#include "InterchangeAssetImportData.h"
#include "Audio/InterchangeAudioPayloadInterface.h"
#include "InterchangeAudioSoundWaveFactoryNode.h"
#include "InterchangeAudioSoundWaveNode.h"
#include "InterchangeImportCommon.h"
#include "InterchangeTranslatorBase.h"
#include "Sound/SoundWave.h"
#include "SoundFileIO/SoundFileIO.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeAudioSoundWaveFactory)

#define LOCTEXT_NAMESPACE "InterchangeAudioSoundWaveFactory"

UClass* UInterchangeAudioSoundWaveFactory::GetFactoryClass() const
{
	return USoundWave::StaticClass();
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeAudioSoundWaveFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeAudioSoundWaveFactory::BeginImportAsset_GameThread);

	FImportAssetResult ImportAssetResult;

	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	const UInterchangeAudioSoundWaveFactoryNode* SoundWaveFactoryNode = Cast<UInterchangeAudioSoundWaveFactoryNode>(Arguments.AssetNode);
	if (!SoundWaveFactoryNode)
	{
		const FText Message = FText::Format(LOCTEXT("InterchangeAudioSoundWaveFactory_ImportSoundWave_FactoryNodeFailedCast", "Failed to cast AssetNode to UInterchangeAudioSoundWaveFactoryNode. AssetNode class: `{0}`")
			, FText::FromString(Arguments.AssetNode->GetObjectClass()->GetName()));
		LogError(Arguments, ImportAssetResult, Message);
		return ImportAssetResult;
	}

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (SoundWaveFactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}
	}

	if (!ensure(SoundWaveFactoryNode->GetTargetNodeCount() == 1))
	{
		const FText Message = FText::Format(LOCTEXT("InterchangeAudioSoundWaveFactory_ImportSoundWave_UnexpectedNumTargets"
			, "Expected one target node, but found {0} target nodes associated with SoundWave factory node `{1}`")
			, SoundWaveFactoryNode->GetTargetNodeCount()
			, FText::FromString(SoundWaveFactoryNode->GetDisplayLabel()));
		LogError(Arguments, ImportAssetResult, Message);
		return ImportAssetResult;
	}

	USoundWave* SoundWaveAsset = nullptr;
	if (ExistingAsset)
	{
		SoundWaveAsset = Cast<USoundWave>(ExistingAsset);
		if (!SoundWaveAsset)
		{
			const FText Message = FText::Format(LOCTEXT("InterchangeAudioSoundWaveFactory_ImportSoundWave_InvalidExistingAsset", "Existing asset `{0}` must be a USoundWave asset.")
				, FText::FromString(ExistingAsset->GetName()));
			LogError(Arguments, ImportAssetResult, Message);
			return ImportAssetResult;
		}

		StopComponentsUsingImportedSound(SoundWaveAsset);
	}
	else
	{
		SoundWaveAsset = NewObject<USoundWave>(Arguments.Parent, GetFactoryClass(), *Arguments.AssetName, RF_Public | RF_Standalone);
	}

	if (!SoundWaveAsset)
	{
		const FText Message = FText::Format(LOCTEXT("InterchangeAudioSoundWaveFactory_ImportSoundWave_CouldNotCreateSoundWaveAsset", "Could not create SoundWave asset for factory node `{0}`")
			, FText::FromString(SoundWaveFactoryNode->GetDisplayLabel()));
		LogError(Arguments, ImportAssetResult, Message);
		return ImportAssetResult;
	}

	ImportAssetResult.ImportedObject = SoundWaveAsset;
	return ImportAssetResult;
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeAudioSoundWaveFactory::ImportAsset_Async(const FImportAssetObjectParams& Arguments)
{
	using namespace UE::Interchange;
	FImportAssetResult ImportAssetResult;

	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	UObject* SoundWaveObject = FFactoryCommon::AsyncFindObject(Arguments.AssetNode, GetFactoryClass(), Arguments.Parent, Arguments.AssetName);
	const bool bIsReimport = Arguments.ReimportObject && SoundWaveObject;
	if (!SoundWaveObject)
	{
		LogError(Arguments, ImportAssetResult, 
			FText::Format(
				LOCTEXT("UInterchangeAudioSoundWaveFactory_ImportSoundWave_NoExistingAsset", "Could not import the SoundWaveAsset '{0}' because the asset does not exist."),
				FText::FromString(Arguments.AssetName)
			)
		);
		return ImportAssetResult;
	}

	USoundWave* SoundWave = Cast<USoundWave>(SoundWaveObject);
	if (!ensure(SoundWave))
	{
		LogError(Arguments, ImportAssetResult, LOCTEXT("InterchangeAudioSoundWaveFactory_ImportSoundWave_FailedToCast", "Could not cast to SoundWave."));
		return ImportAssetResult;
	}
	
	const UInterchangeAudioSoundWaveFactoryNode* SoundWaveFactoryNode = Cast<UInterchangeAudioSoundWaveFactoryNode>(Arguments.AssetNode);
	if (!ensure(SoundWaveFactoryNode))
	{
		const FText Message = FText::Format(LOCTEXT("InterchangeAudioSoundWaveFactory_ImportSoundWave_FactoryNodeFailedCast", "Failed to cast AssetNode to UInterchangeAudioSoundWaveFactoryNode. AssetNode class: `{0}`")
			, FText::FromString(Arguments.AssetNode->GetObjectClass()->GetName()));
		LogError(Arguments, ImportAssetResult, Message);
		return ImportAssetResult;
	}

	TArray<FString> TargetNodeUids;
	SoundWaveFactoryNode->GetTargetNodeUids(TargetNodeUids);
	const UInterchangeBaseNode* TargetNode = Arguments.NodeContainer->GetNode(TargetNodeUids[0]);
	if (!ensure(TargetNode))
	{
		const FText Message = LOCTEXT("InterchangeAudioSoundWaveFactory_ImportSoundWave_InvalidTargetNode", "Target node was null.");
		LogError(Arguments, ImportAssetResult, Message);
		return ImportAssetResult;
	}

	const UInterchangeAudioSoundWaveNode* SoundWaveNode = Cast<UInterchangeAudioSoundWaveNode>(TargetNode);
	if (!SoundWaveNode)
	{
		const FText Message = FText::Format(LOCTEXT("InterchangeAudioSoundWaveFactory_ImportSoundWave_NullSoundWaveNode", "Failed to cast target node `{0}` to a sound wave node.")
			, FText::FromString(TargetNode->GetDisplayLabel()));
		LogError(Arguments, ImportAssetResult, Message);
		return ImportAssetResult;
	}

	TOptional<FString> SourceFilePayloadKey = SoundWaveNode->GetPayloadKey();
	if (!SourceFilePayloadKey.IsSet())
	{
		const FText Message = FText::Format(LOCTEXT("InterchangeAudioSoundWaveFactory_ImportSoundWave_SoundWaveNodeInvalidPayload", "Sound wave node `{0}` doesn't have a valid Payload.")
			, FText::FromString(SoundWaveNode->GetDisplayLabel()));
		LogError(Arguments, ImportAssetResult, Message);
		return ImportAssetResult;
	}

	const IInterchangeAudioPayloadInterface* AudioTranslator = Cast<IInterchangeAudioPayloadInterface>(Arguments.Translator);
	if (!AudioTranslator)
	{
		LogError(Arguments, ImportAssetResult, LOCTEXT("InterchangeAudioSoundWaveFactory_ImportSoundWave_InvalidTranslator", "Translator used doesn't implement IInterchangeAudioPayloadInterface."));
		return ImportAssetResult;
	}

	TOptional<UE::Interchange::FInterchangeAudioPayloadData> AudioPayloadData = AudioTranslator->GetAudioPayloadData(SourceFilePayloadKey.GetValue());
	if (!AudioPayloadData.IsSet())
	{
		LogError(Arguments, ImportAssetResult, FText::Format(LOCTEXT("InterchangeAudioSoundWaveFactory_ImportSoundWave_FailedToGetAudioPayloadData", "Failed to retrieve audio payload from file [{0}]. See output log for more details."), FText::FromString(*SourceFilePayloadKey.GetValue())));
		return ImportAssetResult;
	}

	SetUpSoundWaveFromPayload(Arguments, SoundWave, AudioPayloadData.GetValue(), ImportAssetResult);

	ImportAssetResult.ImportedObject = SoundWave;
	return ImportAssetResult;
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeAudioSoundWaveFactory::EndImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	FImportAssetResult ImportAssetResult;

	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	UObject* ExistingAsset = nullptr;
	FSoftObjectPath ReferenceObject;
	if (Arguments.AssetNode->GetCustomReferenceObject(ReferenceObject))
	{
		ExistingAsset = ReferenceObject.TryLoad();
	}

	//We need a valid imported object
	if (!ExistingAsset)
	{
		return ImportAssetResult;
	}

	if (USoundWave* SoundWaveAsset = Cast<USoundWave>(ExistingAsset))
	{
		constexpr bool bFreeResources = true;
		SoundWaveAsset->InvalidateCompressedData(bFreeResources);

		if (SoundWaveAsset->IsStreaming(nullptr))
		{
			SoundWaveAsset->LoadZerothChunk();
		}

#if WITH_EDITOR
		SoundWaveAsset->PostImport();
#endif
		ImportAssetResult.ImportedObject = SoundWaveAsset;
	}

	return ImportAssetResult;
}

void UInterchangeAudioSoundWaveFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeAudioSoundWaveFactory::SetupObject_GameThread);

	check(IsInGameThread());

	USoundWave* SoundWaveAsset = Cast<USoundWave>(Arguments.ImportedObject);
	if (!SoundWaveAsset)
	{
		return;
	}

#if WITH_EDITOR
	SoundWaveAsset->PreEditChange(nullptr);

	UInterchangeFactoryBaseNode* SoundWaveFactoryNode = Arguments.FactoryNode;
	if (!SoundWaveFactoryNode)
	{
		return;
	}

	if (Arguments.bIsReimport)
	{
		const UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(SoundWaveAsset->AssetImportData);
		const UInterchangeFactoryBaseNode* PreviousNode = nullptr;
		if (InterchangeAssetImportData)
		{
			PreviousNode = InterchangeAssetImportData->GetStoredFactoryNode(InterchangeAssetImportData->NodeUniqueID);
		}

		UInterchangeFactoryBaseNode* CurrentNode = UInterchangeFactoryBaseNode::DuplicateWithObject(SoundWaveFactoryNode, SoundWaveAsset);
		UE::Interchange::FFactoryCommon::ApplyReimportStrategyToAsset(SoundWaveAsset, PreviousNode, CurrentNode, SoundWaveFactoryNode);
	}
	else
	{
		SoundWaveFactoryNode->ApplyAllCustomAttributeToObject(SoundWaveAsset);
	}
#endif

#if WITH_EDITORONLY_DATA
	if (ensure(Arguments.SourceData))
	{
		UE::Interchange::FFactoryCommon::FSetImportAssetDataParameters SetImportAssetDataParameters(
			SoundWaveAsset,
			SoundWaveAsset->AssetImportData,
			Arguments.SourceData,
			Arguments.NodeUniqueID,
			Arguments.NodeContainer,
			Arguments.OriginalPipelines,
			Arguments.Translator);

		SoundWaveAsset->AssetImportData = UE::Interchange::FFactoryCommon::SetImportAssetData(SetImportAssetDataParameters);
	}
#endif
}

void UInterchangeAudioSoundWaveFactory::FinalizeObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeAudioSoundWaveFactory::FinalizeObject_GameThread);

	check(IsInGameThread());

	if (!Arguments.ImportedObject)
	{
		return;
	}

	if (Arguments.bIsReimport)
	{
		for (UAudioComponent* AudioComponent : ComponentsToRestart)
		{
			AudioComponent->Play();
		}
	}

#if WITH_EDITOR
	if (USoundWave* ImportedSoundWaveAsset = Cast<USoundWave>(Arguments.ImportedObject))
	{
		ImportedSoundWaveAsset->SetRedrawThumbnail(true);
	}
#endif

}

bool UInterchangeAudioSoundWaveFactory::GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const
{
#if WITH_EDITORONLY_DATA
	if (const USoundWave* SoundWave = Cast<USoundWave>(Object))
	{
		return UE::Interchange::FFactoryCommon::GetSourceFilenames(SoundWave->AssetImportData.Get(), OutSourceFilenames);
	}
#endif

	return false;
}

bool UInterchangeAudioSoundWaveFactory::SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const
{
#if WITH_EDITORONLY_DATA
	if (const USoundWave* SoundWave = Cast<USoundWave>(Object))
	{
		return UE::Interchange::FFactoryCommon::SetSourceFilename(SoundWave->AssetImportData.Get(), SourceFilename, SourceIndex);
	}
#endif

	return false;
}

void UInterchangeAudioSoundWaveFactory::BackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const USoundWave* SoundWave = Cast<USoundWave>(Object))
	{
		UE::Interchange::FFactoryCommon::BackupSourceData(SoundWave->AssetImportData.Get());
	}
#endif
}

void UInterchangeAudioSoundWaveFactory::ReinstateSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const USoundWave* SoundWave = Cast<USoundWave>(Object))
	{
		UE::Interchange::FFactoryCommon::ReinstateSourceData(SoundWave->AssetImportData.Get());
	}
#endif
}

void UInterchangeAudioSoundWaveFactory::ClearBackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const USoundWave* SoundWave = Cast<USoundWave>(Object))
	{
		UE::Interchange::FFactoryCommon::ClearBackupSourceData(SoundWave->AssetImportData.Get());
	}
#endif
}

void UInterchangeAudioSoundWaveFactory::SetUpSoundWaveFromPayload(const FImportAssetObjectParams& Arguments, USoundWave*& SoundWaveAsset, const UE::Interchange::FInterchangeAudioPayloadData& AudioPayloadData, FImportAssetResult& ImportAssetResult)
{
	if (!SoundWaveAsset)
	{
		const FText Message = LOCTEXT("InterchangeAudioSoundWaveFactory_ImportWaveData_NullAsset", "Sound wave asset was null.");
		LogError(Arguments, ImportAssetResult, Message);
		return;
	}

	if (AudioPayloadData.FactoryInfo.ChannelCount > 2)
	{
		if (!SetUpMultichannelSoundWave(Arguments, SoundWaveAsset, AudioPayloadData, ImportAssetResult))
		{
			const FText Message = LOCTEXT("InterchangeAudioSoundWaveFactory_ImportWaveData_SetUpMultiChanFailure", "Failed to set up multichannel sound wave asset.");
			LogError(Arguments, ImportAssetResult, Message);
			return;
		}
	}
	else
	{
		SoundWaveAsset->bIsAmbisonics = false;

#if WITH_EDITORONLY_DATA
		SoundWaveAsset->ChannelOffsets.Reset();
		SoundWaveAsset->ChannelSizes.Reset();
		SoundWaveAsset->RawData.UpdatePayload(FSharedBuffer::Clone(AudioPayloadData.Buffer.GetData(), AudioPayloadData.Buffer.Num()));
#endif
	}

	SoundWaveAsset->Duration = (float)AudioPayloadData.FactoryInfo.NumFrames / AudioPayloadData.FactoryInfo.SamplesPerSec;
	SoundWaveAsset->SetImportedSampleRate(AudioPayloadData.FactoryInfo.SamplesPerSec);
	SoundWaveAsset->SetSampleRate(AudioPayloadData.FactoryInfo.SamplesPerSec);
	SoundWaveAsset->NumChannels = AudioPayloadData.FactoryInfo.ChannelCount;
	SoundWaveAsset->TotalSamples = AudioPayloadData.FactoryInfo.SamplesPerSec * SoundWaveAsset->Duration;

#if WITH_EDITORONLY_DATA
	if (AudioPayloadData.WaveModInfo.TimecodeInfo)
	{
		SoundWaveAsset->SetTimecodeInfo(*AudioPayloadData.WaveModInfo.TimecodeInfo);
	}
	else
	{
		SoundWaveAsset->SetTimecodeInfo(FSoundWaveTimecodeInfo{});
	}
#endif

}

void UInterchangeAudioSoundWaveFactory::StopComponentsUsingImportedSound(USoundWave* SoundWaveAsset)
{
	if (!SoundWaveAsset)
	{
		return;
	}

	FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();
	if (!AudioDeviceManager)
	{
		return;
	}

	TArray<UAudioComponent*> AudioComponents = ComponentsToRestart;
	AudioDeviceManager->StopSoundsUsingResource(SoundWaveAsset, &AudioComponents);

	if (!SoundWaveAsset->GetResourceData())
	{
		const FName RuntimeFormat = SoundWaveAsset->GetRuntimeFormat();
		SoundWaveAsset->InitAudioResource(RuntimeFormat);
	}

	if (ComponentsToRestart.Num() > 0)
	{
		for (UAudioComponent* AudioComponent : ComponentsToRestart)
		{
			AudioComponent->Stop();
		}
	}
}

bool UInterchangeAudioSoundWaveFactory::SetUpMultichannelSoundWave(const FImportAssetObjectParams& Arguments, USoundWave*& SoundWaveAsset, const UE::Interchange::FInterchangeAudioPayloadData& AudioPayloadData, FImportAssetResult& ImportAssetResult)
{
	using namespace UE::Interchange;

	const FString RootName = Arguments.AssetName;
	const FString AmbiXTag = RootName.Right(6).ToLower();
	const FString FuMaTag = RootName.Right(5).ToLower();

	const bool bIsAmbiX = AmbiXTag == TEXT("_ambix");
	const bool bIsFuMa = FuMaTag == TEXT("_fuma");

	const FSoundWaveFactoryInfo SoundWaveFactoryInfo = AudioPayloadData.FactoryInfo;

	if ((bIsAmbiX || bIsFuMa) && SoundWaveFactoryInfo.ChannelCount != 4)
	{
		const FText Message = LOCTEXT("InterchangeAudioSoundWaveFactory_SetUpMultichan_InvalidChanCountAmbi", "Tried to import ambisonics format file but the file does not contain exactly four channels.");
		LogError(Arguments, ImportAssetResult, Message);
		return false;
	}

	const int16* SampleDataStart = (int16*)AudioPayloadData.Buffer.GetData() + SoundWaveFactoryInfo.SampleDataOffset;
	// Starting or ending on a non-zero sample creates a pop unless you have very specific use cases
	// (doesn't apply to ambi)
	if (!bIsFuMa && !bIsAmbiX)
	{
		for (int32 Chan = 0; Chan < SoundWaveFactoryInfo.ChannelCount; ++Chan)
		{
			const int16* FirstFrame = SampleDataStart;
			const int16* LastFrame = FirstFrame + (SoundWaveFactoryInfo.NumFrames - 1) * SoundWaveFactoryInfo.ChannelCount;

			FirstFrame += Chan;
			LastFrame += Chan;

			if (FirstFrame[0] != 0)
			{
				//Warn->Logf(ELogVerbosity::Warning, TEXT("Channel %d starts with a non zero value (%d) - will likely pop and not loop well: '%s'"), Chan, *FirstFrame, *Name.ToString());
			}

			if (LastFrame[0] != 0)
			{
				//Warn->Logf(ELogVerbosity::Warning, TEXT("Channel %d ends with a non zero value (%d) - will likely pop and not loop well: '%s'"), Chan, *FirstFrame, *Name.ToString());
			}
		}
	}

#if WITH_EDITORONLY_DATA
	SoundWaveAsset->ChannelOffsets.Empty(SPEAKER_Count);
	SoundWaveAsset->ChannelOffsets.AddZeroed(SPEAKER_Count);

	SoundWaveAsset->ChannelSizes.Empty(SPEAKER_Count);
	SoundWaveAsset->ChannelSizes.AddZeroed(SPEAKER_Count);
#endif

	TArray<int32> ChannelIndices;
	if (SoundWaveFactoryInfo.ChannelCount == 4)
	{
		ChannelIndices = {
			SPEAKER_FrontLeft,
			SPEAKER_FrontRight,
			SPEAKER_LeftSurround,
			SPEAKER_RightSurround
		};
	}
	else if (SoundWaveFactoryInfo.ChannelCount == 6)
	{
		ChannelIndices = {
			SPEAKER_FrontLeft,
			SPEAKER_FrontRight,
			SPEAKER_FrontCenter,
			SPEAKER_LowFrequency,
			SPEAKER_LeftSurround,
			SPEAKER_RightSurround
		};
	}
	else if (SoundWaveFactoryInfo.ChannelCount == 8)
	{
		ChannelIndices = {
			SPEAKER_FrontLeft,
			SPEAKER_FrontRight,
			SPEAKER_FrontCenter,
			SPEAKER_LowFrequency,
			SPEAKER_LeftSurround,
			SPEAKER_RightSurround,
			SPEAKER_LeftBack,
			SPEAKER_RightBack
		};
	}
	else
	{
		const FText Message = LOCTEXT("InterchangeAudioSoundWaveFactory_SetUpMultichan_UnknownChanConfig", "Unable to import multichannel file with unsupported channel configuration.");
		LogError(Arguments, ImportAssetResult, Message);
		return false;
	}

	int32 TotalSize = 0;
	TArray<uint8> RawChannelWaveData[SPEAKER_Count];
	TArray<int16> DeinterleavedAudioScratchBuffer;
	const int16* SampleDataBuffer = SampleDataStart;

	check(SoundWaveFactoryInfo.ChannelCount == ChannelIndices.Num());
	for (int32 Chan = 0; Chan < SoundWaveFactoryInfo.ChannelCount; ++Chan)
	{
		DeinterleavedAudioScratchBuffer.Empty();
		for (int32 Frame = 0; Frame < SoundWaveFactoryInfo.NumFrames; ++Frame)
		{
			const int32 SampleIndex = Frame * SoundWaveFactoryInfo.ChannelCount + Chan;
			DeinterleavedAudioScratchBuffer.Add(SampleDataBuffer[SampleIndex]);
		}

		SerializeWaveFile(RawChannelWaveData[Chan]
			, (uint8*)DeinterleavedAudioScratchBuffer.GetData()
			, SoundWaveFactoryInfo.NumFrames * sizeof(int16)
			, 1
			, SoundWaveFactoryInfo.SamplesPerSec);
#if WITH_EDITORONLY_DATA
		SoundWaveAsset->ChannelOffsets[ChannelIndices[Chan]] = TotalSize;
		const int32 ChannelSize = RawChannelWaveData[Chan].Num();
		SoundWaveAsset->ChannelSizes[ChannelIndices[Chan]] = ChannelSize;
		TotalSize += ChannelSize;
#endif
	}

	FUniqueBuffer EditableBuffer = FUniqueBuffer::Alloc(TotalSize);
	uint8* LockedData = (uint8*)EditableBuffer.GetData();
	int32 RawDataOffset = 0;

	if (bIsAmbiX || bIsFuMa)
	{
		check(SoundWaveFactoryInfo.ChannelCount == 4)
		SoundWaveAsset->bIsAmbisonics = true;
	}
	if (bIsFuMa)
	{
		int32 FuMaChannelIndices[4] = { 0, 2, 3, 1 };
		for (const int32 ChannelIndex : FuMaChannelIndices)
		{
			const int32 ChannelSize = RawChannelWaveData[ChannelIndex].Num();
			FMemory::Memcpy(LockedData + RawDataOffset, RawChannelWaveData[ChannelIndex].GetData(), ChannelSize);
			RawDataOffset += ChannelSize;
		}
	}
	else
	{
		for (int32 Chan = 0; Chan < SoundWaveFactoryInfo.ChannelCount; ++Chan)
		{
			const int32 ChannelSize = RawChannelWaveData[Chan].Num();
			FMemory::Memcpy(LockedData + RawDataOffset, RawChannelWaveData[Chan].GetData(), ChannelSize);
			RawDataOffset += ChannelSize;
		}
	}
#if WITH_EDITORONLY_DATA
	SoundWaveAsset->RawData.UpdatePayload(EditableBuffer.MoveToShared());
#endif
	return true;
}

void UInterchangeAudioSoundWaveFactory::LogError(const FImportAssetObjectParams& Arguments, FImportAssetResult& ImportAssetResult, const FText& ErrorText)
{
	UInterchangeResultError_Generic* ErrorMessage = AddMessage<UInterchangeResultError_Generic>();
	if (ensure(ErrorMessage && Arguments.Translator))
	{
		ErrorMessage->AssetType = GetFactoryClass();

		if (const UInterchangeSourceData* SourceData = Arguments.Translator->GetSourceData())
		{
			const FString Filename = SourceData->GetFilename();
			ErrorMessage->SourceAssetName = Arguments.AssetName;
			ErrorMessage->DestinationAssetName = Arguments.AssetName;
			ErrorMessage->InterchangeKey = FPaths::GetBaseFilename(Filename);
		}
		else
		{
			ErrorMessage->InterchangeKey = TEXT("Undefined");
		}

		ImportAssetResult = FImportAssetResult{
			.bIsFactorySkipAsset = true,
			.ImportedObject = nullptr
		};

		ErrorMessage->Text = ErrorText;
	}
}

#undef LOCTEXT_NAMESPACE
