// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Fbx/InterchangeFbxTranslator.h"

#include "AnalyticsEventAttribute.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "HAL/PlatformFileManager.h"
#include "InterchangeAnalyticsHandlerBase.h"
#include "InterchangeCommonAnimationPayload.h"
#include "InterchangeDispatcher.h"
#include "FBX/InterchangeDispatcherFBXTasks.h"
#include "InterchangeImportLog.h"
#include "InterchangeManager.h"
#include "InterchangeTranslatorHelper.h"
#include "Mesh/InterchangeMeshPayload.h"
#include "Misc/AxisDisplayInfo.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeSourceNode.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/LargeMemoryReader.h"
#include "StaticMeshOperations.h"
#include "Texture/InterchangeImageWrapperTranslator.h"
#include "Texture/InterchangeTexturePayloadData.h"
#include "UObject/GCObjectScopeGuard.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeFbxTranslator)

static bool GInterchangeEnableFBXImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnableFBXImport(
	TEXT("Interchange.FeatureFlags.Import.FBX"),
	GInterchangeEnableFBXImport,
	TEXT("Whether FBX support is enabled."),
	ECVF_Default);

static bool GInterchangeEnableFBXLevelImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnableFBXLevelImport(
	TEXT("Interchange.FeatureFlags.Import.FBX.ToLevel"),
	GInterchangeEnableFBXLevelImport,
	TEXT("Whether support for FBX level import is enabled."),
	ECVF_Default);

static bool GInterchangeFBXTranslatorUseWorker = false;
static FAutoConsoleVariableRef CCvarInterchangeFBXTranslatorUseWorker(
	TEXT("Interchange.FeatureFlags.Translator.UseWorker.FBX"),
	GInterchangeFBXTranslatorUseWorker,
	TEXT("Whether FBX translator can be execute in parallel using the InterchangeWorker process."),
	ECVF_Default);

namespace UE::Interchange::Private
{
	static const FString FBXAnalyticsIdentifier = TEXT("Interchange.Usage.Import.FBX");
	
	enum EFBXTranslatorAnalyticsResult : uint8
	{
		Successful,
		InputFileNotFound,
		TranslatorFailed,
		FBXParserFailed,
		InterchangeDispatcherFailed,
		InterchangeWorkerFailed,
	};

	enum EFBXParserType : uint8
	{
		Unspecified,
		FBX_SDK,
		uFBX,
	};

	void  AddAnalytics (const EFBXTranslatorAnalyticsResult& TranslatorAnalyticsResult, TObjectPtr<UInterchangeAnalyticsHandlerBase> FBXAnalyticsHandler, bool bIsUsingInterchangeWorker = false, EFBXParserType FBXParser = EFBXParserType::Unspecified)
	{
		if (!FBXAnalyticsHandler)
		{
			return;
		}

		TArray<FAnalyticsEventAttribute> FBXAnalyticsAttribs;

		if (bIsUsingInterchangeWorker)
		{
			FBXParser = EFBXParserType::FBX_SDK;
		}

		FBXAnalyticsAttribs.Add(FAnalyticsEventAttribute(TEXT("InterchangeWorkerUsed"), bIsUsingInterchangeWorker));

		switch (FBXParser)
		{
		case FBX_SDK:
			FBXAnalyticsAttribs.Add(FAnalyticsEventAttribute(TEXT("FBXParser"), "FBX SDK"));
			break;
		case uFBX:
			FBXAnalyticsAttribs.Add(FAnalyticsEventAttribute(TEXT("FBXParser"), "uFBX"));
			break;
		case Unspecified:
			// Falls through...
		default:
			break;
		}

		switch (TranslatorAnalyticsResult)
		{
		case Successful:
			FBXAnalyticsAttribs.Add(FAnalyticsEventAttribute(TEXT("TranslationStatus"), "Successful."));
			break;
		case InputFileNotFound:
			FBXAnalyticsAttribs.Add(FAnalyticsEventAttribute(TEXT("TranslationStatus"), "[Failed] Input File Not Found."));
			break;
		case TranslatorFailed:
			FBXAnalyticsAttribs.Add(FAnalyticsEventAttribute(TEXT("TranslationStatus"), "[Failed] Interchange Translator Failed."));
			break;
		case FBXParserFailed:
			FBXAnalyticsAttribs.Add(FAnalyticsEventAttribute(TEXT("TranslationStatus"), "[Failed] Parsing Failed."));
			break;
		case InterchangeDispatcherFailed:
			FBXAnalyticsAttribs.Add(FAnalyticsEventAttribute(TEXT("TranslationStatus"), "[Failed] Interchange Dispatcher Failed."));
			break;
		case InterchangeWorkerFailed:
			FBXAnalyticsAttribs.Add(FAnalyticsEventAttribute(TEXT("TranslationStatus"), "[Failed] Interchange Worker Failed."));
			break;
		default:
			break;
		}

		FBXAnalyticsHandler->Append(FBXAnalyticsIdentifier, FBXAnalyticsAttribs);
	}

	void ApplyTranslatorMessage(const UInterchangeFbxTranslator* Translator, const FString& JsonMessage)
	{
		if (UInterchangeResult* InterchangeResult = UInterchangeResult::FromJson(JsonMessage))
		{
			//Downgrade warning message to display log when we are in automation
			if (GIsAutomationTesting && InterchangeResult->IsA(UInterchangeResultWarning::StaticClass()))
			{
				UE_LOGF(LogInterchangeImport, Display, "%ls", *InterchangeResult->GetText().ToString());
			}
			else
			{
				Translator->AddMessage(InterchangeResult);
			}
		}
	}
} //ns UE::Interchange::Private

void UInterchangeFbxTranslatorSettings::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	bUsingLUFCoordinateSystem = AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward;
	CoordinateSystemPolicy = bConvertScene ? (bForceFrontXAxis ? EInterchangeCoordinateSystemPolicy::MatchUpForwardAxes : EInterchangeCoordinateSystemPolicy::MatchUpAxis) : EInterchangeCoordinateSystemPolicy::KeepXYZAxes;
}

#if WITH_EDITOR
void UInterchangeFbxTranslatorSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UInterchangeFbxTranslatorSettings, CoordinateSystemPolicy))
	{
		bConvertScene = CoordinateSystemPolicy != EInterchangeCoordinateSystemPolicy::KeepXYZAxes;
		bForceFrontXAxis = CoordinateSystemPolicy == EInterchangeCoordinateSystemPolicy::MatchUpForwardAxes;
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

#define INTERCHANGE_FBX_PATH TEXT("Interchange/Fbx")
UInterchangeFbxTranslator::UInterchangeFbxTranslator()
{
	Dispatcher = nullptr;
	bUseWorkerImport = false;
	if (HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		//Class default object should not use any resources
		return;
	}
	
	FGuid RandomGuid;
	FPlatformMisc::CreateGuid(RandomGuid);
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	const FString ProjectSavedDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir());
	const FString RamdomGuidDir = RandomGuid.ToString(EGuidFormats::Base36Encoded);
	if (!PlatformFile.DirectoryExists(*ProjectSavedDir))
	{
		PlatformFile.CreateDirectory(*ProjectSavedDir);
	}
	const FString InterchangeDir = FPaths::Combine(ProjectSavedDir, INTERCHANGE_FBX_PATH);
	if (!PlatformFile.DirectoryExists(*InterchangeDir))
	{
		PlatformFile.CreateDirectory(*InterchangeDir);
	}
	ResultFolder = FPaths::Combine(InterchangeDir, RamdomGuidDir);
	if (!PlatformFile.DirectoryExists(*ResultFolder))
	{
		PlatformFile.CreateDirectory(*ResultFolder);
	}

	//Run the import in parallel only if we can start interchange worker
	if (GInterchangeFBXTranslatorUseWorker)
	{
		//Create the dispatcher
		Dispatcher = MakeUnique<UE::Interchange::FInterchangeDispatcher>(ResultFolder, 1);

		if (ensure(Dispatcher.IsValid()))
		{
			Dispatcher->StartProcess();
		}

		if (Dispatcher->IsValid())
		{
			bUseWorkerImport = true;
		}
		else
		{
			Dispatcher.Reset();
		}
	}
}

void UInterchangeFbxTranslator::CleanUpTemporaryFolder()
{
	//Clean up the interchange fbx temporary folder.
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	const FString ProjectSavedDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir());
	if (PlatformFile.DirectoryExists(*ProjectSavedDir))
	{
		const FString InterchangeDir = FPaths::Combine(ProjectSavedDir, INTERCHANGE_FBX_PATH);
		if (PlatformFile.DirectoryExists(*InterchangeDir))
		{
			constexpr bool RequireExists = false;
			//Delete recursively folder's content
			constexpr bool Tree = true;
			IFileManager::Get().DeleteDirectory(*InterchangeDir, RequireExists, Tree);
		}
	}
}

bool UInterchangeFbxTranslator::IsThreadSafe() const
{
	return bUseWorkerImport || (CacheFbxTranslatorSettings ? CacheFbxTranslatorSettings->bUseUfbxParser : false);
}

EInterchangeTranslatorType UInterchangeFbxTranslator::GetTranslatorType() const
{
	return GInterchangeEnableFBXLevelImport ? EInterchangeTranslatorType::Scenes : EInterchangeTranslatorType::Assets;
}

EInterchangeTranslatorAssetType UInterchangeFbxTranslator::GetSupportedAssetTypes() const
{
	//fbx translator support Meshes, Materials and animation
	return EInterchangeTranslatorAssetType::Materials | EInterchangeTranslatorAssetType::Meshes | EInterchangeTranslatorAssetType::Animations;
}

TArray<FString> UInterchangeFbxTranslator::GetSupportedFormats() const
{
#if WITH_EDITOR
	if (GInterchangeEnableFBXImport)
	{
		TArray<FString> Formats{ TEXT("fbx;Filmbox") };
		return Formats;
	}
#endif
	return TArray<FString>{};
}

bool UInterchangeFbxTranslator::Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeFbxTranslator::Translate);
	using namespace UE::Interchange::Private;

	if (AnalyticsHandler)
	{
		// This is to ensure that the keys are not duplicated when the Analytics Event is actually sent.
		// Changing the Translator Settings in the import dialog will call the Translate function on the Translators again.
		AnalyticsHandler->Clear(FBXAnalyticsIdentifier);
	}

	if (HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		//Class default should never be use has an active translator
		ensure(!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject));
		UInterchangeResultError_Generic* ErrorResult = AddMessage<UInterchangeResultError_Generic>();
		ErrorResult->SourceAssetName = SourceData ? SourceData->GetFilename() : FString();
		ErrorResult->Text = NSLOCTEXT("UInterchangeFbxTranslator", "Translate_DoNotUseClassDefault", "Class default should not be use has an active translator.");
		AddAnalytics(EFBXTranslatorAnalyticsResult::TranslatorFailed, AnalyticsHandler, bUseWorkerImport);
		return false;
	}

	//Make sure the hash is compute here in asynchronous mode
	GetSourceData()->GetFileContentHash();
	FString FilePath = GetSourceData()->GetFilename();
	if (!FPaths::FileExists(FilePath))
	{
		AddAnalytics(EFBXTranslatorAnalyticsResult::InputFileNotFound, AnalyticsHandler, bUseWorkerImport);
		return false;
	}

	GetSettings();
	ensure(CacheFbxTranslatorSettings);
	const bool bConvertScene = CacheFbxTranslatorSettings ? CacheFbxTranslatorSettings->bConvertScene : true;
	const bool bForceFrontXAxis = CacheFbxTranslatorSettings ? CacheFbxTranslatorSettings->bForceFrontXAxis : false;
	const bool bConvertSceneUnit = CacheFbxTranslatorSettings ? CacheFbxTranslatorSettings->bConvertSceneUnit : true;
	const bool bKeepFbxNamespace = CacheFbxTranslatorSettings ? CacheFbxTranslatorSettings->bKeepFbxNamespace : false;
	const bool bConsiderClusterBeforePoseForMeshBindPose = CacheFbxTranslatorSettings ? CacheFbxTranslatorSettings->bConsiderClusterBeforePoseForMeshBindPose : false;
	const bool bUseUfbxParser = CacheFbxTranslatorSettings ? CacheFbxTranslatorSettings->bUseUfbxParser : false;

	if(CacheFbxTranslatorSettings)
	{
		UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(&BaseNodeContainer);

		// Store requested front axis direction in source node. It will be used by the factory.
		// If neither bConvertScene nor bForceFrontXAxis, the front axis is set by default to EAxis::Y as the actual orientation is unknown.
		SourceNode->SetCustomSkeletalMeshFrontAxis(int8(CacheFbxTranslatorSettings->bConvertScene ? (CacheFbxTranslatorSettings->bForceFrontXAxis ? EAxis::X : EAxis::Y) : EAxis::Y));
	}

	StartTime = FPlatformTime::Cycles64();
	bool bEncounteredParserErrors = false;
	if (bUseWorkerImport)
	{
		if (!Dispatcher.IsValid())
		{
			AddAnalytics(InterchangeDispatcherFailed, AnalyticsHandler, bUseWorkerImport);
			return false;
		}
		//Create a json command to read the fbx file
		FString JsonCommand = CreateLoadFbxFileCommand(FilePath, bConvertScene, bForceFrontXAxis, bConvertSceneUnit, bKeepFbxNamespace, bConsiderClusterBeforePoseForMeshBindPose);
		int32 TaskIndex = Dispatcher->AddTask(JsonCommand);

		//Blocking call until all tasks are executed
		Dispatcher->WaitAllTaskToCompleteExecution();

		FString WorkerFatalError = Dispatcher->GetInterchangeWorkerFatalError();
		if (!WorkerFatalError.IsEmpty())
		{
			AddMessage(UInterchangeResult::FromJson(WorkerFatalError));
			AddAnalytics(EFBXTranslatorAnalyticsResult::InterchangeWorkerFailed, AnalyticsHandler, bUseWorkerImport);
		}

		UE::Interchange::ETaskState TaskState;
		FString JsonResult;
		TArray<FString> JsonMessages;
		Dispatcher->GetTaskState(TaskIndex, TaskState, JsonResult, JsonMessages);

		// Parse the Json messages into UInterchangeResults
		for (const FString& JsonMessage : JsonMessages)
		{
			if (UInterchangeResult* InterchangeResult = UInterchangeResult::FromJson(JsonMessage))
			{
				
				if (GIsAutomationTesting && InterchangeResult->GetResultType() == EInterchangeResultType::Warning)
				{
					UE_LOGF(LogInterchangeImport, Display, "%ls", *InterchangeResult->GetText().ToString());
				}
				else
				{
					AddMessage(InterchangeResult);
				}

				if (!bEncounteredParserErrors && InterchangeResult->GetResultType() == EInterchangeResultType::Error)
				{
					AddAnalytics(EFBXTranslatorAnalyticsResult::FBXParserFailed, AnalyticsHandler, bUseWorkerImport);
					bEncounteredParserErrors = true;
				}
			}
		}

		if (TaskState != UE::Interchange::ETaskState::ProcessOk)
		{
			AddAnalytics(EFBXTranslatorAnalyticsResult::InterchangeWorkerFailed, AnalyticsHandler, bUseWorkerImport);
			return false;
		}
		//Grab the result file and fill the BaseNodeContainer
		UE::Interchange::FJsonLoadSourceCmd::JsonResultParser ResultParser;
		ResultParser.FromJson(JsonResult);
		FString BaseNodeContainerFilename = ResultParser.GetResultFilename();

		//Parse the filename to fill the container
		BaseNodeContainer.LoadFromFile(BaseNodeContainerFilename);
	}
	else
	{
#if WITH_EDITOR
		FbxParser.Reset(bUseUfbxParser);
		FbxParser.SetResultContainer(Results);
		FbxParser.SetConvertSettings(bConvertScene, bForceFrontXAxis, bConvertSceneUnit, bKeepFbxNamespace, bConsiderClusterBeforePoseForMeshBindPose);
		FbxParser.LoadFbxFile(FilePath, BaseNodeContainer);

		TArray<UInterchangeResult*> TranslatorResults = Results->GetResults();
		if (!TranslatorResults.IsEmpty())
		{
			for (const UInterchangeResult* InterchangeResult : TranslatorResults)
			{
				if (InterchangeResult->GetResultType() == EInterchangeResultType::Error)
				{
					AddAnalytics(
						EFBXTranslatorAnalyticsResult::FBXParserFailed, 
						AnalyticsHandler, 
						bUseWorkerImport, 
						bUseUfbxParser ? EFBXParserType::uFBX : EFBXParserType::FBX_SDK
					);
					bEncounteredParserErrors = true;
					break;
				}
			}
		}
#endif
	}

	// Log time spent to import incoming file in minutes and seconds
	double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

	int ElapsedMin = int(ElapsedSeconds / 60.0);
	ElapsedSeconds -= 60.0 * (double)ElapsedMin;
	FileName = FPaths::GetCleanFilename(FilePath);
#if WITH_EDITOR
	UE_LOGF(LogInterchangeImport, Log, "Translation (%ls) of %ls in [%d min %.3f s]", FbxParser.GetName(), *FileName, ElapsedMin, ElapsedSeconds);
#endif
	
	if (!bEncounteredParserErrors)
	{
		AddAnalytics(
			EFBXTranslatorAnalyticsResult::Successful, 
			AnalyticsHandler, 
			bUseWorkerImport, 
			bUseUfbxParser ? EFBXParserType::uFBX : EFBXParserType::FBX_SDK
		);
	}

	return true;
}

void UInterchangeFbxTranslator::ReleaseSource()
{
	if (Dispatcher.IsValid())
	{
		//Do not block the main thread
		Dispatcher->StopProcess(!IsInGameThread());
	}
#if WITH_EDITOR

	// #interchange_debug
	if (StartTime > 0)
	{
		double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

		int ElapsedMin = int(ElapsedSeconds / 60.0);
		ElapsedSeconds -= 60.0 * (double)ElapsedMin;

		UE_LOGF(LogInterchangeImport, Log, "Imported (%ls) of %ls in [%d min %.3f s]", FbxParser.GetName(), *FileName, ElapsedMin, ElapsedSeconds);

		ElapsedSeconds = FPlatformTime::ToSeconds64(MeshPayloadTime);
		ElapsedMin = int(ElapsedSeconds / 60.0);
		ElapsedSeconds -= 60.0 * (double)ElapsedMin;
		UE_LOGF(LogInterchangeImport, Log, "MeshPayload (%ls) of %ls, [%lld %lld %lld], in [%d min %.3f s]", FbxParser.GetName(), *FileName, MeshMeshCount, MeshTriangleCount, MeshVertexCount, ElapsedMin, ElapsedSeconds);
	}

	FbxParser.ReleaseResources();
#endif
	//Delete the result folder if we are not running with the worker, in the other case the dispatcher will delete the folder on TerminateProcess.
	if (!bUseWorkerImport)
	{
		constexpr bool RequireExists = false;
		//Delete recursively folder's content
		constexpr bool Tree = true;
		IFileManager::Get().DeleteDirectory(*ResultFolder, RequireExists, Tree);
	}
	ResultFolder.Empty();

	if (CacheFbxTranslatorSettings)
	{
		CacheFbxTranslatorSettings->ClearFlags(RF_Standalone);
		CacheFbxTranslatorSettings = nullptr;
	}
}

void UInterchangeFbxTranslator::ImportFinish()
{
	if (Dispatcher.IsValid())
	{
		Dispatcher->TerminateProcess();
	}
}

UInterchangeTranslatorSettings* UInterchangeFbxTranslator::GetSettings() const
{
	if (!CacheFbxTranslatorSettings)
	{
		CacheFbxTranslatorSettings = DuplicateObject<UInterchangeFbxTranslatorSettings>(UInterchangeFbxTranslatorSettings::StaticClass()->GetDefaultObject<UInterchangeFbxTranslatorSettings>(), GetTransientPackage());
		CacheFbxTranslatorSettings->LoadSettings();
		CacheFbxTranslatorSettings->PostDuplicate(false);
		//Since we duplicate the CDO we want to remove the archetype flag
		CacheFbxTranslatorSettings->ClearFlags(RF_ArchetypeObject);
		CacheFbxTranslatorSettings->SetFlags(RF_Standalone);
		CacheFbxTranslatorSettings->ClearInternalFlags(EInternalObjectFlags::Async);
	}
	return CacheFbxTranslatorSettings;
}

void UInterchangeFbxTranslator::SetSettings(const UInterchangeTranslatorSettings* InterchangeTranslatorSettings)
{
	if (CacheFbxTranslatorSettings)
	{
		CacheFbxTranslatorSettings->ClearFlags(RF_Standalone);
		CacheFbxTranslatorSettings->ClearInternalFlags(EInternalObjectFlags::Async);
		CacheFbxTranslatorSettings = nullptr;
	}
	if (const UInterchangeFbxTranslatorSettings* InterchangeFbxTranslatorSettings = Cast<UInterchangeFbxTranslatorSettings>(InterchangeTranslatorSettings))
	{
		CacheFbxTranslatorSettings = DuplicateObject<UInterchangeFbxTranslatorSettings>(InterchangeFbxTranslatorSettings, GetTransientPackage());
		CacheFbxTranslatorSettings->ClearInternalFlags(EInternalObjectFlags::Async);
		CacheFbxTranslatorSettings->SetFlags(RF_Standalone);
		// #interchange_debug
		GetMutableDefault<UInterchangeFbxTranslatorSettings>()->bUseUfbxParser = InterchangeFbxTranslatorSettings->bUseUfbxParser;
	}
}

TOptional<UE::Interchange::FImportImage> UInterchangeFbxTranslator::GetTexturePayloadData(const FString& PayLoadKey, TOptional<FString>& AlternateTexturePath) const
{
	if (PayLoadKey.IsEmpty())
	{
		return TOptional<UE::Interchange::FImportImage>();
	}
#if WITH_EDITOR
	TOptional<TArray64<uint8>> ImageData = TOptional<TArray64<uint8>>();
	if (FbxParser.FetchTexturePayload(PayLoadKey, ImageData))
	{
		UInterchangeImageWrapperTranslator* ImageWrapperTranslator = NewObject<UInterchangeImageWrapperTranslator>(GetTransientPackage(), NAME_None);
		ImageWrapperTranslator->SetResultsContainer(Results);
		TOptional<UE::Interchange::FImportImage> TexturePayloadData = ImageWrapperTranslator->GetTexturePayloadDataFromBuffer(*ImageData);
		ImageWrapperTranslator->ClearInternalFlags(EInternalObjectFlags::Async);

		// todo: TexturePayloadData.GetValue().CompressionSettings = TextureUsedAsNormal ? TC_Normalmap : TC_Default;
		return TexturePayloadData;
	}
#endif

	UE::Interchange::Private::FScopedTranslator ScopedTranslator(PayLoadKey, Results, AnalyticsHandler);
	const IInterchangeTexturePayloadInterface* TextureTranslator = ScopedTranslator.GetPayLoadInterface<IInterchangeTexturePayloadInterface>();
	if (!TextureTranslator)
	{
		// TextureTranslator can can nullptr in case where
		// UInterchangeTranslatorBase::CanImportSourceData implementation return false
		return TOptional<UE::Interchange::FImportImage>();
	}
	AlternateTexturePath = PayLoadKey;
	return TextureTranslator->GetTexturePayloadData(PayLoadKey, AlternateTexturePath);
}

TOptional<UE::Interchange::FMeshPayloadData> UInterchangeFbxTranslator::GetMeshPayloadData(const FInterchangeMeshPayLoadKey& PayLoadKey, const UE::Interchange::FAttributeStorage& PayloadAttributes) const
{
	using namespace UE::Interchange;
	FTransform MeshGlobalTransform;	
	PayloadAttributes.GetAttribute(UE::Interchange::FAttributeKey{ MeshPayload::Attributes::MeshGlobalTransform }, MeshGlobalTransform);

	TSharedPtr<TPromise<TOptional<UE::Interchange::FMeshPayloadData>>> Promise = MakeShared<TPromise<TOptional<UE::Interchange::FMeshPayloadData>>>();

	auto OnPayloadReady = [this, Promise, PayLoadKey](const FString& MeshPayloadFilename)
		{
			//Mesh payload file generation can fail due to invalid Mesh (for ep No Polygons/Only Degenerate Polygons)
			if (!FPaths::FileExists(MeshPayloadFilename))
			{
				UE_LOGF(LogInterchangeImport, Warning, "Expected mesh payload file does not exist for PayloadKey: %ls", *PayLoadKey.UniqueId);

				Promise->SetValue(TOptional<UE::Interchange::FMeshPayloadData>());
				return;
			}

			UE::Interchange::FMeshPayloadData MeshPayloadData;
			MeshPayloadData.MeshDescription.Empty();

			// All sub object should be gone with the reset
			TArray64<uint8> Buffer;
			FFileHelper::LoadFileToArray(Buffer, *MeshPayloadFilename);
			uint8* FileData = Buffer.GetData();
			int64 FileDataSize = Buffer.Num();
			if (FileDataSize < 1)
			{
				// Nothing to load from this file
				Promise->SetValue(TOptional<UE::Interchange::FMeshPayloadData>());
				return;
			}

			switch (PayLoadKey.Type)
			{
			case EInterchangeMeshPayLoadType::STATIC:
			case EInterchangeMeshPayLoadType::SKELETAL:
			{
				// Buffer keeps the ownership of the data, the large memory reader is use to serialize the TMap
				FLargeMemoryReader Ar(FileData, FileDataSize);
				MeshPayloadData.MeshDescription.Serialize(Ar);

				// This is a static mesh payload can contain skinned data if we need to convert skeletalmesh to staticmesh
				bool bFetchSkinnedData = false;
				Ar << bFetchSkinnedData;
				if (bFetchSkinnedData)
				{
					Ar << MeshPayloadData.JointNames;
				}
			}
			break;
			case EInterchangeMeshPayLoadType::MORPHTARGET:
			{
				//Buffer keep the ownership of the data, the large memory reader is use to serialize the TMap
				FLargeMemoryReader Ar(FileData, FileDataSize);
				MeshPayloadData.MeshDescription.Serialize(Ar);
			}
			break;
			case EInterchangeMeshPayLoadType::NONE:
			default:
				break;
			}

			if (!FStaticMeshOperations::ValidateAndFixData(MeshPayloadData.MeshDescription, PayLoadKey.UniqueId))
			{
				UInterchangeResultError_Generic* ErrorResult = AddMessage<UInterchangeResultError_Generic>();
				ErrorResult->SourceAssetName = SourceData ? SourceData->GetFilename() : FString();
				ErrorResult->Text = NSLOCTEXT("UInterchangeFbxTranslator", "GetMeshPayloadData_ValidateMeshDescriptionFail", "Invalid mesh data (NAN) was found and fix to zero. Mesh render can be bad.");
			}

			Promise->SetValue(MoveTemp(MeshPayloadData));
		};

	if (!bUseWorkerImport)
	{
#if WITH_EDITOR
		int64 PayloadStartTime = FPlatformTime::Cycles64();

		UE::Interchange::FMeshPayloadData MeshPayloadData;
		MeshPayloadData.MeshDescription.Empty();
		if (FbxParser.FetchMeshPayload(PayLoadKey.UniqueId, MeshGlobalTransform, MeshPayloadData))
		{
			if (!FStaticMeshOperations::ValidateAndFixData(MeshPayloadData.MeshDescription, PayLoadKey.UniqueId))
			{
				UInterchangeResultError_Generic* ErrorResult = AddMessage<UInterchangeResultError_Generic>();
				ErrorResult->SourceAssetName = SourceData ? SourceData->GetFilename() : FString();
				ErrorResult->Text = NSLOCTEXT("UInterchangeFbxTranslator", "GetMeshPayloadData_ValidateMeshDescriptionFail", "Invalid mesh data (NAN) was found and fix to zero. Mesh render can be bad.");
			}
		}
		else
		{
			// ensure(false);
		}

		Promise->SetValue(MoveTemp(MeshPayloadData));

		MeshPayloadTime += FPlatformTime::Cycles64() - PayloadStartTime;
		MeshMeshCount++;
		MeshTriangleCount += MeshPayloadData.MeshDescription.Triangles().Num();
		MeshVertexCount += MeshPayloadData.MeshDescription.Vertices().Num();
#endif
	}
	else
	{
		if (!Dispatcher.IsValid())
		{
			return TOptional<UE::Interchange::FMeshPayloadData>();
		}

		// Create a json command to read the fbx file
		FString JsonCommand = CreateFetchMeshPayloadFbxCommand(PayLoadKey.UniqueId, MeshGlobalTransform);
		const int32 CreatedTaskIndex = Dispatcher->AddTask(JsonCommand, FInterchangeDispatcherTaskCompleted::CreateLambda([this, Promise, PayLoadKey, OnPayloadReadyClosure = MoveTemp(OnPayloadReady)](const int32 TaskIndex)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeFbxTranslator::GetStaticMeshPayloadData::Dispatcher->AddTaskDone)
					UE::Interchange::ETaskState TaskState;
				FString JsonResult;
				TArray<FString> JsonMessages;
				Dispatcher->GetTaskState(TaskIndex, TaskState, JsonResult, JsonMessages);

				// Parse the Json messages into UInterchangeResults
				for (const FString& JsonMessage : JsonMessages)
				{
					UE::Interchange::Private::ApplyTranslatorMessage(this, JsonMessage);
				}

				if (TaskState != UE::Interchange::ETaskState::ProcessOk)
				{
					Promise->SetValue(TOptional<UE::Interchange::FMeshPayloadData>());
					return;
				}

				// Grab the result file and fill the BaseNodeContainer
				UE::Interchange::FJsonFetchMeshPayloadCmd::JsonResultParser ResultParser;
				ResultParser.FromJson(JsonResult);
				FString MeshPayloadFilename = ResultParser.GetResultFilename();

				OnPayloadReadyClosure(MeshPayloadFilename);
			}));

		// The task was not added to the dispatcher
		if (CreatedTaskIndex == INDEX_NONE)
		{
			Promise->SetValue(TOptional<UE::Interchange::FMeshPayloadData>{});
		}
	}

	//We must stall since the payload async is now control by the interchange task system
	return Promise->GetFuture().Get();
}

TArray<UE::Interchange::FAnimationPayloadData> UInterchangeFbxTranslator::GetAnimationPayloadData(const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries) const
{
	TArray<UE::Interchange::FAnimationPayloadData> PayloadDataResults;
	
	if (PayloadQueries.Num() == 0)
	{
		return PayloadDataResults;
	}

	bool bBakedQueries = PayloadQueries[0].PayloadKey.Type == EInterchangeAnimationPayLoadType::BAKED;

	auto OnPayloadReady = [](const UE::Interchange::FAnimationPayloadQuery& PayloadQuery, const FString& AnimationPayloadFile)
		{
			if (!ensure(FPaths::FileExists(AnimationPayloadFile)))
			{
				// TODO log an error saying the payload file does not exist even if the get payload command succeeded
				return TOptional<UE::Interchange::FAnimationPayloadData>();
			}

			// All sub object should be gone with the reset
			TArray64<uint8> Buffer;
			FFileHelper::LoadFileToArray(Buffer, *AnimationPayloadFile);
			uint8* FileData = Buffer.GetData();
			int64 FileDataSize = Buffer.Num();
			if (FileDataSize < 1)
			{
				// Nothing to load from this file
				return TOptional<UE::Interchange::FAnimationPayloadData>();
			}

			UE::Interchange::FAnimationPayloadData AnimationTransformPayload(PayloadQuery.SceneNodeUniqueID, PayloadQuery.PayloadKey);

			// Buffer keeps the ownership of the data, the large memory reader is use to serialize the TMap
			FLargeMemoryReader Ar(FileData, FileDataSize);

			switch (PayloadQuery.PayloadKey.Type)
			{
				case EInterchangeAnimationPayLoadType::CURVE:
				{
					TArray<FInterchangeCurve> InterchangeCurves;
					Ar << InterchangeCurves;
					AnimationTransformPayload.Curves.AddDefaulted(InterchangeCurves.Num());
					for (int32 CurveIndex = 0; CurveIndex < InterchangeCurves.Num(); ++CurveIndex)
					{
						const FInterchangeCurve& InterchangeCurve = InterchangeCurves[CurveIndex];
						InterchangeCurve.ToRichCurve(AnimationTransformPayload.Curves[CurveIndex]);
					}
				}
				break;
				case EInterchangeAnimationPayLoadType::MORPHTARGETCURVE:
				{
					TArray<FInterchangeCurve> InterchangeCurves;
					Ar << InterchangeCurves;
					Ar << AnimationTransformPayload.InbetweenCurveNames;
					Ar << AnimationTransformPayload.InbetweenFullWeights;
					AnimationTransformPayload.Curves.AddDefaulted(InterchangeCurves.Num());
					for (int32 CurveIndex = 0; CurveIndex < InterchangeCurves.Num(); ++CurveIndex)
					{
						const FInterchangeCurve& InterchangeCurve = InterchangeCurves[CurveIndex];
						InterchangeCurve.ToRichCurve(AnimationTransformPayload.Curves[CurveIndex]);
					}
				}
				break;
				case EInterchangeAnimationPayLoadType::STEPCURVE:
				{
					Ar << AnimationTransformPayload.StepCurves;
				}
				break;
				case EInterchangeAnimationPayLoadType::BAKED:
					AnimationTransformPayload.SerializeBaked(Ar);
					break;
				case EInterchangeAnimationPayLoadType::NONE:
				default:
					break;
			}

			return TOptional<UE::Interchange::FAnimationPayloadData>(AnimationTransformPayload);
		};

	if (!bUseWorkerImport)
	{
#if WITH_EDITOR
		//PayloadQueries are assumed to be of the same PayloadKey.Type.
		if (bBakedQueries)
		{
			//Fetch the animation data for all the queries at the same time:
			FbxParser.FetchAnimationBakeTransformPayloads(PayloadQueries, ResultFolder);

			for (const UE::Interchange::FAnimationPayloadQuery& PayloadQuery : PayloadQueries)
			{
				TOptional<UE::Interchange::FAnimationPayloadData> OptionalPayloadData = OnPayloadReady(PayloadQuery, FbxParser.GetResultPayloadFilepath(PayloadQuery.GetHashString()));
				if (OptionalPayloadData.IsSet())
				{
					PayloadDataResults.Add(OptionalPayloadData.GetValue());
				}
			}
		}
		else
		{
			for (const UE::Interchange::FAnimationPayloadQuery& PayloadQuery : PayloadQueries)
			{
				FbxParser.FetchPayload(PayloadQuery.PayloadKey.UniqueId, ResultFolder);

				TOptional<UE::Interchange::FAnimationPayloadData> OptionalPayloadData = OnPayloadReady(PayloadQuery, FbxParser.GetResultPayloadFilepath(PayloadQuery.PayloadKey.UniqueId));
				if (OptionalPayloadData.IsSet())
				{
					PayloadDataResults.Add(OptionalPayloadData.GetValue());
				}
			}
		}
#endif //WITH_EDITOR
	}
	else
	{
		if (!Dispatcher.IsValid())
		{
			return PayloadDataResults;
		}

		//Init Promises and Futures
		//Promises so Dispatcher can set the Values
		//Futures so we can run the Asynch tasks and then acquire the Values.
		TArray<TFuture<TOptional<UE::Interchange::FAnimationPayloadData>>> AnimationPayloadDataFutures;
		TArray<TSharedPtr<TPromise<TOptional<UE::Interchange::FAnimationPayloadData>>>> AnimationPayloadDataPromises;
		AnimationPayloadDataFutures.Reserve(PayloadQueries.Num());
		for (size_t PayloadQueryIndex = 0; PayloadQueryIndex < PayloadQueries.Num(); PayloadQueryIndex++)
		{
			TSharedPtr<TPromise<TOptional<UE::Interchange::FAnimationPayloadData>>> Promise = MakeShared<TPromise<TOptional<UE::Interchange::FAnimationPayloadData>>>();
			AnimationPayloadDataFutures.Add(Promise->GetFuture());
			AnimationPayloadDataPromises.Add(Promise);
		}

		if (bBakedQueries)
		{
			FString JsonCommand = CreateFetchAnimationBakeTransformPayloadFbxCommand(PayloadQueries);

			const int32 CreatedTaskIndex = Dispatcher->AddTask(JsonCommand, FInterchangeDispatcherTaskCompleted::CreateLambda([this, PayloadQueries, OnPayloadReadyClosure = MoveTemp(OnPayloadReady), &AnimationPayloadDataPromises](const int32 TaskIndex)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeFbxTranslator::GetAnimationCurvePayloadData::Dispatcher->AddTaskDone)
					
					UE::Interchange::ETaskState TaskState;
					FString JsonResult;
					TArray<FString> JsonMessages;
					Dispatcher->GetTaskState(TaskIndex, TaskState, JsonResult, JsonMessages);

					// Parse the Json messages into UInterchangeResults
					for (const FString& JsonMessage : JsonMessages)
					{
						UE::Interchange::Private::ApplyTranslatorMessage(this, JsonMessage);
					}

					if (TaskState != UE::Interchange::ETaskState::ProcessOk)
					{
						//Process failed, set Promises up with OptionalValue.
						for (size_t PayloadQueryIndex = 0; PayloadQueryIndex < PayloadQueries.Num(); PayloadQueryIndex++)
						{
							TSharedPtr<TPromise<TOptional<UE::Interchange::FAnimationPayloadData>>>& Promise = AnimationPayloadDataPromises[PayloadQueryIndex];
							Promise->SetValue(TOptional<UE::Interchange::FAnimationPayloadData>());
						}
						return;
					}

					// Grab the result file and fill the BaseNodeContainer
					UE::Interchange::FJsonFetchAnimationQueriesCmd::JsonAnimationQueriesResultParser ResultParser;
					ResultParser.FromJson(JsonResult);
					const TMap<FString, FString>& HashToFilenames = ResultParser.GetHashToFilenames();

					for (size_t PayloadQueryIndex = 0; PayloadQueryIndex < PayloadQueries.Num(); PayloadQueryIndex++)
					{
						TSharedPtr<TPromise<TOptional<UE::Interchange::FAnimationPayloadData>>>& Promise = AnimationPayloadDataPromises[PayloadQueryIndex];

						const UE::Interchange::FAnimationPayloadQuery& PayloadQuery = PayloadQueries[PayloadQueryIndex];
						if (HashToFilenames.Contains(PayloadQuery.GetHashString()))
						{
							TOptional<UE::Interchange::FAnimationPayloadData> OptionalPayloadData = OnPayloadReadyClosure(PayloadQuery, HashToFilenames[PayloadQuery.GetHashString()]);

							Promise->SetValue(OptionalPayloadData);
						}
						else 
						{
							Promise->SetValue(TOptional<UE::Interchange::FAnimationPayloadData>());
						}
					}
				}));
		}
		else
		{
			for (size_t PayloadQueryIndex = 0; PayloadQueryIndex < PayloadQueries.Num(); PayloadQueryIndex++)
			{
				const UE::Interchange::FAnimationPayloadQuery& PayloadQuery = PayloadQueries[PayloadQueryIndex];
				TSharedPtr<TPromise<TOptional<UE::Interchange::FAnimationPayloadData>>>& Promise = AnimationPayloadDataPromises[PayloadQueryIndex];

				FString JsonCommand = CreateFetchPayloadFbxCommand(PayloadQuery.PayloadKey.UniqueId);

				const int32 CreatedTaskIndex = Dispatcher->AddTask(JsonCommand, FInterchangeDispatcherTaskCompleted::CreateLambda([this, PayloadQuery, OnPayloadReadyClosure = MoveTemp(OnPayloadReady), &Promise](const int32 TaskIndex)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeFbxTranslator::GetAnimationCurvePayloadData::Dispatcher->AddTaskDone)
							UE::Interchange::ETaskState TaskState;
						FString JsonResult;
						TArray<FString> JsonMessages;
						Dispatcher->GetTaskState(TaskIndex, TaskState, JsonResult, JsonMessages);

						// Parse the Json messages into UInterchangeResults
						for (const FString& JsonMessage : JsonMessages)
						{
							UE::Interchange::Private::ApplyTranslatorMessage(this, JsonMessage);
						}

						if (TaskState != UE::Interchange::ETaskState::ProcessOk)
						{
							Promise->SetValue(TOptional<UE::Interchange::FAnimationPayloadData>());
							return;
						}

						// Grab the result file and fill the BaseNodeContainer
						UE::Interchange::FJsonFetchPayloadCmd::JsonResultParser ResultParser;
						ResultParser.FromJson(JsonResult);
						FString AnimationTransformPayloadFilename = ResultParser.GetResultFilename();
						TOptional<UE::Interchange::FAnimationPayloadData> OptionalPayloadData = OnPayloadReadyClosure(PayloadQuery, AnimationTransformPayloadFilename);
						Promise->SetValue(OptionalPayloadData);
					}));
			}
		}

		for (TFuture<TOptional<UE::Interchange::FAnimationPayloadData>>& AnimationPayloadFuture : AnimationPayloadDataFutures)
		{
			TOptional<UE::Interchange::FAnimationPayloadData> OptionalPayloadData = AnimationPayloadFuture.Get();
			if (!OptionalPayloadData.IsSet())
			{
				continue;
			}
			PayloadDataResults.Add(OptionalPayloadData.GetValue());
		}

	}
	
	return PayloadDataResults;
}


FString UInterchangeFbxTranslator::CreateLoadFbxFileCommand(const FString& FbxFilePath, const bool bConvertScene, const bool bForceFrontXAxis, const bool bConvertSceneUnit, const bool bKeepFbxNamespace, const bool bConsiderClusterBeforePoseForMeshBindPose) const
{
	using namespace UE::Interchange;
	FJsonFBXLoadSourceCmd LoadSourceCommand(FJsonFBXLoadSourceCmd::TaskName, FbxFilePath, bConvertScene, bForceFrontXAxis, bConvertSceneUnit, bKeepFbxNamespace, bConsiderClusterBeforePoseForMeshBindPose);
	return LoadSourceCommand.ToJson();
}

FString UInterchangeFbxTranslator::CreateFetchPayloadFbxCommand(const FString& FbxPayloadKey) const
{
	UE::Interchange::FJsonFetchPayloadCmd PayloadCommand(TEXT("FBX"), FbxPayloadKey);
	return PayloadCommand.ToJson();
}

FString UInterchangeFbxTranslator::CreateFetchMeshPayloadFbxCommand(const FString& FbxPayloadKey, const FTransform& MeshGlobalTransform) const
{
	UE::Interchange::FJsonFetchMeshPayloadCmd PayloadCommand(TEXT("FBX"), FbxPayloadKey, MeshGlobalTransform);
	return PayloadCommand.ToJson();
}

FString UInterchangeFbxTranslator::CreateFetchAnimationBakeTransformPayloadFbxCommand(const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries) const
{
	UE::Interchange::FJsonFetchAnimationQueriesCmd PayloadCommand(TEXT("FBX"), UE::Interchange::FAnimationPayloadQuery::ToJson(PayloadQueries));
	return PayloadCommand.ToJson();
}

