// Copyright Epic Games, Inc. All Rights Reserved.

#include "Gltf/InterchangeGLTFMaterial.h"
#include "GLTFMaterial.h"
#include "GLTFTexture.h"

#include "Nodes/InterchangeBaseNodeContainer.h"
#include "InterchangeTextureNode.h"
#include "InterchangeShaderGraphNode.h"
#include "InterchangeMaterialInstanceNode.h"
#include "InterchangeImportModule.h"
#include "InterchangeMaterialDefinitions.h"
#include "InterchangeImportLog.h"

#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "Async/Async.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"

const FString InterchangeGltfMaterialAttributeIdentifier = TEXT("Gltf_MI_AttributeIdentifier_");

namespace UE::Interchange::GLTFMaterials
{

	FString ToString(EShadingModel ShadingModel)
	{
		switch (ShadingModel)
		{
		case DEFAULT:
			return TEXT("Default");
		case UNLIT:
			return TEXT("Unlit");
		case CLEARCOAT:
			return TEXT("ClearCoat");
		case SHEEN:
			return TEXT("Sheen");
		case TRANSMISSION:
			return TEXT("Transmission");
		case SPECULARGLOSSINESS:
			return TEXT("SpecularGlossiness");
		default:
			return TEXT("Unexpected");
		}
	}

	FString ToString(EAlphaMode AlphaMode)
	{
		switch (AlphaMode)
		{
		case EAlphaMode::Opaque:
			return TEXT("Opaque");
		case EAlphaMode::Mask:
			return TEXT("Mask");
		case EAlphaMode::Blend:
			return TEXT("Blend");
		default:
			return TEXT("Unexpected");
		}
	}

	FString GetIdentifier(EShadingModel ShadingModel, EAlphaMode AlphaMode, bool bTwoSided)
	{
		FString Name = TEXT("MI_");

		Name += ToString(ShadingModel);

		if (ShadingModel != EShadingModel::TRANSMISSION)
		{
			Name += TEXT("_") + ToString(AlphaMode);
		}

		Name += bTwoSided ? TEXT("_DS") : TEXT("");

		return Name;
	}

	FString GetMaterialFunctionPath(EShadingModel ShadingModel)
	{
		if (ShadingModelToMaterialInformation.Contains(ShadingModel))
		{
			return ShadingModelToMaterialInformation[ShadingModel].MaterialFunctionPath;
		}

		ensure(false);
		return ShadingModelToMaterialInformation.FindArbitraryElement()->Value.MaterialFunctionPath;
	}

	TArray<FString> GetOutputs(EShadingModel ShadingModel)
	{
		if (ShadingModelToMaterialInformation.Contains(ShadingModel))
		{
			return ShadingModelToMaterialInformation[ShadingModel].MaterialFunctionOutputs;
		}

		ensure(false);
		return ShadingModelToMaterialInformation.FindArbitraryElement()->Value.MaterialFunctionOutputs;
	}

	//Precedence order based on the logic in the InterchangeGenericMaterialPipeline: ClearCoat > Sheen > unlit
	EShadingModel GetShadingModel(GLTF::FMaterial::EShadingModel GltfShadingModel, bool bHasClearCoat, bool bHasSheen, bool bUnlit, bool bHasTransmission)
	{
		if (bUnlit)
		{
			return EShadingModel::UNLIT;
		}

		if (GltfShadingModel == GLTF::FMaterial::EShadingModel::SpecularGlossiness)
		{
			return EShadingModel::SPECULARGLOSSINESS;
		}

		if (bHasTransmission)
		{
			return EShadingModel::TRANSMISSION;
		}

		if (bHasClearCoat)
		{
			return EShadingModel::CLEARCOAT;
		}

		if (bHasSheen)
		{
			return EShadingModel::SHEEN;
		}

		return EShadingModel::DEFAULT;
	}

	FString GetMaterialInstanceInputName(const FString& Name)
	{
		return *InterchangeGltfMaterialAttributeIdentifier + Name;
	}

	FString GetMaterialInputName(const FString& Name)
	{
		return UInterchangeShaderPortsAPI::MakeInputValueKey(Name);
	}

	FVector4f GetTilingMethod(const GLTF::FSampler& Sampler)
	{
		auto GetSingleChannelTilingMethod = [](GLTF::FSampler::EWrap Wrap/*TilingMethod*/)
			{
				switch (Wrap)
				{
					case GLTF::FSampler::EWrap::Repeat:         return 0;
					case GLTF::FSampler::EWrap::ClampToEdge:    return 1;
					case GLTF::FSampler::EWrap::MirroredRepeat: return 2;
					default:                                    return 0;
				}
			};

		float TilingU = GetSingleChannelTilingMethod(Sampler.WrapS);
		float TilingV = GetSingleChannelTilingMethod(Sampler.WrapT);

		return FVector4f(TilingU, TilingV, 0, 0);
	}

	struct FGLTFMaterialProcessor
	{
		UInterchangeBaseNodeContainer& NodeContainer;
		const GLTF::FMaterial& GltfMaterial;
		const TArray<GLTF::FTexture>& Textures;
		
		UInterchangeShaderGraphNode* ShaderGraphNode = nullptr; //Set by the Constructor
		UInterchangeShaderNode* MaterialNode = nullptr;			//Set by ConfigureMaterialNode
		UInterchangeShaderNode* MaterialInstanceNode = nullptr; //Set by ConfigureMaterialInstanceNode

		EShadingModel ShadingModel = EShadingModel::DEFAULT;
		EAlphaMode AlphaMode = EAlphaMode::Opaque;

		FGLTFMaterialProcessor(UInterchangeBaseNodeContainer& InNodeContainer, const GLTF::FMaterial& InGltfMaterial, const TArray<GLTF::FTexture>& InTextures, UInterchangeShaderGraphNode* InShaderGraphNode)
			: NodeContainer(InNodeContainer)
			, GltfMaterial(InGltfMaterial)
			, Textures(InTextures)
			, ShaderGraphNode(InShaderGraphNode)
		{
			//Generate base properties:
			ShadingModel = GetShadingModel(GltfMaterial.ShadingModel, GltfMaterial.bHasClearCoat, GltfMaterial.bHasSheen, GltfMaterial.bIsUnlitShadingModel, GltfMaterial.bHasTransmission);
			AlphaMode = EAlphaMode(GltfMaterial.AlphaMode);

			//Configure Nodes:
			ConfigureMaterialNode();
			ConfigureMaterialInstanceNode();
		}

		////////////////////////////
		//Start of Config Functions:
		////////////////////////////
		void ConfigureMaterialNode()
		{
			FString MaterialFunctionPath = GetMaterialFunctionPath(ShadingModel);
			FString MaterailFunctionFileName = FPaths::GetBaseFilename(MaterialFunctionPath);

			UInterchangeFunctionCallShaderNode* MF_Body_Node = NewObject<UInterchangeFunctionCallShaderNode>(&NodeContainer);
			NodeContainer.SetupNode(MF_Body_Node, UInterchangeShaderNode::MakeNodeUid(MaterailFunctionFileName, ShaderGraphNode->GetUniqueID()), MaterailFunctionFileName, EInterchangeNodeContainerType::TranslatedAsset);
			MF_Body_Node->SetCustomMaterialFunction(MaterialFunctionPath);

			MaterialNode = MF_Body_Node;

			ShaderGraphNode->SetCustomTwoSided(GltfMaterial.bIsDoubleSided);
			if (ShadingModel == EShadingModel::TRANSMISSION)
			{
				ShaderGraphNode->SetCustomScreenSpaceReflections(true);
				ShaderGraphNode->SetCustomTwoSidedTransmission(true);
			}

			TArray<FString> Outputs = GetOutputs(ShadingModel);

			switch (AlphaMode)
			{
			case UE::Interchange::GLTFMaterials::Opaque:
				Outputs.Remove(TEXT("Opacity"));
				Outputs.Remove(TEXT("OpacityMask"));
				break;
			case UE::Interchange::GLTFMaterials::Mask:
				if (ShadingModel != EShadingModel::TRANSMISSION)
				{
					Outputs.Remove(TEXT("Opacity"));
				}
				break;
			case UE::Interchange::GLTFMaterials::Blend:
				Outputs.Remove(TEXT("OpacityMask"));
				break;
			default:
				break;
			}

			for (const FString& Input : Outputs)
			{
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, Input, MF_Body_Node->GetUniqueID(), Input);
			}

			//Set OpacityMaskClipValue if needed:
			if (!GltfMaterial.bIsUnlitShadingModel &&
				GltfMaterial.AlphaMode == GLTF::FMaterial::EAlphaMode::Mask)
			{
				if (ShadingModel == EShadingModel::TRANSMISSION)
				{
					//Transmission (even for Materials) supports AlphaCutoff through the MF_Body instead of the automatic OpacityMaskClipValue
					SetScalar(Inputs::AlphaCutoff, GltfMaterial.AlphaCutoff, 0.f, EProcessType::MATERIAL);
				}
				else
				{
					ShaderGraphNode->SetCustomOpacityMaskClipValue(GltfMaterial.AlphaCutoff);
				}
			}
		}

		void ConfigureMaterialInstanceNode()
		{
			MaterialInstanceNode = ShaderGraphNode;

			FString ParentIdentifier = GetIdentifier(ShadingModel, AlphaMode, GltfMaterial.bIsDoubleSided);
			MaterialInstanceNode->AddStringAttribute(*(InterchangeGltfMaterialAttributeIdentifier + TEXT("ParentIdentifier")), ParentIdentifier);

			//Set AlphaCutOff if needed:
			if (!GltfMaterial.bIsUnlitShadingModel &&
				GltfMaterial.AlphaMode == GLTF::FMaterial::EAlphaMode::Mask)
			{
				//AlphaCutoff
				SetScalar(Inputs::AlphaCutoff, GltfMaterial.AlphaCutoff, 0.f, EProcessType::MATERIALINSTANCE);
			}
		}
		////////////////////////////
		//End of Config Functions:
		////////////////////////////


		///////////////////////////////////
		//Start of Helper Setter Functions:
		///////////////////////////////////
		enum EProcessType : uint8
		{
			MATERIAL = 1,
			MATERIALINSTANCE = 2
		};

		void SetScalar(const FString& Name, float Value, float DefaultValue, int ProcessType = (EProcessType::MATERIAL | EProcessType::MATERIALINSTANCE))
		{
			if (!FMath::IsNearlyEqual(Value, DefaultValue))
			{
				if (ProcessType & EProcessType::MATERIAL)
				{
					MaterialNode->AddFloatAttribute(GetMaterialInputName(Name), Value);
				}

				if (ProcessType & EProcessType::MATERIALINSTANCE)
				{
					MaterialInstanceNode->AddFloatAttribute(GetMaterialInstanceInputName(Name), Value);
				}
			}
		}

		void SetVec3(const FString& Name, const FVector3f& Value, const FVector3f& DefaultValue, int ProcessType = (EProcessType::MATERIAL | EProcessType::MATERIALINSTANCE))
		{
			if (!Value.Equals(DefaultValue, UE_SMALL_NUMBER))
			{

				if (ProcessType & EProcessType::MATERIAL)
				{
					MaterialNode->AddLinearColorAttribute(GetMaterialInputName(Name), Value);
				}

				if (ProcessType & EProcessType::MATERIALINSTANCE)
				{
					MaterialInstanceNode->AddLinearColorAttribute(GetMaterialInstanceInputName(Name), Value);
				}
			}
		}

		void SetVec4(const FString& Name, const FVector4f& Value, const FVector4f& DefaultValue, int ProcessType = (EProcessType::MATERIAL | EProcessType::MATERIALINSTANCE))
		{
			if (!Value.Equals(DefaultValue, UE_SMALL_NUMBER))
			{
				if (ProcessType & EProcessType::MATERIAL)
				{
					MaterialNode->AddLinearColorAttribute(GetMaterialInputName(Name), Value);
				}

				if (ProcessType & EProcessType::MATERIALINSTANCE)
				{
					MaterialInstanceNode->AddLinearColorAttribute(GetMaterialInstanceInputName(Name), Value);
				}
			}
		}

		void SetColor(const FString& Name, const FVector4f& Value, const FVector4f& DefaultValue)
		{
			{//Material specific settings:
				SetVec3(Name + Inputs::PostFix::Color_RGB, FVector3f(Value), FVector3f(DefaultValue), EProcessType::MATERIAL);
				SetScalar(Name + Inputs::PostFix::Color_A, Value.W, DefaultValue.W, EProcessType::MATERIAL);
			}

			{//MaterialInstance specific settings:
				SetVec4(Name, Value, DefaultValue, EProcessType::MATERIALINSTANCE);
			}
		}

		bool SetMap(const FString& Name, const GLTF::FTextureMap& TextureMap)
		{
			if (Textures.IsValidIndex(TextureMap.TextureIndex))
			{
				{//Material specific settings:
					//create TextureObject:
					UInterchangeShaderNode* ColorNode = UInterchangeShaderNode::Create(&NodeContainer, Name, MaterialNode->GetUniqueID());
					ColorNode->SetCustomShaderType(UE::Interchange::Materials::Standard::Nodes::TextureObject::Name.ToString());

					ColorNode->AddStringAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(UE::Interchange::Materials::Standard::Nodes::TextureObject::Inputs::Texture.ToString()), UInterchangeTextureNode::MakeNodeUid(Textures[TextureMap.TextureIndex].UniqueId));

					UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(MaterialNode, Name, ColorNode->GetUniqueID());
				}

				{//MaterialInstance specific settings:
					MaterialInstanceNode->AddStringAttribute(GetMaterialInstanceInputName(Name), UInterchangeTextureNode::MakeNodeUid(Textures[TextureMap.TextureIndex].UniqueId));
				}

				//TexCoord decision in MaterialInstances work based on a Switch Node
				//Currently Only supports UV0 and UV1
				// [0...1) -> UV0
				// [1...2) -> UV1
				// [2...3) -> UV2
				// [3...4) -> UV3
				// else    -> UV0 (defaults to 0)
				SetScalar(Name + Inputs::PostFix::TexCoord, TextureMap.TexCoord, 0.f);

				//Set the TilingMethod:
				SetVec4(Name + Inputs::PostFix::TilingMethod, GetTilingMethod(Textures[TextureMap.TextureIndex].Sampler), FVector4f(0, 0, 0, 0));

				if (TextureMap.bHasTextureTransform)
				{
					{//Material specific settings:
						SetScalar(Name + Inputs::PostFix::OffsetX, TextureMap.TextureTransform.Offset[0], 0.f, EProcessType::MATERIAL);
						SetScalar(Name + Inputs::PostFix::OffsetY, TextureMap.TextureTransform.Offset[1], 0.f, EProcessType::MATERIAL);

						SetScalar(Name + Inputs::PostFix::ScaleX, TextureMap.TextureTransform.Scale[0], 1.f, EProcessType::MATERIAL);
						SetScalar(Name + Inputs::PostFix::ScaleY, TextureMap.TextureTransform.Scale[1], 1.f, EProcessType::MATERIAL);
					}

					{//MaterialInstance specific settings:
						FVector4f OffsetScale(TextureMap.TextureTransform.Offset[0], TextureMap.TextureTransform.Offset[1], TextureMap.TextureTransform.Scale[0], TextureMap.TextureTransform.Scale[1]);
						SetVec4(Name + Inputs::PostFix::OffsetScale, OffsetScale, FVector4f(0, 0, 1, 1), EProcessType::MATERIALINSTANCE);
					}

					if (!FMath::IsNearlyZero(TextureMap.TextureTransform.Rotation))
					{
						float AngleRadians = TextureMap.TextureTransform.Rotation;

						if (AngleRadians < 0.0f)
						{
							AngleRadians = TWO_PI - AngleRadians;
						}

						SetScalar(Name + Inputs::PostFix::Rotation, AngleRadians, 0.f);
					}
				}

				return true;
			}

			return false;
		}

		// Only setting it for MaterialInstances on default for now, the glTF pipeline is using MaterialInstances only.
		//	However if someone disregards the usage of the glTF pipeline then the MF_XYZ_Body functions will be used which don't have configuration inputs.
		void SetBool(const FString& Name, const bool& Value, const bool& DefaultValue = true, int ProcessType = (EProcessType::MATERIALINSTANCE))
		{
			if (Value != DefaultValue)
			{
				if (ProcessType & EProcessType::MATERIAL)
				{
					MaterialNode->AddBooleanAttribute(GetMaterialInputName(Name), Value);
				}

				if (ProcessType & EProcessType::MATERIALINSTANCE)
				{
					MaterialInstanceNode->AddBooleanAttribute(GetMaterialInstanceInputName(Name), Value);
				}
			}
		}
		////////////////////////////////
		//End Of Helper Setter Functions
		////////////////////////////////
		

		////////////////////////////////////////////////////////////////////////////////////
		/// Parse Material Values from GltfMaterial to MaterialNode and MaterialInstanceNode
		////////////////////////////////////////////////////////////////////////////////////
		void ParseMaterialValues()
		{

			// The M_GLTF material disable all static switches, to avoid having all the samplers at once, otherwise platforms like Android fail to compile
			const bool bDefaultValueConfiguration = IInterchangeImportModule::Get().IsSubstrateEnabled() ? false : true;

			// AlphaMode used now by Substrate to handle every case
			SetScalar(Inputs::AlphaMode, float(GltfMaterial.AlphaMode), 0.f);


			if (GltfMaterial.bIsUnlitShadingModel)
			{
				//BaseColorTexture
				//BaseColorTexture_OffsetScale
				//BaseColorTexture_Rotation
				//BaseColorTexture_TexCoord
				//BaseColorTexture_TilingMethod
				bool bHasBaseColorTexture = SetMap(Inputs::BaseColorTexture, GltfMaterial.BaseColor);

				//BaseColorFactor
				SetColor(Inputs::BaseColorFactor, GltfMaterial.BaseColorFactor, FVector4f(1, 1, 1, 1));

				//Configuration
				SetBool(Inputs::Configuration::bHasBaseColorTexture, bHasBaseColorTexture, bDefaultValueConfiguration);
				return;
			}

			{
				//NormalTexture
				//NormalTexture_OffsetScale
				//NormalTexture_Rotation
				//NormalTexture_TexCoord
				//NormalTexture_TilingMethod
				bool bHasNormalTexture = SetMap(Inputs::NormalTexture, GltfMaterial.Normal);

				//NormalScale
				SetScalar(Inputs::NormalScale, GltfMaterial.NormalScale, 1.f);

				//Configuration
				SetBool(Inputs::Configuration::bHasNormalTexture, bHasNormalTexture, bDefaultValueConfiguration);
			}

			if (!GltfMaterial.bHasTransmission)
			{
				//EmissiveTexture
				//EmissiveTexture_OffsetScale
				//EmissiveTexture_Rotation
				//EmissiveTexture_TexCoord
				//EmissiveTexture_TilingMethod
				bool bHasEmissiveTexture = SetMap(Inputs::EmissiveTexture, GltfMaterial.Emissive);

				//EmissiveFactor
				SetVec3(Inputs::EmissiveFactor, GltfMaterial.EmissiveFactor, FVector3f(0, 0, 0));

				//EmissiveStrength
				SetScalar(Inputs::EmissiveStrength, GltfMaterial.bHasEmissiveStrength ? GltfMaterial.EmissiveStrength : 1.f, 1.f);

				//Configuration
				SetBool(Inputs::Configuration::bHasEmissiveTexture, bHasEmissiveTexture, bDefaultValueConfiguration);
			}
			else
			{
				//Configuration
				SetBool(Inputs::Configuration::bHasEmissiveTexture, false);
			}

			{
				//OcclusionTexture
				//OcclusionTexture_OffsetScale
				//OcclusionTexture_Rotation
				//OcclusionTexture_TexCoord
				//OcclusionTexture_TilingMethod
				bool bHasOcclusionTexture = SetMap(Inputs::OcclusionTexture, GltfMaterial.Occlusion);

				//OcclusionStrength
				SetScalar(Inputs::OcclusionStrength, GltfMaterial.OcclusionStrength, 1.f);

				//Configuration
				SetBool(Inputs::Configuration::bHasOcclusionTexture, bHasOcclusionTexture, bDefaultValueConfiguration);
			}

			if (GltfMaterial.ShadingModel == GLTF::FMaterial::EShadingModel::SpecularGlossiness)
			{
				//////////////////////
				//specular glossiness:
				//////////////////////

				bool bHasDiffuseSpecGlossTexture = false;
				{
					//DiffuseTexture
					//DiffuseTexture_OffsetScale
					//DiffuseTexture_Rotation
					//DiffuseTexture_TexCoord
					//DiffuseTexture_TilingMethod
					bHasDiffuseSpecGlossTexture = SetMap(Inputs::DiffuseTexture, GltfMaterial.BaseColor);

					//DiffuseFactor
					SetColor(Inputs::DiffuseFactor, GltfMaterial.BaseColorFactor, FVector4f(1, 1, 1, 1));
				}

				{
					//SpecularGlossinessTexture
					//SpecularGlossinessTexture_OffsetScale
					//SpecularGlossinessTexture_Rotation
					//SpecularGlossinessTexture_TexCoord
					//SpecularGlossinessTexture_TilingMethod
					bHasDiffuseSpecGlossTexture = SetMap(Inputs::SpecularGlossinessTexture, GltfMaterial.SpecularGlossiness.Map) || bHasDiffuseSpecGlossTexture;

					//SpecFactor
					FVector3f SpecularFactor(GltfMaterial.SpecularGlossiness.SpecularFactor[0], GltfMaterial.SpecularGlossiness.SpecularFactor[1], GltfMaterial.SpecularGlossiness.SpecularFactor[2]);
					SetVec3(Inputs::SpecFactor, SpecularFactor, FVector3f(1, 1, 1));

					//GlossinessFactor
					SetScalar(Inputs::GlossinessFactor, GltfMaterial.SpecularGlossiness.GlossinessFactor, 1.f);
				}

				//Configuration
				SetBool(Inputs::Configuration::bHasDiffuseSpecGlossTexture, bHasDiffuseSpecGlossTexture, bDefaultValueConfiguration);

				return;
			}
			else if (GltfMaterial.ShadingModel == GLTF::FMaterial::EShadingModel::MetallicRoughness)
			{
				{
					//BaseColorTexture
					//BaseColorTexture_OffsetScale
					//BaseColorTexture_Rotation
					//BaseColorTexture_TexCoord
					//BaseColorTexture_TilingMethod
					bool bHasBaseColorTexture = SetMap(Inputs::BaseColorTexture, GltfMaterial.BaseColor);

					//BaseColorFactor
					SetColor(Inputs::BaseColorFactor, GltfMaterial.BaseColorFactor, FVector4f(1, 1, 1, 1));

					//Configuration
					SetBool(Inputs::Configuration::bHasBaseColorTexture, bHasBaseColorTexture, bDefaultValueConfiguration);
				}

				{
					//MetallicRoughnessTexture
					//MetallicRoughnessTexture_OffsetScale
					//MetallicRoughnessTexture_Rotation
					//MetallicRoughnessTexture_TexCoord
					//MetallicRoughnessTexture_TilingMethod
					bool bHasMetallicRoughnessTexture = SetMap(Inputs::MetallicRoughnessTexture, GltfMaterial.MetallicRoughness.Map);

					//MetallicFactor
					SetScalar(Inputs::MetallicFactor, GltfMaterial.MetallicRoughness.MetallicFactor, 1.f);

					//RoughnessFactor
					SetScalar(Inputs::RoughnessFactor, GltfMaterial.MetallicRoughness.RoughnessFactor, 1.f);

					//Configuration
					SetBool(Inputs::Configuration::bHasMetallicRoughnessTexture, bHasMetallicRoughnessTexture, bDefaultValueConfiguration);
				}

				if (GltfMaterial.bHasSpecular)
				{
					//SpecularTexture
					//SpecularTexture_OffsetScale
					//SpecularTexture_Rotation
					//SpecularTexture_TexCoord
					//SpecularTexture_TilingMethod
					bool bHasSpecularTexture = SetMap(Inputs::SpecularTexture, GltfMaterial.Specular.SpecularMap);

					//SpecularFactor
					SetScalar(Inputs::SpecularFactor, GltfMaterial.Specular.SpecularFactor, 0.5); //(UE Specular default is 0.5).

					//Configuration
					SetBool(Inputs::Configuration::bHasSpecularTexture, bHasSpecularTexture, bDefaultValueConfiguration);

					//SpecularColorTexture
					//SpecularColorTexture_OffsetScale
					//SpecularColorTexture_Rotation
					//SpecularColorTexture_TexCoord
					//SpecularColorTexture_TilingMethod
					bool bHasSpecularColorTexture = SetMap(Inputs::SpecularColorTexture, GltfMaterial.Specular.SpecularColorMap);

					//SpecularColorFactor
					const FVector3f SpecularColorFactor(GltfMaterial.Specular.SpecularColorFactor.X, GltfMaterial.Specular.SpecularColorFactor.Y, GltfMaterial.Specular.SpecularColorFactor.Z);
					SetVec3(Inputs::SpecularColorFactor, SpecularColorFactor, { 1, 1, 1});

					//Configuration
					SetBool(Inputs::Configuration::bHasSpecularColorTexture, bHasSpecularColorTexture, bDefaultValueConfiguration);
				}
				else
				{
					//Configuration
					SetBool(Inputs::Configuration::bHasSpecularTexture, false);
					SetBool(Inputs::Configuration::bHasSpecularColorTexture, false);
				}

				if (GltfMaterial.Iridescence.bHasIridescence)
				{
					//IridescenceIOR
					SetScalar(Inputs::IridescenceIOR, GltfMaterial.Iridescence.IOR, 1.3f);

					//IridescenceTexture
					//IridescenceTexture_OffsetScale
					//IridescenceTexture_Rotation
					//IridescenceTexture_TexCoord
					//IridescenceTexture_TilingMethod
					bool bHasIridescence = SetMap(Inputs::IridescenceTexture, GltfMaterial.Iridescence.Texture);

					//IridescenceFactor
					SetScalar(Inputs::IridescenceFactor, GltfMaterial.Iridescence.Factor, 0.0f);

					//IridescenceThicknessTexture
					//IridescenceThicknessTexture_OffsetScale
					//IridescenceThicknessTexture_Rotation
					//IridescenceThicknessTexture_TexCoord
					//IridescenceThicknessTexture_TilingMethod
					SetMap(Inputs::IridescenceThicknessTexture, GltfMaterial.Iridescence.Thickness.Texture);

					//IridescenceThicknessMinimum
					SetScalar(Inputs::IridescenceThicknessMinimum, GltfMaterial.Iridescence.Thickness.Minimum, 100.0f);
					//IridescenceThicknessMaximum
					SetScalar(Inputs::IridescenceThicknessMaximum, GltfMaterial.Iridescence.Thickness.Maximum, 400.0f);

					//Configuration
					SetBool(Inputs::Configuration::bHasIridescence, bHasIridescence, bDefaultValueConfiguration);
				}
				else
				{
					//Configuration
					SetBool(Inputs::Configuration::bHasIridescence, false);
				}

				if (GltfMaterial.Anisotropy.bHasAnisotropy)
				{
					//AnisotropyTexture
					//AnisotropyTexture_OffsetScale
					//AnisotropyTexture_Rotation
					//AnisotropyTexture_TexCoord
					//AnisotropyTexture_TilingMethod
					bool bHasAnisotropyTextureAndOrRotation = SetMap(Inputs::AnisotropyTexture, GltfMaterial.Anisotropy.Texture);

					//AnisotropyStrength
					SetScalar(Inputs::AnisotropyStrength, GltfMaterial.Anisotropy.Strength, 0);

					//AnisotropyRotation
					SetScalar(Inputs::AnisotropyRotation, GltfMaterial.Anisotropy.Rotation, 0);
					bHasAnisotropyTextureAndOrRotation |= GltfMaterial.Anisotropy.Rotation != 0;

					//Configuration
					SetBool(Inputs::Configuration::bHasAnisotropyTextureAndOrRotation, bHasAnisotropyTextureAndOrRotation, bDefaultValueConfiguration);
				}
				else
				{
					//Configuration
					SetBool(Inputs::Configuration::bHasAnisotropyTextureAndOrRotation, false);
				}
			}
			else
			{
				ensure(false);
			}

			if (GltfMaterial.bHasIOR)
			{
				//ior
				SetScalar(Inputs::IOR, GltfMaterial.IOR, 1.5f);
			}

			if (GltfMaterial.Volume.bHasVolume)
			{
				//ThicknessTexture
				//ThicknessTexture_OffsetScale
				//ThicknessTexture_Rotation
				//ThicknessTexture_TexCoord
				//ThicknessTexture_TilingMethod
				bool bHasThicknessTexture = SetMap(Inputs::ThicknessTexture, GltfMaterial.Volume.ThicknessTexture);

				//ThicknessFactor
				SetScalar(Inputs::ThicknessFactor, GltfMaterial.Volume.Thickness, 0.f);

				//AttenuationDistance
				SetScalar(Inputs::AttenuationDistance, GltfMaterial.Volume.AttenuationDistance, 10.f);

				//AttenuationColor
				const FVector3f AttenuationColor(GltfMaterial.Volume.AttenuationColor.X, GltfMaterial.Volume.AttenuationColor.Y, GltfMaterial.Volume.AttenuationColor.Z);
				SetVec3(Inputs::AttenuationColor, AttenuationColor, FVector3f::OneVector);

				//Configuration
				SetBool(Inputs::Configuration::bHasThicknessTexture, bHasThicknessTexture, bDefaultValueConfiguration);
			}

			//Transmission ClearCoat and Sheen are Model specific
			// Meaning we don't have to do exclusion configs.
			if (GltfMaterial.bHasTransmission)
			{
				{
					//TransmissionTexture
					//TransmissionTexture_OffsetScale
					//TransmissionTexture_Rotation
					//TransmissionTexture_TexCoord
					//TransmissionTexture_TilingMethod
					bool bHasTransmissionTexture = SetMap(Inputs::TransmissionTexture, GltfMaterial.Transmission.TransmissionMap);

					//TransmissionFactor
					SetScalar(Inputs::TransmissionFactor, GltfMaterial.Transmission.TransmissionFactor, 0.f);

					//AlphaMode
					SetScalar(Inputs::AlphaMode, AlphaMode, EAlphaMode::Blend);

					//Configuration
					SetBool(Inputs::Configuration::bHasTransmissionTexture, bHasTransmissionTexture, bDefaultValueConfiguration);
				}
			}
			else if (GltfMaterial.bHasClearCoat)
			{
				bool bHasClearCoatTexture = false;
				{
					//ClearCoatTexture
					//ClearCoatTexture_OffsetScale
					//ClearCoatTexture_Rotation
					//ClearCoatTexture_TexCoord
					//ClearCoatTexture_TilingMethod
					bHasClearCoatTexture = SetMap(Inputs::ClearCoatTexture, GltfMaterial.ClearCoat.ClearCoatMap);

					//ClearCoatFactor
					SetScalar(Inputs::ClearCoatFactor, GltfMaterial.ClearCoat.ClearCoatFactor, 0.f);
				}

				{
					//ClearCoatRoughnessTexture
					//ClearCoatRoughnessTexture_OffsetScale
					//ClearCoatRoughnessTexture_Rotation
					//ClearCoatRoughnessTexture_TexCoord
					//ClearCoatRoughnessTexture_TilingMethod
					bHasClearCoatTexture = SetMap(Inputs::ClearCoatRoughnessTexture, GltfMaterial.ClearCoat.RoughnessMap) || bHasClearCoatTexture;

					//ClearCoatRoughnessFactor
					SetScalar(Inputs::ClearCoatRoughnessFactor, GltfMaterial.ClearCoat.Roughness, 0.f);
				}

				{
					//ClearCoatNormalTexture
					//ClearCoatNormalTexture_OffsetScale
					//ClearCoatNormalTexture_Rotation
					//ClearCoatNormalTexture_TexCoord
					//ClearCoatNormalTexture_TilingMethod
					bHasClearCoatTexture = SetMap(Inputs::ClearCoatNormalTexture, GltfMaterial.ClearCoat.NormalMap) || bHasClearCoatTexture;

					//ClearCoatNormalFactor
					SetScalar(Inputs::ClearCoatNormalScale, GltfMaterial.ClearCoat.NormalMapUVScale, 1.f);
				}

				//Configuration
				SetBool(Inputs::Configuration::bHasClearCoatTexture, bHasClearCoatTexture, bDefaultValueConfiguration);
			}
			else if (GltfMaterial.bHasSheen)
			{
				{
					//SheenColorTexture
					//SheenColorTexture_OffsetScale
					//SheenColorTexture_Rotation
					//SheenColorTexture_TexCoord
					//SheenColorTexture_TilingMethod
					bool bHasSheenTexture = SetMap(Inputs::SheenColorTexture, GltfMaterial.Sheen.SheenColorMap);

					//SheenColorFactor
					FVector3f SheenColorFactor(GltfMaterial.Sheen.SheenColorFactor[0], GltfMaterial.Sheen.SheenColorFactor[1], GltfMaterial.Sheen.SheenColorFactor[2]);
					SetVec3(Inputs::SheenColorFactor, SheenColorFactor, FVector3f(0, 0, 0));

					//SheenRoughnessTexture
					//SheenRoughnessTexture_OffsetScale
					//SheenRoughnessTexture_Rotation
					//SheenRoughnessTexture_TexCoord
					//SheenRoughnessTexture_TilingMethod
					bHasSheenTexture = SetMap(Inputs::SheenRoughnessTexture, GltfMaterial.Sheen.SheenRoughnessMap) || bHasSheenTexture;

					//SheenRoughnessFactor
					SetScalar(Inputs::SheenRoughnessFactor, GltfMaterial.Sheen.SheenRoughnessFactor, 0.f);

					//Configuration
					SetBool(Inputs::Configuration::bHasSheenTexture, bHasSheenTexture, bDefaultValueConfiguration);
				}
			}
		}
	};

	void HandleGltfMaterial(UInterchangeBaseNodeContainer& NodeContainer, const GLTF::FMaterial& GltfMaterial, const TArray<GLTF::FTexture>& Textures, UInterchangeShaderGraphNode* ShaderGraphNode)
	{
		using namespace UE::Interchange::GLTFMaterials;

		//Constructor of FGLTFMaterialProcessor will configure and generate necessary properties for Parsing:
		FGLTFMaterialProcessor GLTFMaterailProcessor(NodeContainer, GltfMaterial, Textures, ShaderGraphNode);

		GLTFMaterailProcessor.ParseMaterialValues();
	}

	bool AreRequiredPackagesLoaded()
	{
		auto ArePackagesLoaded = [](const TArray<FString>& PackagePaths) -> bool
		{
			bool bAllLoaded = true;

			for (const FString& PackagePath : PackagePaths)
			{
				const FString ObjectPath(FPackageName::ExportTextPathToObjectPath(PackagePath));

				if (FPackageName::DoesPackageExist(ObjectPath))
				{
					if (FSoftObjectPath(ObjectPath).TryLoad())
					{
						continue;
					}
					else
					{
						UE_LOGF(LogInterchangeImport, Warning, "Couldn't load %ls", *PackagePath);
					}
				}
				else
				{
					UE_LOGF(LogInterchangeImport, Warning, "Couldn't find %ls", *PackagePath);
				}

				bAllLoaded = false;
			}

			return bAllLoaded;
		};

		TArray<FString> RequiredPackages = GetRequiredMaterialFunctionPaths();

		static const bool bRequiredPackagesLoaded = ArePackagesLoaded(RequiredPackages);

		return bRequiredPackagesLoaded;
	}
}

