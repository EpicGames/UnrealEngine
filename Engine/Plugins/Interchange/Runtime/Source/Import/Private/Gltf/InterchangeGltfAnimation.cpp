// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Gltf/InterchangeGltfPrivate.h"

#include "InterchangeCommonAnimationPayload.h"
#include "GLTFAccessor.h"
#include "GLTFAnimation.h"
#include "GLTFAsset.h"
#include "GLTFNode.h"
#include "GLTFMesh.h"
#include "InterchangeImportLog.h"
#include "Animation/AnimTypes.h"

#include "Nodes/InterchangeBaseNodeContainer.h"
#include "InterchangeSceneNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangeAnimationTrackSetNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"

namespace UE::Interchange::Gltf::Private
{
	/*
	* According to gltf specification Seconds acquired from Samplers.input:
	* "The values represent time in seconds with time[0] >= 0.0, and strictly increasing values, Index.e., time[n + 1] > time[n]."
	* Meaning the incoming array(Seconds) is sorted.
	*/
	TTuple<int32, int32> GetNeighbourIndices(TArray<float>& Seconds, float SampleTime)
	{
		int32 Lower = 0;
		int32 Higher = 0;

		if (Seconds.Num() == 0)
		{
			return TTuple<int32, int32>(0, 0);
		}
		else if (Seconds.Num() == 1)
		{
			return TTuple<int32, int32>(Lower, Higher);
		}

		if (SampleTime < Seconds[0])
		{
			Lower = Higher = 0;
		}
		else if (SampleTime >= Seconds[Seconds.Num() - 1])
		{
			Lower = Higher = Seconds.Num() - 1;
		}
		else
		{
			for (int32 Index = 0; Index < Seconds.Num() - 1; Index++)
			{
				if ((Seconds[Index] <= SampleTime) && (SampleTime < Seconds[Index + 1]))
				{
					Lower = Index;
					Higher = Index + 1;
					break;
				}
			}
		}

		return TTuple<int32, int32>(Lower, Higher);
	}

	template <typename T>
	T InterpolateValue(TArray<float>& Seconds /*input*/, TArray<T>& SegmentValues /*output*/, int32 kIndex /*lower*/, int32 kp1Index /*higher*/, double CurrentRequestedTimeStamp, GLTF::FAnimation::EInterpolation Interpolation, bool IsRotation)
	{
		if (SegmentValues.Num() <= kIndex || SegmentValues.Num() <= kp1Index || kIndex < 0 || kp1Index < 0)
		{
			UE_LOGF(LogInterchangeImport, Warning, "Interpolation Failed due to invalid kIndices");
			return T();
		}

		if (kIndex == kp1Index)
		{
			if (Interpolation == GLTF::FAnimation::EInterpolation::CubicSpline)
			{
				return SegmentValues[kIndex * 3 + 1];
			}
			return SegmentValues[kp1Index];
		}

		// To match variables against https://registry.khronos.org/glTF/specs/2.0/glTF-2.0.html#appendix-c-interpolation
		// Time := Time
		// TimeDelta := td
		double TimeDelta = Seconds[kp1Index] - Seconds[kIndex];
		double Time = (CurrentRequestedTimeStamp - Seconds[kIndex]) / TimeDelta;

		T Value = {};
		switch (Interpolation)
		{
		case GLTF::FAnimation::EInterpolation::Linear:
		{
			if (IsRotation)
			{
				//Spherical Linear Interpolation
				// a=arccos(|vk⋅vk+1|)
				// s=vk⋅vk+1 / |vk⋅vk+1|

				T vk = SegmentValues[kIndex];
				T vkp1 = SegmentValues[kp1Index];

				double DotProduct = FMath::Clamp(double(vk | vkp1), -1.0, 1.0);
				double a = acos(abs(DotProduct));

				if (FMath::IsNearlyZero(a))
				{
					//Implementation Note
					//	When a is close to zero, spherical linear interpolation turns into regular linear interpolation.
					Value = (1.0 - Time) * vk + Time * vkp1;
				}
				else
				{
					double s = DotProduct / abs(DotProduct);

					// vt=(sin(a(1−t))/sin(a))∗vk + s∗(sin(at)/sin(a))∗vk+1

					Value = (sin(a * (1.0 - Time)) / sin(a)) * vk + s * (sin(a * Time) / sin(a)) * vkp1;
				}
			}
			else
			{
				//Normal Linear Interpolation
				// vt=(1−t)∗vk + t∗vk+1

				T vk = SegmentValues[kIndex];
				T vkp1 = SegmentValues[kp1Index];

				Value = (1.0 - Time) * vk + Time * vkp1;
			}
			break;
		}
		case GLTF::FAnimation::EInterpolation::Step:
		{
			//Step Interpolation
			// vt=vk

			T vk = SegmentValues[kIndex];
			T vkp1 = SegmentValues[kp1Index];
			if (Time < 0.5)
			{
				Value = vk;
			}
			else
			{
				Value = vkp1;
			}
			break;
		}
		case GLTF::FAnimation::EInterpolation::CubicSpline:
		{
			//Cubic Spline Interpolation
			// For each timestamp stored in the animation sampler, there are three associated keyframe values: in-tangent, property Value, and out-tangent.
			// ak, vk, and bk be the in-tangent, the property Value, and the out-tangent of the k-th frame respectively.
			// The interpolated sampler Value vt at the timestamp tc is computed as follows.
			// vt = (2t^3−3t^2 + 1)∗vk 
			//		+ td(t^3−2t^2 + t)∗bk 
			//		+ (−2t^3 + 3t^2)∗vk + 1 
			//		+ td(t^3−t^2)∗ak + 1

			T ak = SegmentValues[kIndex * 3 + 0];
			T vk = SegmentValues[kIndex * 3 + 1];
			T bk = SegmentValues[kIndex * 3 + 2];
			T akp1 = SegmentValues[kp1Index * 3 + 0];
			T vkp1 = SegmentValues[kp1Index * 3 + 1];
			T bkp1 = SegmentValues[kp1Index * 3 + 2];

			Value = (2 * Time * Time * Time - 3 * Time * Time + 1) * vk 
				+ TimeDelta * (Time * Time * Time - 2 * Time * Time + Time) * bk 
				+ (-2 * Time * Time * Time + 3 * Time * Time) * vkp1 
				+ TimeDelta * (Time * Time * Time - Time * Time) * akp1;

			break;
		}

		default:
			break;
		}

		return Value;
	}

	bool ParsePayLoadKey(const GLTF::FAsset& GltfAsset, const FString& PayLoadKey, int32& AnimationIndex, TArray<int32>& ChannelIndices)
	{
		TArray<FString> PayLoadValues;

		PayLoadKey.ParseIntoArray(PayLoadValues, TEXT(":"));

		LexFromString(AnimationIndex, *PayLoadValues[0]);

		if (!GltfAsset.Animations.IsValidIndex(AnimationIndex))
		{
			return false;
		}

		for (size_t Index = 1; Index < PayLoadValues.Num(); Index++)
		{
			int32 ChannelIndex;
			LexFromString(ChannelIndex, *PayLoadValues[Index]);

			if (GltfAsset.Animations[AnimationIndex].Channels.IsValidIndex(ChannelIndex))
			{
				ChannelIndices.Add(ChannelIndex);
			}
		}

		return ChannelIndices.Num() != 0;
	}

	bool GetTransformAnimationPayloadData(const FString& PayLoadKey, const GLTF::FAsset& GltfAsset, FAnimationPayloadData& OutPayloadData)
	{
		//translation, rotation, scale time 3 channels:
		OutPayloadData.Curves.SetNum(9);

		TArray<int32> ChannelIndices;
		int32 AnimationIndex;

		if (!ParsePayLoadKey(GltfAsset, PayLoadKey, AnimationIndex, ChannelIndices))
		{
			return false;
		}
		const GLTF::FAnimation& Animation = GltfAsset.Animations[AnimationIndex];

		for (const int32 ChannelIndex : ChannelIndices)
		{
			const GLTF::FAnimation::FChannel& Channel = Animation.Channels[ChannelIndex];
			const GLTF::FAnimation::FSampler& Sampler = Animation.Samplers[Channel.Sampler];

			TArray<float> FrameTimeBuffer;
			TArray<float> FrameDataBuffer;

			Sampler.Input.GetFloatArray(FrameTimeBuffer);

			ERichCurveInterpMode InterpolationMode = static_cast<ERichCurveInterpMode>(Sampler.Interpolation);

			TFunction<void(TFunction<FVector3f(const float*)>, int32, int32, bool)> ConvertCurve = [InterpolationMode, &FrameTimeBuffer, &FrameDataBuffer, &OutPayloadData](TFunction<FVector3f(const float*)> ConvertSample, int32 CurvesIndexOffset, int32 SampleSize, bool bUnwindRotation)
			{
				int32 ActualSampleSize = SampleSize;
				if (InterpolationMode == ERichCurveInterpMode::RCIM_Cubic)
				{
					ActualSampleSize *= 3;
				}
				const int32 SampleCount = FrameDataBuffer.Num() / ActualSampleSize;

				FRichCurve& Curve0 = OutPayloadData.Curves[CurvesIndexOffset + 0];
				FRichCurve& Curve1 = OutPayloadData.Curves[CurvesIndexOffset + 1];
				FRichCurve& Curve2 = OutPayloadData.Curves[CurvesIndexOffset + 2];

				for (int32 SampleIndex = 0; SampleIndex < SampleCount; ++SampleIndex)
				{


					if (InterpolationMode == ERichCurveInterpMode::RCIM_Cubic)
					{
						const float* RawSampleArrive = FrameDataBuffer.GetData() + (SampleIndex * ActualSampleSize) + 0 * SampleSize;
						const float* RawSampleValue = FrameDataBuffer.GetData() + (SampleIndex * ActualSampleSize) + 1 * SampleSize;
						const float* RawSampleLeave = FrameDataBuffer.GetData() + (SampleIndex * ActualSampleSize) + 2 * SampleSize;

						FVector3f SampleArrive = ConvertSample(RawSampleArrive);
						FVector3f SampleValue = ConvertSample(RawSampleValue);
						FVector3f SampleLeave = ConvertSample(RawSampleLeave);

						FKeyHandle Key0 = Curve0.AddKey(FrameTimeBuffer[SampleIndex], SampleValue[0], bUnwindRotation);
						FRichCurveKey& CurveKey0 = Curve0.GetKey(Key0);
						CurveKey0.ArriveTangent = SampleArrive[0];
						CurveKey0.LeaveTangent = SampleLeave[0];

						FKeyHandle Key1 = Curve1.AddKey(FrameTimeBuffer[SampleIndex], SampleValue[1], bUnwindRotation);
						FRichCurveKey& CurveKey1 = Curve1.GetKey(Key1);
						CurveKey1.ArriveTangent = SampleArrive[1];
						CurveKey1.LeaveTangent = SampleLeave[1];

						FKeyHandle Key2 = Curve2.AddKey(FrameTimeBuffer[SampleIndex], SampleValue[2], bUnwindRotation);
						FRichCurveKey& CurveKey2 = Curve2.GetKey(Key2);
						CurveKey2.ArriveTangent = SampleArrive[2];
						CurveKey2.LeaveTangent = SampleLeave[2];

						Curve0.SetKeyInterpMode(Key0, InterpolationMode);
						Curve1.SetKeyInterpMode(Key1, InterpolationMode);
						Curve2.SetKeyInterpMode(Key2, InterpolationMode);
					}
					else
					{
						const float* RawSample = FrameDataBuffer.GetData() + (SampleIndex * ActualSampleSize);
						FVector3f Sample = ConvertSample(RawSample);

						FKeyHandle Key0 = Curve0.AddKey(FrameTimeBuffer[SampleIndex], Sample[0], bUnwindRotation);
						FKeyHandle Key1 = Curve1.AddKey(FrameTimeBuffer[SampleIndex], Sample[1], bUnwindRotation);
						FKeyHandle Key2 = Curve2.AddKey(FrameTimeBuffer[SampleIndex], Sample[2], bUnwindRotation);

						Curve0.SetKeyInterpMode(Key0, InterpolationMode);
						Curve1.SetKeyInterpMode(Key1, InterpolationMode);
						Curve2.SetKeyInterpMode(Key2, InterpolationMode);
					}
				}
			};

			switch (Channel.Target.Path)
			{
			case GLTF::FAnimation::EPath::Translation:
			{
				FrameDataBuffer.SetNumUninitialized(Sampler.Output.Count * 3);
				Sampler.Output.GetCoordArray(reinterpret_cast<FVector3f*>(FrameDataBuffer.GetData()));
				const float Scale = GltfUnitConversionMultiplier; // Convert m to cm
				ConvertCurve([&Scale](const float* VecData)
					{
						return FVector3f(VecData[0] * Scale, VecData[1] * Scale, VecData[2] * Scale);
					}, 0, 3, false);
				break;
			}
			case GLTF::FAnimation::EPath::Rotation:
			{
				FrameDataBuffer.SetNumUninitialized(Sampler.Output.Count * 4);
				Sampler.Output.GetQuatArray(reinterpret_cast<FVector4f*>(FrameDataBuffer.GetData()));

				struct FHeadingHelper
				{
					FRotator3f Heading;
					bool bHeadingSet = false;
				};
				FHeadingHelper HeadingHelper;
				ConvertCurve([&HeadingHelper](const float* QuatData)
					{
						//GLTF uses quaternions while Uneal uses Eulers for level animations
						// Quaternions can represent multiple Orientations in Eulers.
						// We cannot tell which precise Orientation version was the author of gltf trying to achieve at the moment of creation.
						// For that reason we compare each consecutive orientation and pick the closest (by ManhattanDistance) from the 2 EquivalentRotators.
						// This means that animations of Rotations can be correctly represented with this system, if
						//    every consecutive frame's intended (absolute) rotations are less than 3x180 degrees (aka less than 180 degrees per axis)

						FQuat4f Quat(QuatData[0], QuatData[1], QuatData[2], QuatData[3]);
						FRotator3f Rotator = Quat.Rotator();
						if (HeadingHelper.bHeadingSet)
						{
							FMath::WindRelativeAnglesDegrees(HeadingHelper.Heading.Roll, Rotator.Roll);
							FMath::WindRelativeAnglesDegrees(HeadingHelper.Heading.Pitch, Rotator.Pitch);
							FMath::WindRelativeAnglesDegrees(HeadingHelper.Heading.Yaw, Rotator.Yaw);

							FRotator3f OtherChoice = Rotator.GetEquivalentRotator().GetNormalized();
							float FirstDiff = HeadingHelper.Heading.GetManhattanDistance(Rotator);
							float SecondDiff = HeadingHelper.Heading.GetManhattanDistance(OtherChoice);
							if (SecondDiff < FirstDiff)
							{
								Rotator = OtherChoice;
							}
						}
						else
						{
							HeadingHelper.bHeadingSet = true;
						}

						HeadingHelper.Heading = Rotator;

						return Rotator.Euler();
					}, 3, 4, true);
				break;
			}
			case GLTF::FAnimation::EPath::Scale:
			{
				FrameDataBuffer.SetNumUninitialized(Sampler.Output.Count * 3);
				Sampler.Output.GetCoordArray(reinterpret_cast<FVector3f*>(FrameDataBuffer.GetData()));
				ConvertCurve([](const float* VecData)
					{
						return FVector3f(VecData[0], VecData[1], VecData[2]);
					}, 6, 3, false);
				break;
			}
			case GLTF::FAnimation::EPath::Weights:
			{
				UE_LOGF(LogInterchangeImport, Warning, "Animation[%d]:Channel[%d]: Morph Animation (weight path) type not supported for transform-like animations. Morph animations should be handled through the \"GetMorphTargetAnimationPayloadData\" function.", AnimationIndex, ChannelIndex);
				break;
			}
			default:
				UE_LOGF(LogInterchangeImport, Warning, "Animation type not supported");
				break;
			}
		}
		return true;
	}

	bool GetMorphTargetAnimationPayloadData(const FString& PayLoadKey, const GLTF::FAsset& GltfAsset, FAnimationPayloadData& OutPayloadData)
	{
		OutPayloadData.Curves.SetNum(1);

		TArray<FString> PayLoadKeys;
		PayLoadKey.ParseIntoArray(PayLoadKeys, TEXT(":"));

		if (PayLoadKeys.Num() != 4)
		{
			return false;
		}

		int32 AnimationIndex;
		int32 ChannelIndex;
		int32 MeshIndex;
		int32 MorphTargetIndex;

		LexFromString(AnimationIndex, *PayLoadKeys[0]);
		LexFromString(ChannelIndex, *PayLoadKeys[1]);
		LexFromString(MeshIndex, *PayLoadKeys[2]);
		LexFromString(MorphTargetIndex, *PayLoadKeys[3]);

		if (GltfAsset.Animations.Num() <= AnimationIndex
			|| GltfAsset.Animations[AnimationIndex].Channels.Num() <= ChannelIndex
			|| GltfAsset.Meshes.Num() <= MeshIndex
			|| GltfAsset.Meshes[MeshIndex].MorphTargetNames.Num() <= MorphTargetIndex)
		{
			return false;
		}

		const GLTF::FAnimation& Animation = GltfAsset.Animations[AnimationIndex];
		const GLTF::FAnimation::FChannel& Channel = Animation.Channels[ChannelIndex];
		const GLTF::FAnimation::FSampler& Sampler = Animation.Samplers[Channel.Sampler];
		const GLTF::FMesh& Mesh = GltfAsset.Meshes[MeshIndex];

		TArray<float> FrameTimeBuffer;
		TArray<float> FrameDataBuffer;

		Sampler.Input.GetFloatArray(FrameTimeBuffer);

		ERichCurveInterpMode InterpolationMode = static_cast<ERichCurveInterpMode>(Sampler.Interpolation);

		if (uint32(MorphTargetIndex * FrameDataBuffer.Num()) > Sampler.Output.Count)
		{
			return false;
		}

		//framebuffer is :
		//morphTarget(n)_frame0 morphTarget(n+1)_frame0 morphTarget(n+2)_frame0 .... morphTarget(n)_frame1 morphTarget(n+1)_frame1 morphTarget(n+2)_frame1 .....
		// CUBIC:
		//morphTarget(n)_frame0_in morphTarget(n)_frame0 morphTarget(n)_frame0_out morphTarget(n+1)_frame0_in morphTarget(n+1)_frame0 morphTarget(n+1)_frame0_out morphTarget(n+2)_frame0_in morphTarget(n+2)_frame0 morphTarget(n+2)_frame0_out
		//in/out values are currently ignored (in Interchange pipelines)
		FrameDataBuffer.SetNumUninitialized(Sampler.Output.Count * (InterpolationMode == ERichCurveInterpMode::RCIM_Cubic ? 3 : 1));
		Sampler.Output.GetFloatArray(reinterpret_cast<float*>(FrameDataBuffer.GetData()));

		FRichCurve& Curve = OutPayloadData.Curves[0];

		int32 RowMultiplier = Mesh.MorphTargetNames.Num() * (InterpolationMode == ERichCurveInterpMode::RCIM_Cubic ? 3 : 1);

		for (int32 SampleIndex = 0; SampleIndex < FrameTimeBuffer.Num(); SampleIndex++)
		{
			if (InterpolationMode == ERichCurveInterpMode::RCIM_Cubic)
			{
				FKeyHandle Key = Curve.AddKey(FrameTimeBuffer[SampleIndex], FrameDataBuffer[MorphTargetIndex + (SampleIndex * RowMultiplier) + 1]);
				FRichCurveKey& CurveKey = Curve.GetKey(Key);
				CurveKey.ArriveTangent = FrameDataBuffer[MorphTargetIndex + (SampleIndex * RowMultiplier) + 0];
				CurveKey.LeaveTangent = FrameDataBuffer[MorphTargetIndex + (SampleIndex * RowMultiplier) + 2];

				Curve.SetKeyInterpMode(Key, InterpolationMode);
			}
			else
			{
				FKeyHandle Key = Curve.AddKey(FrameTimeBuffer[SampleIndex], FrameDataBuffer[MorphTargetIndex + (SampleIndex * RowMultiplier)]);

				Curve.SetKeyInterpMode(Key, InterpolationMode);
			}
		}

		return true;
	}

	void GetT0Transform(const GLTF::FAnimation& GltfAnimation, const GLTF::FNode& AnimatedNode, const TArray<int32>& ChannelIndices, FTransform& OutTransform)
	{
		int32 LowerIndex = 0;
		int32 HigherIndex = 0;
		double RequestedTime = 0.;

		FVector3f TranslationData = FVector3f(OutTransform.GetTranslation());
		FQuat4f RotationData = FQuat4f(OutTransform.GetRotation());
		FVector3f ScaleData = FVector3f(OutTransform.GetScale3D());

		for (const int32& ChannelIndex : ChannelIndices)
		{
			const GLTF::FAnimation::FChannel& Channel = GltfAnimation.Channels[ChannelIndex];
			const GLTF::FAnimation::FSampler& Sampler = GltfAnimation.Samplers[Channel.Sampler];

			TArray<float> Seconds;
			Sampler.Input.GetFloatArray(Seconds);

			if (Seconds.Num() == 0)
			{
				continue;
			}

			switch (Channel.Target.Path)
			{
				case GLTF::FAnimation::EPath::Translation:
				{
					TArray<FVector3f> TranslationBuffer;
					TranslationBuffer.SetNumUninitialized(Sampler.Output.Count);
					Sampler.Output.GetCoordArray(TranslationBuffer.GetData());
					const float Scale = GltfUnitConversionMultiplier; // Convert m to cm
					for (FVector3f& Position : TranslationBuffer)
					{
						Position *= Scale;
					}

					TranslationData = InterpolateValue(Seconds, TranslationBuffer, LowerIndex, HigherIndex, RequestedTime, Sampler.Interpolation, false);
					break;
				}
				case GLTF::FAnimation::EPath::Rotation:
				{
					TArray<FQuat4f> RotationBuffer;
					TArray<FVector4f> RotationBufferTemp;
					RotationBufferTemp.SetNumUninitialized(Sampler.Output.Count);
					RotationBuffer.SetNumUninitialized(Sampler.Output.Count);

					Sampler.Output.GetQuatArray(RotationBufferTemp.GetData());

					for (size_t Index = 0; Index < Sampler.Output.Count; Index++)
					{
						RotationBuffer[Index] = FQuat4f(RotationBufferTemp[Index][0], RotationBufferTemp[Index][1], RotationBufferTemp[Index][2], RotationBufferTemp[Index][3]);
					}

					RotationData = InterpolateValue(Seconds, RotationBuffer, LowerIndex, HigherIndex, RequestedTime, Sampler.Interpolation, true);
					RotationData.Normalize();
					break;
				}
				case GLTF::FAnimation::EPath::Scale:
				{
					TArray<FVector3f> ScaleBuffer;
					ScaleBuffer.SetNumUninitialized(Sampler.Output.Count);
					Sampler.Output.GetCoordArray(ScaleBuffer.GetData());

					ScaleData = InterpolateValue(Seconds, ScaleBuffer, LowerIndex, HigherIndex, RequestedTime, Sampler.Interpolation, false);

					break;
				}
				default:
					UE_LOGF(LogInterchangeImport, Warning, "Animation type not supported");
					break;
			}
		}

		OutTransform = FTransform(
			FQuat4d(RotationData),
			FVector3d(TranslationData),
			FVector3d(ScaleData));
	}

	bool GetBakedAnimationTransformPayloadData(const FString& PayLoadKey, const GLTF::FAsset& GltfAsset, FAnimationPayloadData& PayloadData)
	{
		const double BakeInterval = 1.0 / PayloadData.BakeFrequency;
		const double SequenceLength = FMath::Max<double>(PayloadData.RangeEndTime - PayloadData.RangeStartTime, MINIMUM_ANIMATION_LENGTH);
		int32 FrameCount = FMath::RoundToInt32(SequenceLength * PayloadData.BakeFrequency);
		int32 BakeKeyCount = FrameCount + 1;

		TArray<int32> ChannelIndices;
		int32 AnimationIndex;

		if (!ParsePayLoadKey(GltfAsset, PayLoadKey, AnimationIndex, ChannelIndices))
		{
			return false;
		}
		const GLTF::FAnimation& GltfAnimation = GltfAsset.Animations[AnimationIndex];

		//buffers to use for final Transform:
		TArray<FVector3f> TranslationData;
		TArray<FQuat4f> RotationData;
		TArray<FVector3f> ScaleData;

		//initial transforms:
		//channels are targetting the same node:
		bool BaseTransformsAcquired = false;

		for (const int32& ChannelIndex : ChannelIndices)
		{
			const GLTF::FAnimation::FChannel& Channel = GltfAnimation.Channels[ChannelIndex];
			const GLTF::FAnimation::FSampler& Sampler = GltfAnimation.Samplers[Channel.Sampler];

			if (!BaseTransformsAcquired)
			{
				//on the first valid channel id we set the targeted nodes default transforms
				//the default transforms will be overwritten by the provided animation data, IF they are provided
				BaseTransformsAcquired = true;
				FVector3f JointBaseTranslation(Channel.Target.Node.Transform.GetTranslation());
				FQuat4f JointBaseRotation(Channel.Target.Node.Transform.GetRotation());
				FVector3f JointBaseScale(Channel.Target.Node.Transform.GetScale3D());

				TranslationData.Init(JointBaseTranslation, BakeKeyCount);
				RotationData.Init(JointBaseRotation, BakeKeyCount);
				ScaleData.Init(JointBaseScale, BakeKeyCount);
			}

			TArray<float> Seconds;
			Sampler.Input.GetFloatArray(Seconds);

			if (Seconds.Num() == 0)
			{
				continue;
			}

			switch (Channel.Target.Path)
			{
			case GLTF::FAnimation::EPath::Translation:
			{
				TArray<FVector3f> TranslationBuffer;
				TranslationBuffer.SetNumUninitialized(Sampler.Output.Count);
				Sampler.Output.GetCoordArray(TranslationBuffer.GetData());
				const float Scale = GltfUnitConversionMultiplier; // Convert m to cm
				for (FVector3f& Position : TranslationBuffer)
				{
					Position *= Scale;
				}

				for (size_t Index = 0; Index < BakeKeyCount; Index++)
				{
					double CurrentRequestedTimeStamp = Index * BakeInterval;
					TTuple<int32, int32> LowerHigherIndices = GetNeighbourIndices(Seconds, CurrentRequestedTimeStamp);
					TranslationData[Index] = InterpolateValue(Seconds, TranslationBuffer, LowerHigherIndices.Key, LowerHigherIndices.Value, CurrentRequestedTimeStamp, Sampler.Interpolation, false);
				}
				break;
			}
			case GLTF::FAnimation::EPath::Rotation:
			{
				TArray<FQuat4f> RotationBuffer;
				TArray<FVector4f> RotationBufferTemp;
				RotationBufferTemp.SetNumUninitialized(Sampler.Output.Count);
				RotationBuffer.SetNumUninitialized(Sampler.Output.Count);

				Sampler.Output.GetQuatArray(RotationBufferTemp.GetData());

				for (size_t Index = 0; Index < Sampler.Output.Count; Index++)
				{
					RotationBuffer[Index] = FQuat4f(RotationBufferTemp[Index][0], RotationBufferTemp[Index][1], RotationBufferTemp[Index][2], RotationBufferTemp[Index][3]);
				}

				//When the animation sampler targets a node’s rotation property, the interpolated quaternion MUST be normalized before applying the result to the node’s rotation.
				for (size_t Index = 0; Index < BakeKeyCount; Index++)
				{
					double CurrentRequestedTimeStamp = Index * BakeInterval;
					TTuple<int32, int32> LowerHigherIndices = GetNeighbourIndices(Seconds, CurrentRequestedTimeStamp);
					RotationData[Index] = InterpolateValue(Seconds, RotationBuffer, LowerHigherIndices.Key, LowerHigherIndices.Value, CurrentRequestedTimeStamp, Sampler.Interpolation, true);
					RotationData[Index].Normalize();
				}
				break;
			}
			case GLTF::FAnimation::EPath::Scale:
			{
				TArray<FVector3f> ScaleBuffer;
				ScaleBuffer.SetNumUninitialized(Sampler.Output.Count);
				Sampler.Output.GetCoordArray(ScaleBuffer.GetData());

				for (size_t Index = 0; Index < BakeKeyCount; Index++)
				{
					double CurrentRequestedTimeStamp = Index * BakeInterval;
					TTuple<int32, int32> LowerHigherIndices = GetNeighbourIndices(Seconds, CurrentRequestedTimeStamp);
					ScaleData[Index] = InterpolateValue(Seconds, ScaleBuffer, LowerHigherIndices.Key, LowerHigherIndices.Value, CurrentRequestedTimeStamp, Sampler.Interpolation, false);
				}

				break;
			}
			default:
				UE_LOGF(LogInterchangeImport, Warning, "Animation type not supported.");
				break;
			}
		}

		for (size_t Index = 0; Index < BakeKeyCount; Index++)
		{
			PayloadData.Transforms.Add(FTransform(
				FQuat4d(RotationData[Index]),
				FVector3d(TranslationData[Index]),
				FVector3d(ScaleData[Index])));
		}

		return true;
	}

	namespace AnimationHelpers
	{
		struct FAnimationHandler
		{
			UInterchangeBaseNodeContainer& NodeContainer;
			int32 AnimationIndex;
			const GLTF::FAnimation& GLTFAnimation;
			const TArray<GLTF::FNode>& GLTFNodes;
			const TArray<GLTF::FSkinInfo>& GLTFSkins;
			const TMap<const GLTF::FNode*, FString>& GLTFNodeToInterchangeUidMap; //UInterchangeGLTFTranslator::NodeUidMap

			TMap<FString, UInterchangeSkeletalAnimationTrackNode*> RootJointIndexToTrackNodeMap;

			FAnimationHandler(UInterchangeBaseNodeContainer& InNodeContainer, 
				int32 InAnimationIndex, const GLTF::FAnimation& InGLTFAnimation, 
				const TArray<GLTF::FNode>& InGLTFNodes,
				const TArray<GLTF::FSkinInfo>& InGLTFSkins,
				const TMap<const GLTF::FNode*, FString>& InGLTFNodeToInterchangeUidMap)
				: NodeContainer(InNodeContainer)
				, AnimationIndex(InAnimationIndex)
				, GLTFAnimation(InGLTFAnimation)
				, GLTFNodes(InGLTFNodes)
				, GLTFSkins(InGLTFSkins)
				, GLTFNodeToInterchangeUidMap(InGLTFNodeToInterchangeUidMap)
			{}

			struct FAnimationTimes
			{
				inline static const double DefaultFrameRate = 30.0;
				inline static const TArray<FFrameRate> UsualFrameRates = { 
					UE::Interchange::Animation::ConvertSampleRateToFrameRate(DefaultFrameRate), //30
					
					UE::Interchange::Animation::ConvertSampleRateToFrameRate(24.),
					FFrameRate(24000, 1001),								//23.976
					UE::Interchange::Animation::ConvertSampleRateToFrameRate(25),
					FFrameRate(30000, 1001),								//29.97
					UE::Interchange::Animation::ConvertSampleRateToFrameRate(48),
					UE::Interchange::Animation::ConvertSampleRateToFrameRate(50),
					UE::Interchange::Animation::ConvertSampleRateToFrameRate(60),
					FFrameRate(60000, 1001),								//59.94
					UE::Interchange::Animation::ConvertSampleRateToFrameRate(72),
					UE::Interchange::Animation::ConvertSampleRateToFrameRate(90),
					UE::Interchange::Animation::ConvertSampleRateToFrameRate(100),
					UE::Interchange::Animation::ConvertSampleRateToFrameRate(120),
					UE::Interchange::Animation::ConvertSampleRateToFrameRate(144),
					UE::Interchange::Animation::ConvertSampleRateToFrameRate(165),
					UE::Interchange::Animation::ConvertSampleRateToFrameRate(240),
					UE::Interchange::Animation::ConvertSampleRateToFrameRate(480)
				};

				//@ StartTime = 0;
				//from gltf documentation: https://registry.khronos.org/glTF/specs/2.0/glTF-2.0.html#animations
				//Implementation Note
				//	For example, if the earliest sampler input for an animation is t = 10, a client implementation must begin playback of that animation channel at t = 0 with output clamped to the first available output value.
				double StartTime = 0.0;
				double StopTime = 0.0;
			};


			UInterchangeSkeletalAnimationTrackNode* AcquireTrackNode(const FString& SkeletonNodeUid, FAnimationTimes& AnimationTimes)
			{
				UInterchangeSkeletalAnimationTrackNode* TrackNode = nullptr;
				if (RootJointIndexToTrackNodeMap.Contains(SkeletonNodeUid))
				{
					TrackNode = RootJointIndexToTrackNodeMap[SkeletonNodeUid];

					TrackNode->GetCustomAnimationStopTime(AnimationTimes.StopTime);
				}
				else
				{
					TrackNode = NewObject< UInterchangeSkeletalAnimationTrackNode >(&NodeContainer);
					FString TrackNodeUid = "\\SkeletalAnimation\\" + SkeletonNodeUid + "_" + LexToString(AnimationIndex);
					NodeContainer.SetupNode(TrackNode, TrackNodeUid, GLTFAnimation.Name, EInterchangeNodeContainerType::TranslatedAsset);
					TrackNode->SetCustomSkeletonNodeUid(SkeletonNodeUid);

					for (const TPair<FString, FString>& Extra : GLTFAnimation.Extras)
					{
						UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(TrackNode, Extra.Key, Extra.Value, TOptional<FString>());
					}

					RootJointIndexToTrackNodeMap.Add(SkeletonNodeUid, TrackNode);
				}

				return TrackNode;
			};

			bool ValidateChannelSampler(const GLTF::FAnimation::FChannel& Channel, const GLTF::FAnimation::FSampler& Sampler)
			{
				if (Sampler.Interpolation == GLTF::FAnimation::EInterpolation::CubicSpline)
				{
					if (Sampler.Input.Count != 3 * Sampler.Output.Count)
					{
						return false;

					}
				}
				else
				{
					if (Channel.Target.Path != GLTF::FAnimation::EPath::Weights && (Sampler.Input.Count != Sampler.Output.Count))
					{
						return false;
					}
				}
				return true;
			};

			void UpdateAnimationTimes(const GLTF::FAnimation::FSampler& Sampler, FAnimationTimes& AnimationTimes)
			{
				TArray<float> Seconds;
				Sampler.Input.GetFloatArray(Seconds);

				if (Seconds.Num() > 0)
				{
					float CurrentStopTime = Seconds[Seconds.Num() - 1];
					if (AnimationTimes.StopTime < CurrentStopTime)
					{
						AnimationTimes.StopTime = CurrentStopTime;
					}
				}
			};

			void ValidateAnimationStopAndFrameRate(const FString& AnimationName, double& AnimationStopTime, double& FrameRate)
			{
				auto AtLeast1FrameLong = [&AnimationStopTime, &FrameRate]()
					{
						double FrameLength = 1. / FrameRate;
						if (AnimationStopTime < FrameLength)
						{
							AnimationStopTime = FrameLength;
						}
					};

				for (const FFrameRate& FrameRateCandidate : FAnimationTimes::UsualFrameRates)
				{
					const FFrameTime FrameTime = FrameRateCandidate.AsFrameTime(AnimationStopTime);
					const FFrameTime RoundedFrameTime = FrameTime.RoundToFrame();
					float DeltaTiming = FMath::Abs(RoundedFrameTime.AsDecimal() * FrameRateCandidate.AsInterval() - AnimationStopTime);
					//0.001 represents 1ms.
					if (FMath::IsNearlyZero(DeltaTiming, 1e-3) || FMath::IsNearlyEqual(DeltaTiming, FrameRateCandidate.AsInterval(), 1e-3))
					{
						FrameRate = FrameRateCandidate.AsDecimal();

						const float SubFrame = FrameTime.GetSubFrame();
						if (!FMath::IsNearlyZero(SubFrame, KINDA_SMALL_NUMBER) && !FMath::IsNearlyEqual(SubFrame, 1.0f, KINDA_SMALL_NUMBER))
						{
							//Update timing to align with FrameTimings for UE this might be needed due to:
							//Exporters sometimes round the timings to certain decimals (for example to ms)
							// For example 20 frames on 30 FPS becomes 0.66666666666666666666666666666667
							//	Which then might be truncated to 0.666 or rounded to 0.667.
							// To check for this scenario we actually do comparison on the SubFrame in seconds with 0.001 delta evaluation.
							// To fix this we re-align the AnimationStopTime here:
							AnimationStopTime = FrameRateCandidate.AsSeconds(FrameRateCandidate.AsFrameTime(AnimationStopTime).RoundToFrame());
						}

						AtLeast1FrameLong();

						if (FrameRate != FAnimationTimes::DefaultFrameRate)
						{
							UE_LOG(LogInterchangeImport, Display, TEXT("Animation [%s] FrameRate identified as %.3f"), *AnimationName, FrameRate);
						}
						return;
					}
				}

				
				FrameRate = FAnimationTimes::DefaultFrameRate;
				AtLeast1FrameLong();
				UE_LOG(LogInterchangeImport, Display, TEXT("Animation [%s] FrameRate could not be identified. Using default %f."), *AnimationName, FrameRate);
			}

			void AcquireJointsWithBindPose(int32 CurrentIndex, TSet<int32>& Joints)
			{
				if (GLTFNodes.IsValidIndex(CurrentIndex))
				{
					const GLTF::FNode& CurrentNode = GLTFNodes[CurrentIndex];

					if (CurrentNode.Type == GLTF::FNode::EType::Joint)
					{
						if (CurrentNode.bHasLocalBindPose)
						{
							Joints.Add(CurrentIndex);
						}

						for (int32 ChildIndex : CurrentNode.Children)
						{
							AcquireJointsWithBindPose(ChildIndex, Joints);
						}
					}
				}
			}

			void ProcessRiggedAnimations(const TMap<int32, TMap<int32, TSet<int32>>>& RiggedAnimations)
			{
				for (const TPair<int32, TMap<int32, TSet<int32>>>& RiggedAnimation : RiggedAnimations)
				{
					int32 NodeIndex = RiggedAnimation.Key;
					const GLTF::FNode* GLTFNode = &GLTFNodes[NodeIndex];

					//Acquire Node Interchange UniqueID:
					const FString* NodeUidPtr = GLTFNodeToInterchangeUidMap.Find(GLTFNode);
					if (!ensure(NodeUidPtr))
					{
						continue;
					}

					FAnimationTimes AnimationTimes;

					//Acquire/Create UInterchangeSkeletalAnimationTrackNode:
					UInterchangeSkeletalAnimationTrackNode* TrackNode = AcquireTrackNode(*NodeUidPtr, AnimationTimes);

					//iterate (AnimatedNodeIndex, [Channels])
					for (const TTuple<int32, TSet<int32>>& AnimatedNodeIndexToChannelIndices : RiggedAnimation.Value)
					{
						int32 AnimatedNodeIndex = AnimatedNodeIndexToChannelIndices.Key;
						const FString* AnimatedNodeUidPtr = GLTFNodeToInterchangeUidMap.Find(&GLTFNodes[AnimatedNodeIndex]);
						if (!ensure(AnimatedNodeUidPtr))
						{
							continue;
						}

						//double PreviousStopTime = StopTime;
						//check channel length and build payload
						FString Payload = TEXT("");
						for (int32 ChannelIndex : AnimatedNodeIndexToChannelIndices.Value)
						{
							const GLTF::FAnimation::FChannel& Channel = GLTFAnimation.Channels[ChannelIndex];
							const GLTF::FAnimation::FSampler& Sampler = GLTFAnimation.Samplers[Channel.Sampler];

							if (!ValidateChannelSampler(Channel, Sampler))
							{
								// if any of the channels are corrupt the joint will not receive any of the  animation data
								UE_LOGF(LogInterchangeImport, Warning, "glTF sampler corrupt. Input and Output do not meet expectations.");
								break;
							}

							UpdateAnimationTimes(Sampler, AnimationTimes);

							if (Channel.Target.Path != GLTF::FAnimation::EPath::Weights)
							{
								Payload += ":" + LexToString(ChannelIndex);
							}
						}

						if (Payload.Len() > 0)
						{
							Payload = LexToString(AnimationIndex) + Payload;
							TrackNode->SetAnimationPayloadKeyForSceneNodeUid(*AnimatedNodeUidPtr, Payload, EInterchangeAnimationPayLoadType::BAKED);
						}
					}

					//set animation length:
					TrackNode->SetCustomAnimationStartTime(AnimationTimes.StartTime);

					double FrameRate = FAnimationTimes::DefaultFrameRate;
					double AnimationStopTime = AnimationTimes.StopTime;
					ValidateAnimationStopAndFrameRate(TrackNode->GetDisplayLabel(), AnimationStopTime, FrameRate);
					TrackNode->SetCustomAnimationSampleRate(FrameRate);
					TrackNode->SetCustomAnimationStopTime(AnimationStopTime);
				}
			}

			void ProcessMorphTargetAnimations(const TMap<int32, TMap<int32, TSet<int32>>>& MorphTargetAnimations)
			{
				ProcessRiggedAnimations(MorphTargetAnimations);

				for (const TPair<int32, TMap<int32, TSet<int32>>>& RiggedAnimation : MorphTargetAnimations)
				{
					int32 NodeIndex = RiggedAnimation.Key;
					const GLTF::FNode* GLTFNode = &GLTFNodes[NodeIndex];

					//Acquire Interchange UniqueID:
					const FString* NodeUidPtr = GLTFNodeToInterchangeUidMap.Find(GLTFNode);
					if (!ensure(NodeUidPtr))
					{
						continue;
					}

					FAnimationTimes AnimationTimes;

					//Acquire/Create UInterchangeSkeletalAnimationTrackNode:
					UInterchangeSkeletalAnimationTrackNode* TrackNode = AcquireTrackNode(*NodeUidPtr, AnimationTimes);

					//iterate (AnimatedNodeIndex, [Channels])
					for (const TTuple<int32, TSet<int32>>& AnimatedNodeIndexToChannelIndices : RiggedAnimation.Value)
					{
						int32 AnimatedNodeIndex = AnimatedNodeIndexToChannelIndices.Key;
						const FString* AnimatedNodeUidPtr = GLTFNodeToInterchangeUidMap.Find(&GLTFNodes[AnimatedNodeIndex]);
						if (!ensure(AnimatedNodeUidPtr))
						{
							continue;
						}

						const TSet<int32>& Channels = AnimatedNodeIndexToChannelIndices.Value;
						ensure(Channels.Num() == 1);

						//Find SceneNode that references the MeshNode:
						if (const UInterchangeSceneNode* ConstSceneMeshActorNode = Cast< UInterchangeSceneNode >(NodeContainer.GetNode(*AnimatedNodeUidPtr)))
						{
							FString SkeletalMeshUid;
							if (ConstSceneMeshActorNode->GetCustomAssetInstanceUid(SkeletalMeshUid))
							{
								if (const UInterchangeMeshNode* MeshNode = Cast< UInterchangeMeshNode >(NodeContainer.GetNode(SkeletalMeshUid)))
								{
									TArray<FString> MorphTargetDependencies;
									MeshNode->GetMorphTargetDependencies(MorphTargetDependencies);
									for (const FString& MorphTargetDependencyUid : MorphTargetDependencies)
									{
										if (const UInterchangeMeshNode* MorphTargetNodeConst = Cast< UInterchangeMeshNode >(NodeContainer.GetNode(MorphTargetDependencyUid)))
										{
											if (MorphTargetNodeConst->GetPayLoadKey().IsSet())
											{
												FInterchangeMeshPayLoadKey PayLoadKey = MorphTargetNodeConst->GetPayLoadKey().GetValue();
												FString PayLoadKeyUniqueId = LexToString(AnimationIndex) + TEXT(":") + LexToString(*Channels.begin()) + TEXT(":") + PayLoadKey.UniqueId;

												TrackNode->SetAnimationPayloadKeyForMorphTargetNodeUid(MorphTargetDependencyUid, PayLoadKeyUniqueId, EInterchangeAnimationPayLoadType::MORPHTARGETCURVE);
											}
										}
									}
								}
							}
						}
					}
				}
			}

			void ProcessRigidAnimation(const TMap<int32, TSet<int32>>& RigidAnimation)
			{
				if (RigidAnimation.Num() == 0)
				{
					return;
				}

				UInterchangeAnimationTrackSetNode* TrackSetNode = NewObject< UInterchangeAnimationTrackSetNode >(&NodeContainer);

				const FString AnimTrackSetNodeUid = TEXT("\\Animation\\") + GLTFAnimation.UniqueId;
				NodeContainer.SetupNode(TrackSetNode, AnimTrackSetNodeUid, GLTFAnimation.Name, EInterchangeNodeContainerType::TranslatedAsset);

				for (const TTuple<int32, TSet<int32>>& NodeChannelsEntry : RigidAnimation)
				{
					if (NodeChannelsEntry.Value.Num() == 0)
					{
						continue;
					}

					const GLTF::FNode& GltfNode = GLTFNodes[NodeChannelsEntry.Key];
					const FString* NodeUid = GLTFNodeToInterchangeUidMap.Find(&GltfNode);
					if (!ensure(NodeUid))
					{
						continue;
					}

					UInterchangeTransformAnimationTrackNode* TransformAnimTrackNode = NewObject< UInterchangeTransformAnimationTrackNode >(&NodeContainer);

					const FString TransformAnimTrackNodeName = FString::Printf(TEXT("%s_%s"), *GltfNode.Name, *GLTFAnimation.Name);
					const FString TransformAnimTrackNodeUid = TEXT("\\AnimationTrack\\") + TransformAnimTrackNodeName;

					NodeContainer.SetupNode(TransformAnimTrackNode, TransformAnimTrackNodeUid, TransformAnimTrackNodeName, EInterchangeNodeContainerType::TranslatedAsset);

					TransformAnimTrackNode->SetCustomActorDependencyUid(*NodeUid);

					FString PayloadKey = FString::FromInt(AnimationIndex);

					constexpr int32 TranslationChannel = 0x0001 | 0x0002 | 0x0004;
					constexpr int32 RotationChannel = 0x0008 | 0x0010 | 0x0020;
					constexpr int32 ScaleChannel = 0x0040 | 0x0080 | 0x0100;

					int32 UsedChannels = 0;

					for (int32 ChannelIndex : NodeChannelsEntry.Value)
					{
						PayloadKey += TEXT(":") + FString::FromInt(ChannelIndex);

						const GLTF::FAnimation::FChannel& Channel = GLTFAnimation.Channels[ChannelIndex];

						switch (Channel.Target.Path)
						{
						case GLTF::FAnimation::EPath::Translation:
						{
							UsedChannels |= TranslationChannel;
						} break;

						case GLTF::FAnimation::EPath::Rotation:
						{
							UsedChannels |= RotationChannel;
						} break;

						case GLTF::FAnimation::EPath::Scale:
						{
							UsedChannels |= ScaleChannel;
						} break;
						default: break;
						}
					}

					TransformAnimTrackNode->SetCustomAnimationPayloadKey(PayloadKey, EInterchangeAnimationPayLoadType::CURVE);
					TransformAnimTrackNode->SetCustomUsedChannels(UsedChannels);

					TrackSetNode->AddCustomAnimationTrackUid(TransformAnimTrackNodeUid);
				}
			}

			void Process()
			{
				TMap<int32, TSet<int32>> RigidAnimation;							// (AnimatedNodeIndex, [Channels])
				TMap<int32, TMap<int32, TSet<int32>>> RiggedAnimations;				// (SkeletonRootIndex, (AnimatedNodeIndex, [Channels])
				TMap<int32, TMap<int32, TSet<int32>>> MorphTargetAnimations;		// (UEAnimatedNodeIndex, (AnimatedNodeIndex, [Channels]) //Channel is a Target.Path==Weight channel.
																					// glTF allows MorphTargets on StaticMeshes as well.
																					// However UE only does MorphTargets on Skeletals, for this reason we presume Skeletals and Skeletal Animations for Morph Target Animations.

				// Important: AnimatedNodes where bHasLocalBindPose==true
				TMap<int32, TSet<int32>> SkeletonRootToAnimatedJointNodeIndicesMap;	// (SkeletonRootIndex, [AnimatedNodeIndices])

				for (int32 ChannelIndex = 0; ChannelIndex < GLTFAnimation.Channels.Num(); ++ChannelIndex)
				{
					const GLTF::FAnimation::FChannel& Channel = GLTFAnimation.Channels[ChannelIndex];
					const GLTF::FNode& AnimatedNode = Channel.Target.Node;
					int32 AnimatedNodeIndex = AnimatedNode.Index;

					bool bMorphTargetAnimation = Channel.Target.Path == GLTF::FAnimation::EPath::Weights;
					bool bSkeletalAnimation = AnimatedNode.Type == GLTF::FNode::EType::Joint && GLTFNodes.IsValidIndex(AnimatedNode.RootJointIndex);

					if (bMorphTargetAnimation)
					{
						int32 UEAnimatedNodeIndex = AnimatedNodeIndex;
						if (AnimatedNode.Type == GLTF::FNode::EType::MeshSkinned)
						{
							const GLTF::FSkinInfo& Skin = GLTFSkins[AnimatedNode.Skindex];
							if (GLTFSkins.IsValidIndex(AnimatedNode.Skindex)
								&& GLTFSkins[AnimatedNode.Skindex].Joints.Num()
								&& GLTFNodes.IsValidIndex(GLTFSkins[AnimatedNode.Skindex].Joints[0]))
							{
								UEAnimatedNodeIndex = GLTFNodes[Skin.Joints[0]].RootJointIndex;
							}
						}
						else if (AnimatedNode.Type == GLTF::FNode::EType::Joint)
						{
							UEAnimatedNodeIndex = AnimatedNode.RootJointIndex;
						}

						TMap<int32, TSet<int32>>& AnimatedNodeIndicesToChannelIndices = MorphTargetAnimations.FindOrAdd(UEAnimatedNodeIndex);
						TSet<int32>& ChannelIndices = AnimatedNodeIndicesToChannelIndices.FindOrAdd(AnimatedNodeIndex);
						ChannelIndices.Add(ChannelIndex);
					}
					else if (bSkeletalAnimation)
					{
						TMap<int32, TSet<int32>>& AnimatedNodeIndicesToChannelIndices = RiggedAnimations.FindOrAdd(AnimatedNode.RootJointIndex);
						TSet<int32>& ChannelIndices = AnimatedNodeIndicesToChannelIndices.FindOrAdd(AnimatedNodeIndex);
						ChannelIndices.Add(ChannelIndex);
					}
					else
					{
						//Rigid animation:
						TSet<int32>& ChannelIndices = RigidAnimation.FindOrAdd(AnimatedNodeIndex);
						ChannelIndices.Add(ChannelIndex);
					}

					if (bSkeletalAnimation)
					{
						TSet<int32>& AnimatedJointNodeIndices = SkeletonRootToAnimatedJointNodeIndicesMap.FindOrAdd(AnimatedNode.RootJointIndex);
						AnimatedJointNodeIndices.Add(AnimatedNode.Index);
					}
				}

				ProcessRiggedAnimations(RiggedAnimations);
				ProcessMorphTargetAnimations(MorphTargetAnimations);
				ProcessRigidAnimation(RigidAnimation);
			}
		};
	}

	void HandleGLTFAnimations(UInterchangeBaseNodeContainer& NodeContainer,
		TArray<GLTF::FAnimation> Animations,
		const TArray<GLTF::FNode>& GLTFNodes,
		const TArray<GLTF::FSkinInfo>& GLTFSkins,
		const TMap<const GLTF::FNode*, FString>& GLTFNodeToInterchangeUidMap)
	{
		using namespace AnimationHelpers;

		for (int32 AnimationIndex = 0; AnimationIndex < Animations.Num(); AnimationIndex++)
		{
			FAnimationHandler AnimationHadler(NodeContainer, AnimationIndex, Animations[AnimationIndex], GLTFNodes, GLTFSkins, GLTFNodeToInterchangeUidMap);
			AnimationHadler.Process();
		}
	}
}