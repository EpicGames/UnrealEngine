// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Gltf/InterchangeGltfPrivate.h"

#include "GLTFAsset.h"
#include "GLTFMaterial.h"
#include "GLTFMesh.h"
#include "GLTFMeshFactory.h"
#include "GLTFNode.h"
#include "StaticMeshAttributes.h"
#include "SkeletalMeshAttributes.h"

#include "StaticMeshOperations.h"
#include "SkeletalMeshOperations.h"

#include "Mesh/InterchangeMeshPayload.h"

#include "InterchangeImportLog.h"

namespace UE::Interchange::Gltf::Private
{
	/*
	* For a given node index calculates the Joint Transformations
	*/
	void CalculateJointTransformations(const TArray<GLTF::FNode>& Nodes, int32 CurrentIndex, FTransform& CurrentTransform)
	{
		while (Nodes.IsValidIndex(CurrentIndex) && Nodes[CurrentIndex].Type == GLTF::FNode::EType::Joint)
		{
			CurrentTransform *= Nodes[CurrentIndex].Transform;
			CurrentIndex = Nodes[CurrentIndex].ParentIndex;
		}
	}

	int32 GetRootNodeIndex(const GLTF::FAsset& GltfAsset, const TArray<int32>& NodeIndices)
	{
		int32 ClosestIndex = -1;
		int32 ClosestRouteLength = INT32_MAX;
		for (int32 NodeIndex : NodeIndices)
		{
			int32 CurrentNodeIndexTracker = NodeIndex;
			int32 CurrentRouteLength = 0;
			while (GltfAsset.Nodes[CurrentNodeIndexTracker].ParentIndex != INDEX_NONE)
			{
				CurrentRouteLength++;
				CurrentNodeIndexTracker = GltfAsset.Nodes[CurrentNodeIndexTracker].ParentIndex;
			}
			if (CurrentRouteLength < ClosestRouteLength)
			{
				ClosestRouteLength = CurrentRouteLength;
				ClosestIndex = NodeIndex;
			}
		}
		return ClosestIndex;
	}

	void PatchPolygonGroups(FMeshDescription& MeshDescription, const GLTF::FAsset& GltfAsset)
	{
		// Patch polygon groups material slot names to match Interchange expectations (rename material slots from indices to material names)
		{
			FStaticMeshAttributes StaticMeshAttributes(MeshDescription);

			for (int32 MaterialSlotIndex = 0; MaterialSlotIndex < StaticMeshAttributes.GetPolygonGroupMaterialSlotNames().GetNumElements(); ++MaterialSlotIndex)
			{
				int32 MaterialIndex = 0;
				LexFromString(MaterialIndex, *StaticMeshAttributes.GetPolygonGroupMaterialSlotNames()[MaterialSlotIndex].ToString());

				if (GltfAsset.Materials.IsValidIndex(MaterialIndex))
				{
					const FString MaterialName = GltfAsset.Materials[MaterialIndex].Name;
					StaticMeshAttributes.GetPolygonGroupMaterialSlotNames()[MaterialSlotIndex] = *MaterialName;
				}
			}
		}
	}

	void PrintGltfMessages(GLTF::FMeshFactory& MeshFactory)
	{
		// Add glTF errors and warnings to the Interchange results
		for (const GLTF::FLogMessage& LogMessage : MeshFactory.GetLogMessages())
		{
			switch (LogMessage.Get<0>())
			{
			case GLTF::EMessageSeverity::Error:
			{
				UE_LOGF(LogInterchangeImport, Error, "glTF Mesh Import Error message: %ls", *LogMessage.Get<1>().ToString());
			}
			break;
			case GLTF::EMessageSeverity::Warning:
			{
				UE_LOGF(LogInterchangeImport, Warning, "glTF Mesh Import Warning message: %ls", *LogMessage.Get<1>().ToString());
			}
			break;
			case GLTF::EMessageSeverity::Display:
			{
				UE_LOGF(LogInterchangeImport, Display, "glTF Mesh Import Display message: %ls", *LogMessage.Get<1>().ToString());
			}
			break;
			default:
				break;
			}
		}
	}

	bool GetSkeletalMeshDescriptionForPayLoadKey(const GLTF::FAsset& GltfAsset, const FString& PayLoadKey, const FTransform& MeshGlobalTransform, FMeshDescription& MeshDescription, TArray<FString>* OutJointUniqueNames)
	{
		TArray<FString> PayLoadKeys;
		PayLoadKey.ParseIntoArray(PayLoadKeys, TEXT(":"));
		TMap<int32, TArray<int32>> MeshIndexToSkinIndicesMap;

		int32 MeshIndex = 0;
		for (const FString& SubPayLoadKey : PayLoadKeys)
		{
			int32 MeshAndSkinIndex = 0;
			LexFromString(MeshAndSkinIndex, *SubPayLoadKey);

			MeshIndex = MeshAndSkinIndex & 0xFFFF;
			const int32 SkinIndex = MeshAndSkinIndex >> 16;

			if (!GltfAsset.Meshes.IsValidIndex(MeshIndex) || !GltfAsset.Skins.IsValidIndex(SkinIndex))
			{
				continue;
			}

			TArray<int32>& SkinIndices = MeshIndexToSkinIndicesMap.FindOrAdd(MeshIndex);
			SkinIndices.Add(SkinIndex);
		}
		if (MeshIndexToSkinIndicesMap.Array().Num() > 1)
		{
			//invalid scenario, because its indicating that the skeletal mesh payload key contains multiple mesh targets.
			return false;
		}

		if (MeshIndexToSkinIndicesMap.Array().Num() == 0)
		{
			return false;
		}

		FMeshDescription BaseMeshDescription;

		const GLTF::FMesh& GltfMesh = GltfAsset.Meshes[MeshIndex];
		GLTF::FMeshFactory MeshFactory;
		MeshFactory.SetUniformScale(GltfUnitConversionMultiplier); // GLTF is in meters while UE is in centimeters
		MeshFactory.FillMeshDescription(GltfMesh, MeshGlobalTransform, &BaseMeshDescription);

		PatchPolygonGroups(BaseMeshDescription, GltfAsset);

		TArray<FMeshDescription> SkinnedMeshDescriptions;
		TMap<int32, TArray<FString>> SkinVsLocalJointNames;
		const TArray<int32>& SkinIndices = MeshIndexToSkinIndicesMap[MeshIndex];

		for (size_t Index = 0; Index < SkinIndices.Num(); Index++)
		{
			FMeshDescription& SkinnedMeshDescription = SkinnedMeshDescriptions.Add_GetRef(BaseMeshDescription);
			const int32 SkinIndex = SkinIndices[Index];

			if (SkinIndex == INDEX_NONE)
			{
				continue;
			}
			const GLTF::FSkinInfo& Skin = GltfAsset.Skins[SkinIndex];

			FSkeletalMeshAttributes SkeletalMeshAttributes(SkinnedMeshDescription);
			SkeletalMeshAttributes.Register(true);

			using namespace UE::AnimationCore;
			TMap<FVertexID, TArray<FBoneWeight>> RawBoneWeights;

			//Add the influence data in the skeletalmesh description
			FSkinWeightsVertexAttributesRef VertexSkinWeights = SkeletalMeshAttributes.GetVertexSkinWeights();

			const TArray<GLTF::FMeshFactory::FIndexVertexIdMap>& PositionIndexToVertexIdPerPrim = MeshFactory.GetPositionIndexToVertexIdPerPrim();

			TArray<FString>& LocalJointNames = SkinVsLocalJointNames.FindOrAdd(SkinIndex);
			for (size_t JoinIndex = 0; JoinIndex < Skin.Joints.Num(); JoinIndex++)
			{
				OutJointUniqueNames->Add(GltfAsset.Nodes[Skin.Joints[JoinIndex]].Name);
				LocalJointNames.Add(GltfAsset.Nodes[Skin.Joints[JoinIndex]].Name);
			}

			for (int32 PrimIndex = 0; PrimIndex < GltfMesh.Primitives.Num(); ++PrimIndex)
			{
				const GLTF::FPrimitive& Prim = GltfMesh.Primitives[PrimIndex];

				TArray<GLTF::FJointInfluence> JointInfluences;
				Prim.GetJointInfluences(JointInfluences);

				const GLTF::FMeshFactory::FIndexVertexIdMap& PositionIndexToVertexId = PositionIndexToVertexIdPerPrim[PrimIndex];

				for (int32 PositionIndex = 0; PositionIndex < JointInfluences.Num(); ++PositionIndex)
				{
					const FVertexID* VertexID = PositionIndexToVertexId.Find(PositionIndex);
					if (!VertexID || !ensure(SkinnedMeshDescription.IsVertexValid(*VertexID)))
					{
						continue;
					}

					for (int32 BoneIndex = 0; BoneIndex < 4; ++BoneIndex)
					{
						const int32 JointIndex = JointInfluences[PositionIndex].ID[BoneIndex];
						const int32 NodeIndex = Skin.Joints[JointIndex];
						const float Weight = JointInfluences[PositionIndex].Weight[BoneIndex];

						const GLTF::FNode* Bone = &GltfAsset.Nodes[NodeIndex];

						TArray<FBoneWeight>* BoneWeights = RawBoneWeights.Find(*VertexID);
						if (BoneWeights)
						{
							// Do we already have a weight for this bone?
							bool bShouldAdd = true;
							for (int32 WeightIndex = 0; WeightIndex < BoneWeights->Num(); WeightIndex++)
							{
								FBoneWeight& BoneWeight = (*BoneWeights)[WeightIndex];
								if (BoneWeight.GetBoneIndex() == JointIndex)
								{
									if (BoneWeight.GetWeight() < Weight)
									{
										BoneWeight.SetWeight(Weight);
									}
									bShouldAdd = false;
									break;
								}
							}
							if (bShouldAdd)
							{
								BoneWeights->Add(FBoneWeight(JointIndex, Weight));
							}
						}
						else
						{
							RawBoneWeights.Add(*VertexID).Add(FBoneWeight(JointIndex, Weight));
						}
					}
				}
			}

			// Add all the raw bone weights. This will cause the weights to be sorted and re-normalized after culling to max influences.
			for (const TTuple<FVertexID, TArray<FBoneWeight>>& Item : RawBoneWeights)
			{
				VertexSkinWeights.Set(Item.Key, Item.Value);
			}
		}

		FStaticMeshOperations::FAppendSettings AppendSettings;
		for (int32 ChannelIdx = 0; ChannelIdx < FStaticMeshOperations::FAppendSettings::MAX_NUM_UV_CHANNELS; ++ChannelIdx)
		{
			AppendSettings.bMergeUVChannels[ChannelIdx] = true;
		}
		for (size_t Index = 1; Index < SkinnedMeshDescriptions.Num(); Index++)
		{
			const int32 VertexOffset = SkinnedMeshDescriptions[0].Vertices().Num();
			FSkeletalMeshOperations::FSkeletalMeshAppendSettings SkeletalMeshAppendSettings;
			SkeletalMeshAppendSettings.SourceVertexIDOffset = VertexOffset;
			const int32 LocalJointCount = SkinVsLocalJointNames[SkinIndices[Index]].Num();
			SkeletalMeshAppendSettings.SourceRemapBoneIndex.AddZeroed(LocalJointCount);
			for (int32 LocalJointIndex = 0; LocalJointIndex < LocalJointCount; ++LocalJointIndex)
			{
				SkeletalMeshAppendSettings.SourceRemapBoneIndex[LocalJointIndex] = LocalJointIndex;
				const FString& LocalJointName = SkinVsLocalJointNames[SkinIndices[Index]][LocalJointIndex];
				for (int32 GlobalJointNamesIndex = 0; GlobalJointNamesIndex < OutJointUniqueNames->Num(); ++GlobalJointNamesIndex)
				{
					FString temp = (*OutJointUniqueNames)[GlobalJointNamesIndex];
					if (temp.Equals(LocalJointName))
					{
						SkeletalMeshAppendSettings.SourceRemapBoneIndex[LocalJointIndex] = GlobalJointNamesIndex;
						break;
					}
				}
			}

			FStaticMeshOperations::AppendMeshDescription(SkinnedMeshDescriptions[Index], SkinnedMeshDescriptions[0], AppendSettings);
			FSkeletalMeshOperations::AppendSkinWeight(SkinnedMeshDescriptions[Index], SkinnedMeshDescriptions[0], SkeletalMeshAppendSettings);
		}

		MeshDescription = SkinnedMeshDescriptions[0];

		PrintGltfMessages(MeshFactory);

		return true;
	}

	bool GetStaticMeshPayloadDataForPayLoadKey(const GLTF::FAsset& GltfAsset, const FString& PayLoadKey, const FTransform& MeshGlobalTransform, FMeshDescription& MeshDescription)
	{
		TArray<FString> PayLoadKeys;
		PayLoadKey.ParseIntoArray(PayLoadKeys, TEXT(":"));

		if (PayLoadKeys.Num() == 0)
		{
			return false;
		}

		int32 MeshIndex = 0;
		LexFromString(MeshIndex, *PayLoadKeys[0]);

		if (!GltfAsset.Meshes.IsValidIndex(MeshIndex))
		{
			return false;
		}

		const GLTF::FMesh& GltfMesh = GltfAsset.Meshes[MeshIndex];

		TArray<float> MorphTargetWeights;
		if (PayLoadKeys.Num() == 2)
		{
			int32 MorphTargetIndex;
			LexFromString(MorphTargetIndex, *PayLoadKeys[1]);

			MorphTargetWeights.SetNumZeroed(GltfMesh.MorphTargetNames.Num());

			if (MorphTargetIndex >= MorphTargetWeights.Num())
			{
				return false;
			}

			MorphTargetWeights[MorphTargetIndex] = 1.0f;
		}

		GLTF::FMeshFactory MeshFactory;
		MeshFactory.SetUniformScale(GltfUnitConversionMultiplier); // GLTF is in meters while UE is in centimeters
		MeshFactory.FillMeshDescription(GltfMesh, MeshGlobalTransform, &MeshDescription, MorphTargetWeights);

		PatchPolygonGroups(MeshDescription, GltfAsset);

		PrintGltfMessages(MeshFactory);

		return true;
	}
}