// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class UInterchangeBaseNodeContainer;
struct FMeshDescription;

namespace GLTF
{
	struct FAsset;
	struct FAnimation;
	struct FNode;
	struct FSkinInfo;
}

namespace UE::Interchange
{
	struct FAnimationPayloadData;
	struct FMeshPayloadData;

	namespace Gltf::Private
	{
		static float GltfUnitConversionMultiplier = 100.f;

		// Animation related functions
		bool GetTransformAnimationPayloadData(const FString& PayLoadKey, const GLTF::FAsset& GltfAsset, FAnimationPayloadData& OutPayloadData);
		bool GetMorphTargetAnimationPayloadData(const FString& PayLoadKey, const GLTF::FAsset& GltfAsset, FAnimationPayloadData& OutPayloadData);
		void GetT0Transform(const GLTF::FAnimation& GltfAnimation, const GLTF::FNode& AnimatedNode, const TArray<int32>& ChannelIndices, FTransform& OutTransform);
		bool GetBakedAnimationTransformPayloadData(const FString& PayLoadKey, const GLTF::FAsset& GltfAsset, FAnimationPayloadData& PayloadData);
		// 

		// Mesh related functions
		int32 GetRootNodeIndex(const GLTF::FAsset& GltfAsset, const TArray<int32>& NodeIndices);

		bool GetSkeletalMeshDescriptionForPayLoadKey(const GLTF::FAsset& GltfAsset, const FString& PayLoadKey, const FTransform& MeshGlobalTransform,
			FMeshDescription& MeshDescription, TArray<FString>* OutJointUniqueNames);

		bool GetStaticMeshPayloadDataForPayLoadKey(const GLTF::FAsset& GltfAsset, const FString& PayLoadKey, const FTransform& MeshGlobalTransform
		, FMeshDescription& MeshDescription);
		//

		//Process/Handle GLTF Animations:
		void HandleGLTFAnimations(UInterchangeBaseNodeContainer& NodeContainer,
			TArray<GLTF::FAnimation> Animations,
			const TArray<GLTF::FNode>& GLTFNodes,
			const TArray<GLTF::FSkinInfo>& GLTFSkins,
			const TMap<const GLTF::FNode*, FString>& GLTFNodeToInterchangeUidMap);
	}
}