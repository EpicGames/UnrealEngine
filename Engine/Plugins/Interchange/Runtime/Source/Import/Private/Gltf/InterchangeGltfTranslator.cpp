// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Gltf/InterchangeGltfTranslator.h"

#include "GLTFAccessor.h"
#include "GLTFAnimation.h"
#include "GLTFAsset.h"
#include "GLTFMesh.h"
#include "GLTFMeshFactory.h"
#include "GLTFNode.h"
#include "GLTFReader.h"
#include "GLTFTexture.h"

#include "InterchangeAnalyticsHandlerBase.h"
#include "InterchangeAnimationTrackSetNode.h"
#include "InterchangeCameraNode.h"
#include "InterchangeImportLog.h"
#include "InterchangeLightNode.h"
#include "InterchangeManager.h"
#include "InterchangeMaterialDefinitions.h"
#include "InterchangeMeshNode.h"
#include "InterchangeJointNode.h"
#include "InterchangeShaderGraphNode.h"
#include "InterchangeTexture2DNode.h"
#include "InterchangeTranslatorHelper.h"
#include "InterchangeVariantSetNode.h"
#include "InterchangeTextureLightProfileNode.h"
#include "Nodes/InterchangeSourceNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"

#include "Sections/MovieScene3DTransformSection.h"
#include "Texture/InterchangeImageWrapperTranslator.h"

#include "Algo/Find.h"
#include "Async/ParallelFor.h"
#include "Misc/App.h"
#include "StaticMeshAttributes.h"
#include "SkeletalMeshAttributes.h"
#include "UObject/GCObjectScopeGuard.h"

#include "StaticMeshOperations.h"
#include "SkeletalMeshOperations.h"

#include "Gltf/InterchangeGltfPrivate.h"
#include "Gltf/InterchangeGLTFMaterial.h"

#include "EngineAnalytics.h"
#include "Engine/RendererSettings.h"

#include "IESConverter.h"
#include "Texture/TextureTranslatorUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGltfTranslator)

#define LOCTEXT_NAMESPACE "InterchangeGLTFTranslator"

static const TArray<FString> ImporterSupportedExtensions = {
	/* Lights */
	GLTF::ToString(GLTF::EExtension::KHR_LightsPunctual),
	GLTF::ToString(GLTF::EExtension::KHR_Lights),
	GLTF::ToString(GLTF::EExtension::EXT_LightsIES),
	/* Variants */
	GLTF::ToString(GLTF::EExtension::KHR_MaterialsVariants),
	/* Materials */
	GLTF::ToString(GLTF::EExtension::KHR_MaterialsUnlit),
	GLTF::ToString(GLTF::EExtension::KHR_MaterialsIOR),
	GLTF::ToString(GLTF::EExtension::KHR_MaterialsClearCoat),
	GLTF::ToString(GLTF::EExtension::KHR_MaterialsTransmission),
	GLTF::ToString(GLTF::EExtension::KHR_MaterialsSheen),
	GLTF::ToString(GLTF::EExtension::KHR_MaterialsSpecular),
	GLTF::ToString(GLTF::EExtension::KHR_MaterialsPbrSpecularGlossiness),
	GLTF::ToString(GLTF::EExtension::KHR_MaterialsEmissiveStrength),
	GLTF::ToString(GLTF::EExtension::KHR_MaterialsIridescence),
	GLTF::ToString(GLTF::EExtension::KHR_MaterialsVolume),
	GLTF::ToString(GLTF::EExtension::MSFT_PackingOcclusionRoughnessMetallic),
	GLTF::ToString(GLTF::EExtension::MSFT_PackingNormalRoughnessMetallic),
	/* Textures */
	GLTF::ToString(GLTF::EExtension::KHR_TextureTransform),

	/* Mesh */
	GLTF::ToString(GLTF::EExtension::KHR_MeshQuantization),
	GLTF::ToString(GLTF::EExtension::KHR_DracoMeshCompression)
};

namespace UE::Interchange::Gltf::Private
{
	EInterchangeTextureWrapMode ConvertWrap(GLTF::FSampler::EWrap Wrap)
	{
		switch (Wrap)
		{
		case GLTF::FSampler::EWrap::Repeat:
			return EInterchangeTextureWrapMode::Wrap;
		case GLTF::FSampler::EWrap::MirroredRepeat:
			return EInterchangeTextureWrapMode::Mirror;
		case GLTF::FSampler::EWrap::ClampToEdge:
			return EInterchangeTextureWrapMode::Clamp;

		default:
			return EInterchangeTextureWrapMode::Wrap;
		}
	}

	EInterchangeTextureFilterMode ConvertFilter(GLTF::FSampler::EFilter Filter)
	{
		switch (Filter)
		{
			case GLTF::FSampler::EFilter::Nearest:
				return EInterchangeTextureFilterMode::Nearest;
			case GLTF::FSampler::EFilter::LinearMipmapNearest:
				return EInterchangeTextureFilterMode::Bilinear;
			case GLTF::FSampler::EFilter::LinearMipmapLinear:
				return EInterchangeTextureFilterMode::Trilinear;
				// Other glTF filter values have no direct correlation to Unreal
			default:
				return EInterchangeTextureFilterMode::Default;
		}
	}

	bool CheckForVariants(const GLTF::FMesh& Mesh, int32 VariantCount, int32 MaterialCount)
	{
		for (const GLTF::FPrimitive& Primitive : Mesh.Primitives)
		{
			for (const GLTF::FVariantMapping& VariantMapping : Primitive.VariantMappings)
			{
				if (FMath::IsWithin(VariantMapping.MaterialIndex, 0, MaterialCount))
				{
					for (int32 VariantIndex : VariantMapping.VariantIndices)
					{
						if (FMath::IsWithin(VariantIndex, 0, VariantCount))
						{
							return true;
						}
					}
				}
			}
		}

		return false;
	}

	void ScaleNodeTranslations(TArray<GLTF::FNode>& Nodes, float Scale)
	{
		for (GLTF::FNode& Node : Nodes)
		{
			Node.Transform.SetTranslation(Node.Transform.GetTranslation() * Scale);
			Node.LocalBindPose.SetTranslation(Node.LocalBindPose.GetTranslation() * Scale);
		}
	}

	enum TranslationResult : int32
	{
		SUCCESSFULL = 0,
		INPUT_FILE_NOTFOUND,
		GLTFREADER_FAILED,
		NOTSUPPORTED_EXTENSION_FOUND
	};
	void AddAnalytics(const TranslationResult& TranslationResult,
		TObjectPtr<UInterchangeAnalyticsHandlerBase> AnalyticsHandler,
		const GLTF::FAsset& Asset = GLTF::FAsset(),
		const FText& GLTFReaderLogMessage = FText::GetEmpty())
	{
		if (!AnalyticsHandler)
		{
			return;
		}

		TMap<FString, FString> MetadataExtras;
		for (GLTF::FMetadata::FExtraData ExtraData : Asset.Metadata.Extras)
		{
			MetadataExtras.Add(ExtraData.Name, ExtraData.Value);
		}

		TSet<FString> AllExtensions;
		AllExtensions.Append(Asset.ExtensionsUsed);
		AllExtensions.Append(Asset.ExtensionsRequired);

		TArray<FString> ExtensionsSupported;
		TArray<FString> ExtensionsUnsupported;

		for (const FString& Extension : AllExtensions)
		{
			if (ImporterSupportedExtensions.Find(Extension) == INDEX_NONE)
			{
				ExtensionsUnsupported.Add(Extension);
			}
			else
			{
				ExtensionsSupported.Add(Extension);
			}
		}

		const FString GLTFAnalyticsIdentifier = TEXT("Interchange.Usage.Import.GLTF");
		TArray<FAnalyticsEventAttribute> GLTFAnalytics;
		if (Asset.ExtensionsUsed.Num() > 0)				GLTFAnalytics.Add(FAnalyticsEventAttribute(TEXT("ExtensionsUsed"), Asset.ExtensionsUsed));
		if (Asset.ExtensionsRequired.Num() > 0)			GLTFAnalytics.Add(FAnalyticsEventAttribute(TEXT("ExtensionsRequired"), Asset.ExtensionsRequired));
		if (ExtensionsSupported.Num() > 0)				GLTFAnalytics.Add(FAnalyticsEventAttribute(TEXT("ExtensionsSupported"), ExtensionsSupported));
		if (ExtensionsUnsupported.Num() > 0)			GLTFAnalytics.Add(FAnalyticsEventAttribute(TEXT("ExtensionsUnsupported"), ExtensionsUnsupported));
		if (Asset.Metadata.GeneratorName.Len() > 0)		GLTFAnalytics.Add(FAnalyticsEventAttribute(TEXT("MetaData.GeneratorName"), Asset.Metadata.GeneratorName));
		if (MetadataExtras.Num() > 0)					GLTFAnalytics.Add(FAnalyticsEventAttribute(TEXT("MetaData.Extras"), MetadataExtras));
		/*Version is always set at this point.*/
		GLTFAnalytics.Add(FAnalyticsEventAttribute(TEXT("MetaData.Version"), Asset.Metadata.Version));
		if (Asset.HasAbnormalInverseBindMatrices)		GLTFAnalytics.Add(FAnalyticsEventAttribute(TEXT("HasAbnormalInverseBindMatrices"), true));

		switch (TranslationResult)
		{
		case SUCCESSFULL:
			GLTFAnalytics.Add(FAnalyticsEventAttribute(TEXT("TranslationStatus"), "Successful."));
			break;
		case INPUT_FILE_NOTFOUND:
			GLTFAnalytics.Add(FAnalyticsEventAttribute(TEXT("TranslationStatus"), "[Failed] Input File Not Found."));
			break;
		case GLTFREADER_FAILED:
			GLTFAnalytics.Add(FAnalyticsEventAttribute(TEXT("TranslationStatus"), "[Failed] Parsing error."));
			break;
		case NOTSUPPORTED_EXTENSION_FOUND:
			GLTFAnalytics.Add(FAnalyticsEventAttribute(TEXT("TranslationStatus"), "[Failed] Unsupported Extension Found."));
			break;
		default:
			break;
		}

		AnalyticsHandler->Append(GLTFAnalyticsIdentifier, GLTFAnalytics);
	};
}

void UInterchangeGLTFTranslator::HandleGltfNode( UInterchangeBaseNodeContainer& NodeContainer, const GLTF::FNode& GltfNode, const FString& ParentNodeUid, const int32 NodeIndex, 
	bool &bHasVariants, TArray<int32>& SkinnedMeshNodes, TSet<int>& UnusedMeshIndices,
	const TMap<int32, FTransform>& T0Transforms, const FString& SceneNodeUid) const
{
	using namespace UE::Interchange::Gltf::Private;

	const FString NodeUid = ParentNodeUid + TEXT("\\") + GltfNode.UniqueId;

	const UInterchangeSceneNode* ParentSceneNode = Cast< UInterchangeSceneNode >( NodeContainer.GetNode( ParentNodeUid ) );

	UInterchangeSceneNode* InterchangeSceneNode;
	if (GltfNode.Type == GLTF::FNode::EType::Joint)
	{
		InterchangeSceneNode = NewObject< UInterchangeJointNode >(&NodeContainer);
	}
	else
	{
		InterchangeSceneNode = NewObject< UInterchangeSceneNode >(&NodeContainer);
	}

	const FString ParentNodeUIDToUse = (GltfNode.Type == GLTF::FNode::EType::MeshSkinned) ? SceneNodeUid : ParentNodeUid;
	NodeContainer.SetupNode(InterchangeSceneNode, NodeUid, GltfNode.Name, EInterchangeNodeContainerType::TranslatedScene, ParentNodeUIDToUse);
	InterchangeSceneNode->SetAssetName(GltfNode.UniqueId);

	for (const TPair<FString, FString>& Extra : GltfNode.Extras)
	{
		UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(InterchangeSceneNode, Extra.Key, Extra.Value, TOptional<FString>());
	}

	NodeUidMap.Add( &GltfNode, NodeUid );

	FTransform Transform = GltfNode.Transform;

	Transform.SetTranslation( Transform.GetTranslation());

	switch ( GltfNode.Type )
	{
		case GLTF::FNode::EType::MeshSkinned:
		{
			SkinnedMeshNodes.Add(NodeIndex);

			if (GltfNode.ParentIndex != INDEX_NONE)
			{
				UE_LOGF(LogInterchangeImport, Warning, "Node [%ls] with a skinned mesh is not root. Parent transforms will not affect a skinned mesh.", *GltfNode.Name);
			}

			if (!bHasVariants && GltfAsset.Variants.Num() > 0)
			{
				bHasVariants |= CheckForVariants(GltfAsset.Meshes[GltfNode.MeshIndex], GltfAsset.Variants.Num(), GltfAsset.Materials.Num());
			}

			{//Set Morph Target Curve Weights
				const TArray<FString>& MorphTargetNames = GltfAsset.Meshes[GltfNode.MeshIndex].MorphTargetNames;
				int32 MorphTargetNamesCount = MorphTargetNames.Num();
				const TArray<float>& MorphTargetWeights = (GltfNode.MorphTargetWeights.Num() > 0) ? GltfNode.MorphTargetWeights : GltfAsset.Meshes[GltfNode.MeshIndex].MorphTargetWeights;

				if (MorphTargetWeights.Num() == MorphTargetNamesCount)
				{
					for (int32 MorphTargetIndex = 0; MorphTargetIndex < MorphTargetNamesCount; MorphTargetIndex++)
					{
						InterchangeSceneNode->SetMorphTargetCurveWeight(MorphTargetNames[MorphTargetIndex], MorphTargetWeights[MorphTargetIndex]);
					}
				}
				else
				{
					UE_LOGF(LogInterchangeImport, Warning, "glTF Node [%ls] Import Warning: The glTF node's MorphTargetNames count does not match its MorphTargetWeights count.", *GltfNode.UniqueId);
				}
			}

			break;
		}

		case GLTF::FNode::EType::Joint:
		{
			UInterchangeJointNode* JointNode = CastChecked<UInterchangeJointNode>(InterchangeSceneNode);

			if (GltfNode.bHasLocalBindPose)
			{
				JointNode->SetBindPoseLocalTransform(&NodeContainer, GltfNode.LocalBindPose);
			}

			if (GltfAsset.Animations.Num() == 0 || !T0Transforms.Contains(GltfNode.Index))
			{
				//If no animations present, use Local Transform for T0
				JointNode->SetTimeZeroLocalTransform(&NodeContainer, GltfNode.Transform);
			}
			else
			{
				JointNode->SetTimeZeroLocalTransform(&NodeContainer, T0Transforms[GltfNode.Index]);
			}
			break;
		}

		case GLTF::FNode::EType::Mesh:
		{
			if ( GltfAsset.Meshes.IsValidIndex( GltfNode.MeshIndex ) )
			{
				UInterchangeMeshNode* MeshNode = HandleGltfMesh(NodeContainer, GltfAsset.Meshes[GltfNode.MeshIndex], GltfNode.MeshIndex, UnusedMeshIndices);

				InterchangeSceneNode->SetCustomAssetInstanceUid( MeshNode->GetUniqueID() );
				if (GltfAsset.Meshes[GltfNode.MeshIndex].MorphTargetNames.Num() > 0)
				{
					const TArray<FString>& MorphTargetNames = GltfAsset.Meshes[GltfNode.MeshIndex].MorphTargetNames;
					int32 MorphTargetNamesCount = MorphTargetNames.Num();
					const TArray<float>& MorphTargetWeights = (GltfNode.MorphTargetWeights.Num() > 0) ? GltfNode.MorphTargetWeights : GltfAsset.Meshes[GltfNode.MeshIndex].MorphTargetWeights;
					
					if (MorphTargetWeights.Num() == MorphTargetNamesCount)
					{
						for (int32 MorphTargetIndex = 0; MorphTargetIndex < MorphTargetNamesCount; MorphTargetIndex++)
						{
							InterchangeSceneNode->SetMorphTargetCurveWeight(MorphTargetNames[MorphTargetIndex], MorphTargetWeights[MorphTargetIndex]);
						}
					}
					else
					{
						UE_LOGF(LogInterchangeImport, Warning, "glTF Node [%ls] Import Warning: The glTF node's MorphTargetNames count does not match its MorphTargetWeights count.", *GltfNode.UniqueId);
					}
				}

				if (!bHasVariants && GltfAsset.Variants.Num() > 0)
				{
					bHasVariants |= CheckForVariants(GltfAsset.Meshes[ GltfNode.MeshIndex ], GltfAsset.Variants.Num(), GltfAsset.Materials.Num());
				}
			}
			break;
		}

		case GLTF::FNode::EType::Camera:
		{
			Transform.ConcatenateRotation(FRotator(0, -90, 0).Quaternion());

			if ( GltfAsset.Cameras.IsValidIndex( GltfNode.CameraIndex ) )
			{
				const FString CameraNodeUid = TEXT("\\Camera\\") + GltfAsset.Cameras[ GltfNode.CameraIndex ].UniqueId;
				const FString Prefix = GltfAsset.Cameras[GltfNode.CameraIndex].Name + TEXT("_");
				InterchangeSceneNode->SetCustomAssetInstanceUid( CameraNodeUid );

				for (const TPair<FString, FString>& Extra : GltfAsset.Cameras[GltfNode.CameraIndex].Extras)
				{
					UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(InterchangeSceneNode, Prefix + Extra.Key, Extra.Value, TOptional<FString>());
				}
			}
			break;
		}

		case GLTF::FNode::EType::Light:
		{
			Transform.ConcatenateRotation(FRotator(0, -90, 0).Quaternion());

			if (GltfAsset.LightsIES.IsValidIndex(GltfNode.LightIES.Index))
			{
				FString LightIESInstanceHash = GltfNode.LightIES.GetHash();
				const GLTF::FLightIES& LightIES = GltfAsset.LightsIES[GltfNode.LightIES.Index];
				//As LightIES values can be instance specific, we create LightIES on Demand

				const FString LightIESUid = TEXT("\\LightIES\\") + GltfAsset.LightsIES[GltfNode.LightIES.Index].UniqueId;

				auto SetupLightIESInstanceNode = [&GltfNode, &LightIESUid, &NodeContainer](UInterchangeLightNode* LightNode, const FString& LightIESInstanceUid, const FString& DisplayLabel, const EInterchangeNodeContainerType& ContainerType)
					{
						NodeContainer.SetupNode(LightNode, LightIESInstanceUid, DisplayLabel, ContainerType);

						if (GltfNode.LightIES.bHasColor)
						{
							LightNode->SetCustomLightColor(FLinearColor(GltfNode.LightIES.Color));
						}

						if (GltfNode.LightIES.bHasIntensityMultiplier)
						{
							LightNode->SetCustomUseIESBrightness(true);
							LightNode->SetCustomIESBrightnessScale(GltfNode.LightIES.IntensityMultipler);
						}

						LightNode->SetCustomIESTexture(LightIESUid);
					};

				if (!GltfAsset.Lights.IsValidIndex(GltfNode.LightIndex))
				{//No Base Light
					const FString LightIESInstanceUid = TEXT("\\LightIESInstance\\LightIES-") + LightIES.UniqueId + TEXT("\\") + LightIESInstanceHash;
				
					if (!NodeContainer.GetNode(LightIESInstanceUid))
					{
						UInterchangePointLightNode* LightIESInstanceNode = NewObject< UInterchangePointLightNode >(&NodeContainer);
						SetupLightIESInstanceNode(LightIESInstanceNode, LightIESInstanceUid, GltfAsset.LightsIES[GltfNode.LightIES.Index].Name, EInterchangeNodeContainerType::TranslatedAsset);

						//Default is Candelas with 1 intensity:
						LightIESInstanceNode->SetCustomIntensityUnits(EInterchangeLightUnits::Candelas);
						LightIESInstanceNode->SetCustomIntensity(1.f);
					}

					InterchangeSceneNode->SetCustomAssetInstanceUid(LightIESInstanceUid);
				}
				else
				{//Base Light defined, needs duplication in order to be able to set the IES Instance.
					const FString LightIESInstanceUid = TEXT("\\LightIESInstance\\Light-") + GltfAsset.Lights[GltfNode.LightIndex].UniqueId + TEXT("_LightIES-") + LightIES.UniqueId + TEXT("\\") + LightIESInstanceHash;

					if (!NodeContainer.GetNode(LightIESInstanceUid))
					{
						const FString LightNodeUid = TEXT("\\Light\\") + GltfAsset.Lights[GltfNode.LightIndex].UniqueId;

						const UInterchangeBaseNode* OriginalLightBaseNode = NodeContainer.GetNode(LightNodeUid);
						if (const UInterchangeLightNode* OriginalLightNode = Cast<UInterchangeLightNode>(OriginalLightBaseNode))
						{

							//Duplicate the originalLightNode so we can set the LightProfile on it:
							if (UInterchangeLightNode* DuplicatedObject = Cast<UInterchangeLightNode>(StaticDuplicateObject(OriginalLightNode, &NodeContainer)))
							{
								SetupLightIESInstanceNode(DuplicatedObject, LightIESInstanceUid, DuplicatedObject->GetDisplayLabel(), DuplicatedObject->GetNodeContainerType());
							}
						}
					}

					InterchangeSceneNode->SetCustomAssetInstanceUid(LightIESInstanceUid);
				}
			}
			else if ( GltfAsset.Lights.IsValidIndex( GltfNode.LightIndex ) )
			{
				const FString LightNodeUid = TEXT("\\Light\\") + GltfAsset.Lights[ GltfNode.LightIndex ].UniqueId;
				InterchangeSceneNode->SetCustomAssetInstanceUid( LightNodeUid );
			}
		}

		case GLTF::FNode::EType::Transform:
		default:
		{
			break;
		}
	}

	InterchangeSceneNode->SetCustomLocalTransform(&NodeContainer, Transform);

	for ( const int32 ChildIndex : GltfNode.Children )
	{
		if ( GltfAsset.Nodes.IsValidIndex( ChildIndex ) )
		{
			HandleGltfNode( NodeContainer, GltfAsset.Nodes[ ChildIndex ], NodeUid, ChildIndex, bHasVariants, SkinnedMeshNodes, UnusedMeshIndices, T0Transforms, SceneNodeUid);
		}
	}
}

EInterchangeTranslatorType UInterchangeGLTFTranslator::GetTranslatorType() const
{
	return EInterchangeTranslatorType::Scenes;
}

EInterchangeTranslatorAssetType UInterchangeGLTFTranslator::GetSupportedAssetTypes() const
{
	//gltf translator support Meshes and Materials
	return EInterchangeTranslatorAssetType::Materials | EInterchangeTranslatorAssetType::Meshes | EInterchangeTranslatorAssetType::Animations;
}

TArray<FString> UInterchangeGLTFTranslator::GetSupportedFormats() const
{
	if (IsInGameThread())
	{
		static bool bRequiredPackageLoaded = []() -> bool
			{
				if (!UE::Interchange::GLTFMaterials::AreRequiredPackagesLoaded())
				{
					UE_LOGF(LogInterchangeImport, Warning, "UInterchangeGLTFPipeline: Some required packages are missing. Material import might be wrong.");
					return false;
				}

				return true;
			}();
	}

	static TArray<FString> GltfExtensions{
		TEXT("gltf;GL Transmission Format"),
		TEXT("glb;GL Transmission Format (Binary)")
	};

	return GltfExtensions;
}

bool UInterchangeGLTFTranslator::Translate( UInterchangeBaseNodeContainer& NodeContainer ) const
{
	using namespace UE::Interchange::Gltf::Private;

	FString FilePath = GetSourceData()->GetFilename();
	if ( !FPaths::FileExists( FilePath ) )
	{
		AddAnalytics(TranslationResult::INPUT_FILE_NOTFOUND, AnalyticsHandler);
		return false;
	}

	GLTF::FFileReader GltfFileReader;

	const bool bLoadImageData = false;
	const bool bLoadMetaData = true;
	GltfFileReader.ReadFile( FilePath, bLoadImageData, bLoadMetaData, const_cast< UInterchangeGLTFTranslator* >( this )->GltfAsset );

	UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(&NodeContainer);
	SourceNode->SetExtraInformation(TEXT("File Units"), TEXT("meter"));
	SourceNode->SetExtraInformation(TEXT("File Axis Direction"), TEXT("Y-UP (RH)"));
	if (GltfAsset.Metadata.GeneratorName.Len() > 0)
	{
		SourceNode->SetExtraInformation(TEXT("Generator Name"), GltfAsset.Metadata.GeneratorName);

		bool bVersionStringFound = false;
		int32 CharSpaceIndex = -1;
		if (GltfAsset.Metadata.GeneratorName.FindLastChar(' ', CharSpaceIndex))
		{
			if ((CharSpaceIndex < GltfAsset.Metadata.GeneratorName.Len() - 1) && GltfAsset.Metadata.GeneratorName[CharSpaceIndex + 1] == 'v')
			{
				const int32 VersionStringCount = GltfAsset.Metadata.GeneratorName.Len() - (CharSpaceIndex + 1);
				const FString ApplicationVersionString = GltfAsset.Metadata.GeneratorName.Right(VersionStringCount);
				const FString ApplicationNameString = GltfAsset.Metadata.GeneratorName.Left(CharSpaceIndex);

				// To achieve uniformity across all translators.
				SourceNode->SetExtraInformation(UE::Interchange::FSourceNodeExtraInfoStaticData::GetApplicationNameExtraInfoKey(), ApplicationNameString);
				SourceNode->SetExtraInformation(UE::Interchange::FSourceNodeExtraInfoStaticData::GetApplicationVersionExtraInfoKey(), ApplicationVersionString);
				bVersionStringFound = true;
			}
		}
		
		if (!bVersionStringFound)
		{
			SourceNode->SetExtraInformation(UE::Interchange::FSourceNodeExtraInfoStaticData::GetApplicationNameExtraInfoKey(), GltfAsset.Metadata.GeneratorName);
		}
	}
	for (const GLTF::FMetadata::FExtraData& Extra : GltfAsset.Metadata.Extras)
	{
		SourceNode->SetExtraInformation(Extra.Name, Extra.Value);
	}

	{
		//For sockets, crete the Axis System conversion inverse:

		FMatrix glTFToUE5Matrix = FMatrix(
			FPlane(1, 0, 0, 0),  // X-axis remains the same
			FPlane(0, 0, 1, 0),  // Y-axis becomes Z-axis
			FPlane(0, 1, 0, 0),  // Z-axis becomes Y-axis
			FPlane(0, 0, 0, 1)   // W (homogeneous coordinate)
		);
		FTransform AxisConversionInverseTransform = FTransform(glTFToUE5Matrix.Inverse());
		SourceNode->SetCustomAxisConversionInverseTransform(AxisConversionInverseTransform);
	}

	const FString FileName = GltfAsset.Name;

	//Required Extension Check:
	TArray<FString> NotSupportedRequiredExtensions;
	if (GltfAsset.ExtensionsRequired.Num() != 0)
	{
		for (const FString& RequiredExtension : GltfAsset.ExtensionsRequired)
		{
			if (ImporterSupportedExtensions.Find(RequiredExtension) == INDEX_NONE)
			{
				NotSupportedRequiredExtensions.Add(RequiredExtension);
			}
		}
	}

	//Check if ReadFile failed:
	TArray<GLTF::FLogMessage> GLTFReadFileLogMessages = GltfFileReader.GetLogMessages();
	for (GLTF::FLogMessage& LogMessage : GLTFReadFileLogMessages)
	{
		if (LogMessage.Key == GLTF::EMessageSeverity::Error)
		{
			UInterchangeResultError_Generic* ErrorResult = AddMessage< UInterchangeResultError_Generic >();
			ErrorResult->SourceAssetName = FileName;
			ErrorResult->Text = LogMessage.Value;

			AddAnalytics(TranslationResult::GLTFREADER_FAILED, AnalyticsHandler, GltfAsset, LogMessage.Value);
			return false;
		}
	}

	//In case of non supported extensions fail out:
	if (NotSupportedRequiredExtensions.Num() > 0)
	{
		FString NotSupportedRequiredExtensionsStringified;
		for (const FString& NotSupportedExtension : NotSupportedRequiredExtensions)
		{
			if (NotSupportedRequiredExtensionsStringified.Len() > 0)
			{
				NotSupportedRequiredExtensionsStringified += ", ";
			}
			NotSupportedRequiredExtensionsStringified += NotSupportedExtension;
		}

		UInterchangeResultError_Generic* ErrorResult = AddMessage< UInterchangeResultError_Generic >();
		ErrorResult->SourceAssetName = FileName;
		ErrorResult->Text = FText::Format(
			LOCTEXT("UnsupportedRequiredExtensions", "Not all required extensions are supported. (Unsupported extensions: {0})"),
			FText::FromString(NotSupportedRequiredExtensionsStringified));

		AddAnalytics(TranslationResult::NOTSUPPORTED_EXTENSION_FOUND, AnalyticsHandler, GltfAsset);
		return false;
	}

	ScaleNodeTranslations(const_cast<UInterchangeGLTFTranslator*>(this)->GltfAsset.Nodes, GltfUnitConversionMultiplier);

	//Check Normal Textures:
	TSet<int32> NormalTextureIndices;
	{
		auto AddTextureIndex = [&](int32 TextureIndex)
		{
			if (GltfAsset.Textures.IsValidIndex(TextureIndex))
			{
				NormalTextureIndices.Add(TextureIndex);
			}
		};

		for (const GLTF::FMaterial& GltfMaterial : GltfAsset.Materials)
		{
			AddTextureIndex(GltfMaterial.Normal.TextureIndex);
			AddTextureIndex(GltfMaterial.ClearCoat.NormalMap.TextureIndex);
			AddTextureIndex(GltfMaterial.Packing.NormalMap.TextureIndex);
		}
	}

	// Textures
	{
		int32 TextureIndex = 0;
		for ( const GLTF::FTexture& GltfTexture : GltfAsset.Textures )
		{
			// The glTF reader enforces the spec on the image format for buffers, URIs and file paths
			// Skip the texture is the glTF reader has not recognized the format
			if (GltfTexture.Source.Format == GLTF::FImage::EFormat::Unknown)
			{
				UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();

				FText TextMessage;
				if (GltfTexture.Source.FilePath.IsEmpty())
				{
					Message->Text = FText::Format(LOCTEXT("TextureCreationFailed", "The image format of the buffer for texture {0} is not supported."), FText::FromString(GltfTexture.Name));
				}
				else
				{
					Message->SourceAssetName = GetSourceData()->GetFilename();
					Message->Text = FText::Format(
						LOCTEXT("TextureCreationFailedFromFile", "The extension of the image file, {0}, for texture {1} is not supported."),
						FText::FromString(GltfTexture.Source.FilePath), FText::FromString(GltfTexture.Name));
				}

				continue;
			}

			const FString TextureNodeUid = UInterchangeTextureNode::MakeNodeUid(GltfTexture.UniqueId);
			UInterchangeTexture2DNode* TextureNode = UInterchangeTexture2DNode::Create(&NodeContainer, TextureNodeUid, GltfTexture.Name);

			TextureNode->SetCustomFilter(ConvertFilter(GltfTexture.Sampler.MinFilter));

			bool TextureUsedAsNormal = NormalTextureIndices.Contains(TextureIndex);

			if (TextureUsedAsNormal)
			{
				//According to GLTF documentation the normal maps are right handed (following OpenGL convention),
				//however UE expects left handed normal maps, this can be resolved by flipping the green channel of the normal textures:
				//(based on https://github.com/KhronosGroup/glTF-Sample-Models/tree/master/2.0/NormalTangentTest#problem-flipped-y-axis-or-flipped-green-channel)
				TextureNode->SetCustombFlipGreenChannel(true);
			}

			FString Payloadkey = LexToString(TextureIndex++) + TEXT(":") + LexToString(TextureUsedAsNormal);
			TextureNode->SetPayLoadKey(Payloadkey);

			TextureNode->SetCustomWrapU( UE::Interchange::Gltf::Private::ConvertWrap( GltfTexture.Sampler.WrapS ) );
			TextureNode->SetCustomWrapV( UE::Interchange::Gltf::Private::ConvertWrap( GltfTexture.Sampler.WrapT ) );

			for (const TPair<FString, FString>& Extra : GltfTexture.Source.Extras)
			{
				UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(TextureNode, Extra.Key, Extra.Value, TOptional<FString>());
			}
			for (const TPair<FString, FString>& Extra : GltfTexture.Extras)
			{
				UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(TextureNode, Extra.Key, Extra.Value, TOptional<FString>());
			}
		}
	}

	//IES
	{
		for (const GLTF::FLightIES& LightIES : GltfAsset.LightsIES)
		{
			UInterchangeTextureLightProfileNode* TextureLightProfileNode = NewObject< UInterchangeTextureLightProfileNode>(&NodeContainer, UInterchangeTextureLightProfileNode::StaticClass());

			FString LightIESUid = TEXT("\\LightIES\\") + LightIES.UniqueId;

			NodeContainer.SetupNode(TextureLightProfileNode, LightIESUid, LightIES.Name, EInterchangeNodeContainerType::TranslatedAsset);
			TextureLightProfileNode->SetPayLoadKey(FString::FromInt(LightIES.Index));

		}
	}

	// Meshes
	TSet<FString> MaterialsUsedOnMeshesWithVertexColor;
	TSet<int> UnusedGltfMeshIndices;
	{
		int32 MeshIndex = 0;
		for (const GLTF::FMesh& GltfMesh : GltfAsset.Meshes)
		{
			UnusedGltfMeshIndices.Add(MeshIndex++);

			if (GltfMesh.HasColors())
			{
				for (int32 PrimitiveCounter = 0; PrimitiveCounter < GltfMesh.Primitives.Num(); PrimitiveCounter++)
				{
					const GLTF::FPrimitive& Primitive = GltfMesh.Primitives[PrimitiveCounter];

					if (GltfAsset.Materials.IsValidIndex(Primitive.MaterialIndex))
					{
						const FString ShaderGraphNodeUid = UInterchangeShaderGraphNode::MakeNodeUid(GltfAsset.Materials[Primitive.MaterialIndex].UniqueId);
						MaterialsUsedOnMeshesWithVertexColor.Add(ShaderGraphNodeUid);
					}
				}
			}
		}
	}

	// Materials
	{
		int32 MaterialIndex = 0;
		for ( const GLTF::FMaterial& GltfMaterial : GltfAsset.Materials )
		{
			//Based on the gltf specification the basecolor and emissive and specular textures have SRGB colors:
			SetTextureSRGB(NodeContainer, GltfMaterial.BaseColor, true);
			SetTextureSRGB(NodeContainer, GltfMaterial.Emissive, true);
			SetTextureSRGB(NodeContainer, GltfMaterial.Specular.SpecularColorMap, true);
			SetTextureSRGB(NodeContainer, GltfMaterial.Specular.SpecularMap, true);		//Technically SpecularTexture is only using the Alpha channel, but could be packed onto other textures
																						//Parent material's expectation is that the SpecularMap is SRGB (as the expectation is that it is packed onto the SpecularColorTexture.)
			//Textures that are expected to use Scalar outputs we want to set them as SRGB false explicitly, based on UInterchangeGenericMaterialPipeline::HandleTextureNode
			SetTextureSRGB(NodeContainer, GltfMaterial.MetallicRoughness.Map, false);
			SetTextureSRGB(NodeContainer, GltfMaterial.Occlusion, false);
			SetTextureSRGB(NodeContainer, GltfMaterial.ClearCoat.ClearCoatMap, false);
			SetTextureSRGB(NodeContainer, GltfMaterial.ClearCoat.RoughnessMap, false);
			SetTextureSRGB(NodeContainer, GltfMaterial.Transmission.TransmissionMap, false);

			// TODO: Spec specifies linear space, but it seems to give better results in sRGB especially on the Dragon
			SetTextureSRGB(NodeContainer, GltfMaterial.Volume.ThicknessTexture, true);

			const FString ShaderGraphNodeUid = UInterchangeShaderGraphNode::MakeNodeUid(GltfMaterial.UniqueId);
			bool bUseVertexColor = MaterialsUsedOnMeshesWithVertexColor.Contains(ShaderGraphNodeUid);

			UInterchangeShaderGraphNode* ShaderGraphNode = UInterchangeShaderGraphNode::Create(&NodeContainer, GltfMaterial.UniqueId);
			ShaderGraphNode->SetDisplayLabel(GltfMaterial.Name);

			for (const TPair<FString, FString>& Extra : GltfMaterial.Extras)
			{
				UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(ShaderGraphNode, Extra.Key, Extra.Value, TOptional<FString>());
			}

			UE::Interchange::GLTFMaterials::HandleGltfMaterial(NodeContainer, GltfMaterial, GltfAsset.Textures, ShaderGraphNode);
			
			++MaterialIndex;
		}
	}

	// Cameras
	{
		int32 CameraIndex = 0;
		for ( const GLTF::FCamera& GltfCamera : GltfAsset.Cameras )
		{
			UInterchangeStandardCameraNode* CameraNode = NewObject< UInterchangeStandardCameraNode >(&NodeContainer);
			FString CameraNodeUid = TEXT("\\Camera\\") + GltfCamera.UniqueId;

			NodeContainer.SetupNode(CameraNode, CameraNodeUid, GltfCamera.Name, EInterchangeNodeContainerType::TranslatedAsset);

			if (GltfCamera.bIsPerspective)
			{
				CameraNode->SetCustomProjectionMode(EInterchangeCameraProjectionType::Perspective);

				CameraNode->SetCustomFieldOfView(FMath::RadiansToDegrees(GltfCamera.Perspective.Fov));
				CameraNode->SetCustomAspectRatio(GltfCamera.Perspective.AspectRatio);
			}
			else
			{
				CameraNode->SetCustomProjectionMode(EInterchangeCameraProjectionType::Orthographic);

				CameraNode->SetCustomWidth(GltfCamera.Orthographic.XMagnification * GltfUnitConversionMultiplier);
				CameraNode->SetCustomNearClipPlane(GltfCamera.ZNear * GltfUnitConversionMultiplier);
				CameraNode->SetCustomFarClipPlane(GltfCamera.ZFar * GltfUnitConversionMultiplier);

				CameraNode->SetCustomAspectRatio(GltfCamera.Orthographic.XMagnification / GltfCamera.Orthographic.YMagnification);
			}

			++CameraIndex;
		}
	}

	// Lights
	{
		int32 LightIndex = 0;
		for ( const GLTF::FLight& GltfLight : GltfAsset.Lights )
		{
			FString LightNodeUid = TEXT("\\Light\\") + GltfLight.UniqueId;

			switch (GltfLight.Type)
			{
			case GLTF::FLight::EType::Directional:
			{
				UInterchangeDirectionalLightNode* LightNode = NewObject< UInterchangeDirectionalLightNode >(&NodeContainer);
				NodeContainer.SetupNode(LightNode, LightNodeUid, GltfLight.Name, EInterchangeNodeContainerType::TranslatedAsset);

				LightNode->SetCustomLightColor(FLinearColor(GltfLight.Color));
				LightNode->SetCustomIntensity(GltfLight.Intensity);

				++LightIndex;
			}
				break;
			case GLTF::FLight::EType::Point:
			{
				UInterchangePointLightNode* LightNode = NewObject< UInterchangePointLightNode >(&NodeContainer);
				NodeContainer.SetupNode(LightNode, LightNodeUid, GltfLight.Name, EInterchangeNodeContainerType::TranslatedAsset);
				
				LightNode->SetCustomIntensityUnits(EInterchangeLightUnits::Candelas);
				LightNode->SetCustomLightColor(FLinearColor(GltfLight.Color));
				LightNode->SetCustomIntensity(GltfLight.Intensity);
				
				LightNode->SetCustomAttenuationRadius(GltfLight.Range * GltfUnitConversionMultiplier);

				++LightIndex;
			}
				break;
			case GLTF::FLight::EType::Spot:
			{
				UInterchangeSpotLightNode* LightNode = NewObject< UInterchangeSpotLightNode >(&NodeContainer);
				NodeContainer.SetupNode(LightNode, LightNodeUid, GltfLight.Name, EInterchangeNodeContainerType::TranslatedAsset);

				LightNode->SetCustomIntensityUnits(EInterchangeLightUnits::Candelas);
				LightNode->SetCustomLightColor(FLinearColor(GltfLight.Color));
				LightNode->SetCustomIntensity(GltfLight.Intensity);

				LightNode->SetCustomInnerConeAngle(FMath::RadiansToDegrees(GltfLight.Spot.InnerConeAngle));
				LightNode->SetCustomOuterConeAngle(FMath::RadiansToDegrees(GltfLight.Spot.OuterConeAngle));
				
				++LightIndex;
			}
				break;
			default:
				break;
			}
		}
	}

	// Cache created scene nodes UIDs to use later for animation binding
	bool bHasVariants = false;

	// Scenes
	{
		//Generate T0 Transforms
		TMap<int32, FTransform> T0Transforms;
		if (GltfAsset.Animations.Num() > 0)
		{
			const GLTF::FAnimation& Animation = GltfAsset.Animations[0];

			//Only Skeletal Animations (no Morph Animations as those do not produce FTransforms)
			TMap<int32, TArray<int32>> AnimatedNodesIndexToChannelIndices;//(AnimatedNodeIndex, [Channels])
			for (int32 ChannelIndex = 0; ChannelIndex < Animation.Channels.Num(); ++ChannelIndex)
			{
				const GLTF::FAnimation::FChannel& Channel = Animation.Channels[ChannelIndex];

				if (Channel.Target.Node.Type == GLTF::FNode::EType::Joint)
				{
					TArray<int32>& ChannelIndices = AnimatedNodesIndexToChannelIndices.FindOrAdd(Channel.Target.Node.Index);
					ChannelIndices.Add(ChannelIndex);
				}
			}

			for (const TPair<int32, TArray<int32>>& AnimatedNodeIndexToChannelIndices : AnimatedNodesIndexToChannelIndices)
			{
				FTransform T0Transform;
				UE::Interchange::Gltf::Private::GetT0Transform(Animation, GltfAsset.Nodes[AnimatedNodeIndexToChannelIndices.Key], AnimatedNodeIndexToChannelIndices.Value, T0Transform);
				T0Transforms.Emplace(AnimatedNodeIndexToChannelIndices.Key, T0Transform);
			}
		}

		int32 SceneIndex = 0;
		for ( const GLTF::FScene& GltfScene : GltfAsset.Scenes )
		{
			UInterchangeSceneNode* SceneNode = NewObject< UInterchangeSceneNode >( &NodeContainer );

			FString SceneName = GltfScene.Name;

			FString SceneNodeUid = TEXT("\\Scene\\") + GltfScene.UniqueId;

			NodeContainer.SetupNode(SceneNode, SceneNodeUid, SceneName, EInterchangeNodeContainerType::TranslatedScene);

			for (const TPair<FString, FString>& Extra : GltfScene.Extras)
			{
				UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(SceneNode, Extra.Key, Extra.Value, TOptional<FString>());
			}

			//All scene node should have a valid local transform
			SceneNode->SetCustomLocalTransform(&NodeContainer, FTransform::Identity);

			TArray<int32> SkinnedMeshNodes;
			for ( const int32 NodeIndex : GltfScene.Nodes )
			{
				if ( GltfAsset.Nodes.IsValidIndex( NodeIndex ) )
				{
					HandleGltfNode( NodeContainer, GltfAsset.Nodes[ NodeIndex ], SceneNodeUid, NodeIndex, bHasVariants, SkinnedMeshNodes, UnusedGltfMeshIndices, T0Transforms, SceneNodeUid);
				}
			}

			// Skeletons:
			HandleGltfSkeletons( NodeContainer, SceneNodeUid, SkinnedMeshNodes, UnusedGltfMeshIndices );
		}
	}

	// Animations
	{
		UE::Interchange::Gltf::Private::HandleGLTFAnimations(NodeContainer, GltfAsset.Animations, GltfAsset.Nodes, GltfAsset.Skins, NodeUidMap);
	}

	// Variants
	// Note: Variants are not supported yet in game play mode
	if (bHasVariants)
	{
		HandleGltfVariants(NodeContainer, FileName);
	}

	// Add glTF errors and warnings to the Interchange results
	for ( const GLTF::FLogMessage& LogMessage : GltfFileReader.GetLogMessages() )
	{
		UInterchangeResult* Result = nullptr;

		switch ( LogMessage.Get<0>() )
		{
		case GLTF::EMessageSeverity::Error :
			{
				UInterchangeResultError_Generic* ErrorResult = AddMessage< UInterchangeResultError_Generic >();
				ErrorResult->Text = LogMessage.Get<1>();
				Result = ErrorResult;
			}
			break;
		case GLTF::EMessageSeverity::Warning:
			{
				UInterchangeResultWarning_Generic* WarningResult = AddMessage< UInterchangeResultWarning_Generic >();
				WarningResult->Text = LogMessage.Get<1>();
				Result = WarningResult;
			}
			break;
		case GLTF::EMessageSeverity::Display:
			{
				UInterchangeResultDisplay_Generic* DisplayResult = AddMessage< UInterchangeResultDisplay_Generic >();
				DisplayResult->Text = LogMessage.Get<1>();
				Result = DisplayResult;
			}
			break;
		default:
			break;
		}

		if ( Result )
		{
			Result->SourceAssetName = FileName;
		}
	}

	// Create any Mesh Nodes for meshes that have not been used and just in the gltf as an asset:
	TSet<int> UnusedMeshIndices = UnusedGltfMeshIndices;
	for (int UnusedMeshIndex : UnusedMeshIndices)
	{
		HandleGltfMesh(NodeContainer, GltfAsset.Meshes[UnusedMeshIndex], UnusedMeshIndex, UnusedGltfMeshIndices);
	}

	if (UnusedGltfMeshIndices.Num() != 0)
	{
		UE_LOGF(LogInterchangeImport, Warning, "glTF Mesh Import Warning: glTF mesh usage expectations are not met.");
	}

	AddAnalytics(TranslationResult::SUCCESSFULL, AnalyticsHandler, GltfAsset);
	return true;
}

TOptional< UE::Interchange::FImportImage > UInterchangeGLTFTranslator::GetTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const
{
	TArray<FString> PayloadKeys;
	PayloadKey.ParseIntoArray(PayloadKeys, TEXT(":"));

	if (PayloadKeys.Num() == 0)
	{
		return TOptional< UE::Interchange::FImportImage >();
	}

	int32 TextureIndex = 0;
	LexFromString( TextureIndex, *PayloadKeys[0]);

	if ( !GltfAsset.Textures.IsValidIndex( TextureIndex ) )
	{
		return TOptional< UE::Interchange::FImportImage >();
	}

	const GLTF::FTexture& GltfTexture = GltfAsset.Textures[ TextureIndex ];

	TOptional<UE::Interchange::FImportImage> TexturePayloadData = TOptional<UE::Interchange::FImportImage>();

	if (GltfTexture.Source.FilePath.IsEmpty())
	{
		// Embedded texture -- try using ImageWrapper to decode it
		TArray64<uint8> ImageData(GltfTexture.Source.Data, GltfTexture.Source.DataByteLength);
		UInterchangeImageWrapperTranslator* ImageWrapperTranslator = NewObject<UInterchangeImageWrapperTranslator>(GetTransientPackage(), NAME_None);
		ImageWrapperTranslator->SetResultsContainer(Results);
		TexturePayloadData = ImageWrapperTranslator->GetTexturePayloadDataFromBuffer(ImageData);
		ImageWrapperTranslator->ClearInternalFlags(EInternalObjectFlags::Async);
	}
	else
	{
		const FString TextureFilePath = FPaths::ConvertRelativePathToFull(GltfTexture.Source.FilePath);
		UE::Interchange::Private::FScopedTranslator ScopedTranslator(TextureFilePath, Results, AnalyticsHandler);
		const IInterchangeTexturePayloadInterface* TextureTranslator = ScopedTranslator.GetPayLoadInterface<IInterchangeTexturePayloadInterface>();
		if (!ensure(TextureTranslator))
		{
			return TexturePayloadData;
		}
		AlternateTexturePath = TextureFilePath;
		TexturePayloadData = TextureTranslator->GetTexturePayloadData(PayloadKey, AlternateTexturePath);
	}

	if (PayloadKeys.Num() == 2 && TexturePayloadData.IsSet())
	{
		bool TextureUsedAsNormal = false;
		LexFromString(TextureUsedAsNormal, *PayloadKeys[1]);

		TexturePayloadData.GetValue().CompressionSettings = TextureUsedAsNormal ? TC_Normalmap : TC_Default;
	}

	return TexturePayloadData;
}

TOptional<UE::Interchange::FAnimationPayloadData> UInterchangeGLTFTranslator::GetAnimationPayloadData(const UE::Interchange::FAnimationPayloadQuery& PayloadQuery) const
{
	UE::Interchange::FAnimationPayloadData AnimationPayLoadData(PayloadQuery.SceneNodeUniqueID, PayloadQuery.PayloadKey);
	TOptional<UE::Interchange::FAnimationPayloadData> Result;
	switch (PayloadQuery.PayloadKey.Type)
	{
	case EInterchangeAnimationPayLoadType::CURVE:
		if (UE::Interchange::Gltf::Private::GetTransformAnimationPayloadData(PayloadQuery.PayloadKey.UniqueId, GltfAsset, AnimationPayLoadData))
		{
			Result = AnimationPayLoadData;
		}
		break;
	case EInterchangeAnimationPayLoadType::MORPHTARGETCURVE:
		if (UE::Interchange::Gltf::Private::GetMorphTargetAnimationPayloadData(PayloadQuery.PayloadKey.UniqueId, GltfAsset, AnimationPayLoadData))
		{
			Result = AnimationPayLoadData;
		}
		break;
	case EInterchangeAnimationPayLoadType::BAKED:
		AnimationPayLoadData.BakeFrequency = PayloadQuery.TimeDescription.BakeFrequency;
		AnimationPayLoadData.RangeStartTime = PayloadQuery.TimeDescription.RangeStartSecond;
		AnimationPayLoadData.RangeEndTime = PayloadQuery.TimeDescription.RangeStopSecond;
		if (UE::Interchange::Gltf::Private::GetBakedAnimationTransformPayloadData(PayloadQuery.PayloadKey.UniqueId, GltfAsset, AnimationPayLoadData))
		{
			Result = AnimationPayLoadData;
		}
		break;
	case EInterchangeAnimationPayLoadType::STEPCURVE:
	case EInterchangeAnimationPayLoadType::NONE:
	default:
		break;
	}

	return Result;
}

TArray<UE::Interchange::FAnimationPayloadData> UInterchangeGLTFTranslator::GetAnimationPayloadData(const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries) const
{
	TArray<TOptional<UE::Interchange::FAnimationPayloadData>> AnimationPayloadOptionals;
	int32 PayloadCount = PayloadQueries.Num();
	AnimationPayloadOptionals.AddDefaulted(PayloadCount);
	
	const int32 BatchSize = 10;
	if (PayloadQueries.Num() > BatchSize)
	{
		const int32 NumBatches = (PayloadCount / BatchSize) + 1;
		ParallelFor(NumBatches, [&](int32 BatchIndex)
			{
				int32 PayloadIndexOffset = BatchIndex * BatchSize;
				for (int32 PayloadIndex = PayloadIndexOffset; PayloadIndex < PayloadIndexOffset + BatchSize; ++PayloadIndex)
				{
					if (PayloadQueries.IsValidIndex(PayloadIndex))
					{
						AnimationPayloadOptionals[PayloadIndex] = GetAnimationPayloadData(PayloadQueries[PayloadIndex]);
					}
				}
			}, EParallelForFlags::BackgroundPriority);// ParallelFor
	}
	else
	{
		for (int32 PayloadIndex = 0; PayloadIndex < PayloadCount; ++PayloadIndex)
		{
			if (PayloadQueries.IsValidIndex(PayloadIndex))
			{
				AnimationPayloadOptionals[PayloadIndex] = GetAnimationPayloadData(PayloadQueries[PayloadIndex]);
			}
		}
	}

	TArray<UE::Interchange::FAnimationPayloadData> AnimationPayloads;
	for (TOptional<UE::Interchange::FAnimationPayloadData>& OptionalPayloadData : AnimationPayloadOptionals)
	{
		if (!OptionalPayloadData.IsSet())
		{
			continue;
		}
		AnimationPayloads.Add(OptionalPayloadData.GetValue());
	}

	return AnimationPayloads;
}

void UInterchangeGLTFTranslator::SetTextureSRGB(UInterchangeBaseNodeContainer& NodeContainer, const GLTF::FTextureMap& TextureMap, bool bSRGB) const
{
	if (GltfAsset.Textures.IsValidIndex(TextureMap.TextureIndex))
	{
		const FString TextureUid = UInterchangeTextureNode::MakeNodeUid(GltfAsset.Textures[TextureMap.TextureIndex].UniqueId);
		if (UInterchangeTextureNode* TextureNode = const_cast<UInterchangeTextureNode*>(Cast<UInterchangeTextureNode>(NodeContainer.GetNode(TextureUid))))
		{
			bool bExistingSRGBSetting;
			if (TextureNode->GetCustomSRGB(bExistingSRGBSetting))
			{
				if (bExistingSRGBSetting != bSRGB)
				{
					UE_LOGF(LogInterchangeImport, Warning, "UInterchangeGLTFPipeline: Unexpected SRGB/Linear setting on Texture: %ls.", *TextureNode->GetDisplayLabel());
				}
			}
			TextureNode->SetCustomSRGB(bSRGB);
		}
	}
}

TOptional<UE::Interchange::FVariantSetPayloadData> UInterchangeGLTFTranslator::GetVariantSetPayloadData(const FString& PayloadKey) const
{
	using namespace UE::Interchange;

	TOptional<FVariantSetPayloadData> Result;

	TArray<FString> PayloadTokens;

	// We need two indices to build the payload: index of LevelVariantSet and index of VariantSetIndex
	if (GltfAsset.Variants.Num() + 1 != PayloadKey.ParseIntoArray(PayloadTokens, TEXT(";")))
	{
		// Invalid payload
		return Result;
	}

	//FString PayloadKey = FileName;
	for (int32 Index = 0; Index < GltfAsset.Variants.Num(); ++Index)
	{
		if (PayloadTokens[Index + 1] != GltfAsset.Variants[Index])
		{
			// Invalid payload
			return Result;
		}
	}

	FVariantSetPayloadData PayloadData;
	if (this->GetVariantSetPayloadData(PayloadData))
	{
		Result = MoveTemp(PayloadData);
	}

	return Result;
}

void UInterchangeGLTFTranslator::HandleGltfVariants(UInterchangeBaseNodeContainer& NodeContainer, const FString& FileName) const
{
	UInterchangeVariantSetNode* VariantSetNode = nullptr;
	VariantSetNode = NewObject<UInterchangeVariantSetNode>(&NodeContainer);

	FString VariantSetNodeUid = TEXT("\\VariantSet\\") + FileName;
	NodeContainer.SetupNode(VariantSetNode, VariantSetNodeUid, FileName, EInterchangeNodeContainerType::TranslatedScene);

	VariantSetNode->SetCustomDisplayText(FileName);

	FString PayloadKey = FileName;
	for (int32 Index = 0; Index < GltfAsset.Variants.Num(); ++Index)
	{
		PayloadKey += TEXT(";") + GltfAsset.Variants[Index];
	}
	VariantSetNode->SetCustomVariantsPayloadKey(PayloadKey);

	TFunction<void(const TArray<int32>& Nodes)> CollectDependencies;
	CollectDependencies = [this, VariantSetNode, &CollectDependencies](const TArray<int32>& Nodes) -> void
	{
		const TArray<GLTF::FMaterial>& Materials = this->GltfAsset.Materials;

		for (const int32 NodeIndex : Nodes)
		{
			if (this->GltfAsset.Nodes.IsValidIndex(NodeIndex))
			{
				const GLTF::FNode& GltfNode = this->GltfAsset.Nodes[NodeIndex];

				if (GltfNode.Type == GLTF::FNode::EType::Mesh && this->GltfAsset.Meshes.IsValidIndex(GltfNode.MeshIndex))
				{
					const GLTF::FMesh& Mesh = this->GltfAsset.Meshes[GltfNode.MeshIndex];
					const FString* NodeUidPtr = this->NodeUidMap.Find(&GltfNode);
					if (!ensure(NodeUidPtr))
					{
						continue;
					}

					VariantSetNode->AddCustomDependencyUid(*NodeUidPtr);

					for (const GLTF::FPrimitive& Primitive : Mesh.Primitives)
					{
						if (Primitive.VariantMappings.Num() > 0)
						{
							for (const GLTF::FVariantMapping& VariantMapping : Primitive.VariantMappings)
							{
								if (!ensure(Materials.IsValidIndex(VariantMapping.MaterialIndex)))
								{
									continue;
								}

								const GLTF::FMaterial& GltfMaterial = Materials[VariantMapping.MaterialIndex];
								const FString MaterialUid = UInterchangeShaderGraphNode::MakeNodeUid(GltfMaterial.UniqueId);

								VariantSetNode->AddCustomDependencyUid(MaterialUid);

							}
						}
					}
				}

				if (GltfNode.Children.Num() > 0)
				{
					CollectDependencies(GltfNode.Children);
				}
			}
		}
	};

	for (const GLTF::FScene& GltfScene : GltfAsset.Scenes)
	{
		CollectDependencies(GltfScene.Nodes);
	}

	UInterchangeSceneVariantSetsNode* SceneVariantSetsNode = NewObject<UInterchangeSceneVariantSetsNode>(&NodeContainer);

	FString SceneVariantSetsNodeUid = TEXT("\\SceneVariantSets\\") + FileName;

	NodeContainer.SetupNode(SceneVariantSetsNode, SceneVariantSetsNodeUid, FileName, EInterchangeNodeContainerType::TranslatedScene);

	SceneVariantSetsNode->AddCustomVariantSetUid(VariantSetNodeUid);
}

bool UInterchangeGLTFTranslator::GetVariantSetPayloadData(UE::Interchange::FVariantSetPayloadData& PayloadData) const
{
	using namespace UE;

	PayloadData.Variants.SetNum(GltfAsset.Variants.Num());

	TMap<const TCHAR*, Interchange::FVariant*, FDefaultSetAllocator, TStringPointerMapKeyFuncs_DEPRECATED<const TCHAR*, Interchange::FVariant*>> VariantMap;
	VariantMap.Reserve(GltfAsset.Variants.Num());

	for (int32 VariantIndex = 0; VariantIndex < GltfAsset.Variants.Num(); ++VariantIndex)
	{
		const FString& VariantName = GltfAsset.Variants[VariantIndex];
		PayloadData.Variants[VariantIndex].DisplayText = VariantName;
		VariantMap.Add(*VariantName, &PayloadData.Variants[VariantIndex]);
	}

	TFunction<void(const TArray<int32>& Nodes)> BuildPayloadData;
	BuildPayloadData = [this, VariantMap = MoveTemp(VariantMap), &BuildPayloadData](const TArray<int32>& Nodes) -> void
	{
		const TArray<FString>& VariantNames = this->GltfAsset.Variants;
		const TArray<GLTF::FMaterial>& Materials = this->GltfAsset.Materials;

		for (const int32 NodeIndex : Nodes)
		{
			if (!ensure(this->GltfAsset.Nodes.IsValidIndex(NodeIndex)))
			{
				continue;
			}

			const GLTF::FNode& GltfNode = this->GltfAsset.Nodes[NodeIndex];

			if (GltfNode.Type == GLTF::FNode::EType::Mesh && this->GltfAsset.Meshes.IsValidIndex(GltfNode.MeshIndex))
			{
				const GLTF::FMesh& Mesh = this->GltfAsset.Meshes[GltfNode.MeshIndex];
				const FString* NodeUidPtr = this->NodeUidMap.Find(&GltfNode);
				if (!ensure(NodeUidPtr))
				{
					continue;
				}

				for (const GLTF::FPrimitive& Primitive : Mesh.Primitives)
				{
					for (const GLTF::FVariantMapping& VariantMapping : Primitive.VariantMappings)
					{
						if (!ensure(Materials.IsValidIndex(VariantMapping.MaterialIndex)))
						{
							continue;
						}

						const GLTF::FMaterial& GltfMaterial = Materials[VariantMapping.MaterialIndex];
						const FString MaterialNodeUid = UInterchangeShaderGraphNode::MakeNodeUid(GltfMaterial.UniqueId);

						for (int32 VariantIndex : VariantMapping.VariantIndices)
						{
							if (!ensure(VariantMap.Contains(*VariantNames[VariantIndex])))
							{
								continue;
							}

							// This is on par with the Datasmith GLTF translator but might be wrong.
							// Each primitive should be a section of the static mesh and 
							// TODO: Revisit creation of static mesh and handling of variants: UE-159945.
							Interchange::FVariantPropertyCaptureData PropertyCaptureData;

							PropertyCaptureData.Category = Interchange::EVariantPropertyCaptureCategory::Material;
							PropertyCaptureData.ObjectUid = MaterialNodeUid;

							Interchange::FVariant& VariantData = *(VariantMap[*VariantNames[VariantIndex]]);

							Interchange::FVariantBinding& Binding = VariantData.Bindings.AddDefaulted_GetRef();

							Binding.TargetUid = *NodeUidPtr;
							Binding.Captures.Add(PropertyCaptureData);
						}

					}
				}
			}

			if (GltfNode.Children.Num() > 0)
			{
				BuildPayloadData(GltfNode.Children);
			}
		}
	};

	for (const GLTF::FScene& GltfScene : GltfAsset.Scenes)
	{
		BuildPayloadData(GltfScene.Nodes);
	}

	return true;
}

TOptional<UE::Interchange::FMeshPayloadData> UInterchangeGLTFTranslator::GetMeshPayloadData(const FInterchangeMeshPayLoadKey& PayLoadKey, const UE::Interchange::FAttributeStorage& PayloadAttributes) const
{
	using namespace UE::Interchange;
	FTransform MeshGlobalTransform;
	PayloadAttributes.GetAttribute(UE::Interchange::FAttributeKey{ MeshPayload::Attributes::MeshGlobalTransform }, MeshGlobalTransform);

	UE::Interchange::FMeshPayloadData MeshPayloadData;
	bool bSuccessfullAcquisition = false;

	switch (PayLoadKey.Type)
	{
	case EInterchangeMeshPayLoadType::STATIC:
		bSuccessfullAcquisition = UE::Interchange::Gltf::Private::GetStaticMeshPayloadDataForPayLoadKey(GltfAsset, PayLoadKey.UniqueId, MeshGlobalTransform, MeshPayloadData.MeshDescription);
		break;
	case EInterchangeMeshPayLoadType::SKELETAL:
		bSuccessfullAcquisition = UE::Interchange::Gltf::Private::GetSkeletalMeshDescriptionForPayLoadKey(GltfAsset, PayLoadKey.UniqueId, MeshGlobalTransform, MeshPayloadData.MeshDescription, &MeshPayloadData.JointNames);
		break;
	case EInterchangeMeshPayLoadType::MORPHTARGET:
		//GLTF handles morph targets as simple Meshes
		bSuccessfullAcquisition = UE::Interchange::Gltf::Private::GetStaticMeshPayloadDataForPayLoadKey(GltfAsset, PayLoadKey.UniqueId, MeshGlobalTransform, MeshPayloadData.MeshDescription);
		break;
	case EInterchangeMeshPayLoadType::NONE:
	default:
		break;
	}

	if (bSuccessfullAcquisition)
	{
		if (!FStaticMeshOperations::ValidateAndFixData(MeshPayloadData.MeshDescription, PayLoadKey.UniqueId))
		{
			UInterchangeResultError_Generic* ErrorResult = AddMessage<UInterchangeResultError_Generic>();
			ErrorResult->SourceAssetName = SourceData ? SourceData->GetFilename() : FString();
			ErrorResult->Text = LOCTEXT("GetMeshPayloadData_ValidateMeshDescriptionFail", "Invalid mesh data (NAN) was found and changed to zero. This may affect the mesh rendering.");
		}
		return MeshPayloadData;
	}
	return TOptional<UE::Interchange::FMeshPayloadData>();
}

void UInterchangeGLTFTranslator::HandleGltfSkeletons(UInterchangeBaseNodeContainer& NodeContainer, const FString& SceneNodeUid, const TArray<int32>& SkinnedMeshNodes, TSet<int>& UnusedMeshIndices) const
{
	TMap<int32, TMap<int32, TArray<int32>>> MeshIndexToRootJointGroupedSkinnedMeshNodesMap;

	//group SkinnedMeshNodes based on Joint Root Parents and Mesh indices
	//this is needed in order to figure out how many duplications do we need for a given mesh
	for (int32 SkinnedMeshNodeIndex : SkinnedMeshNodes)
	{
		const GLTF::FNode& SkinnedMeshNode = GltfAsset.Nodes[SkinnedMeshNodeIndex];

		TMap<int32, TArray<int32>>& RootJointGroupedSkinnedMeshNodes = MeshIndexToRootJointGroupedSkinnedMeshNodesMap.FindOrAdd(SkinnedMeshNode.MeshIndex);

		//get the SkinnedMeshNode's skin's first joint as the starting ground and find the top most root joint for it:
		if (!GltfAsset.Skins.IsValidIndex(SkinnedMeshNode.Skindex)
			|| !GltfAsset.Skins[SkinnedMeshNode.Skindex].Joints.IsValidIndex(0)
			|| !GltfAsset.Nodes.IsValidIndex(GltfAsset.Skins[SkinnedMeshNode.Skindex].Joints[0]))
		{
			continue;
		}
		
		int32 RootSkinJointIndex = UE::Interchange::Gltf::Private::GetRootNodeIndex(GltfAsset, GltfAsset.Skins[SkinnedMeshNode.Skindex].Joints);
		if (!GltfAsset.Nodes.IsValidIndex(RootSkinJointIndex))
		{
			continue;
		}

		//basedon that root joint group the SkinnedMeshNodes:
		TArray<int32>& GroupedSkinnedMeshNodes = RootJointGroupedSkinnedMeshNodes.FindOrAdd(RootSkinJointIndex);
		GroupedSkinnedMeshNodes.Add(SkinnedMeshNodeIndex);
	}

	for (const TTuple<int32, TMap<int32, TArray<int32>>>& MeshIndexRootJointGroups : MeshIndexToRootJointGroupedSkinnedMeshNodesMap)
	{
		const TMap<int32, TArray<int32>>& RootJointGroupedSkinnedMeshNodes = MeshIndexRootJointGroups.Value;

		int MeshIndex = MeshIndexRootJointGroups.Key;

		//iterate through the groups:
		//rootjoint , array<skinnedMeshNodes>
		for (const TTuple<int32, TArray<int32>>& RootJointToSkinnedMeshNodes : RootJointGroupedSkinnedMeshNodes)
		{
			//Duplicate MeshNode for each group:
			int32 RootSkinJointNodeIndex = RootJointToSkinnedMeshNodes.Key;
			int32 RootJointNodeIndex = GltfAsset.Nodes[RootSkinJointNodeIndex].RootJointIndex;

			//Skeletal Mesh's naming policy: (Mesh.Name)_(RootJointNode.Name) naming policy:
			FString SkeletalName = GltfAsset.Meshes[MeshIndex].Name + TEXT("_") + GltfAsset.Nodes[RootSkinJointNodeIndex].Name;
			FString SkeletalId = GltfAsset.Meshes[MeshIndex].UniqueId + TEXT("_") + GltfAsset.Nodes[RootSkinJointNodeIndex].UniqueId;

			UInterchangeMeshNode* SkeletalMeshNode = HandleGltfMesh(NodeContainer, GltfAsset.Meshes[MeshIndex], MeshIndex, UnusedMeshIndices, SkeletalName, SkeletalId);
			SkeletalMeshNode->SetSkinnedMesh(true);

			//set the root joint node as the skeletonDependency:
			const GLTF::FNode& RootJointNode = GltfAsset.Nodes[RootJointNodeIndex];
			const FString* SkeletonNodeUid = NodeUidMap.Find(&RootJointNode);
			if (ensure(SkeletonNodeUid))
			{
				SkeletalMeshNode->SetSkeletonDependencyUid(*SkeletonNodeUid);
			}

			TSet<int32> SkinIndices;
			//generate payload key:
			//of template:
			//"LexToString(SkinnedMeshNode.MeshIndex | (SkinnedMeshNode.Skindex << 16))":"LexToString(SkinnedMeshNode.MeshIndex | (SkinnedMeshNode.Skindex << 16))".....
			FString Payload = TEXT("");
			for (int32 SkinnedMeshIndex : RootJointToSkinnedMeshNodes.Value)
			{
				const GLTF::FNode& SkinnedMeshNode = GltfAsset.Nodes[SkinnedMeshIndex];
				if (Payload.Len() > 0)
				{
					Payload += TEXT(":");
				}
				
				Payload += LexToString(SkinnedMeshNode.MeshIndex | (SkinnedMeshNode.Skindex << 16));
				SkinIndices.Add(SkinnedMeshNode.Skindex);
			}
			SkeletalMeshNode->SetPayLoadKey(Payload, EInterchangeMeshPayLoadType::SKELETAL);

			for (const int32& SkinIndex : SkinIndices)
			{
				if (GltfAsset.Skins.IsValidIndex(SkinIndex))
				{
					const FString Prefix = GltfAsset.Skins[SkinIndex].Name + TEXT("_");
					for (const TPair<FString, FString>& Extra : GltfAsset.Skins[SkinIndex].Extras)
					{
						UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(SkeletalMeshNode, Prefix + Extra.Key, Extra.Value, TOptional<FString>());
					}
				}
			}

			//set the mesh actor node's custom asset instance uid to the new duplicated mesh
			//if there are more than one skins, then choose the topmost (root node of the collection, top most in a hierarchical tree term) occurance of SkinnedMeshIndex
			int32 MeshActorNodeIndex = UE::Interchange::Gltf::Private::GetRootNodeIndex(GltfAsset, RootJointToSkinnedMeshNodes.Value);
			const GLTF::FNode& MeshActorNode = GltfAsset.Nodes[MeshActorNodeIndex];
			const FString* SceneMeshActorNodeUid = NodeUidMap.Find(&MeshActorNode);
			if (const UInterchangeSceneNode* ConstSceneMeshActorNode = Cast< UInterchangeSceneNode >(NodeContainer.GetNode(*SceneMeshActorNodeUid)))
			{
				UInterchangeSceneNode* SceneMeshNode = const_cast<UInterchangeSceneNode*>(ConstSceneMeshActorNode);
				if (ensure(SceneMeshNode))
				{
					SceneMeshNode->SetCustomAssetInstanceUid(SkeletalMeshNode->GetUniqueID());
				}
			}
		}
	}
}

UInterchangeMeshNode* UInterchangeGLTFTranslator::HandleGltfMesh(UInterchangeBaseNodeContainer& NodeContainer,
	const GLTF::FMesh& GltfMesh, int MeshIndex,
	TSet<int>& UnusedMeshIndices,
	const FString& SkeletalName/*If set it creates the mesh even if it was already created (for Skeletals)*/,
	const FString& SkeletalId) const
{
	FString MeshName = SkeletalName.Len() ? SkeletalName : GltfMesh.Name;
	FString MeshNodeUid = TEXT("\\Mesh\\") + (SkeletalId.Len() ? SkeletalId : GltfMesh.UniqueId);

	//check if Node already exist with MeshNodeUid:
	if (const UInterchangeMeshNode* Node = Cast< UInterchangeMeshNode >(NodeContainer.GetNode(MeshNodeUid)))
	{
		UInterchangeMeshNode* MeshNode = const_cast<UInterchangeMeshNode*>(Node);
		if (ensure(MeshNode))
		{
			return MeshNode;
		}
	}

	//to track which meshes we have to generate a mesh node for at the end of Translate:
	UnusedMeshIndices.Remove(MeshIndex);

	//Create Mesh Node:
	UInterchangeMeshNode* MeshNode = NewObject< UInterchangeMeshNode >(&NodeContainer);
	NodeContainer.SetupNode(MeshNode, MeshNodeUid, MeshName, EInterchangeNodeContainerType::TranslatedAsset);

	int32 PrimitiveIndex = 0;
	for (const GLTF::FPrimitive& Primitive : GltfMesh.Primitives)
	{
		FString Prefix = TEXT("Primitive[") + FString::FromInt(PrimitiveIndex) + TEXT("]_");
		for (const TPair<FString, FString>& Extra : Primitive.Extras)
		{
			UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(MeshNode, Prefix + Extra.Key, Extra.Value, TOptional<FString>());
		}

		PrimitiveIndex++;
	}
	for (const TPair<FString, FString>& Extra : GltfMesh.Extras)
	{
		UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(MeshNode, Extra.Key, Extra.Value, TOptional<FString>());
	}

	//Generate Mesh Payload:
	FString PayloadKey = LexToString(MeshIndex);
	MeshNode->SetPayLoadKey(PayloadKey, EInterchangeMeshPayLoadType::STATIC);

	//Set Slot Material Dependencies:
	for (int32 PrimitiveCounter = 0; PrimitiveCounter < GltfMesh.Primitives.Num(); PrimitiveCounter++)
	{
		const GLTF::FPrimitive& Primitive = GltfMesh.Primitives[PrimitiveCounter];

		// Assign materials
		if (GltfAsset.Materials.IsValidIndex(Primitive.MaterialIndex))
		{
			const FString MaterialName = GltfAsset.Materials[Primitive.MaterialIndex].Name;
			const FString ShaderGraphNodeUid = UInterchangeShaderGraphNode::MakeNodeUid(GltfAsset.Materials[Primitive.MaterialIndex].UniqueId);
			MeshNode->SetSlotMaterialDependencyUid(MaterialName, ShaderGraphNodeUid);
		}
	}

	//Generate Morph Target Meshes:
	if (GltfMesh.MorphTargetNames.Num() > 0)
	{
		for (int32 MorphTargetIndex = 0; MorphTargetIndex < GltfMesh.MorphTargetNames.Num(); MorphTargetIndex++)
		{
			//check if MorphTarget mesh was already created or not:
			FString MorphTargetName = GltfMesh.MorphTargetNames[MorphTargetIndex]; //Morph Target Names are validated to be unique (GLTFAsset::GenerateNames)

			//Add the MorphTargetName as a dependency to original mesh:
			MeshNode->SetMorphTargetDependencyUid(MorphTargetName);

			//check if Node already exist with MorphTargetName(uid):
			if (const UInterchangeMeshNode* Node = Cast< UInterchangeMeshNode >(NodeContainer.GetNode(MorphTargetName)))
			{
				continue;
			}

			//create MorphTargetMeshNode:
			UInterchangeMeshNode* MorphTargetMeshNode = NewObject< UInterchangeMeshNode >(&NodeContainer);
			NodeContainer.SetupNode(MorphTargetMeshNode, MorphTargetName, MorphTargetName, EInterchangeNodeContainerType::TranslatedAsset);

			//Generate Payload:
			FString MorphTargetPayLoadKey = LexToString(MeshIndex) + TEXT(":") + LexToString(MorphTargetIndex);
			MorphTargetMeshNode->SetPayLoadKey(MorphTargetPayLoadKey, EInterchangeMeshPayLoadType::MORPHTARGET);

			//set mesh as a morph target:
			MorphTargetMeshNode->SetMorphTarget(true);
			MorphTargetMeshNode->SetMorphTargetName(MorphTargetName);

			//Set Slot Material Dependencies:
			for (int32 PrimitiveCounter = 0; PrimitiveCounter < GltfMesh.Primitives.Num(); PrimitiveCounter++)
			{
				const GLTF::FPrimitive& Primitive = GltfMesh.Primitives[PrimitiveCounter];

				// Assign materials
				if (GltfAsset.Materials.IsValidIndex(Primitive.MaterialIndex))
				{
					const FString MaterialName = GltfAsset.Materials[Primitive.MaterialIndex].Name;
					const FString ShaderGraphNodeUid = UInterchangeShaderGraphNode::MakeNodeUid(GltfAsset.Materials[Primitive.MaterialIndex].UniqueId);
					MorphTargetMeshNode->SetSlotMaterialDependencyUid(MaterialName, ShaderGraphNodeUid);
				}
			}
		}
	}

	return MeshNode;
}

UInterchangeGLTFTranslator::UInterchangeGLTFTranslator()
{
	if (!HasAllFlags(RF_ClassDefaultObject))
	{
		bRenderSettingsClearCoatEnableSecondNormal = GetDefault<URendererSettings>()->bClearCoatEnableSecondNormal != 0;
	}
}

TOptional<UE::Interchange::FImportLightProfile> UInterchangeGLTFTranslator::GetLightProfilePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const
{
	using namespace UE::Interchange;

	int32 IndexLightIES = INDEX_NONE;
	LexFromString(IndexLightIES, *PayloadKey);

	if (!GltfAsset.LightsIES.IsValidIndex(IndexLightIES))
	{
		return {};
	}
	
	const bool bUseBuffer = GltfAsset.LightsIES[IndexLightIES].BufferViewIndex != INDEX_NONE;
	
	//faking the filepath for buffered version
	const FString FilePath = bUseBuffer ? TEXT("FakeFor_glTF_IES_Buffer_Import.ies") : FPaths::ConvertRelativePathToFull(GltfAsset.LightsIES[IndexLightIES].FilePath);

	UInterchangeSourceData* PayloadSourceData = UInterchangeManager::GetInterchangeManager().CreateSourceData(FilePath);
	FGCObjectScopeGuard ScopedSourceData(PayloadSourceData);
	if (!PayloadSourceData)
	{
		return TOptional<UE::Interchange::FImportLightProfile>();
	}

	PayloadSourceData->ClearInternalFlags(EInternalObjectFlags::Async);

	UInterchangeTranslatorBase* SourceTranslator = UInterchangeManager::GetInterchangeManager().GetTranslatorForSourceData(PayloadSourceData);
	FGCObjectScopeGuard ScopedSourceTranslator(SourceTranslator);
	const IInterchangeTextureLightProfilePayloadInterface* TextureTranslator = Cast< IInterchangeTextureLightProfilePayloadInterface >(SourceTranslator);
	if (!ensure(TextureTranslator))
	{
		return TOptional<UE::Interchange::FImportLightProfile>();
	}

	SourceTranslator->ClearInternalFlags(EInternalObjectFlags::Async);
	SourceTranslator->SetResultsContainer(Results);

	return bUseBuffer ? TextureTranslator->GetLightProfilePayloadData(GltfAsset.LightsIES[IndexLightIES].Data, GltfAsset.LightsIES[IndexLightIES].DataByteLength) : TextureTranslator->GetLightProfilePayloadData(FilePath, AlternateTexturePath);
}

#undef LOCTEXT_NAMESPACE
