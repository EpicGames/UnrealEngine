// Copyright Epic Games, Inc. All Rights Reserved.

#include "Groom/InterchangeGroomBindingFactory.h"

#include "Engine/SkeletalMesh.h"
#include "GeometryCache.h"
#include "Groom/InterchangeGroomPayloadInterface.h"
#include "GroomAsset.h"
#include "GroomBindingAsset.h"
#include "GroomBuilder.h"
#include "GroomImportOptions.h"

#include "InterchangeGroomBindingFactoryNode.h"
#include "InterchangeGroomBindingNode.h"
#include "InterchangeGroomFactoryNode.h"
#include "InterchangeImportCommon.h"
#include "InterchangeImportLog.h"
#include "InterchangeMeshFactoryNode.h"
#include "InterchangeSourceData.h"
#include "InterchangeTranslatorBase.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGroomBindingFactory)

#define LOCTEXT_NAMESPACE "InterchangeGroomBindingFactory"

UClass* UInterchangeGroomBindingFactory::GetFactoryClass() const
{
	return UGroomBindingAsset::StaticClass();
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeGroomBindingFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeGroomBindingFactory::BeginImportAsset_GameThread);

	FImportAssetResult ImportAssetResult;

#if WITH_EDITOR
	auto LogGroomFactoryError = [this, &Arguments, &ImportAssetResult](const FText& Info)
		{
			UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
			Message->SourceAssetName = Arguments.SourceData->GetFilename();
			Message->DestinationAssetName = Arguments.AssetName;
			Message->AssetType = GetFactoryClass();
			Message->Text = FText::Format(
				LOCTEXT("GroomFactory_Failure", "UInterchangeGroomFactory: Could not create Groom asset '{0}'. Reason: {1}"),
				FText::FromString(Arguments.AssetName),
				Info
			);
			ImportAssetResult.bIsFactorySkipAsset = true;
		};

	const IInterchangeGroomPayloadInterface* GroomPayloadInterface = Cast<IInterchangeGroomPayloadInterface>(Arguments.Translator);
	if (!GroomPayloadInterface)
	{
		LogGroomFactoryError(LOCTEXT("GroomFactory_NoPayloadInterface", "The translator does not implement IInterchangeGroomPayloadInterface"));
		return ImportAssetResult;
	}

	UInterchangeGroomBindingFactoryNode* GroomBindingFactoryNode = Cast<UInterchangeGroomBindingFactoryNode>(Arguments.AssetNode);
	if (!GroomBindingFactoryNode)
	{
		LogGroomFactoryError(LOCTEXT("GroomBindingFactory_AssetNodeNull", "Asset node parameter is not a UInterchangeGroomBindingFactoryNode."));
		return ImportAssetResult;
	}

	const UClass* ObjectClass = GroomBindingFactoryNode->GetObjectClass();
	if (!ObjectClass || !ObjectClass->IsChildOf(GetFactoryClass()))
	{
		LogGroomFactoryError(LOCTEXT("GroomBindingFactory_NodeClassMissmatch", "Asset node parameter class doesn't derive the UGroomBindingAsset class."));
		return ImportAssetResult;
	}

	// Retrieve the groom asset dependency
	FSoftObjectPath GroomAssetPath;
	FSoftObjectPath TargetMeshAssetPath;

	bool bSuccess = false;
	TSet<FString> Dependencies = GroomBindingFactoryNode->GetFactoryDependencies();
	for (const FString& Dependency : Dependencies)
	{
		if (const UInterchangeGroomFactoryNode* GroomFactoryNode = Cast<UInterchangeGroomFactoryNode>(Arguments.NodeContainer->GetNode(Dependency)))
		{
			bSuccess = GroomFactoryNode && GroomFactoryNode->GetCustomReferenceObject(GroomAssetPath);
			if (!bSuccess)
			{
				LogGroomFactoryError(LOCTEXT("GroomBindingFactory_MissingGroomAssetDependency", "Could not retrieve groom asset dependency."));
				return ImportAssetResult;
			}
		}
		if (const UInterchangeMeshFactoryNode* MeshFactoryNode = Cast<UInterchangeMeshFactoryNode>(Arguments.NodeContainer->GetNode(Dependency)))
		{
			bSuccess = MeshFactoryNode && MeshFactoryNode->GetCustomReferenceObject(TargetMeshAssetPath);
			if (!bSuccess)
			{
				LogGroomFactoryError(LOCTEXT("GroomBindingFactory_MissingMeshAssetDependency", "Could not retrieve target mesh asset dependency."));
				return ImportAssetResult;
			}
		}
	}

	UGroomAsset* GroomAsset= Cast<UGroomAsset>(GroomAssetPath.TryLoad());
	if (!GroomAsset)
	{
		LogGroomFactoryError(LOCTEXT("GroomBindingFactory_MissingGroomAsset", "Could not retrieve groom asset."));
		return ImportAssetResult;
	}

	UObject* MeshAsset = TargetMeshAssetPath.TryLoad();
	if (!MeshAsset)
	{
		LogGroomFactoryError(LOCTEXT("GroomBindingFactory_MissingMeshAsset", "Could not retrieve mesh asset."));
		return ImportAssetResult;
	}

	if (!Cast<USkeletalMesh>(MeshAsset) && !Cast<UGeometryCache>(MeshAsset))
	{
		LogGroomFactoryError(LOCTEXT("GroomBindingFactory_WrongMeshAsset", "The mesh asset is not a SkeletalMesh nor a GeometryCache."));
		return ImportAssetResult;
	}

	EGroomBindingMeshType GroomBindingType = Cast<USkeletalMesh>(MeshAsset) ? EGroomBindingMeshType::SkeletalMesh : EGroomBindingMeshType::GeometryCache;

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (GroomBindingFactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}
	}

	UGroomBindingAsset* GroomBindingAsset = nullptr;
	if (!ExistingAsset)
	{
		GroomBindingAsset = NewObject<UGroomBindingAsset>(Arguments.Parent, *Arguments.AssetName, RF_Public | RF_Standalone);
	}
	else
	{
		GroomBindingAsset = Cast<UGroomBindingAsset>(ExistingAsset);
	}

	if (!GroomBindingAsset)
	{
		LogGroomFactoryError(LOCTEXT("GroomFactory_GroomBindingCreationFailure", "Could not allocate new GroomBindingAsset, a possible different asset already exists."));
		return ImportAssetResult;
	}

	GroomBindingAsset->SetGroomBindingType(GroomBindingType);
	GroomBindingAsset->SetGroom(GroomAsset);
	if (GroomBindingAsset->GetGroomBindingType() == EGroomBindingMeshType::SkeletalMesh)
	{
		GroomBindingAsset->SetTargetSkeletalMesh(Cast<USkeletalMesh>(MeshAsset));
	}
	else
	{
		GroomBindingAsset->SetTargetGeometryCache(Cast<UGeometryCache>(MeshAsset));
	}
	GroomBindingAsset->GetHairGroupsPlatformData().Reserve(GroomAsset->GetHairGroupsPlatformData().Num());
	int32 NumInterpolationPoints = 100;
	if (GroomBindingFactoryNode->GetNumInterpolationPoints(NumInterpolationPoints))
	{
		GroomBindingAsset->SetNumInterpolationPoints(NumInterpolationPoints);
	}

	GroomBindingAsset->Build();

	ImportAssetResult.ImportedObject = GroomBindingAsset;

#endif // WITH_EDITOR

	return ImportAssetResult;
}

#undef LOCTEXT_NAMESPACE