// Copyright Epic Games, Inc. All Rights Reserved.

#include "Groom/InterchangeGroomCacheFactory.h"

#include "Groom/InterchangeGroomPayloadData.h"
#include "Groom/InterchangeGroomPayloadInterface.h"
#include "GroomBuilder.h"
#include "GroomCache.h"
#include "GroomImportOptions.h"
#include "InterchangeGroomCacheFactoryNode.h"
#include "InterchangeGroomFactoryNode.h"
#include "InterchangeGroomNode.h"
#include "InterchangeImportCommon.h"
#include "InterchangeImportLog.h"
#include "InterchangeSourceData.h"
#include "InterchangeTranslatorBase.h"
#include "Misc/ScopedSlowTask.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGroomCacheFactory)

#define LOCTEXT_NAMESPACE "InterchangeGroomCacheFactory"

UClass* UInterchangeGroomCacheFactory::GetFactoryClass() const
{
	return UGroomCache::StaticClass();
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeGroomCacheFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeGroomCacheFactory::BeginImportAsset_GameThread);

	FImportAssetResult ImportAssetResult;

	auto LogGroomCacheFactoryError = [this, &Arguments, &ImportAssetResult](const FText& Info)
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName = Arguments.SourceData->GetFilename();
		Message->DestinationAssetName = Arguments.AssetName;
		Message->AssetType = GetFactoryClass();
		Message->Text = FText::Format(
			LOCTEXT("GroomCacheFactory_Failure", "UInterchangeGroomCacheFactory: Could not create Groom Cache '{0}'. Reason: {1}"),
			FText::FromString(Arguments.AssetName),
			Info
		);
		ImportAssetResult.bIsFactorySkipAsset = true;
	};

	const IInterchangeGroomPayloadInterface* GroomPayloadInterface = Cast<IInterchangeGroomPayloadInterface>(Arguments.Translator);
	if (!GroomPayloadInterface)
	{
		LogGroomCacheFactoryError(LOCTEXT("GroomCacheFactory_NoPayloadInterface", "The translator does not implement IInterchangeGroomPayloadInterface"));
		return ImportAssetResult;
	}

	UInterchangeGroomCacheFactoryNode* GroomCacheFactoryNode = Cast<UInterchangeGroomCacheFactoryNode>(Arguments.AssetNode);
	if (!GroomCacheFactoryNode)
	{
		LogGroomCacheFactoryError(LOCTEXT("GroomCacheFactory_AssetNodeNull", "Asset node parameter is not a UInterchangeGroomCacheFactoryNode."));
		return ImportAssetResult;
	}

	const UClass* ObjectClass = GroomCacheFactoryNode->GetObjectClass();
	if (!ObjectClass || !ObjectClass->IsChildOf(GetFactoryClass()))
	{
		LogGroomCacheFactoryError(LOCTEXT("GroomCacheFactory_NodeClassMissmatch", "Asset node parameter class doesn't derive the UGroomCache class."));
		return ImportAssetResult;
	}

	TArray<FString> GroomUids;
	GroomCacheFactoryNode->GetTargetNodeUids(GroomUids);
	if (GroomUids.Num() == 0)
	{
		LogGroomCacheFactoryError(LOCTEXT("GroomCacheFactory_MissingTargetNodeId", "Groom cache factory node is missing the target node id."));
		return ImportAssetResult;
	}

	const FString& GroomUid = GroomUids[0]; // the groom pipeline adds only one target node
	const UInterchangeBaseNode* Node = Arguments.NodeContainer->GetNode(GroomUid);
	const UInterchangeGroomNode* GroomNode = Cast<const UInterchangeGroomNode>(Node);
	if (!GroomNode)
	{
		LogGroomCacheFactoryError(LOCTEXT("GroomCacheFactory_MissingGroomNode", "Groom node not found."));
		return ImportAssetResult;
	}

	TOptional<FInterchangeGroomPayloadKey> OptionalPayloadKey = GroomNode->GetPayloadKey();
	if (!ensure(OptionalPayloadKey.IsSet()))
	{
		LogGroomCacheFactoryError(LOCTEXT("GroomCacheFactory_MissingPayloadKey", "Groom node has no payload key."));
		return ImportAssetResult;
	}

	// Retrieve the groom asset dependency
	// First check if a groom asset was specifically selected by the user
	FSoftObjectPath GroomAssetPath;
	if (!GroomCacheFactoryNode->GetCustomGroomAssetPath(GroomAssetPath))
	{
		// Otherwise, the user wanted to import the groom asset
		bool bSuccess = false;
		TSet<FString> Dependencies = GroomCacheFactoryNode->GetFactoryDependencies();
		if (const FString* Dependency = Dependencies.FindArbitraryElement())
		{
			const UInterchangeGroomFactoryNode* GroomFactoryNode = Cast<UInterchangeGroomFactoryNode>(Arguments.NodeContainer->GetNode(*Dependency));
			bSuccess = GroomFactoryNode && GroomFactoryNode->GetCustomReferenceObject(GroomAssetPath);
		}

		if (!bSuccess)
		{
			LogGroomCacheFactoryError(LOCTEXT("GroomCacheFactory_MissingGroomAssetDependency", "Could not retrieve groom asset dependency."));
			return ImportAssetResult;
		}
	}

	UGroomAsset* GroomAssetForCache = Cast<UGroomAsset>(GroomAssetPath.TryLoad());
	if (!GroomAssetForCache)
	{
		LogGroomCacheFactoryError(LOCTEXT("GroomCacheFactory_MissingGroomAsset", "Could not retrieve groom asset."));
		return ImportAssetResult;
	}

	// Reconstruct and validate the GroomAnimationInfo
	FGroomAnimationInfo AnimInfo;
	int32 Value = 0;
	if (GroomCacheFactoryNode->GetCustomNumFrames(Value))
	{
		AnimInfo.NumFrames = Value;
	}

	if (GroomCacheFactoryNode->GetCustomStartFrame(Value))
	{
		AnimInfo.StartFrame = Value;
	}

	if (GroomCacheFactoryNode->GetCustomEndFrame(Value))
	{
		AnimInfo.EndFrame = Value;
	}

	double FrameRate = 0;
	if (GroomCacheFactoryNode->GetCustomFrameRate(FrameRate))
	{
		if (FrameRate != 0)
		{
			AnimInfo.SecondsPerFrame = static_cast<float>(1 / FrameRate);
		}
		else
		{
			// Fallback to 24 fps
			AnimInfo.SecondsPerFrame = static_cast<float>(1.0 / 24.0);
		}
	}

	EInterchangeGroomCacheAttributes Attributes;
	if (GroomCacheFactoryNode->GetCustomGroomCacheAttributes(Attributes))
	{
		AnimInfo.Attributes = static_cast<EGroomCacheAttributes>(Attributes);
	}

	if (!AnimInfo.IsValid())
	{
		LogGroomCacheFactoryError(LOCTEXT("GroomCacheFactory_InvalidAnimInfo", "Invalid number of frames, start/end frame, frame rate or no attribute is animated."));
		return ImportAssetResult;
	}

	AnimInfo.Duration = AnimInfo.NumFrames * AnimInfo.SecondsPerFrame;
	AnimInfo.StartTime = AnimInfo.StartFrame * AnimInfo.SecondsPerFrame;
	AnimInfo.EndTime = AnimInfo.EndFrame * AnimInfo.SecondsPerFrame;

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (GroomCacheFactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}
	}

	UGroomCache* GroomCache = Cast<UGroomCache>(ExistingAsset);
	if (!GroomCache)
	{
		GroomCache = NewObject<UGroomCache>(Arguments.Parent, *Arguments.AssetName, RF_Public | RF_Standalone);
	}
	if (!GroomCache)
	{
		LogGroomCacheFactoryError(LOCTEXT("GroomCacheFactory_GroomCreationFailure", "Could not allocate new GroomCache."));
		return ImportAssetResult;
	}

	static_assert((int)EGroomCacheType::None == (int)EInterchangeGroomCacheImportType::None);
	static_assert((int)EGroomCacheType::Strands == (int)EInterchangeGroomCacheImportType::Strands);
	static_assert((int)EGroomCacheType::Guides == (int)EInterchangeGroomCacheImportType::Guides);

	// Initialize groom cache
	EInterchangeGroomCacheImportType ImportType;
	if (!GroomCacheFactoryNode->GetCustomGroomCacheImportType(ImportType))
	{
		LogGroomCacheFactoryError(LOCTEXT("GroomCacheFactory_NoImportType", "No GroomCache type was specified for import."));
		return ImportAssetResult;
	}
	EGroomCacheType CacheType = static_cast<EGroomCacheType>(ImportType);
	GroomCache->Initialize(CacheType);

	const int32 NumFrames = AnimInfo.NumFrames + 1;
	FScopedSlowTask SlowTask(NumFrames, LOCTEXT("ImportGroomCache", "Importing GroomCache frames"));
	SlowTask.MakeDialog();

	const TArray<FHairGroupPlatformData>& GroupPlatformData = GroomAssetForCache->GetHairGroupsPlatformData();
	const FInterchangeGroomPayloadKey& PayloadKey = OptionalPayloadKey.GetValue();

	FGroomCacheProcessor CacheProcessor(CacheType, AnimInfo.Attributes);
	bool bSuccess = true;
	FText ImportErrorMessage;

	// Ref. FGroomCacheImporter::ImportGroomCache
	// 
	// Each frame the HairDescription payload is translated and processed into GroomCacheInputData
	// If the frame groom data fails the validation, the import is aborted
	// 
	// Sample one extra frame so that we can interpolate between EndFrame - 1 and EndFrame
	for (int32 FrameIndex = AnimInfo.StartFrame; FrameIndex < AnimInfo.EndFrame + 1; ++FrameIndex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGroomCacheImporter::ImportGroomCache::OneFrame);

		const uint32 CurrentFrame = FrameIndex - AnimInfo.StartFrame;

		FTextBuilder TextBuilder;
		TextBuilder.AppendLineFormat(LOCTEXT("ImportGroomCacheFrame", "Importing {2} frame {0} of {1}"), FText::AsNumber(CurrentFrame), FText::AsNumber(NumFrames), FText::FromString(Arguments.AssetName));
		SlowTask.EnterProgressFrame(1.f, TextBuilder.ToText());

		// Get the payload for this frame
		FInterchangeGroomPayloadKey FramePayloadKey = PayloadKey;
		FramePayloadKey.FrameNumber = CurrentFrame;
		TOptional<UE::Interchange::FGroomPayloadData> GroomPayload = GroomPayloadInterface->GetGroomPayloadData(FramePayloadKey);

		if (!GroomPayload.IsSet())
		{
			bSuccess = false;
			ImportErrorMessage = FText::Format(LOCTEXT("GroomCacheFactory_NoPayload", "Missing payload for frame {0}"), FrameIndex);
			break;
		}

		FHairDescriptionGroups HairDescriptionGroups;
		// Do not add extra control points at the end of curve when hair strip geometry is enabled. This is because groom cache data are 
		// serialized within the uasset (i.e. do not use intermediate cached/build data), and need the asset to be compatible with or without hair strip geometry.
		if (!FGroomBuilder::BuildHairDescriptionGroups(GroomPayload->HairDescription, HairDescriptionGroups, false /*bAllowAddEndControlPoint*/))
		{
			bSuccess = false;
			ImportErrorMessage = LOCTEXT("GroomCacheFactory_HairDescriptionGroupsBuildFailure", "Failed to build HairDescriptionGroups.");
			break;
		}

		const uint32 GroupCount = HairDescriptionGroups.HairGroups.Num();
		if (GroupCount != GroupPlatformData.Num())
		{
			bSuccess = false;
			ImportErrorMessage = FText::Format(LOCTEXT("GroomCacheFactory_GroupCountMismatch", "GroomCache does not have the same number of groups as the static groom ({0} instead of {1})."), GroupCount, GroupPlatformData.Num());
			break;
		}

		TArray<FHairGroupInfoWithVisibility> HairGroupsInfo = GroomAssetForCache->GetHairGroupsInfo();
		TArray<FGroomCacheInputData> CacheInputDatas;
		CacheInputDatas.SetNum(GroupCount);
		for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
		{
			FGroomBuilder::BuildData(
				HairDescriptionGroups.HairGroups[GroupIndex],
				GroomAssetForCache->GetHairGroupsInterpolation()[GroupIndex],
				HairGroupsInfo[GroupIndex],
				CacheInputDatas[GroupIndex].Strands,
				CacheInputDatas[GroupIndex].Guides);
		}

		// Validate that the GroomCache has the same topology as the static groom
		for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
		{
			if (CacheType == EGroomCacheType::Strands)
			{
				// When UsesTriangleStrips is enabled, we add an extra control point at the end of each curve in the groom asset.
				// Since the groom cache needs to be independent of these settings (since the data is serialized directly into the asset
				// and not rebuilt from a description), it does not contains these extra control points.
				// We account for them for validation here
				const bool bHasExtraPoint = GroupPlatformData[GroupIndex].Strands.BulkData.HasFlags(FHairStrandsBulkData::DataFlags_HasExtraEndingPoint);
				const uint32 ExtraControlPointCount = bHasExtraPoint ? CacheInputDatas[GroupIndex].Strands.GetNumCurves() : 0u;

				const bool bCurveCountMatch = CacheInputDatas[GroupIndex].Strands.GetNumCurves() == GroupPlatformData[GroupIndex].Strands.BulkData.GetNumCurves();
				const bool bPointCountMatch = CacheInputDatas[GroupIndex].Strands.GetNumPoints() + ExtraControlPointCount == GroupPlatformData[GroupIndex].Strands.BulkData.GetNumPoints();

				if (!bCurveCountMatch || !bPointCountMatch)
				{
					bSuccess = false;
					ImportErrorMessage = FText::Format(LOCTEXT("GroomCacheFactory_StrandsValidationFailure", 
						"GroomCache frame {0} does not have the same number of curves ({1}) or vertices ({2}) for the strands as the static groom ({3} and {4} respectively)."),
						FrameIndex, CacheInputDatas[GroupIndex].Strands.GetNumCurves(), CacheInputDatas[GroupIndex].Strands.GetNumPoints() + ExtraControlPointCount,
						GroupPlatformData[GroupIndex].Strands.BulkData.GetNumCurves(), GroupPlatformData[GroupIndex].Strands.BulkData.GetNumPoints());
					break;
				}
			}
			else if (CacheType == EGroomCacheType::Guides)
			{
				const bool bHasExtraPoint = GroupPlatformData[GroupIndex].Guides.BulkData.HasFlags(FHairStrandsBulkData::DataFlags_HasExtraEndingPoint);
				const uint32 ExtraControlPointCount = bHasExtraPoint ? CacheInputDatas[GroupIndex].Guides.GetNumCurves() : 0u;

				const bool bCurveCountMatch = CacheInputDatas[GroupIndex].Guides.GetNumCurves() == GroupPlatformData[GroupIndex].Guides.BulkData.GetNumCurves();
				const bool bPointCountMatch = CacheInputDatas[GroupIndex].Guides.GetNumPoints() + ExtraControlPointCount == GroupPlatformData[GroupIndex].Guides.BulkData.GetNumPoints();

				if (!bCurveCountMatch || !bPointCountMatch)
				{
					bSuccess = false;
					ImportErrorMessage = FText::Format(LOCTEXT("GroomCacheFactory_GuidesValidationFailure",
						"GroomCache frame {0} does not have the same number of curves ({1}) or vertices ({2}) for the guides as the static groom ({3} and {4} respectively)."),
						FrameIndex, CacheInputDatas[GroupIndex].Guides.GetNumCurves(), CacheInputDatas[GroupIndex].Guides.GetNumPoints() + ExtraControlPointCount,
						GroupPlatformData[GroupIndex].Guides.BulkData.GetNumCurves(), GroupPlatformData[GroupIndex].Guides.BulkData.GetNumPoints());
					break;
				}
			}
		}

		if (!bSuccess)
		{
			break;
		}

		CacheProcessor.AddGroomSample(MoveTemp(CacheInputDatas));
	}

	if (!bSuccess)
	{
		if (GroomCache != ExistingAsset)
		{
			GroomCache->MarkAsGarbage();
		}

		LogGroomCacheFactoryError(ImportErrorMessage);
		return ImportAssetResult;
	}

	// Finalize groom cache
	CacheProcessor.TransferChunks(GroomCache);
	GroomCache->SetGroomAnimationInfo(AnimInfo);

	ImportAssetResult.ImportedObject = GroomCache;

	return ImportAssetResult;
}

void UInterchangeGroomCacheFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeGroomCacheFactory::SetupObject_GameThread);

	Super::SetupObject_GameThread(Arguments);

#if WITH_EDITORONLY_DATA
	if (ensure(Arguments.ImportedObject && Arguments.SourceData))
	{
		// We must call the Update of the asset source file in the main thread because UAssetImportData::Update execute some delegate we do not control
		UGroomCache* GroomCache = CastChecked<UGroomCache>(Arguments.ImportedObject);

		UE::Interchange::FFactoryCommon::FUpdateImportAssetDataParameters UpdateImportAssetDataParameters(GroomCache, GroomCache->AssetImportData, Arguments.SourceData, Arguments.NodeUniqueID, Arguments.NodeContainer, Arguments.OriginalPipelines, Arguments.Translator);
		GroomCache->AssetImportData = UE::Interchange::FFactoryCommon::UpdateImportAssetData(UpdateImportAssetDataParameters);
	}
#endif
}

bool UInterchangeGroomCacheFactory::GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const
{
#if WITH_EDITORONLY_DATA
	if (const UGroomCache* GroomCache = Cast<UGroomCache>(Object))
	{
		return UE::Interchange::FFactoryCommon::GetSourceFilenames(GroomCache->AssetImportData, OutSourceFilenames);
	}
#endif

	return false;
}

bool UInterchangeGroomCacheFactory::SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const
{
#if WITH_EDITORONLY_DATA
	if (const UGroomCache* GroomCache = Cast<UGroomCache>(Object))
	{
		return UE::Interchange::FFactoryCommon::SetSourceFilename(GroomCache->AssetImportData, SourceFilename, SourceIndex);
	}
#endif

	return false;
}

void UInterchangeGroomCacheFactory::BackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UGroomCache* GroomCache = Cast<UGroomCache>(Object))
	{
		UE::Interchange::FFactoryCommon::BackupSourceData(GroomCache->AssetImportData);
	}
#endif
}

void UInterchangeGroomCacheFactory::ReinstateSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UGroomCache* GroomCache = Cast<UGroomCache>(Object))
	{
		UE::Interchange::FFactoryCommon::ReinstateSourceData(GroomCache->AssetImportData);
	}
#endif
}

void UInterchangeGroomCacheFactory::ClearBackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UGroomCache* GroomCache = Cast<UGroomCache>(Object))
	{
		UE::Interchange::FFactoryCommon::ClearBackupSourceData(GroomCache->AssetImportData);
	}
#endif
}

#undef LOCTEXT_NAMESPACE
