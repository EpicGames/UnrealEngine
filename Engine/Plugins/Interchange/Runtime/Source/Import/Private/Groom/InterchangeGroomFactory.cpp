// Copyright Epic Games, Inc. All Rights Reserved.

#include "Groom/InterchangeGroomFactory.h"

#include "Groom/InterchangeGroomPayloadInterface.h"
#include "GroomAsset.h"
#include "GroomBuilder.h"
#include "GroomImportOptions.h"
#if WITH_EDITOR
#include "HairStrandsImporter.h"
#endif

#include "InterchangeGroomFactoryNode.h"
#include "InterchangeGroomNode.h"
#include "InterchangeImportCommon.h"
#include "InterchangeImportLog.h"
#include "InterchangeSourceData.h"
#include "InterchangeTranslatorBase.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGroomFactory)

#define LOCTEXT_NAMESPACE "InterchangeGroomFactory"

UClass* UInterchangeGroomFactory::GetFactoryClass() const
{
	return UGroomAsset::StaticClass();
}

void UInterchangeGroomFactory::CreatePayloadTasks(const FImportAssetObjectParams& Arguments, bool bAsync, TArray<TSharedPtr<UE::Interchange::FInterchangeTaskBase>>& PayloadTasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeGroomFactory::CreatePayloadTasks);

	UInterchangeGroomFactoryNode* GroomFactoryNode = Cast<UInterchangeGroomFactoryNode>(Arguments.AssetNode);
	if (GroomFactoryNode == nullptr)
	{
		return;
	}

	const IInterchangeGroomPayloadInterface* GroomPayloadInterface = Cast<IInterchangeGroomPayloadInterface>(Arguments.Translator);
	if (!GroomPayloadInterface)
	{
		UE_LOGF(LogInterchangeImport, Error, "Cannot import groom asset. The translator does not implement IInterchangeGroomPayloadInterface.");
		return;
	}

	TArray<FString> GroomUids;
	GroomFactoryNode->GetTargetNodeUids(GroomUids);
	if (GroomUids.Num() == 0)
	{
		return;
	}

	const FString& GroomUid = GroomUids[0]; // the groom pipeline adds only one target node
	const UInterchangeBaseNode* Node = Arguments.NodeContainer->GetNode(GroomUid);
	if (const UInterchangeGroomNode* GroomNode = Cast<const UInterchangeGroomNode>(Node))
	{
		TOptional<FInterchangeGroomPayloadKey> OptionalPayloadKey = GroomNode->GetPayloadKey();
		if (!ensure(OptionalPayloadKey.IsSet()))
		{
			UE_LOGF(LogInterchangeImport, Warning, "Empty groom reference payload when importing Groom asset %ls.", *Arguments.AssetName);
			return;
		}

		FInterchangeGroomPayloadKey& PayloadKey = OptionalPayloadKey.GetValue();

		TSharedPtr<UE::Interchange::FInterchangeTaskLambda, ESPMode::ThreadSafe> TaskGetMeshPayload = MakeShared<UE::Interchange::FInterchangeTaskLambda, ESPMode::ThreadSafe>(
			bAsync ? UE::Interchange::EInterchangeTaskThread::AsyncThread : UE::Interchange::EInterchangeTaskThread::GameThread,
			[&GroomPayload = GroomPayload, GroomPayloadInterface, PayloadKey]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeGroomFactory::GetGroomPayloadDataTask);
				GroomPayload = GroomPayloadInterface->GetGroomPayloadData(PayloadKey);
			});

		PayloadTasks.Add(TaskGetMeshPayload);
	}
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeGroomFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeGroomFactory::BeginImportAsset_GameThread);

	FImportAssetResult ImportAssetResult;

#if WITH_EDITOR
	auto LogGroomFactoryError = [this, &Arguments, &ImportAssetResult](const FText& Info)
		{
			UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
			Message->SourceAssetName = Arguments.SourceData->GetFilename();
			Message->DestinationAssetName = Arguments.AssetName;
			Message->AssetType = GetFactoryClass();
			Message->Text = FText::Format(
				LOCTEXT("GroomFactory_Failure", "UInterchangeGroomFactory: Could not create Groom asset '{0}'. Reason: {1}"),
				FText::FromString(Arguments.AssetName),
				Info
			);
			ImportAssetResult.bIsFactorySkipAsset = true;
		};

	const IInterchangeGroomPayloadInterface* GroomPayloadInterface = Cast<IInterchangeGroomPayloadInterface>(Arguments.Translator);
	if (!GroomPayloadInterface)
	{
		LogGroomFactoryError(LOCTEXT("GroomFactory_NoPayloadInterface", "The translator does not implement IInterchangeGroomPayloadInterface"));
		return ImportAssetResult;
	}

	UInterchangeGroomFactoryNode* GroomFactoryNode = Cast<UInterchangeGroomFactoryNode>(Arguments.AssetNode);
	if (!GroomFactoryNode)
	{
		LogGroomFactoryError(LOCTEXT("GroomFactory_AssetNodeNull", "Asset node parameter is not a UInterchangeGroomFactoryNode."));
		return ImportAssetResult;
	}

	const UClass* ObjectClass = GroomFactoryNode->GetObjectClass();
	if (!ObjectClass || !ObjectClass->IsChildOf(GetFactoryClass()))
	{
		LogGroomFactoryError(LOCTEXT("GroomFactory_NodeClassMissmatch", "Asset node parameter class doesn't derive the UGroomAsset class."));
		return ImportAssetResult;
	}

	if (!GroomPayload.IsSet())
	{
		LogGroomFactoryError(LOCTEXT("GroomFactory_NoPayload", "No groom payload could be translated from the source."));
		return ImportAssetResult;
	}

	FHairDescription& HairDescription = GroomPayload->HairDescription;
	if (!HairDescription.IsValid())
	{
		LogGroomFactoryError(LOCTEXT("GroomFactory_InvalidHairDescription", "No valid hair description found in groom payload."));
		return ImportAssetResult;
	}

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (GroomFactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}
	}

	UGroomAsset* GroomAsset = Cast<UGroomAsset>(ExistingAsset);
	if (!GroomAsset)
	{
		GroomAsset = NewObject<UGroomAsset>(Arguments.Parent, *Arguments.AssetName, RF_Public | RF_Standalone);
	}
	if (!GroomAsset)
	{
		LogGroomFactoryError(LOCTEXT("GroomFactory_GroomCreationFailure", "Could not allocate new GroomAsset."));
		return ImportAssetResult;
	}

	// Extract the groom groups info from the hair description to get the number of groups
	FHairDescriptionGroups GroupsDescription;
	FGroomBuilder::BuildHairDescriptionGroups(HairDescription, GroupsDescription);

	FHairGroupsInterpolation GroupInterpolationSettings;
	GroomFactoryNode->GetCustomGroupInterpolationSettings(GroupInterpolationSettings);

	TArray<FHairGroupsInterpolation> GroomInterpolationSettings;
	GroomInterpolationSettings.Init(GroupInterpolationSettings, GroupsDescription.HairGroups.Num());
	UGroomImportOptions* ImportOptions = CreateGroomImportOptions(GroupsDescription, GroomInterpolationSettings);

	FHairImportContext HairImportContext(ImportOptions);
	UGroomAsset* ImportedGroomAsset = FHairStrandsImporter::ImportHair(HairImportContext, HairDescription, GroomAsset);
	if (ImportedGroomAsset)
	{
		ImportAssetResult.ImportedObject = ImportedGroomAsset;
	}
	else
	{
		GroomAsset->MarkAsGarbage();
	}
#endif // WITH_EDITOR

	return ImportAssetResult;
}

void UInterchangeGroomFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeGroomFactory::SetupObject_GameThread);

	Super::SetupObject_GameThread(Arguments);

#if WITH_EDITORONLY_DATA
	if (ensure(Arguments.ImportedObject && Arguments.SourceData))
	{
		// We must call the Update of the asset source file in the main thread because UAssetImportData::Update execute some delegate we do not control
		UGroomAsset* GroomAsset = CastChecked<UGroomAsset>(Arguments.ImportedObject);

		UE::Interchange::FFactoryCommon::FUpdateImportAssetDataParameters UpdateImportAssetDataParameters(GroomAsset, GroomAsset->AssetImportData, Arguments.SourceData, Arguments.NodeUniqueID, Arguments.NodeContainer, Arguments.OriginalPipelines, Arguments.Translator);
		GroomAsset->AssetImportData = UE::Interchange::FFactoryCommon::UpdateImportAssetData(UpdateImportAssetDataParameters);
	}
#endif
}

bool UInterchangeGroomFactory::GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const
{
#if WITH_EDITORONLY_DATA
	if (const UGroomAsset* GroomAsset = Cast<UGroomAsset>(Object))
	{
		return UE::Interchange::FFactoryCommon::GetSourceFilenames(GroomAsset->AssetImportData, OutSourceFilenames);
	}
#endif

	return false;
}

bool UInterchangeGroomFactory::SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const
{
#if WITH_EDITORONLY_DATA
	if (const UGroomAsset* GroomAsset = Cast<UGroomAsset>(Object))
	{
		return UE::Interchange::FFactoryCommon::SetSourceFilename(GroomAsset->AssetImportData, SourceFilename, SourceIndex);
	}
#endif

	return false;
}

void UInterchangeGroomFactory::BackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UGroomAsset* GroomAsset = Cast<UGroomAsset>(Object))
	{
		UE::Interchange::FFactoryCommon::BackupSourceData(GroomAsset->AssetImportData);
	}
#endif
}

void UInterchangeGroomFactory::ReinstateSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UGroomAsset* GroomAsset = Cast<UGroomAsset>(Object))
	{
		UE::Interchange::FFactoryCommon::ReinstateSourceData(GroomAsset->AssetImportData);
	}
#endif
}

void UInterchangeGroomFactory::ClearBackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UGroomAsset* GroomAsset = Cast<UGroomAsset>(Object))
	{
		UE::Interchange::FFactoryCommon::ClearBackupSourceData(GroomAsset->AssetImportData);
	}
#endif
}

#undef LOCTEXT_NAMESPACE
