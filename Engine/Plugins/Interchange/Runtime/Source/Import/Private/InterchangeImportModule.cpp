// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeImportModule.h"

#include "Animation/InterchangeAnimSequenceFactory.h"
#include "Animation/InterchangeLevelSequenceFactory.h"
#include "CoreMinimal.h"
#include "Engine/Engine.h"
#include "Engine/RendererSettings.h"
#include "Fbx/InterchangeFbxTranslator.h"
#include "Gltf/InterchangeGltfTranslator.h"
#include "Groom/InterchangeGroomBindingFactory.h"
#include "Groom/InterchangeGroomCacheFactory.h"
#include "Groom/InterchangeGroomFactory.h"
#include "InterchangeImportLog.h"
#include "InterchangeManager.h"
#include "Audio/InterchangeAudioSoundWaveFactory.h"
#include "Audio/Formats/InterchangeAudioTranslator_AIF.h"
#include "Audio/Formats/InterchangeAudioTranslator_AIFF.h"
#include "Audio/Formats/InterchangeAudioTranslator_OGG.h"
#include "Audio/Formats/InterchangeAudioTranslator_FLAC.h"
#include "Audio/Formats/InterchangeAudioTranslator_OPUS.h"
#include "Audio/Formats/InterchangeAudioTranslator_MP3.h"
#include "Audio/Formats/InterchangeAudioTranslator_WAV.h"
#include "Material/InterchangeMaterialFactory.h"
#include "MaterialX/InterchangeMaterialXTranslator.h"
#include "Mesh/InterchangeGeometryCacheFactory.h"
#include "Mesh/InterchangeOBJTranslator.h"
#include "Mesh/InterchangePhysicsAssetFactory.h"
#include "Mesh/InterchangeSkeletalMeshFactory.h"
#include "Mesh/InterchangeSkeletonFactory.h"
#include "Mesh/InterchangeStaticMeshFactory.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "Scene/InterchangeActorFactory.h"
#include "Scene/InterchangeCameraActorFactory.h"
#include "Scene/InterchangeDecalActorFactory.h"
#include "Scene/InterchangeGeometryCacheActorFactory.h"
#include "Scene/InterchangeHeterogeneousVolumeActorFactory.h"
#include "Scene/InterchangeLevelFactory.h"
#include "Scene/InterchangeLevelInstanceActorFactory.h"
#include "Scene/InterchangeLightActorFactory.h"
#include "Scene/InterchangeSceneImportAssetFactory.h"
#include "Scene/InterchangeSceneVariantSetsFactory.h"
#include "Scene/InterchangeSkeletalMeshActorFactory.h"
#include "Scene/InterchangeSkyLightActorFactory.h"
#include "Scene/InterchangeStaticMeshActorFactory.h"
#include "SpecularProfile/InterchangeSpecularProfileFactory.h"
#include "Texture/InterchangeDDSTranslator.h"
#include "Texture/InterchangeIESTranslator.h"
#include "Texture/InterchangeImageWrapperTranslator.h"
#include "Texture/InterchangeJPGTranslator.h"
#include "Texture/InterchangePSDTranslator.h"
#include "Texture/InterchangeTextureFactory.h"
#include "Texture/InterchangeUEJPEGTranslator.h"
#include "Volume/InterchangeSparseVolumeTextureFactory.h"

DEFINE_LOG_CATEGORY(LogInterchangeImport);

static bool GInterchangeEnableSubstrate = true;
static FAutoConsoleVariableRef CCvarInterchangeEnableSubstrateSupport(
	TEXT("Interchange.FeatureFlags.Import.Substrate"),
	GInterchangeEnableSubstrate,
	TEXT("Enable or disable support of Substrate with Interchange (only works if Substrate is enabled in the Project Settings). Enabled by default."),
	ECVF_Default);

class FInterchangeImportModule : public IInterchangeImportModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool IsSubstrateEnabled() const override
	{
		return bIsSubstrateEnabled && GInterchangeEnableSubstrate;
	}
	virtual bool IsSubstrateAdaptiveGBufferEnabled() const override
	{
		return bIsSubstrateAdaptiveGBufferEnabled;
	}

private:

	bool bIsSubstrateEnabled = false;
	bool bIsSubstrateAdaptiveGBufferEnabled = false;
};

IMPLEMENT_MODULE(FInterchangeImportModule, InterchangeImport)



void FInterchangeImportModule::StartupModule()
{
	if (const URendererSettings* RendererSettings = GetDefault<URendererSettings>())
	{
		bIsSubstrateEnabled = RendererSettings->bEnableSubstrate;
		bIsSubstrateAdaptiveGBufferEnabled = RendererSettings->SubstrateGBufferFormat == ESubstrateStorageFormat::Type::AdaptiveBuffer;
	}

	FInterchangeImportMaterialAsyncHelper& InterchangeMaterialAsyncHelper = FInterchangeImportMaterialAsyncHelper::GetInstance();

	auto RegisterItems = []()
	{
		UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();

		//Register the translators
		//Scenes
#if WITH_EDITOR
		UInterchangeFbxTranslator::CleanUpTemporaryFolder();
		InterchangeManager.RegisterTranslator(UInterchangeFbxTranslator::StaticClass());
#endif
		InterchangeManager.RegisterTranslator(UInterchangeGLTFTranslator::StaticClass());
		InterchangeManager.RegisterTranslator(UInterchangeOBJTranslator::StaticClass());

		// Audio
		InterchangeManager.RegisterTranslator(UInterchangeAudioTranslator_AIF::StaticClass());
		InterchangeManager.RegisterTranslator(UInterchangeAudioTranslator_AIFF::StaticClass());
		InterchangeManager.RegisterTranslator(UInterchangeAudioTranslator_OGG::StaticClass());
		InterchangeManager.RegisterTranslator(UInterchangeAudioTranslator_FLAC::StaticClass());
		InterchangeManager.RegisterTranslator(UInterchangeAudioTranslator_OPUS::StaticClass());
		InterchangeManager.RegisterTranslator(UInterchangeAudioTranslator_MP3::StaticClass());
		InterchangeManager.RegisterTranslator(UInterchangeAudioTranslator_WAV::StaticClass());

		//Materials
		InterchangeManager.RegisterTranslator(UInterchangeMaterialXTranslator::StaticClass());

		//Textures
		InterchangeManager.RegisterTranslator(UInterchangeImageWrapperTranslator::StaticClass());
		InterchangeManager.RegisterTranslator(UInterchangeDDSTranslator::StaticClass());
		InterchangeManager.RegisterTranslator(UInterchangeUEJPEGTranslator::StaticClass());
		InterchangeManager.RegisterTranslator(UInterchangeJPGTranslator::StaticClass());
		InterchangeManager.RegisterTranslator(UInterchangePSDTranslator::StaticClass());
		InterchangeManager.RegisterTranslator(UInterchangeIESTranslator::StaticClass());

		//Register the factories
		InterchangeManager.RegisterFactory(UInterchangeTextureFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeMaterialFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeMaterialFunctionFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeSkeletonFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeSkeletalMeshFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeStaticMeshFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeGeometryCacheFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeGroomFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeGroomCacheFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeGroomBindingFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangePhysicsAssetFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeActorFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeLevelFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeLevelInstanceActorFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeLevelSequenceFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeAnimSequenceFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeCineCameraActorFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeCameraActorFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeStaticMeshActorFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeDecalActorFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeSkeletalMeshActorFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeGeometryCacheActorFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeSceneVariantSetsFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeLightActorFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeSkyLightActorFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeSceneImportAssetFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeSpecularProfileFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeHeterogeneousVolumeActorFactory::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeAudioSoundWaveFactory::StaticClass());
#if WITH_EDITOR
		// Editor-only because SVT internals like UStreamableSparseVolumeTexture::AppendFrame are editor-only
		InterchangeManager.RegisterFactory(UInterchangeSparseVolumeTextureFactory::StaticClass());
#endif
	};

	if (GEngine)
	{
		RegisterItems();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterItems);
	}

	UInterchangeManager::SetInterchangeImportEnabled(true);
}


void FInterchangeImportModule::ShutdownModule()
{
	UInterchangeManager::SetInterchangeImportEnabled(false);
	UInterchangeFbxTranslator::CleanUpTemporaryFolder();
}



