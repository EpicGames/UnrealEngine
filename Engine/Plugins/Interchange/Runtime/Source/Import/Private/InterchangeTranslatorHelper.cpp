// Copyright Epic Games, Inc. All Rights Reserved. 

#include "InterchangeTranslatorHelper.h"

#include "CoreGlobals.h"
#include "InterchangeManager.h"
#include "InterchangeResultsContainer.h"
#include "InterchangeSourceData.h"
#include "InterchangeTranslatorBase.h"
#include "UObject/Object.h"

namespace UE::Interchange::Private
{
	FScopedTranslator::FScopedTranslator(const FString& PayLoadKey, UInterchangeResultsContainer* Results, TObjectPtr<UInterchangeAnalyticsHandlerBase> AnalyticsHandler)
	{
		PayloadSourceData = UInterchangeManager::GetInterchangeManager().CreateSourceData(PayLoadKey);
		if (PayloadSourceData)
		{
			PayloadSourceData->SetInternalFlags(EInternalObjectFlags::Async);

			SourceTranslator = UInterchangeManager::GetInterchangeManager().GetTranslatorForSourceData(PayloadSourceData);
			if (SourceTranslator)
			{
				SourceTranslator->AnalyticsHandler = AnalyticsHandler;
				SourceTranslator->SetInternalFlags(EInternalObjectFlags::Async);
				if (Results)
				{
					SourceTranslator->SetResultsContainer(Results);
				}
			}
		}
	}

	FScopedTranslator::~FScopedTranslator()
	{
		if (PayloadSourceData)
		{
			PayloadSourceData->ClearInternalFlags(EInternalObjectFlags::Async);
		}

		if (SourceTranslator)
		{
			SourceTranslator->ClearInternalFlags(EInternalObjectFlags::Async);
		}

		SourceTranslator = nullptr;
		PayloadSourceData = nullptr;
	}

	bool FScopedTranslator::IsValid() const
	{
		return SourceTranslator != nullptr;
	}
} //namespace UE::Interchange::Private
