// Copyright Epic Games, Inc. All Rights Reserved.
#include "Material/InterchangeMaterialFactory.h"

#include "Engine/RendererSettings.h"
#include "Engine/SpecularProfile.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeDecalMaterialFactoryNode.h"
#include "InterchangeImportCommon.h"
#include "InterchangeImportLog.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeShaderGraphNode.h"
#include "InterchangeSourceData.h"
#include "InterchangeSparseVolumeTextureFactoryNode.h"
#include "InterchangeSpecularProfileFactoryNode.h"
#include "InterchangeTextureFactoryNode.h"
#include "InterchangeTextureNode.h"
#include "InterchangeTranslatorBase.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionClearCoatNormalCustomOutput.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionMakeMaterialAttributes.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionSubstrate.h"
#include "Materials/MaterialExpressionSwitch.h"
#include "Materials/MaterialExpressionSwitch.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionThinTranslucentMaterialOutput.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInterface.h"
#include "Misc/PackageName.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "UObject/ObjectRedirector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeMaterialFactory)


#if WITH_EDITORONLY_DATA
#include "EditorFramework/AssetImportData.h"
#endif //WITH_EDITORONLY_DATA

#if WITH_EDITOR
#include "MaterialEditingLibrary.h"
#endif //WITH_EDITOR

#define LOCTEXT_NAMESPACE "InterchangeMaterialFactory"

namespace UE::Interchange::MaterialFactory::Internal
{
	void SetupSpecularProfile(const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments, const UInterchangeFactoryBaseNode* FactoryNode, UObject* MaterialInstanceOrSlabBSDFExpression);

#if WITH_EDITOR
	/**
	 * Finds a UMaterialExpression class by name.
	 * @param ClassName		The name of the class to look for (ie:UClass*->GetName()).
	 * @return				A sub class of UMaterialExpression or nullptr.
	 */
	TSubclassOf<UMaterialExpression> FindExpressionClass(const TCHAR* ClassName)
	{
		check(ClassName);

		UClass* MaterialExpressionClass = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::EnsureIfAmbiguous);

		if (!MaterialExpressionClass)
		{
			if (UObjectRedirector* RenamedClassRedirector = FindFirstObject<UObjectRedirector>(ClassName, EFindFirstObjectOptions::EnsureIfAmbiguous))
			{
				MaterialExpressionClass = CastChecked<UClass>(RenamedClassRedirector->DestinationObject);
			}
		}

		if (MaterialExpressionClass && MaterialExpressionClass->IsChildOf<UMaterialExpression>())
		{
			return MaterialExpressionClass;
		}
		else
		{
			return nullptr;
		}
	}

	int32 GetInputIndex(UMaterialExpression& MaterialExpression, const FString& InputName)
	{
		for (FExpressionInputIterator It{ &MaterialExpression }; It; ++It)
		{
			// MaterialFuncCall appends the type to the input name when calling GetInputName
			// and the InputName in FExpressionInput is optional so we'll check both here to be safe
			// Sometimes we have to retrieve the input name as is because some Expressions override it based on different factors and we have no mechanism to keep tracks of it
			if (MaterialExpression.GetInputName(It.Index) == *InputName ||
				It->InputName == *InputName ||
				MaterialExpression.UMaterialExpression::GetInputName(It.Index) == *InputName)
			{
				return It.Index;
			}
		}

		return INDEX_NONE;
	}

	int32 GetOutputIndex(UMaterialExpression& MaterialExpression, const FString& OutputName)
	{
		// Check whether OutputName stores an index
		int32 ExpressionOutputIndex = UInterchangeShaderPortsAPI::GetOutputIndexFromName(OutputName);
		if (ExpressionOutputIndex != INDEX_NONE)
		{
			if (MaterialExpression.GetOutputs().IsValidIndex(ExpressionOutputIndex))
			{
				return ExpressionOutputIndex;
			}
		}

		ExpressionOutputIndex = 0;

		for (const FExpressionOutput& ExpressionOutput : MaterialExpression.GetOutputs())
		{
			if (ExpressionOutput.OutputName == *OutputName)
			{
				return ExpressionOutputIndex;
			}

			++ExpressionOutputIndex;
		}

		return 0; // Consider 0 as the default output to connect to since most expressions have a single output
	}

	void SetupFunctionCallExpression(const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments, const UInterchangeMaterialExpressionFactoryNode& ExpressionNode, UMaterialExpressionMaterialFunctionCall* FunctionCallExpression)
	{
		FInterchangeImportMaterialAsyncHelper& AsyncHelper = FInterchangeImportMaterialAsyncHelper::GetInstance();
		if (const UInterchangeMaterialFunctionCallExpressionFactoryNode* FunctionCallFactoryNode = Cast<UInterchangeMaterialFunctionCallExpressionFactoryNode>(&ExpressionNode))
		{
			FString MaterialFactoryNodeUid;
			FunctionCallFactoryNode->GetCustomMaterialFunctionDependency(MaterialFactoryNodeUid);

			if (const UInterchangeMaterialFunctionFactoryNode* MaterialFactoryNode = Cast<UInterchangeMaterialFunctionFactoryNode>(Arguments.NodeContainer->GetNode(MaterialFactoryNodeUid)))
			{
				FSoftObjectPath ReferenceObject;
				MaterialFactoryNode->GetCustomReferenceObject(ReferenceObject);
				if (UMaterialFunctionInterface* MaterialFunction = Cast<UMaterialFunctionInterface>(ReferenceObject.TryLoad()))
				{
					AsyncHelper.UpdateFromFunctionResource(MaterialFunction);
					FunctionCallExpression->SetMaterialFunction(MaterialFunction);
				}
			}
		}

		AsyncHelper.UpdateFromFunctionResource(FunctionCallExpression);
	}

	void SetupTextureExpression(const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments, const UInterchangeMaterialExpressionFactoryNode* ExpressionNode, UMaterialExpressionTextureBase* TextureExpression)
	{
		using namespace UE::Interchange::Materials::Standard::Nodes::TextureSample;

		const bool bIsAParameter = UInterchangeShaderPortsAPI::HasParameter(ExpressionNode, Inputs::Texture);
		const FString InputKey = bIsAParameter
			? UInterchangeShaderPortsAPI::MakeInputParameterKey(Inputs::Texture.ToString())
			: UInterchangeShaderPortsAPI::MakeInputValueKey(Inputs::Texture.ToString());

		FString TextureFactoryNodeUid;
		// It is possible that the Texture Object Parameter was used that uses KeyValueProperty to reference the texture instead of extra texture input connection.
		if (!ExpressionNode->GetStringAttribute(InputKey, TextureFactoryNodeUid))
		{
			FString PayloadKey;
			UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute_FString(ExpressionNode, Inputs::Texture.ToString(), TextureFactoryNodeUid, PayloadKey);
		}

		if (const UInterchangeTextureFactoryNode* TextureFactoryNode = Cast<UInterchangeTextureFactoryNode>(Arguments.NodeContainer->GetNode(TextureFactoryNodeUid)))
		{
			FSoftObjectPath ReferenceObject;
			TextureFactoryNode->GetCustomReferenceObject(ReferenceObject);
			if (UTexture* Texture = Cast<UTexture>(ReferenceObject.TryLoad()))
			{
				TextureExpression->Texture = Texture;
			}
		}

		TextureExpression->AutoSetSampleType();
	}

	UMaterialExpression* CreateMaterialExpression(UMaterial* Material, UMaterialFunction* MaterialFunction, const TSubclassOf<UMaterialExpression>& ExpressionClass)
	{
		UObject* const SelectedAsset = nullptr;
		const int32 NodePosX = 0;
		const int32 NodePosY = 0;
		const bool bAllowMarkingPackageDirty = false;

		return UMaterialEditingLibrary::CreateMaterialExpressionEx(Material, MaterialFunction, ExpressionClass, SelectedAsset, NodePosX, NodePosY, bAllowMarkingPackageDirty);
	}

	template<class T>
	T* CreateMaterialExpression(UMaterial* Material, UMaterialFunction* MaterialFunction)
	{
		return Cast<T>(CreateMaterialExpression(Material, MaterialFunction, T::StaticClass()));
	}


	class FMaterialExpressionBuilder
	{
	public:
		FMaterialExpressionBuilder(UMaterial* InMaterial, UMaterialFunction* InMaterialFunction, const UInterchangeFactoryBase::FImportAssetObjectParams& InArguments, TObjectPtr<UInterchangeResultsContainer> InResultsContainer)
			: Material(InMaterial)
			, MaterialFunction(InMaterialFunction)
			, Arguments(InArguments)
			, ResultsContainer(InResultsContainer)
		{
			check((Material || MaterialFunction) && !(Material && MaterialFunction));

			//We need to put in place a better mechanism for the reimport, for the moment let's delete all material expressions to avoid duplicates
			DeleteAllMaterialExpressions();
		}
		UMaterialExpression* CreateExpressionsForNode(const UInterchangeMaterialExpressionFactoryNode& ExpressionNode)
		{
			UMaterialExpression* MaterialExpression = Expressions.FindRef(ExpressionNode.GetUniqueID());

			if (MaterialExpression)
			{
				return MaterialExpression;
			}

			MaterialExpression = CreateExpression(ExpressionNode);
			if (!MaterialExpression)
			{
				return nullptr;
			}

			// Special Case for MaterialExpressionSwitch, since the inputs are in an array
			if (MaterialExpression->IsA<UMaterialExpressionSwitch>())
			{
				using namespace UE::Interchange::Materials::Standard::Nodes;
				if (int32 InputCount; ExpressionNode.GetInt32Attribute(Switch::Attributes::InputCount.ToString(), InputCount))
				{
					UMaterialExpressionSwitch* MaterialExpressionSwitch = Cast<UMaterialExpressionSwitch>(MaterialExpression);
					MaterialExpressionSwitch->Inputs.Empty();
					MaterialExpressionSwitch->Inputs.Reserve(InputCount);
					for (int32 Index = 0; Index < InputCount; ++Index)
					{
						if (FString InputName; ExpressionNode.GetStringAttribute(Switch::Inputs::InputName.ToString() + FString::FromInt(Index), InputName))
						{
							MaterialExpressionSwitch->Inputs.Emplace(FSwitchCustomInput{ FName{InputName} });
						}
					}
				}
			}

			// We store the path of the Material Expression, in order for Pipelines to retrieve them later in the ExecutePostFactoryImport
			using namespace UE::Interchange::Materials;
			const_cast<UInterchangeMaterialExpressionFactoryNode*>(&ExpressionNode)->AddStringAttribute(Factory::Expression::Path.ToString(), MaterialExpression->GetPathName());

			Expressions.Add(ExpressionNode.GetUniqueID()) = MaterialExpression;

			TArray<FString> Inputs;
			UInterchangeShaderPortsAPI::GatherInputs(&ExpressionNode, Inputs);

			for (const FString& InputName : Inputs)
			{
				FString ConnectedExpressionUid;
				FString OutputName;
				if (UInterchangeShaderPortsAPI::GetInputConnection(&ExpressionNode, InputName, ConnectedExpressionUid, OutputName))
				{
					const UInterchangeMaterialExpressionFactoryNode* ConnectedExpressionNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(ConnectedExpressionUid));
					if (ConnectedExpressionNode)
					{
						UMaterialExpression* ConnectedExpression = Expressions.FindRef(ConnectedExpressionUid);
						if (!ConnectedExpression)
						{
							ConnectedExpression = CreateExpressionsForNode(*ConnectedExpressionNode);
							// This case is particularly observed when there are broken function references and there are Empty Material Function Names.
							if (!ConnectedExpression)
							{
								continue;
							}
						}

						const int32 InputIndex = GetInputIndex(*MaterialExpression, InputName);
						const int32 OutputIndex = GetOutputIndex(*ConnectedExpression, OutputName);

						if (InputIndex != INDEX_NONE)
						{
							FExpressionInput* ExpressionInput = MaterialExpression->GetInput(InputIndex);
							if (ExpressionInput)
							{
								ConnectedExpression->ConnectExpression(ExpressionInput, OutputIndex);
							}
						}
						else
						{
							PostMessage<UInterchangeResultError_Generic>(FText::Format(
								LOCTEXT("InputNotFound", "Invalid input {0} for material expression node {1}."),
								FText::FromString(InputName),
								FText::FromString(ExpressionNode.GetDisplayLabel())
							));
						}
					}
				}
			}

			return MaterialExpression;
		}

	private:
		UMaterialExpression* CreateExpression(const UInterchangeMaterialExpressionFactoryNode& ExpressionNode)
		{
			FString ExpressionClassName;
			ExpressionNode.GetCustomExpressionClassName(ExpressionClassName);

			TSubclassOf<UMaterialExpression> ExpressionClass = FindExpressionClass(*ExpressionClassName);

			if (!ExpressionClass.Get())
			{
				PostMessage<UInterchangeResultError_Generic>(FText::Format(
					LOCTEXT("ExpressionClassNotFound", "Invalid class {0} for material expression node {1}."),
					FText::FromString(ExpressionClassName),
					FText::FromString(ExpressionNode.GetDisplayLabel())
				));
				return nullptr;
			}

			UMaterialExpression* MaterialExpression = CreateMaterialExpression(Material, MaterialFunction, ExpressionClass);

			if (!MaterialExpression)
			{
				PostMessage<UInterchangeResultError_Generic>(FText::Format(
					LOCTEXT("MaterialExpressionCreationFailed", "Failed to create {0} object for material expression node {1}."),
					FText::FromString(ExpressionClassName),
					FText::FromString(ExpressionNode.GetDisplayLabel())
				));
				return nullptr;
			}

			if (Material)
			{
				if (FNameProperty* Property = FindFProperty<FNameProperty>(MaterialExpression->GetClass(), GET_MEMBER_NAME_CHECKED(UMaterialExpressionParameter, ParameterName)))
				{
					const FString MaterialExpressionName = ExpressionNode.GetDisplayLabel();
					FName ParameterName;
					if (MaterialExpressionName == TEXT("Null"))
					{
						const int32 RightChopIndex = FString(TEXT("MaterialExpression")).Len();
						FString ExpressionName = ExpressionClassName.RightChop(RightChopIndex);
						ParameterName = FName(*(ExpressionName + TEXT("_(") + LexToString(Material->GetExpressions().Num()) + TEXT(")")));
					}
					else
					{
						ParameterName = FName(*MaterialExpressionName);
					}
					// Set the parameter name if the material expression has one (some material expressions don't inherit from UMaterialExpressionParameter, ie: UMaterialExpressionTextureSampleParameter
					*(Property->ContainerPtrToValuePtr<FName>(MaterialExpression)) = ParameterName;
				}
			}

			ExpressionNode.ApplyAllCustomAttributeToObject(MaterialExpression);

			if (UMaterialExpressionTextureBase* TextureExpression = Cast<UMaterialExpressionTextureBase>(MaterialExpression))
			{
				SetupTextureExpression(Arguments, &ExpressionNode, TextureExpression);
			}
			else if (UMaterialExpressionMaterialFunctionCall* FunctionCallExpression = Cast<UMaterialExpressionMaterialFunctionCall>(MaterialExpression))
			{
				SetupFunctionCallExpression(Arguments, ExpressionNode, FunctionCallExpression);
			}
			else if (UMaterialExpressionSubstrateSlabBSDF* SlabBSDFExpression = Cast<UMaterialExpressionSubstrateSlabBSDF>(MaterialExpression))
			{
				SetupSpecularProfile(Arguments, &ExpressionNode, SlabBSDFExpression);
			}

			return MaterialExpression;
		}

		void DeleteAllMaterialExpressions()
		{
			if (Material)
			{
				// Copy the Material Expressions in a TArray, otherwise working directly on the TArrayView messes up with the expressions
				// and doesn't make a full clean up, especially when we're not in the Game Thread
				if (TArray<UMaterialExpression*> MaterialExpressions(Material->GetExpressions()); !MaterialExpressions.IsEmpty())
				{
					Material->Modify();
					for (UMaterialExpression* MaterialExpression : MaterialExpressions)
					{
						MaterialExpression->Modify();
						Material->GetExpressionCollection().RemoveExpression(MaterialExpression);
						Material->RemoveExpressionParameter(MaterialExpression);
						// Make sure the deleted expression is caught by gc
						MaterialExpression->MarkAsGarbage();
					}

					Material->MarkPackageDirty();
				}
			}
		}

		template<class TInterchangeResultType>
		void PostMessage(FText&& MessageText)
		{
			if (ResultsContainer)
			{
				TInterchangeResultType* Result = ResultsContainer->Add<TInterchangeResultType>();
				Result->Text = MoveTemp(MessageText);
			}
		}

	private:
		UMaterial* Material = nullptr;
		UMaterialFunction* MaterialFunction = nullptr;
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments;
		TMap<FString, UMaterialExpression*> Expressions;
		TObjectPtr<UInterchangeResultsContainer> ResultsContainer;
	};
#endif // #if WITH_EDITOR

	/**
	 * Try loading the asset with treating the Path as a package path, and fallback to NodeContainer if it fails.
	 * @type AssetType the type to load
	 * @type AssetFactoryType asset factory to use to get the referenced object
	 * @param NodeContainer container that stores all the nodes
	 * @param PathString path or UID of a node
	 *
	 * @returns the loaded asset if there is any
	 */
	template<typename AssetType, typename AssetFactoryType>
	AssetType* TryLoadAsset(const UInterchangeBaseNodeContainer& NodeContainer, FString PathString)
	{
		AssetType* OutAsset = nullptr;
		if (FPackageName::IsValidObjectPath(PathString))
		{
			FSoftObjectPath AssetPath(PathString);
			OutAsset = Cast<AssetType>(AssetPath.TryLoad());
		}
		else if (const AssetFactoryType* FactoryNode = Cast<AssetFactoryType>(NodeContainer.GetNode(PathString)))
		{
			FSoftObjectPath ReferenceObject;
			FactoryNode->GetCustomReferenceObject(ReferenceObject);
			OutAsset = Cast<AssetType>(ReferenceObject.TryLoad());
		}

		return OutAsset;
	}


	void UpdateParameterBool(UMaterialInstance& MaterialInstance, const FString& InputName, const UInterchangeMaterialInstanceFactoryNode& FactoryNode, bool bIsAParameter)
	{
#if WITH_EDITORONLY_DATA
		const FName ParameterName = *InputName;
		bool bInstanceValue;
		FGuid Uid;
		if (MaterialInstance.GetStaticSwitchParameterValue(ParameterName, bInstanceValue, Uid))
		{
			bool bInputValue = false;
			const FString InputKey = bIsAParameter ? UInterchangeShaderPortsAPI::MakeInputParameterKey(InputName) : UInterchangeShaderPortsAPI::MakeInputValueKey(InputName);

			if (FactoryNode.GetBooleanAttribute(InputKey, bInputValue))
			{
				if (bInputValue != bInstanceValue)
				{
					if (UMaterialInstanceConstant* MaterialInstanceConstant = Cast<UMaterialInstanceConstant>(&MaterialInstance))
					{
						MaterialInstanceConstant->SetStaticSwitchParameterValueEditorOnly(ParameterName, bInputValue);
					}
				}
			}
		}
#endif // #if WITH_EDITORONLY_DATA
	}

	void UpdateParameterFloat(UMaterialInstance& MaterialInstance, const FString& InputName, const UInterchangeMaterialInstanceFactoryNode& FactoryNode, bool bIsAParameter)
	{
		const FName ParameterName = *InputName;
		float InstanceValue;

		if (MaterialInstance.GetScalarParameterValue(ParameterName, InstanceValue))
		{
			float InputValue = 0.f;
			const FString InputKey = bIsAParameter ? UInterchangeShaderPortsAPI::MakeInputParameterKey(InputName) : UInterchangeShaderPortsAPI::MakeInputValueKey(InputName);
			if (FactoryNode.GetFloatAttribute(InputKey, InputValue))
			{
				if (!FMath::IsNearlyEqual(InputValue, InstanceValue))
				{
#if WITH_EDITOR
					if (UMaterialInstanceConstant* MaterialInstanceConstant = Cast<UMaterialInstanceConstant>(&MaterialInstance))
					{
						MaterialInstanceConstant->SetScalarParameterValueEditorOnly(ParameterName, InputValue);
					}
					else
#endif // #if WITH_EDITOR
						if (UMaterialInstanceDynamic* MaterialInstanceDynamic = Cast<UMaterialInstanceDynamic>(&MaterialInstance))
						{
							MaterialInstanceDynamic->SetScalarParameterValue(ParameterName, InputValue);
						}
				}
			}
		}
	}

	void UpdateParameterLinearColor(UMaterialInstance& MaterialInstance, const FString& InputName, const UInterchangeMaterialInstanceFactoryNode& FactoryNode, bool bIsAParameter)
	{
		const FName ParameterName = *InputName;
		FLinearColor InstanceValue;

		if (MaterialInstance.GetVectorParameterValue(ParameterName, InstanceValue))
		{
			FLinearColor InputValue;
			const FString InputKey = bIsAParameter ? UInterchangeShaderPortsAPI::MakeInputParameterKey(InputName) : UInterchangeShaderPortsAPI::MakeInputValueKey(InputName);
			if (FactoryNode.GetLinearColorAttribute(InputKey, InputValue))
			{
				if (!InputValue.Equals(InstanceValue))
				{
#if WITH_EDITOR
					if (UMaterialInstanceConstant* MaterialInstanceConstant = Cast<UMaterialInstanceConstant>(&MaterialInstance))
					{
						MaterialInstanceConstant->SetVectorParameterValueEditorOnly(ParameterName, InputValue);
					}
					else
#endif // #if WITH_EDITOR
					if (UMaterialInstanceDynamic* MaterialInstanceDynamic = Cast<UMaterialInstanceDynamic>(&MaterialInstance))
					{
						MaterialInstanceDynamic->SetVectorParameterValue(ParameterName, InputValue);
					}
				}
			}
		}
	}

	void UpdateParameterTexture(UMaterialInstance& MaterialInstance, const FString& InputName, const UInterchangeMaterialInstanceFactoryNode& FactoryNode, const UInterchangeBaseNodeContainer& NodeContainer, bool bIsAParameter)
	{
		const FName ParameterName = *InputName;
		UTexture * TextureInstanceValue;
		USparseVolumeTexture* SparseInstanceValue;

		if (MaterialInstance.GetTextureParameterValue(ParameterName, TextureInstanceValue))
		{
			FString InputValue;
			const FString InputKey = bIsAParameter ? UInterchangeShaderPortsAPI::MakeInputParameterKey(InputName)
												   : UInterchangeShaderPortsAPI::MakeInputValueKey(InputName);
			if (FactoryNode.GetStringAttribute(InputKey, InputValue))
			{
				if (const UInterchangeTextureFactoryNode* TextureFactoryNode = Cast<UInterchangeTextureFactoryNode>(NodeContainer.GetFactoryNode(InputValue)))
				{
					FSoftObjectPath ReferenceObject;
					TextureFactoryNode->GetCustomReferenceObject(ReferenceObject);
					if (UTexture* InputTexture = Cast<UTexture>(ReferenceObject.TryLoad()))
					{
						if (InputTexture != TextureInstanceValue)
						{
#if WITH_EDITOR
							if (UMaterialInstanceConstant* MaterialInstanceConstant = Cast<UMaterialInstanceConstant>(&MaterialInstance))
							{
								MaterialInstanceConstant->SetTextureParameterValueEditorOnly(ParameterName, InputTexture);
							}
							else
#endif	  // #if WITH_EDITOR
							if (UMaterialInstanceDynamic* MaterialInstanceDynamic = Cast<UMaterialInstanceDynamic>(&MaterialInstance))
							{
								MaterialInstanceDynamic->SetTextureParameterValue(ParameterName, InputTexture);
							}
						}
					}
				}
			}
		}
		else if (MaterialInstance.GetSparseVolumeTextureParameterValue(ParameterName, SparseInstanceValue))
		{
			FString InputValue;
			const FString InputKey = bIsAParameter ? UInterchangeShaderPortsAPI::MakeInputParameterKey(InputName)
												   : UInterchangeShaderPortsAPI::MakeInputValueKey(InputName);
			if (FactoryNode.GetStringAttribute(InputKey, InputValue))
			{
				if (const UInterchangeSparseVolumeTextureFactoryNode* SparseTextureNode = Cast<UInterchangeSparseVolumeTextureFactoryNode>(NodeContainer.GetFactoryNode(InputValue)))
				{
					FSoftObjectPath ReferenceObject;
					SparseTextureNode->GetCustomReferenceObject(ReferenceObject);
					if (USparseVolumeTexture* InputTexture = Cast<USparseVolumeTexture>(ReferenceObject.TryLoad()))
					{
						if (InputTexture != SparseInstanceValue)
						{
#if WITH_EDITOR
							if (UMaterialInstanceConstant* MaterialInstanceConstant = Cast<UMaterialInstanceConstant>(&MaterialInstance))
							{
								MaterialInstanceConstant->SetSparseVolumeTextureParameterValueEditorOnly(ParameterName, InputTexture);
							}
							else
#endif	  // #if WITH_EDITOR
							if (UMaterialInstanceDynamic* MaterialInstanceDynamic = Cast<UMaterialInstanceDynamic>(&MaterialInstance))
							{
								MaterialInstanceDynamic->SetSparseVolumeTextureParameterValue(ParameterName, InputTexture);
							}
						}
					}
				}
			}
		}
	}

	void OverrideBoolParameter(const UInterchangeMaterialInstanceFactoryNode& FactoryNode, const FString& AttributeKey, UMaterialInstance& MaterialInstance, const FName& ParameterName)
	{
		bool AttributeValue;
		if (FactoryNode.GetBooleanAttribute(AttributeKey, AttributeValue))
		{
#if WITH_EDITOR
			if (UMaterialInstanceConstant* MaterialInstanceConstant = Cast<UMaterialInstanceConstant>(&MaterialInstance))
			{
				MaterialInstanceConstant->SetStaticSwitchParameterValueEditorOnly(ParameterName, AttributeValue);
			}
			else
#endif // #if WITH_EDITOR
			if (UMaterialInstanceDynamic* MaterialInstanceDynamic = Cast<UMaterialInstanceDynamic>(&MaterialInstance))
			{
				//TODO: Log Error
				ensure(false);
			}
		}
	}

	void OverrideScalarParameter(const UInterchangeMaterialInstanceFactoryNode& FactoryNode, const FString& AttributeKey, UMaterialInstance& MaterialInstance, const FName& ParameterName)
	{
		float AttributeValue;
		if (FactoryNode.GetFloatAttribute(AttributeKey, AttributeValue))
		{
#if WITH_EDITOR
			if (UMaterialInstanceConstant* MaterialInstanceConstant = Cast<UMaterialInstanceConstant>(&MaterialInstance))
			{
				MaterialInstanceConstant->SetScalarParameterValueEditorOnly(ParameterName, AttributeValue);
			}
			else
#endif // #if WITH_EDITOR
			if (UMaterialInstanceDynamic* MaterialInstanceDynamic = Cast<UMaterialInstanceDynamic>(&MaterialInstance))
			{
				MaterialInstanceDynamic->SetScalarParameterValue(ParameterName, AttributeValue);
			}
		}
	}

	void OverrideVectorParameter(const UInterchangeMaterialInstanceFactoryNode& FactoryNode, const FString& AttributeKey, UMaterialInstance& MaterialInstance, const FName& ParameterName)
	{
		FLinearColor AttributeValue;
		if (FactoryNode.GetLinearColorAttribute(AttributeKey, AttributeValue))
		{
#if WITH_EDITOR
			if (UMaterialInstanceConstant* MaterialInstanceConstant = Cast<UMaterialInstanceConstant>(&MaterialInstance))
			{
				MaterialInstanceConstant->SetVectorParameterValueEditorOnly(ParameterName, AttributeValue);
			}
			else
#endif // #if WITH_EDITOR
			if (UMaterialInstanceDynamic* MaterialInstanceDynamic = Cast<UMaterialInstanceDynamic>(&MaterialInstance))
			{
				MaterialInstanceDynamic->SetVectorParameterValue(ParameterName, AttributeValue);
			}
		}
	}

	void OverrideTextureParameter(const UInterchangeMaterialInstanceFactoryNode& FactoryNode, const FString& AttributeKey, const UInterchangeBaseNodeContainer& NodeContainer, UMaterialInstance& MaterialInstance, const FName& ParameterName)
	{
		// The String Attribute is required to be a TextureFactoryUid as opposed to a TextureUid to correctly override.
		FString TextureFactoryUid;
		if (FactoryNode.GetStringAttribute(AttributeKey, TextureFactoryUid))
		{
			if (const UInterchangeTextureFactoryNode* TextureFactoryNode = Cast<UInterchangeTextureFactoryNode>(NodeContainer.GetNode(TextureFactoryUid)))
			{
				FSoftObjectPath ReferenceObject;
				TextureFactoryNode->GetCustomReferenceObject(ReferenceObject);
				if (UTexture* InputTexture = Cast<UTexture>(ReferenceObject.TryLoad()))
				{
#if WITH_EDITOR
					if (UMaterialInstanceConstant* MaterialInstanceConstant = Cast<UMaterialInstanceConstant>(&MaterialInstance))
					{
						MaterialInstanceConstant->SetTextureParameterValueEditorOnly(ParameterName, InputTexture);
					}
					else
#endif // #if WITH_EDITOR
					if (UMaterialInstanceDynamic* MaterialInstanceDynamic = Cast<UMaterialInstanceDynamic>(&MaterialInstance))
					{
						MaterialInstanceDynamic->SetTextureParameterValue(ParameterName, InputTexture);
					}
				}
			}
		}
	}

	UTexture* GetVirtualTextureStreamingMatchedTexture(uint8 DesiredVirtualTextureStreaming, UTexture* UsedTexture)
	{
		if (DesiredVirtualTextureStreaming == UsedTexture->VirtualTextureStreaming)
		{
			return nullptr;
		}

		auto GetExistingPackage = [](const FString& PackageName)
			{
				//Try to find the package in memory
				UPackage* Pkg = FindPackage(nullptr, *PackageName);
				if (!Pkg)
				{
					//Try to load the package from disk
					Pkg = LoadPackage(nullptr, *PackageName, LOAD_NoWarn | LOAD_Quiet);
				}

				if (Pkg)
				{
					Pkg->FullyLoad();
				}

				return Pkg;
			};

		//Duplicate and Converts texture to what the DefaultTexture's setting requires (if needed)
		//in case No VirtualTextureStreaming  => make the Texture's duplicate into a nonVT (and the name should be suffixed with "_nonVT")
		//in case VirtualTextureStreaming     => make the Texture's duplicate into a VT (and the name should be suffixed with "_VT")

		FString Suffix = DesiredVirtualTextureStreaming ? TEXT("_VT") : TEXT("_nonVT");

		FString OriginalPackageName = UsedTexture->GetPackage()->GetName();
		FString ConvertedPackageName = OriginalPackageName + Suffix;
		FString ConvertedAssetName = FPaths::GetCleanFilename(ConvertedPackageName);

		UPackage* ToBeConvertedPackage = GetExistingPackage(ConvertedPackageName);
		UObject* ToBeConvertedObject = ToBeConvertedPackage ? StaticFindObject(nullptr, ToBeConvertedPackage, *ConvertedAssetName) : nullptr;

		{
			//object exists with the desired ConvertedName but its not a texture:
			uint32 Counter = 1;
			while (ToBeConvertedObject && !Cast<UTexture>(ToBeConvertedObject))
			{
				ConvertedPackageName += FString::FromInt(Counter);
				Counter++;

				ToBeConvertedPackage = GetExistingPackage(ConvertedPackageName);
				ToBeConvertedObject = ToBeConvertedPackage ? StaticFindObject(nullptr, ToBeConvertedPackage, *ConvertedAssetName) : nullptr;
			}
		}

		if (!ToBeConvertedObject)
		{
			UPackage* NewPackage = CreatePackage(*ConvertedPackageName);
			NewPackage->SetPackageFlags(PKG_NewlyCreated);

			ConvertedAssetName = FPaths::GetCleanFilename(ConvertedPackageName);
			ToBeConvertedObject = StaticDuplicateObject(UsedTexture, NewPackage, *ConvertedAssetName);
		}

		if (!ToBeConvertedObject)
		{
			ensure(false);
			return nullptr;
		}

		UTexture* ToBeConvertedTexture = Cast<UTexture>(ToBeConvertedObject);
		if (!ToBeConvertedTexture)
		{
			ensure(false);
			return nullptr;
		}

		if (ToBeConvertedTexture->VirtualTextureStreaming != DesiredVirtualTextureStreaming)
		{
#if WITH_EDITOR
			FPropertyChangedEvent PropertyChangeEvent(UTexture::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTexture, VirtualTextureStreaming)));
			ToBeConvertedTexture->Modify();
			ToBeConvertedTexture->VirtualTextureStreaming = DesiredVirtualTextureStreaming;
			ToBeConvertedTexture->PostEditChangeProperty(PropertyChangeEvent);
#else
			ToBeConvertedTexture->VirtualTextureStreaming = DesiredVirtualTextureStreaming;
#endif
		}

		return ToBeConvertedTexture;
	}

	void SetupSpecularProfile(const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments, const UInterchangeFactoryBaseNode* FactoryNode, UObject* MaterialInstanceOrSlabBSDFExpression)
	{
		using namespace UE::Interchange::Materials;

		if (FString SpecularProfileUid; FactoryNode->GetStringAttribute(SubstrateMaterial::SpecularProfile.ToString(), SpecularProfileUid))
		{
			FString SpecularProfileFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(SpecularProfileUid);
			if (const UInterchangeSpecularProfileFactoryNode* SpecularProfileFactoryNode = Cast<UInterchangeSpecularProfileFactoryNode>(Arguments.NodeContainer->GetNode(SpecularProfileFactoryNodeUid)))
			{
				if (FSoftObjectPath ReferenceObject; SpecularProfileFactoryNode->GetCustomReferenceObject(ReferenceObject))
				{
					if (USpecularProfile* SpecularProfile = Cast<USpecularProfile>(ReferenceObject.TryLoad()))
					{
						if (UMaterialExpressionSubstrateSlabBSDF* SlabBSDFExpression = Cast<UMaterialExpressionSubstrateSlabBSDF>(MaterialInstanceOrSlabBSDFExpression))
						{
							SlabBSDFExpression->SpecularProfile = SpecularProfile;
						}
						else if (UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MaterialInstanceOrSlabBSDFExpression))
						{
							MaterialInstance->SpecularProfileOverride = SpecularProfile;
							MaterialInstance->bOverrideSpecularProfile = true;
						}

					}
				}
			}
		}
	}

	void OverrideBlendMode(UMaterialInstance& MaterialInstance, const UInterchangeMaterialInstanceFactoryNode& FactoryNode)
	{
		if (TEnumAsByte<EBlendMode> BlendMode; FactoryNode.GetCustomBlendMode(BlendMode))
		{
			MaterialInstance.BasePropertyOverrides.bOverride_BlendMode = true;
			MaterialInstance.BasePropertyOverrides.BlendMode = BlendMode;
			MaterialInstance.BlendMode = BlendMode;
		}
		// If the factory node does not have a blend mode, disable blend mode override
		else
		{
			MaterialInstance.BasePropertyOverrides.bOverride_BlendMode = false;
		}
	}

	void OverrideTwoSided(UMaterialInstance& MaterialInstance, const UInterchangeMaterialInstanceFactoryNode& FactoryNode)
	{
		if (bool bTwoSided; FactoryNode.GetCustomTwoSided(bTwoSided))
		{
			MaterialInstance.BasePropertyOverrides.bOverride_TwoSided = true;
			MaterialInstance.BasePropertyOverrides.TwoSided = bTwoSided;
			MaterialInstance.TwoSided = bTwoSided;
		}
		else
		{
			MaterialInstance.BasePropertyOverrides.bOverride_TwoSided = false;
		}
	}

	void OverrideEnableTessellation(UMaterialInstance& MaterialInstance, const UInterchangeMaterialInstanceFactoryNode& FactoryNode)
	{
		if (bool bEnableTessellation; FactoryNode.GetCustomEnableTessellation(bEnableTessellation))
		{
			MaterialInstance.BasePropertyOverrides.bOverride_bEnableTessellation = true;
			MaterialInstance.BasePropertyOverrides.bEnableTessellation = bEnableTessellation;
			MaterialInstance.bEnableTessellation = bEnableTessellation;
		}
		else
		{
			MaterialInstance.BasePropertyOverrides.bOverride_bEnableTessellation = false;
		}
	}
}

namespace UE::Interchange::Materials::HashUtils
{

	const TCHAR* FInterchangeMaterialInstanceOverridesAPI::ExpressionNameAttributeKey = TEXT("MaterialExpressionNameOverride");
	const TCHAR* FInterchangeMaterialInstanceOverridesAPI::OverrideParameterPrefix = TEXT("LeafInput");
	const TCHAR* FInterchangeMaterialInstanceOverridesAPI::OverrideParameterSeparator = TEXT(":");
	const TCHAR* FInterchangeMaterialInstanceOverridesAPI::OverrideHashSeparator = TEXT("_");

	FString FInterchangeMaterialInstanceOverridesAPI::MakeOverrideParameterName(UE::Interchange::EAttributeTypes AttributeType, int32 Hash, bool Prefix /*= true*/)
	{
		TStringBuilder<128> StringBuilder;
		if (Prefix)
		{
			StringBuilder.Append(OverrideParameterPrefix);
			StringBuilder.Append(OverrideParameterSeparator);
		}
		StringBuilder.Append(AttributeTypeToString(AttributeType));
		StringBuilder.Append(OverrideHashSeparator);
		StringBuilder.Append(FString::FromInt(Hash));
		return StringBuilder.ToString();
	}

	FString FInterchangeMaterialInstanceOverridesAPI::MakeOverrideParameterName(const FString& DisplayLabel)
	{
		TStringBuilder<128> StringBuilder;
		StringBuilder.Append(OverrideParameterPrefix);
		StringBuilder.Append(OverrideParameterSeparator);
		StringBuilder.Append(DisplayLabel);
		return StringBuilder.ToString();
	}

	FString FInterchangeMaterialInstanceOverridesAPI::MakeExpressionNameString()
	{
		return ExpressionNameAttributeKey;
	}

	bool FInterchangeMaterialInstanceOverridesAPI::GetOverrideParameterName(const UE::Interchange::FAttributeKey& AttributeKey, FString& OverrideParameterName)
	{
		FString Prefix;
		if (AttributeKey.Key.Split(OverrideParameterSeparator, &Prefix, &OverrideParameterName))
		{
			return true;
		}
		return false;
	}

	bool FInterchangeMaterialInstanceOverridesAPI::HasMaterialExpressionNameOverride(const UInterchangeBaseNode* BaseNode)
	{
		return BaseNode->HasAttribute(UE::Interchange::FAttributeKey(MakeExpressionNameString()));
	}

	void FInterchangeMaterialInstanceOverridesAPI::GatherLeafInputs(const UInterchangeBaseNode* BaseNode, TArray<UE::Interchange::FAttributeKey>& OutLeafInputAttributeKeys)
	{
		TArray<UE::Interchange::FAttributeKey> AttributeKeys;
		BaseNode->GetAttributeKeys(AttributeKeys);

		OutLeafInputAttributeKeys.Empty();
		for (const UE::Interchange::FAttributeKey& AttributeKey : AttributeKeys)
		{
			if (AttributeKey.Key.StartsWith(OverrideParameterPrefix))
			{
				OutLeafInputAttributeKeys.Add(AttributeKey);
			}
		}
	}
}


UClass* UInterchangeMaterialFactory::GetFactoryClass() const
{
	return UMaterialInterface::StaticClass();
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeMaterialFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeMaterialFactory::BeginImportAsset_GameThread);
	FImportAssetResult ImportAssetResult;
	UObject* Material = nullptr;

	auto CouldNotCreateMaterialLog = [this, &Arguments, &ImportAssetResult](const FText& Info)
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName = Arguments.SourceData->GetFilename();
		Message->DestinationAssetName = Arguments.AssetName;
		Message->AssetType = GetFactoryClass();
		Message->Text = FText::Format(LOCTEXT("MatFactory_CouldNotCreateMat", "Failed to handle Material {0}. Reason: {1}"), FText::FromString(Arguments.AssetName), Info);
		bSkipImport = true;
		ImportAssetResult.bIsFactorySkipAsset = true;
	};

	const FText MissMatchClassText = LOCTEXT("MatFactory_CouldNotCreateMat_MissMatchClass", "Mismatch between Interchange material factory node class and factory class.");

	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		CouldNotCreateMaterialLog(MissMatchClassText);
		return ImportAssetResult;
	}

	const UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode = Cast<UInterchangeBaseMaterialFactoryNode>(Arguments.AssetNode);
	if (MaterialFactoryNode == nullptr)
	{
		CouldNotCreateMaterialLog(LOCTEXT("MatFactory_CouldNotCreateMat_CannotCastFactoryNode", "Cannot cast Interchange factory node to UInterchangeBaseMaterialFactoryNode."));
		return ImportAssetResult;
	}

	const UClass* MaterialClass = MaterialFactoryNode->GetObjectClass();
	if (!ensure(MaterialClass && MaterialClass->IsChildOf(GetFactoryClass())))
	{
		CouldNotCreateMaterialLog(MissMatchClassText);
		return ImportAssetResult;
	}

	const bool bIsReimport = Arguments.ReimportObject != nullptr;

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (MaterialFactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}

		if (!ExistingAsset && MaterialFactoryNode->IsA<UInterchangeMaterialReferenceFactoryNode>())
		{
			CouldNotCreateMaterialLog(FText::Format(
				LOCTEXT("MatFactory_CouldNotCreateMat_CannotFindReferencedMaterial", "Could not find referenced material '{0}'"),
				FText::FromString(ReferenceObject.ToString())
			));
			return ImportAssetResult;
		}
	}

	auto GetParentMaterial = [&Arguments](const UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode) -> UMaterialInterface*
	{
		using namespace UE::Interchange::MaterialFactory::Internal;
		UMaterialInterface* ParentMaterialAsset = nullptr;

		FString PathOrUid;
		if (!MaterialInstanceFactoryNode->GetCustomParent(PathOrUid))
		{
			return nullptr;
		}

		return TryLoadAsset<UMaterialInterface, UInterchangeFactoryBaseNode>(*Arguments.NodeContainer, PathOrUid);
	};

	bool bCanImportMaterial = true;
	MaterialFactoryNode->GetCustomIsMaterialImportEnabled(bCanImportMaterial);
	// create a new material or overwrite existing asset, if possible
	if (!ExistingAsset && bCanImportMaterial)
	{
		if (MaterialClass->IsChildOf<UMaterialInstanceDynamic>())
		{
			if (const UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode = Cast<UInterchangeMaterialInstanceFactoryNode>(MaterialFactoryNode))
			{
				if(UMaterialInterface* ParentMaterial = GetParentMaterial(MaterialInstanceFactoryNode))				
				{
					Material = UMaterialInstanceDynamic::Create(ParentMaterial, Arguments.Parent, *Arguments.AssetName);
				}
			}
		}
		else
		{
			Material = NewObject<UObject>(Arguments.Parent, MaterialClass, *Arguments.AssetName, RF_Public | RF_Standalone);
		}
	}
	else if (ExistingAsset && ExistingAsset->GetClass()->IsChildOf(MaterialClass))
	{
		//This is a reimport, we are just re-updating the source data
		Material = ExistingAsset;
		//We allow override of existing materials only if the translator is a pure material translator or the user directly ask to re-import this object
		if (!bIsReimport && Arguments.Translator->GetSupportedAssetTypes() != EInterchangeTranslatorAssetType::Materials)
		{
			//Do not override the material asset
			ImportAssetResult.bIsFactorySkipAsset = true;
			bSkipImport = true;
		}
	}

	if (!Material)
	{
		if (bCanImportMaterial)
		{
			CouldNotCreateMaterialLog(LOCTEXT("MatFactory_CouldNotCreateMat_MaterialCreationFail", "Material creation failed"));
		}
		return ImportAssetResult;
	}

#if WITH_EDITOR
	if (!bSkipImport)
	{
		Material->PreEditChange(nullptr);
	}
#endif //WITH_EDITOR

	if (bSkipImport)
	{
		// Don't assign to ImportAssetResult.ImportedObject here or else Interchange will dirty the object
		return ImportAssetResult;
	}

	// Setup material instance based on reimport policy if applicable
	if (const UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode = Cast<UInterchangeMaterialInstanceFactoryNode>(MaterialFactoryNode))
	{
		if (UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Material))
		{
			EReimportStrategyFlags ReimportStrategyFlags = MaterialFactoryNode->GetReimportStrategyFlags();
			bool bApplyPipelineProperties = !Arguments.ReimportObject || ReimportStrategyFlags != EReimportStrategyFlags::ApplyNoProperties;

			UE::Interchange::MaterialFactory::Internal::SetupSpecularProfile(Arguments, MaterialFactoryNode, MaterialInstance);

#if WITH_EDITORONLY_DATA
			// For the time being, reimport policies are only enforced on UMaterialInstanceConstant
			if (Arguments.ReimportObject && MaterialInstance->IsA<UMaterialInstanceConstant>())
			{
				if(UInterchangeAssetImportData* AssetImportData = Cast<UInterchangeAssetImportData>(MaterialInstance->AssetImportData))
				{
					UInterchangeMaterialInstanceFactoryNode* PreviousNode = Cast<UInterchangeMaterialInstanceFactoryNode>(AssetImportData->GetStoredFactoryNode(AssetImportData->NodeUniqueID));

					if(PreviousNode)
					{
						FString PreviousParentPath;
						FString ParentPath;
						if(PreviousNode->GetCustomParent(PreviousParentPath) && MaterialInstanceFactoryNode->GetCustomParent(ParentPath))
						{
							if(ParentPath == PreviousParentPath)
							{
								SetupReimportedMaterialInstance(*MaterialInstance, *Arguments.NodeContainer, *MaterialInstanceFactoryNode, *PreviousNode);
								FStaticParameterSet StaticParameters;
								MaterialInstance->GetStaticParameterValues(StaticParameters);
								MaterialInstance->UpdateStaticPermutation(StaticParameters);
								bApplyPipelineProperties = false;
							}
						}
					}
				}
			}
#endif

			if (bApplyPipelineProperties)
			{
#if WITH_EDITOR
				if (UMaterialInstanceConstant* MaterialInstanceConstant = Cast<UMaterialInstanceConstant>(MaterialInstance))
				{
					if (UMaterialInterface* ParentMaterialAsset = GetParentMaterial(MaterialInstanceFactoryNode))
					{
						MaterialInstanceConstant->SetParentEditorOnly(ParentMaterialAsset);
						MaterialInstanceConstant->PostEditChange();
					}
					else
					{
						UE_LOGF(LogInterchangeImport, Error, "No parent material was found.")
					}
				}
#endif
				SetupMaterialInstance(*MaterialInstance, *Arguments.NodeContainer, *MaterialInstanceFactoryNode, !Arguments.ReimportObject);
			}
		}
	}
	else if (const UInterchangeDecalMaterialFactoryNode* DecalMaterialFactoryNode = Cast<UInterchangeDecalMaterialFactoryNode>(MaterialFactoryNode))
	{
#if WITH_EDITOR
		const FSoftObjectPath DecalMaterialParent(TEXT("/InterchangeAssets/Materials/DecalMaterial.DecalMaterial"));
		const FName DiffuseTextureParameterName(TEXT("DecalTexture"));
		const FName NormalTextureParameterName(TEXT("NormalTexture"));
		if (UMaterialInstanceConstant* MaterialInstanceConstant = Cast<UMaterialInstanceConstant>(Material))
		{
			if (UMaterial* ParentMaterial = Cast<UMaterial>(DecalMaterialParent.TryLoad()))
			{
				using namespace UE::Interchange::MaterialFactory::Internal;
				MaterialInstanceConstant->SetParentEditorOnly(ParentMaterial);

				FString DiffuseTexturePath;
				if (DecalMaterialFactoryNode->GetCustomDiffuseTexturePath(DiffuseTexturePath))
				{
					if (UTexture* InputTexture = TryLoadAsset<UTexture, UInterchangeTextureFactoryNode>(*Arguments.NodeContainer, DiffuseTexturePath))
					{
						MaterialInstanceConstant->SetTextureParameterValueEditorOnly(DiffuseTextureParameterName, InputTexture);
					}
				}

				FString NormalTexturePath;
				if (DecalMaterialFactoryNode->GetCustomNormalTexturePath(NormalTexturePath))
				{
					if (UTexture* InputTexture = TryLoadAsset<UTexture, UInterchangeTextureFactoryNode>(*Arguments.NodeContainer, NormalTexturePath))
					{
						MaterialInstanceConstant->SetTextureParameterValueEditorOnly(NormalTextureParameterName, InputTexture);
					}
				}
			}
			else
			{
				UE_LOGF(LogInterchangeImport, Error, "Invalid Decal Material Parent Path. Can't create a Decal Material Instance.");
			}
		}
#endif // #if WITH_EDITOR
	}

	ImportAssetResult.ImportedObject = Material;
	return ImportAssetResult;
}


UInterchangeFactoryBase::FImportAssetResult UInterchangeMaterialFactory::ImportAsset_Async(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeMaterialFactory::ImportAsset_Async);
	FImportAssetResult ImportAssetResult;
	ImportAssetResult.bIsFactorySkipAsset = bSkipImport;

	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode = Cast<UInterchangeBaseMaterialFactoryNode>(Arguments.AssetNode);
	if (MaterialFactoryNode == nullptr)
	{
		return ImportAssetResult;
	}

	UObject* MaterialObject = UE::Interchange::FFactoryCommon::AsyncFindObject(MaterialFactoryNode, GetFactoryClass(), Arguments.Parent, Arguments.AssetName);

	//Do not override an asset we skip
	if (bSkipImport)
	{
		ImportAssetResult.ImportedObject = MaterialObject;
		return ImportAssetResult;
	}
	const bool bReimport = Arguments.ReimportObject && MaterialObject;

	if (!MaterialObject)
	{
		UE_LOGF(LogInterchangeImport, Error, "Could not import the Material asset %ls because the asset does not exist.", *Arguments.AssetName);
		return ImportAssetResult;
	}

	UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(MaterialObject);
	if (!ensure(MaterialInterface))
	{
		UE_LOGF(LogInterchangeImport, Error, "Could not cast to Material asset %ls.", *Arguments.AssetName);
		return ImportAssetResult;
	}

	// Currently re-import of UMaterial will not touch the material at all
	//TODO: Design a re-import process for the material (expressions and input connections)
	if(!Arguments.ReimportObject)
	{
		if (MaterialInterface)
		{
#if WITH_EDITOR
			if (UMaterial* Material = Cast<UMaterial>(MaterialObject))
			{
				SetupMaterial(Material, Arguments, MaterialFactoryNode);
			}
#endif // #if WITH_EDITOR

			MaterialFactoryNode->ApplyAllCustomAttributeToObject(MaterialObject);
		}
	}
#if WITH_EDITORONLY_DATA
	else if(UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MaterialObject))
	{
		//Apply the re import strategy
		UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(MaterialInstance->AssetImportData);
		UInterchangeFactoryBaseNode* PreviousNode = nullptr;
		if(InterchangeAssetImportData)
		{
			PreviousNode = InterchangeAssetImportData->GetStoredFactoryNode(InterchangeAssetImportData->NodeUniqueID);
		}
		UInterchangeFactoryBaseNode* CurrentNode = NewObject<UInterchangeMaterialFactoryNode>(GetTransientPackage());
		UInterchangeBaseNode::CopyStorage(MaterialFactoryNode, CurrentNode);
		CurrentNode->FillAllCustomAttributeFromObject(MaterialInstance);
		UE::Interchange::FFactoryCommon::ApplyReimportStrategyToAsset(MaterialInstance, PreviousNode, CurrentNode, MaterialFactoryNode);
	}
#endif // #if WITH_EDITORONLY_DATA

	//Getting the file Hash will cache it into the source data
	Arguments.SourceData->GetFileContentHash();

	return ImportAssetResult;
}

/* This function is call in the completion task on the main thread, use it to call main thread post creation step for your assets*/
void UInterchangeMaterialFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeMaterialFactory::SetupObject_GameThread);
	check(IsInGameThread());
	Super::SetupObject_GameThread(Arguments);

	if (bSkipImport)
	{
		return;
	}

	if (ensure(Arguments.ImportedObject && Arguments.SourceData))
	{
		//We must call the Update of the asset source file in the main thread because UAssetImportData::Update execute some delegate we do not control
		UMaterialInterface* ImportedMaterialInterface = CastChecked<UMaterialInterface>(Arguments.ImportedObject);

#if WITH_EDITOR
		//Update the samplers type in case the textures were changed during their SetupObject_GameThread
		if (UMaterial* ImportedMaterial = Cast<UMaterial>(ImportedMaterialInterface))
		{
			TMap<UTexture*, UTexture*> OriginalToConvertedTextureMaps; //for opacity when bEnableVirtualTextureOpacityMask is false
			if (!GetDefault<URendererSettings>()->bEnableVirtualTextureOpacityMask)
			{
				//Virtual textures are not supported in the OpacityMask slot, convert any textures back to a regular texture.
				TArray<UTexture*> OutOpacityMaskTextures;
				if (ImportedMaterialInterface->GetTexturesInPropertyChain(MP_OpacityMask, OutOpacityMaskTextures, nullptr, nullptr))
				{
					for (UTexture* CurrentTexture : OutOpacityMaskTextures)
					{
						if (UTexture* ConvertedTexture = UE::Interchange::MaterialFactory::Internal::GetVirtualTextureStreamingMatchedTexture(0 /*false*/, CurrentTexture))
						{
							OriginalToConvertedTextureMaps.Emplace(CurrentTexture, ConvertedTexture);
						}
					}
				}
			}

			for (UMaterialExpression* Expression : ImportedMaterial->GetExpressions())
			{
				if (UMaterialExpressionTextureBase* TextureSample = Cast<UMaterialExpressionTextureBase>(Expression))
				{
					if (TextureSample->Texture && OriginalToConvertedTextureMaps.Contains(TextureSample->Texture))
					{
						TextureSample->Texture = OriginalToConvertedTextureMaps[TextureSample->Texture];
					}
					TextureSample->AutoSetSampleType();
				}
			}
		}
#endif // WITH_EDITOR

		if (UMaterialInstance* ImportedMaterial = Cast<UMaterialInstance>(ImportedMaterialInterface))
		{
			for (struct FTextureParameterValue& TextureParameterValue : ImportedMaterial->TextureParameterValues)
			{
				if (!TextureParameterValue.ParameterValue)
				{
					continue;
				}

				UTexture* DefaultTexture = nullptr;
				if (!ImportedMaterial->GetTextureParameterDefaultValue(TextureParameterValue.ParameterInfo.Name, DefaultTexture))
				{
					continue;
				}

				if (!DefaultTexture)
				{
					continue;
				}

				if (UTexture* ConvertedTexture = UE::Interchange::MaterialFactory::Internal::GetVirtualTextureStreamingMatchedTexture(DefaultTexture->VirtualTextureStreaming, TextureParameterValue.ParameterValue))
				{
					TextureParameterValue.ParameterValue = ConvertedTexture;
				}
			}
		}

#if WITH_EDITORONLY_DATA
		UE::Interchange::FFactoryCommon::FUpdateImportAssetDataParameters UpdateImportAssetDataParameters(ImportedMaterialInterface
																										  , ImportedMaterialInterface->AssetImportData
																										  , Arguments.SourceData
																										  , Arguments.NodeUniqueID
																										  , Arguments.NodeContainer
																										  , Arguments.OriginalPipelines
																										  , Arguments.Translator);

		ImportedMaterialInterface->AssetImportData = UE::Interchange::FFactoryCommon::UpdateImportAssetData(UpdateImportAssetDataParameters);
#endif // WITH_EDITORONLY_DATA
	}
}

bool UInterchangeMaterialFactory::GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const
{
#if WITH_EDITORONLY_DATA
	if (const UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(Object))
	{
		return UE::Interchange::FFactoryCommon::GetSourceFilenames(MaterialInterface->AssetImportData.Get(), OutSourceFilenames);
	}
#endif

	return false;
}

bool UInterchangeMaterialFactory::SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const
{
#if WITH_EDITORONLY_DATA
	if (const UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(Object))
	{
		return UE::Interchange::FFactoryCommon::SetSourceFilename(MaterialInterface->AssetImportData.Get(), SourceFilename, SourceIndex);
	}
#endif

	return false;
}

void UInterchangeMaterialFactory::BackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(Object))
	{
		UE::Interchange::FFactoryCommon::BackupSourceData(MaterialInterface->AssetImportData.Get());
	}
#endif
}

void UInterchangeMaterialFactory::ReinstateSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(Object))
	{
		UE::Interchange::FFactoryCommon::ReinstateSourceData(MaterialInterface->AssetImportData.Get());
	}
#endif
}
void UInterchangeMaterialFactory::ClearBackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(Object))
	{
		UE::Interchange::FFactoryCommon::ClearBackupSourceData(MaterialInterface->AssetImportData.Get());
	}
#endif
}

#if WITH_EDITOR
void UInterchangeMaterialFactory::SetupMaterial(UMaterial* Material, const FImportAssetObjectParams& Arguments, const UInterchangeBaseMaterialFactoryNode* BaseMaterialFactoryNode)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeMaterialFactory::SetupMaterial);

	using namespace UE::Interchange::Materials;
	using namespace UE::Interchange::MaterialFactory::Internal;

	const UInterchangeMaterialFactoryNode* MaterialFactoryNode = Cast<UInterchangeMaterialFactoryNode>(BaseMaterialFactoryNode);

	{//Screen Space Reflections:
		if (bool bScreenSpaceReflections; MaterialFactoryNode->GetCustomScreenSpaceReflections(bScreenSpaceReflections))
		{
			Material->bScreenSpaceReflections = bScreenSpaceReflections;
		}
	}

	FMaterialExpressionBuilder Builder(Material, nullptr, Arguments, Results.Get());

	// Substrate
	if(UInterchangeShaderPortsAPI::HasInput(MaterialFactoryNode, SubstrateMaterial::Parameters::FrontMaterial))
	{
		// Front Material
		{
			FString ExpressionNodeUid;
			FString OutputName;

			UInterchangeShaderPortsAPI::GetInputConnection(MaterialFactoryNode, SubstrateMaterial::Parameters::FrontMaterial.ToString(), ExpressionNodeUid, OutputName);

			const UInterchangeMaterialExpressionFactoryNode* FrontMaterial = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(ExpressionNodeUid));

			if(FrontMaterial)
			{
				if(UMaterialExpression* FrontMaterialExpression = Builder.CreateExpressionsForNode(*FrontMaterial))
				{
					if(FExpressionInput* FrontMaterialInput = Material->GetExpressionInputForProperty(MP_FrontMaterial))
					{
						FrontMaterialExpression->ConnectExpression(FrontMaterialInput, GetOutputIndex(*FrontMaterialExpression, OutputName));
					}
				}
			}
		}

		// Opacity Mask
		if(UInterchangeShaderPortsAPI::HasInput(MaterialFactoryNode, SubstrateMaterial::Parameters::OpacityMask))
		{
			FString ExpressionNodeUid;
			FString OutputName;

			UInterchangeShaderPortsAPI::GetInputConnection(MaterialFactoryNode, SubstrateMaterial::Parameters::OpacityMask.ToString(), ExpressionNodeUid, OutputName);

			const UInterchangeMaterialExpressionFactoryNode* OpacityMask = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(ExpressionNodeUid));

			if(OpacityMask)
			{
				if(UMaterialExpression* OpacityMaskExpression = Builder.CreateExpressionsForNode(*OpacityMask))
				{
					if(FExpressionInput* OpacityMaskInput = Material->GetExpressionInputForProperty(MP_OpacityMask))
					{
						OpacityMaskExpression->ConnectExpression(OpacityMaskInput, GetOutputIndex(*OpacityMaskExpression, OutputName));
					}
				}
			}
		}

		// Displacement
		if(UInterchangeShaderPortsAPI::HasInput(MaterialFactoryNode, SubstrateMaterial::Parameters::Displacement))
		{
			FString ExpressionNodeUid;
			FString OutputName;

			UInterchangeShaderPortsAPI::GetInputConnection(MaterialFactoryNode, SubstrateMaterial::Parameters::Displacement.ToString(), ExpressionNodeUid, OutputName);

			const UInterchangeMaterialExpressionFactoryNode* Displacement = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(ExpressionNodeUid));

			if(Displacement)
			{
				if(UMaterialExpression* DisplacementExpression = Builder.CreateExpressionsForNode(*Displacement))
				{
					if(FExpressionInput* DisplacementInput = Material->GetExpressionInputForProperty(MP_Displacement))
					{
						DisplacementExpression->ConnectExpression(DisplacementInput, GetOutputIndex(*DisplacementExpression, OutputName));
						Material->bEnableTessellation = true;

						if (float DisplacementCenter; MaterialFactoryNode->GetCustomDisplacementCenter(DisplacementCenter))
						{
							Material->DisplacementScaling.Center = DisplacementCenter;
						}

						if (float DisplacementMagnitude; MaterialFactoryNode->GetCustomDisplacementMagnitude(DisplacementMagnitude))
						{
							Material->DisplacementScaling.Magnitude = DisplacementMagnitude;
						}
					}
				}
			}
		}

		// Occlusion
		if(UInterchangeShaderPortsAPI::HasInput(MaterialFactoryNode, SubstrateMaterial::Parameters::Occlusion))
		{
			FString ExpressionNodeUid;
			FString OutputName;

			UInterchangeShaderPortsAPI::GetInputConnection(MaterialFactoryNode, SubstrateMaterial::Parameters::Occlusion.ToString(), ExpressionNodeUid, OutputName);

			const UInterchangeMaterialExpressionFactoryNode* Occlusion = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(ExpressionNodeUid));

			if(Occlusion)
			{
				if(UMaterialExpression* OcclusionExpression = Builder.CreateExpressionsForNode(*Occlusion))
				{
					if(FExpressionInput* OcclusionInput = Material->GetExpressionInputForProperty(MP_AmbientOcclusion))
					{
						OcclusionExpression->ConnectExpression(OcclusionInput, GetOutputIndex(*OcclusionExpression, OutputName));
					}
				}
			}
		}

		// Refraction
		if (UInterchangeShaderPortsAPI::HasInput(MaterialFactoryNode, Common::Parameters::Refraction))
		{
			FString ExpressionNodeUid;
			FString OutputName;

			UInterchangeShaderPortsAPI::GetInputConnection(MaterialFactoryNode, Common::Parameters::Refraction.ToString(), ExpressionNodeUid, OutputName);

			const UInterchangeMaterialExpressionFactoryNode* Refraction = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(ExpressionNodeUid));

			if (Refraction)
			{
				const UInterchangeMaterialExpressionFactoryNode* RefractionNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(ExpressionNodeUid));

				if (RefractionNode)
				{
					if (UMaterialExpression* RefractionExpression = Builder.CreateExpressionsForNode(*RefractionNode))
					{
						if (FExpressionInput* RefractionInput = Material->GetExpressionInputForProperty(MP_Refraction))
						{
							RefractionExpression->ConnectExpression(RefractionInput, GetOutputIndex(*RefractionExpression, OutputName));
						}
					}
				}
			}
		}

		UMaterialEditingLibrary::LayoutMaterialExpressions(Material);

		return;
	}

	if (UInterchangeShaderPortsAPI::HasInput(MaterialFactoryNode, Common::Parameters::BxDF))
	{
		FString ExpressionNodeUid;
		FString OutputName;

		UInterchangeShaderPortsAPI::GetInputConnection(MaterialFactoryNode, Common::Parameters::BxDF.ToString(), ExpressionNodeUid, OutputName);

		const UInterchangeMaterialExpressionFactoryNode* MaterialAttributes = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(ExpressionNodeUid));

		if (MaterialAttributes)
		{
			if (UMaterialExpression* MaterialAttributesExpression = Builder.CreateExpressionsForNode(*MaterialAttributes))
			{
				if (FExpressionInput* MaterialAttributesInput = Material->GetExpressionInputForProperty(MP_MaterialAttributes))
				{
					MaterialAttributesExpression->ConnectExpression(MaterialAttributesInput, GetOutputIndex(*MaterialAttributesExpression, OutputName));
					Material->bUseMaterialAttributes = true;
				}
			}
		}

		UMaterialEditingLibrary::LayoutMaterialExpressions(Material);

		return;
	}

	// Base Color
	{
		FString BaseColorUid;
		FString OutputName;

		if (MaterialFactoryNode->GetBaseColorConnection(BaseColorUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* BaseColor = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(BaseColorUid));

			if (BaseColor)
			{
				if (UMaterialExpression* BaseColorExpression = Builder.CreateExpressionsForNode(*BaseColor))
				{
					if (FExpressionInput* BaseColorInput = Material->GetExpressionInputForProperty(MP_BaseColor))
					{
						BaseColorExpression->ConnectExpression(BaseColorInput, GetOutputIndex(*BaseColorExpression, OutputName));
					}
				}
			}
		}
	}

	// Metallic
	{
		FString MetallicUid;
		FString OutputName;

		if (MaterialFactoryNode->GetMetallicConnection(MetallicUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* MetallicNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(MetallicUid));

			if (MetallicNode)
			{
				if (UMaterialExpression* MetallicExpression = Builder.CreateExpressionsForNode(*MetallicNode))
				{
					if (FExpressionInput* MetallicInput = Material->GetExpressionInputForProperty(MP_Metallic))
					{
						MetallicExpression->ConnectExpression(MetallicInput, GetOutputIndex(*MetallicExpression, OutputName));
					}
				}
			}
		}
	}

	// Specular
	{
		FString SpecularUid;
		FString OutputName;

		if (MaterialFactoryNode->GetSpecularConnection(SpecularUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* SpecularNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(SpecularUid));

			if (SpecularNode)
			{
				if (UMaterialExpression* SpecularExpression = Builder.CreateExpressionsForNode(*SpecularNode))
				{
					if (FExpressionInput* SpecularInput = Material->GetExpressionInputForProperty(MP_Specular))
					{
						SpecularExpression->ConnectExpression(SpecularInput, GetOutputIndex(*SpecularExpression, OutputName));
					}
				}
			}
		}
	}

	// Roughness
	{
		FString RoughnessUid;
		FString OutputName;

		if (MaterialFactoryNode->GetRoughnessConnection(RoughnessUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* Roughness = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(RoughnessUid));

			if (Roughness)
			{
				if (UMaterialExpression* RoughnessExpression = Builder.CreateExpressionsForNode(*Roughness))
				{
					if (FExpressionInput* RoughnessInput = Material->GetExpressionInputForProperty(MP_Roughness))
					{
						RoughnessExpression->ConnectExpression(RoughnessInput, GetOutputIndex(*RoughnessExpression, OutputName));
					}
				}
			}
		}
	}

	// Anisotropy
	{
		FString AnisotropyUid;
		FString OutputName;

		if (MaterialFactoryNode->GetAnisotropyConnection(AnisotropyUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* Anisotropy = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(AnisotropyUid));

			if (Anisotropy)
			{
				if (UMaterialExpression* AnisotropyExpression = Builder.CreateExpressionsForNode(*Anisotropy))
				{
					if (FExpressionInput* AnisotropyInput = Material->GetExpressionInputForProperty(MP_Anisotropy))
					{
						AnisotropyExpression->ConnectExpression(AnisotropyInput, GetOutputIndex(*AnisotropyExpression, OutputName));
					}
				}
			}
		}
	}

	// Emissive
	{
		FString EmissiveColorUid;
		FString OutputName;

		if (MaterialFactoryNode->GetEmissiveColorConnection(EmissiveColorUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* EmissiveColor = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(EmissiveColorUid));

			if (EmissiveColor)
			{
				if (UMaterialExpression* EmissiveExpression = Builder.CreateExpressionsForNode(*EmissiveColor))
				{
					if (FExpressionInput* EmissiveInput = Material->GetExpressionInputForProperty(MP_EmissiveColor))
					{
						EmissiveExpression->ConnectExpression(EmissiveInput, GetOutputIndex(*EmissiveExpression, OutputName));
					}
				}
			}
		}
	}

	// Normal
	{
		FString NormalUid;
		FString OutputName;

		if (MaterialFactoryNode->GetNormalConnection(NormalUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* NormalNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(NormalUid));

			if (NormalNode)
			{
				if (UMaterialExpression* NormalExpression = Builder.CreateExpressionsForNode(*NormalNode))
				{
					if (FExpressionInput* NormalInput = Material->GetExpressionInputForProperty(MP_Normal))
					{
						NormalExpression->ConnectExpression(NormalInput, GetOutputIndex(*NormalExpression, OutputName));
					}
				}
			}
		}
	}

	// Tangent
	{
		FString TangentUid;
		FString OutputName;

		if(MaterialFactoryNode->GetTangentConnection(TangentUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* TangentNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(TangentUid));

			if(TangentNode)
			{
				if(UMaterialExpression* TangentExpression = Builder.CreateExpressionsForNode(*TangentNode))
				{
					if(FExpressionInput* TangentInput = Material->GetExpressionInputForProperty(MP_Tangent))
					{
						TangentExpression->ConnectExpression(TangentInput, GetOutputIndex(*TangentExpression, OutputName));
					}
				}
			}
		}
	}

	// Subsurface
	{
		FString SubsurfaceColorUid;
		FString OutputName;

		if(MaterialFactoryNode->GetSubsurfaceConnection(SubsurfaceColorUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* SubsurfaceColorNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(SubsurfaceColorUid));

			if(SubsurfaceColorNode)
			{
				if(UMaterialExpression* SubsurfaceColorExpression = Builder.CreateExpressionsForNode(*SubsurfaceColorNode))
				{
					if(FExpressionInput* SubsurfaceColorInput = Material->GetExpressionInputForProperty(MP_SubsurfaceColor))
					{
						SubsurfaceColorExpression->ConnectExpression(SubsurfaceColorInput, GetOutputIndex(*SubsurfaceColorExpression, OutputName));
					}
				}
			}
		}
	}

	// Opacity
	{
		FString OpacityUid;
		FString OutputName;

		if (MaterialFactoryNode->GetOpacityConnection(OpacityUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* OpacityNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(OpacityUid));

			if (OpacityNode)
			{
				if (UMaterialExpression* OpacityExpression = Builder.CreateExpressionsForNode(*OpacityNode))
				{
					FExpressionInput* OpacityInput = Material->GetExpressionInputForProperty(MP_Opacity);

					TEnumAsByte<EBlendMode> BlendMode;
					if (MaterialFactoryNode->GetCustomBlendMode(BlendMode))
					{
						if (BlendMode == EBlendMode::BLEND_Masked)
						{
							OpacityInput = Material->GetExpressionInputForProperty(MP_OpacityMask);
						}
					}

					if (OpacityInput)
					{
						OpacityExpression->ConnectExpression(OpacityInput, GetOutputIndex(*OpacityExpression, OutputName));
					}
				}
			}
		}
	}

	// Occlusion
	{
		FString OcclusionUid;
		FString OutputName;

		if (MaterialFactoryNode->GetOcclusionConnection(OcclusionUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* OcclusionNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(OcclusionUid));

			if (OcclusionNode)
			{
				if (UMaterialExpression* OcclusionExpression = Builder.CreateExpressionsForNode(*OcclusionNode))
				{
					if (FExpressionInput* OcclusionInput = Material->GetExpressionInputForProperty(MP_AmbientOcclusion))
					{
						OcclusionExpression->ConnectExpression(OcclusionInput, GetOutputIndex(*OcclusionExpression, OutputName));
					}
				}
			}
		}
	}

	// Refraction
	{
		FString RefractionUid;
		FString OutputName;

		if (MaterialFactoryNode->GetRefractionConnection(RefractionUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* RefractionNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(RefractionUid));

			if (RefractionNode)
			{
				if (UMaterialExpression* RefractionExpression = Builder.CreateExpressionsForNode(*RefractionNode))
				{
					if (FExpressionInput* RefractionInput = Material->GetExpressionInputForProperty(MP_Refraction))
					{
						RefractionExpression->ConnectExpression(RefractionInput, GetOutputIndex(*RefractionExpression, OutputName));
					}
				}
			}
		}
	}

	// Clear Coat
	{
		FString ClearCoatUid;
		FString OutputName;

		if (MaterialFactoryNode->GetClearCoatConnection(ClearCoatUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* ClearCoatNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(ClearCoatUid));

			if (ClearCoatNode)
			{
				if (UMaterialExpression* ClearCoatExpression = Builder.CreateExpressionsForNode(*ClearCoatNode))
				{
					if (FExpressionInput* ClearCoatInput = Material->GetExpressionInputForProperty(MP_CustomData0))
					{
						ClearCoatExpression->ConnectExpression(ClearCoatInput, GetOutputIndex(*ClearCoatExpression, OutputName));
					}
				}
			}
		}
	}

	// Clear Coat Roughness
	{
		FString ClearCoatRoughnessUid;
		FString OutputName;

		if (MaterialFactoryNode->GetClearCoatRoughnessConnection(ClearCoatRoughnessUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* ClearCoatRoughnessNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(ClearCoatRoughnessUid));

			if (ClearCoatRoughnessNode)
			{
				if (UMaterialExpression* ClearCoatRoughnessExpression = Builder.CreateExpressionsForNode(*ClearCoatRoughnessNode))
				{
					if (FExpressionInput* ClearCoatRoughnessInput = Material->GetExpressionInputForProperty(MP_CustomData1))
					{
						ClearCoatRoughnessExpression->ConnectExpression(ClearCoatRoughnessInput, GetOutputIndex(*ClearCoatRoughnessExpression, OutputName));
					}
				}
			}
		}
	}

	// Clear Coat Normal
	{
		FString ClearCoatNormalUid;
		FString OutputName;

		if (MaterialFactoryNode->GetClearCoatNormalConnection(ClearCoatNormalUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* ClearCoatNormalNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(ClearCoatNormalUid));

			if (ClearCoatNormalNode)
			{
				if (UMaterialExpression* ClearCoatNormalExpression = Builder.CreateExpressionsForNode(*ClearCoatNormalNode))
				{
					UMaterialExpression* ClearCoatNormalCustomOutput = CreateMaterialExpression(Material, nullptr, UMaterialExpressionClearCoatNormalCustomOutput::StaticClass());

					ClearCoatNormalExpression->ConnectExpression(ClearCoatNormalCustomOutput->GetInput(0), GetOutputIndex(*ClearCoatNormalExpression, OutputName));
				}
			}
		}
	}

	// Thin Translucent
	{
		FString TransmissionColorUid, SurfaceCoverageUid;
		FString TransmissionColorOutputName, SurfaceCoverageOutputName;

		const bool bHasTransmissionColor = MaterialFactoryNode->GetTransmissionColorConnection(TransmissionColorUid, TransmissionColorOutputName);
		const bool bHasSurfaceCoverage = MaterialFactoryNode->GetSurfaceCoverageConnection(SurfaceCoverageUid, SurfaceCoverageOutputName);
		if(bHasTransmissionColor || bHasSurfaceCoverage)
		{
			const UInterchangeMaterialExpressionFactoryNode* TransmissionColorNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(TransmissionColorUid));
			const UInterchangeMaterialExpressionFactoryNode* SurfaceCoverageNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(SurfaceCoverageUid));

			UMaterialExpression* ThinTranslucentMaterialOutput = nullptr;

			if(TransmissionColorNode)
			{
				if(UMaterialExpression* TransmissionColorExpression = Builder.CreateExpressionsForNode(*TransmissionColorNode))
				{
					ThinTranslucentMaterialOutput = CreateMaterialExpression(Material, nullptr, UMaterialExpressionThinTranslucentMaterialOutput::StaticClass());

					TransmissionColorExpression->ConnectExpression(ThinTranslucentMaterialOutput->GetInput(0), GetOutputIndex(*TransmissionColorExpression, TransmissionColorOutputName));
				}
			}

			if(SurfaceCoverageNode)
			{
				if(UMaterialExpression* SurfaceCoverageExpression = Builder.CreateExpressionsForNode(*SurfaceCoverageNode))
				{
					if(!ThinTranslucentMaterialOutput)
					{
						ThinTranslucentMaterialOutput = CreateMaterialExpression(Material, nullptr, UMaterialExpressionThinTranslucentMaterialOutput::StaticClass());
					}

					SurfaceCoverageExpression->ConnectExpression(ThinTranslucentMaterialOutput->GetInput(1), GetOutputIndex(*SurfaceCoverageExpression, SurfaceCoverageOutputName));
				}
			}
		}
	}

	// Fuzz Color
	{
		FString FuzzColorUid;
		FString OutputName;

		if (MaterialFactoryNode->GetFuzzColorConnection(FuzzColorUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* FuzzColorNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(FuzzColorUid));

			if (FuzzColorNode)
			{
				if (UMaterialExpression* FuzzColorExpression = Builder.CreateExpressionsForNode(*FuzzColorNode))
				{
					if (FExpressionInput* FuzzColorInput = Material->GetExpressionInputForProperty(MP_SubsurfaceColor))
					{
						FuzzColorExpression->ConnectExpression(FuzzColorInput, GetOutputIndex(*FuzzColorExpression, OutputName));
					}
				}
			}
		}
	}

	// Cloth
	{
		FString ClothUid;
		FString OutputName;

		if (MaterialFactoryNode->GetClothConnection(ClothUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* ClothNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(ClothUid));

			if (ClothNode)
			{
				if (UMaterialExpression* ClothExpression = Builder.CreateExpressionsForNode(*ClothNode))
				{
					if (FExpressionInput* ClothInput = Material->GetExpressionInputForProperty(MP_CustomData0))
					{
						ClothExpression->ConnectExpression(ClothInput, GetOutputIndex(*ClothExpression, OutputName));
					}
				}
			}
		}
	}

	// Displacement
	{
		FString DisplacementUid;
		FString OutputName;

		if(MaterialFactoryNode->GetDisplacementConnection(DisplacementUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* DisplacementNode = Cast<UInterchangeMaterialExpressionFactoryNode>(Arguments.NodeContainer->GetNode(DisplacementUid));

			if(DisplacementNode)
			{
				if(UMaterialExpression* DisplacementExpression = Builder.CreateExpressionsForNode(*DisplacementNode))
				{
					if(FExpressionInput* DisplacementInput = Material->GetExpressionInputForProperty(MP_Displacement))
					{
						DisplacementExpression->ConnectExpression(DisplacementInput, GetOutputIndex(*DisplacementExpression, OutputName));
						Material->bEnableTessellation = true;

						if (float DisplacementCenter; MaterialFactoryNode->GetCustomDisplacementCenter(DisplacementCenter))
						{
							Material->DisplacementScaling.Center = DisplacementCenter;
						}

						if (float DisplacementMagnitude; MaterialFactoryNode->GetCustomDisplacementMagnitude(DisplacementMagnitude))
						{
							Material->DisplacementScaling.Magnitude = DisplacementMagnitude;
						}
					}
				}
			}
		}
	}

	UMaterialEditingLibrary::LayoutMaterialExpressions(Material);
}
#endif // #if WITH_EDITOR

void UInterchangeMaterialFactory::SetupMaterialInstance(UMaterialInstance& MaterialInstance, const UInterchangeBaseNodeContainer& NodeContainer, const UInterchangeMaterialInstanceFactoryNode& FactoryNode, bool bResetInstance)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeMaterialFactory::SetupMaterialInstance);
	using namespace UE::Interchange::MaterialFactory::Internal;

	if (bResetInstance)
	{
		// Clear all material instance's parameters before applying new ones
#if WITH_EDITOR
		if (UMaterialInstanceConstant* MaterialInstanceConstant = Cast<UMaterialInstanceConstant>(&MaterialInstance))
		{
			MaterialInstanceConstant->ClearParameterValuesEditorOnly();
		}
		else
#endif
		if (UMaterialInstanceDynamic* MaterialInstanceDynamic = Cast<UMaterialInstanceDynamic>(&MaterialInstance))
		{
			MaterialInstanceDynamic->ClearParameterValues();
		}
	}

	TArray<FString> Inputs;
	UInterchangeShaderPortsAPI::GatherInputs(&FactoryNode, Inputs);

	for (const FString& InputName : Inputs)
	{
		const FName ParameterName = *InputName;
		const bool bIsAParameter = UInterchangeShaderPortsAPI::HasParameter(&FactoryNode, ParameterName);

		switch (UInterchangeShaderPortsAPI::GetInputType(&FactoryNode, InputName, bIsAParameter))
		{
		case UE::Interchange::EAttributeTypes::Bool:
			UpdateParameterBool(MaterialInstance, InputName, FactoryNode, bIsAParameter);
			break;
		case UE::Interchange::EAttributeTypes::Float:
			UpdateParameterFloat(MaterialInstance, InputName, FactoryNode, bIsAParameter);
			break;
		case UE::Interchange::EAttributeTypes::LinearColor:
			UpdateParameterLinearColor(MaterialInstance, InputName, FactoryNode, bIsAParameter);
			break;
		case UE::Interchange::EAttributeTypes::String:
			UpdateParameterTexture(MaterialInstance, InputName, FactoryNode, NodeContainer, bIsAParameter);
			break;
		}
	}

	TArray<UE::Interchange::FAttributeKey> LeafInputAttributeKeys;
	UE::Interchange::Materials::HashUtils::FInterchangeMaterialInstanceOverridesAPI::GatherLeafInputs(&FactoryNode, LeafInputAttributeKeys);

	for (const UE::Interchange::FAttributeKey& AttributeKey : LeafInputAttributeKeys)
	{
		FString OverrideParameterName;
		if (UE::Interchange::Materials::HashUtils::FInterchangeMaterialInstanceOverridesAPI::GetOverrideParameterName(AttributeKey, OverrideParameterName))
		{
			const FName ParameterName(OverrideParameterName);
			switch (FactoryNode.GetAttributeType(AttributeKey))
			{
			case UE::Interchange::EAttributeTypes::Bool:
			{
				OverrideBoolParameter(FactoryNode, AttributeKey.Key, MaterialInstance, ParameterName);
			}
			break;
			case UE::Interchange::EAttributeTypes::Float:
			{
				OverrideScalarParameter(FactoryNode, AttributeKey.Key, MaterialInstance, ParameterName);
			}
			break;
			case UE::Interchange::EAttributeTypes::LinearColor:
			{
				OverrideVectorParameter(FactoryNode, AttributeKey.Key, MaterialInstance, ParameterName);
			}
			break;
			case UE::Interchange::EAttributeTypes::String:
			{
				OverrideTextureParameter(FactoryNode, AttributeKey.Key, NodeContainer, MaterialInstance, ParameterName);
			}
			break;
			}
		}
	}

	OverrideBlendMode(MaterialInstance, FactoryNode);
	OverrideTwoSided(MaterialInstance, FactoryNode);
	OverrideEnableTessellation(MaterialInstance, FactoryNode);
}

void UInterchangeMaterialFactory::SetupReimportedMaterialInstance(UMaterialInstance& MaterialInstance, const UInterchangeBaseNodeContainer& NodeContainer, const UInterchangeMaterialInstanceFactoryNode& FactoryNode, const UInterchangeMaterialInstanceFactoryNode& PreviousFactoryNode)
{
	using namespace UE::Interchange::MaterialFactory::Internal;

	const EReimportStrategyFlags ReimportStrategy = FactoryNode.GetReimportStrategyFlags();
	if (ReimportStrategy == EReimportStrategyFlags::ApplyNoProperties)
	{
		return;
	}

	TArray<FString> Inputs;
	UInterchangeShaderPortsAPI::GatherInputs(&FactoryNode, Inputs);

	for (const FString& InputName : Inputs)
	{
		bool bIsAParameter = UInterchangeShaderPortsAPI::HasParameter(&FactoryNode, *InputName);
		const FString AttributKey = bIsAParameter ? UInterchangeShaderPortsAPI::MakeInputParameterKey(InputName) : UInterchangeShaderPortsAPI::MakeInputValueKey(InputName);

		FGuid Uid;
		switch (UInterchangeShaderPortsAPI::GetInputType(&FactoryNode, InputName, bIsAParameter))
		{
#if WITH_EDITORONLY_DATA
		case UE::Interchange::EAttributeTypes::Bool:
			{
				bool bInstanceValue;
				if (MaterialInstance.GetStaticSwitchParameterValue(*InputName, bInstanceValue, Uid))
				{
					bool bPreviousInputValue = false;
					if (!PreviousFactoryNode.GetBooleanAttribute(AttributKey, bPreviousInputValue))
					{
						bPreviousInputValue = bInstanceValue;
					}

					if (ReimportStrategy == EReimportStrategyFlags::ApplyPipelineProperties || bInstanceValue == bPreviousInputValue)
					{
						UpdateParameterBool(MaterialInstance, InputName, FactoryNode, bIsAParameter);
					}
				}
			}
			break;
#endif
		case UE::Interchange::EAttributeTypes::Float:
			{
				bool bUpdateParameter = ReimportStrategy == EReimportStrategyFlags::ApplyPipelineProperties;
				float PreviousValue;
				if (PreviousFactoryNode.GetFloatAttribute(AttributKey, PreviousValue))
				{
					float CurrentValue;
					if (MaterialInstance.GetScalarParameterValue(*InputName, CurrentValue))
					{
						bUpdateParameter |= FMath::IsNearlyEqual(PreviousValue, CurrentValue);
					}
				}

				if(bUpdateParameter)
				{
					UpdateParameterFloat(MaterialInstance, InputName, FactoryNode, bIsAParameter);
				}
			}
			break;
		case UE::Interchange::EAttributeTypes::LinearColor:
			{
				bool bUpdateParameter = ReimportStrategy == EReimportStrategyFlags::ApplyPipelineProperties;
				FLinearColor PreviousValue;
				if (PreviousFactoryNode.GetLinearColorAttribute(AttributKey, PreviousValue))
				{
					FLinearColor CurrentValue;
					if (MaterialInstance.GetVectorParameterValue(*InputName, CurrentValue))
					{
						bUpdateParameter |= PreviousValue.Equals(CurrentValue);
					}
				}

				if (bUpdateParameter)
				{
					UpdateParameterLinearColor(MaterialInstance, InputName, FactoryNode, bIsAParameter);
				}
			}
			break;
		case UE::Interchange::EAttributeTypes::String:
			{
				bool bUpdateParameter = ReimportStrategy == EReimportStrategyFlags::ApplyPipelineProperties;
				FString PreviousValue;
				if (PreviousFactoryNode.GetStringAttribute(AttributKey, PreviousValue))
				{
					UTexture* CurrentValue;
					if (MaterialInstance.GetTextureParameterValue(*InputName, CurrentValue))
					{
						FSoftObjectPath PreviousObjectPath(PreviousValue);
						bUpdateParameter |= PreviousObjectPath == FSoftObjectPath(CurrentValue);
					}
				}

				if (bUpdateParameter)
				{
					UpdateParameterTexture(MaterialInstance, InputName, FactoryNode, NodeContainer, bIsAParameter);
				}
			}
			break;
		}
	}

	auto OverrideProperty = [&MaterialInstance, &FactoryNode, &PreviousFactoryNode, ReimportStrategy]
	(auto GetCustomProperty, auto OverrideValue, auto OverrideFlag, auto Override)
	{
		using T = std::remove_cv_t<std::remove_reference_t<decltype(OverrideValue)>>;

		bool bUpdateParameter = ReimportStrategy == EReimportStrategyFlags::ApplyPipelineProperties;
		// If previous factory node has a property (blend mode, twosided, etc.), check whether it is the same as the material instance's one
		if (T Property{}; (PreviousFactoryNode.*GetCustomProperty)(Property))
		{
			if (OverrideFlag)
			{
				bUpdateParameter |= (T(OverrideValue) == Property);
			}
		}
		// If there was no property override in previous import and things have not changed, update is allowed
		else if (!OverrideFlag)
		{
			bUpdateParameter |= true;
		}

		if (bUpdateParameter)
		{
			Override(MaterialInstance, FactoryNode);
		}
	};

	OverrideProperty(&UInterchangeMaterialInstanceFactoryNode::GetCustomBlendMode, MaterialInstance.BasePropertyOverrides.BlendMode, MaterialInstance.BasePropertyOverrides.bOverride_BlendMode, OverrideBlendMode);
	OverrideProperty(&UInterchangeMaterialInstanceFactoryNode::GetCustomTwoSided, bool(MaterialInstance.BasePropertyOverrides.TwoSided), MaterialInstance.BasePropertyOverrides.bOverride_TwoSided, OverrideTwoSided);
}

UClass* UInterchangeMaterialFunctionFactory::GetFactoryClass() const
{
	return UMaterialFunctionInterface::StaticClass();
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeMaterialFunctionFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeMaterialFunctionFactory::BeginImportAsset_GameThread);
	FImportAssetResult ImportAssetResult;
	UObject* Material = nullptr;

	auto CouldNotCreateMaterialLog = [this, &Arguments, &ImportAssetResult](const FText& Info)
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName = Arguments.SourceData->GetFilename();
		Message->DestinationAssetName = Arguments.AssetName;
		Message->AssetType = GetFactoryClass();
		Message->Text = FText::Format(LOCTEXT("MatFunc_CouldNotCreateMat", "Could not create Material asset {0}. Reason: {1}"), FText::FromString(Arguments.AssetName), Info);
		bSkipImport = true;
		ImportAssetResult.bIsFactorySkipAsset = true;
	};

	const FText MissMatchClassText = LOCTEXT("MatFunc_CouldNotCreateMat_MissMatchClass", "Mismatch between Interchange material factory node class and factory class.");

	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		CouldNotCreateMaterialLog(MissMatchClassText);
		return ImportAssetResult;
	}

	const UInterchangeMaterialFunctionFactoryNode* MaterialFactoryNode = Cast<UInterchangeMaterialFunctionFactoryNode>(Arguments.AssetNode);
	if (!ensure(MaterialFactoryNode))
	{
		CouldNotCreateMaterialLog(LOCTEXT("MatFunc_CouldNotCreateMat_CannotCastFactoryNode", "Cannot cast Interchange factory node to UInterchangeBaseMaterialFactoryNode."));
		return ImportAssetResult;
	}

	const UClass* MaterialClass = MaterialFactoryNode->GetObjectClass();
	if (!ensure(MaterialClass && MaterialClass->IsChildOf(GetFactoryClass())))
	{
		CouldNotCreateMaterialLog(MissMatchClassText);
		return ImportAssetResult;
	}

	const bool bIsReimport = Arguments.ReimportObject != nullptr;

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (MaterialFactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}
	}

	// create a new material or overwrite existing asset, if possible
	if (!ExistingAsset)
	{
		Material = NewObject<UObject>(Arguments.Parent, MaterialClass, *Arguments.AssetName, RF_Public | RF_Standalone);
	}
	else if (ExistingAsset->GetClass()->IsChildOf(MaterialClass))
	{
		//This is a reimport, we are just re-updating the source data
		Material = ExistingAsset;
		//We allow override of existing materials only if the translator is a pure material translator
		if (!bIsReimport && Arguments.Translator->GetSupportedAssetTypes() != EInterchangeTranslatorAssetType::Materials)
		{
			//Do not override the asset
			bSkipImport = true;
			ImportAssetResult.bIsFactorySkipAsset = bSkipImport;
		}
	}

	if (!Material)
	{
		CouldNotCreateMaterialLog(LOCTEXT("MatFunc_CouldNotCreateMat_MaterialCreationFail", "Material creation failed."));
		return ImportAssetResult;
	}

#if WITH_EDITOR
	if (!bSkipImport)
	{
		Material->PreEditChange(nullptr);
	}
#endif //WITH_EDITOR

	ImportAssetResult.ImportedObject = Material;
	return ImportAssetResult;
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeMaterialFunctionFactory::ImportAsset_Async(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeMaterialFunctionFactory::ImportAsset_Async);
	FImportAssetResult ImportAssetResult;
	ImportAssetResult.bIsFactorySkipAsset = bSkipImport;

	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	const UInterchangeMaterialFunctionFactoryNode* MaterialFactoryNode = Cast<UInterchangeMaterialFunctionFactoryNode>(Arguments.AssetNode);
	if (!ensure(MaterialFactoryNode))
	{
		return ImportAssetResult;
	}

	UObject* MaterialObject = UE::Interchange::FFactoryCommon::AsyncFindObject(Arguments.AssetNode, GetFactoryClass(), Arguments.Parent, Arguments.AssetName);

	//Do not override an asset we skip
	if (bSkipImport)
	{
		ImportAssetResult.ImportedObject = MaterialObject;
		return ImportAssetResult;
	}

	const bool bReimport = Arguments.ReimportObject && MaterialObject;

	if (!MaterialObject)
	{
		UE_LOGF(LogInterchangeImport, Error, "Could not import the Material asset %ls because the asset does not exist.", *Arguments.AssetName);
		return ImportAssetResult;
	}

	//Currently material re-import will not touch the material at all
	//TODO design a re-import process for the material (expressions and input connections)
	if (!Arguments.ReimportObject)
	{
		if (UMaterialFunction* MaterialFunction = Cast<UMaterialFunction>(MaterialObject))
		{
#if WITH_EDITOR
			SetupMaterial(MaterialFunction, Arguments, MaterialFactoryNode);
#endif // #if WITH_EDITOR

			MaterialFactoryNode->ApplyAllCustomAttributeToObject(MaterialObject);
		}
	}

	//Getting the file Hash will cache it into the source data
	Arguments.SourceData->GetFileContentHash();

	ImportAssetResult.ImportedObject = MaterialObject;
	return ImportAssetResult;
}

/* This function is call in the completion task on the main thread, use it to call main thread post creation step for your assets*/
void UInterchangeMaterialFunctionFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeMaterialFunctionFactory::SetupObject_GameThread);
	Super::SetupObject_GameThread(Arguments);

	if (bSkipImport)
	{
		return;
	}

#if WITH_EDITORONLY_DATA
	if (ensure(Arguments.ImportedObject && Arguments.SourceData))
	{
		UMaterialFunction* ImportedMaterialFunction = CastChecked<UMaterialFunction>(Arguments.ImportedObject);

		if (UMaterialFunctionEditorOnlyData* Data = ImportedMaterialFunction->GetEditorOnlyData())
		{
			//Update the samplers type in case the textures were changed during their SetupObject_GameThread
			for (UMaterialExpression* Expression : Data->ExpressionCollection.Expressions)
			{
				if (UMaterialExpressionTextureBase* TextureSample = Cast<UMaterialExpressionTextureBase>(Expression))
				{
					TextureSample->AutoSetSampleType();
				}
			}
		}
	}
#endif
}

#if WITH_EDITOR
void UInterchangeMaterialFunctionFactory::SetupMaterial(UMaterialFunction* MaterialFunction, const FImportAssetObjectParams& Arguments, const UInterchangeMaterialFunctionFactoryNode* MaterialFunctionFactoryNode)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeMaterialFunctionFactory::SetupMaterial);
	using namespace UE::Interchange::MaterialFactory::Internal;
	using namespace UE::Interchange::Materials;

	FMaterialExpressionBuilder Builder(nullptr, MaterialFunction, Arguments, Results.Get());

	UMaterialExpressionFunctionOutput* Output = CreateMaterialExpression<UMaterialExpressionFunctionOutput>(nullptr, MaterialFunction);

	UMaterialExpressionMakeMaterialAttributes* Attrib = CreateMaterialExpression<UMaterialExpressionMakeMaterialAttributes>(nullptr, MaterialFunction);
	Attrib->ConnectExpression(Output->GetInput(0), 0);

	TFunction<bool(const FString& , FExpressionInput*)> ConnectInput;
	ConnectInput = [&MaterialFunctionFactoryNode, &Builder, NodeContainer = Arguments.NodeContainer](const FString& InputName, FExpressionInput* Input) -> bool
	{
		FString ExpressionNodeUid;
		FString OutputName;

		if (MaterialFunctionFactoryNode->GetInputConnection(InputName, ExpressionNodeUid, OutputName))
		{
			const UInterchangeMaterialExpressionFactoryNode* ExpressionFactoryNode = Cast<UInterchangeMaterialExpressionFactoryNode>(NodeContainer->GetNode(ExpressionNodeUid));

			if (ExpressionFactoryNode)
			{
				if (UMaterialExpression* MaterialExpression = Builder.CreateExpressionsForNode(*ExpressionFactoryNode))
				{
					MaterialExpression->ConnectExpression(Input, 0);
					return true;
				}
			}
		}

		return false;
	};

	// Only material functions with BSDF output are supported for now
	if (!ConnectInput(Common::Parameters::BxDF.ToString(), Attrib->GetInput(0)))
	{
		ConnectInput(PBRMR::Parameters::BaseColor.ToString(), &Attrib->BaseColor);
		ConnectInput(PBRMR::Parameters::Metallic.ToString(), &Attrib->Metallic);
		ConnectInput(PBRMR::Parameters::Specular.ToString(), &Attrib->Specular);
		ConnectInput(PBRMR::Parameters::Roughness.ToString(), &Attrib->Roughness);
		ConnectInput(PBRMR::Parameters::EmissiveColor.ToString(), &Attrib->EmissiveColor);
		ConnectInput(PBRMR::Parameters::Normal.ToString(), &Attrib->Normal);
		ConnectInput(PBRMR::Parameters::Anisotropy.ToString(), &Attrib->Anisotropy);
		ConnectInput(PBRMR::Parameters::Tangent.ToString(), &Attrib->Tangent);
		ConnectInput(PBRMR::Parameters::IndexOfRefraction.ToString(), &Attrib->Refraction);
		ConnectInput(PBRMR::Parameters::Occlusion.ToString(), &Attrib->AmbientOcclusion);
		ConnectInput(ClearCoat::Parameters::ClearCoat.ToString(), &Attrib->ClearCoat);
		ConnectInput(ClearCoat::Parameters::ClearCoatRoughness.ToString(), &Attrib->ClearCoatRoughness);
		ConnectInput(Subsurface::Parameters::SubsurfaceColor.ToString(), &Attrib->SubsurfaceColor);
	}

	UMaterialEditingLibrary::LayoutMaterialFunctionExpressions(MaterialFunction);

	MaterialFunction->UpdateDependentFunctionCandidates();
}

void FInterchangeImportMaterialAsyncHelper::UpdateFromFunctionResource(UMaterialExpressionMaterialFunctionCall* MaterialFunctionCall)
{
	FScopeLock Lock(&UpdatedMaterialFunctionCallsLock);
	{
		MaterialFunctionCall->UpdateFromFunctionResource();
	}
}

void FInterchangeImportMaterialAsyncHelper::UpdateFromFunctionResource(UMaterialFunctionInterface* MaterialFunction)
{
	FScopeLock Lock(&UpdatedMaterialFunctionsLock);
	{
		MaterialFunction->UpdateFromFunctionResource();
	}
}
#endif // #if WITH_EDITOR

FInterchangeImportMaterialAsyncHelper& FInterchangeImportMaterialAsyncHelper::GetInstance()
{
	static FInterchangeImportMaterialAsyncHelper Instance;
	return Instance;
}

#undef LOCTEXT_NAMESPACE
