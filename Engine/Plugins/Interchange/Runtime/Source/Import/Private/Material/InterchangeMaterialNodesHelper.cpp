// Copyright Epic Games, Inc. All Rights Reserved. 
#include "Material/InterchangeMaterialNodesHelper.h"

#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"
#include "InterchangeShaderGraphNode.h"
#include "InterchangeMaterialDefinitions.h"

namespace UE::Interchange::Materials::Private
{
	bool FMaterialNodesHelper::SetupScalarParameter(UInterchangeBaseNodeContainer& BaseNodeContainer, UInterchangeShaderGraphNode* ShaderGraphNode, const FString& InputName, const float& AttributeValue)
	{
		using namespace UE::Interchange::Materials::Standard::Nodes;

		UInterchangeShaderNode* ExpressionNode = UInterchangeShaderNode::Create(&BaseNodeContainer, InputName, ShaderGraphNode->GetUniqueID());
		if (!ensure(ExpressionNode))
		{
			// TODO: Log error
			return false;
		}

		ExpressionNode->AddFloatInput(ScalarParameter::Attributes::DefaultValue.ToString(), AttributeValue,/*bIsAParameter = */ true);
		ExpressionNode->SetCustomShaderType(ScalarParameter::Name.ToString());

		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(ShaderGraphNode, InputName, ExpressionNode->GetUniqueID());
		return true;
	}

	bool FMaterialNodesHelper::SetupVectorParameter(UInterchangeBaseNodeContainer& BaseNodeContainer, UInterchangeShaderGraphNode* ShaderGraphNode, const FString& InputName, const FLinearColor& AttributeValue)
	{
		using namespace UE::Interchange::Materials::Standard::Nodes;

		UInterchangeShaderNode* ExpressionNode = UInterchangeShaderNode::Create(&BaseNodeContainer, InputName, ShaderGraphNode->GetUniqueID());
		if (!ensure(ExpressionNode))
		{
			// TODO: Log error
			return false;
		}

		ExpressionNode->AddLinearColorInput(VectorParameter::Attributes::DefaultValue.ToString(), AttributeValue,/*bIsAParameter = */ true);
		ExpressionNode->SetCustomShaderType(VectorParameter::Name.ToString());
		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(ShaderGraphNode, InputName, ExpressionNode->GetUniqueID());
		return true;
	}
}