// Copyright Epic Games, Inc. All Rights Reserved. 

#include "MaterialX/InterchangeMaterialXTranslator.h"
#include "InterchangeTranslatorHelper.h"
#include "MaterialX/MaterialXUtils/MaterialXManager.h"
#include "Nodes/InterchangeSourceNode.h"
#include "UDIMUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeMaterialXTranslator)

static bool GInterchangeEnableMaterialXImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnableMaterialXImport(
	TEXT("Interchange.FeatureFlags.Import.MTLX"),
	GInterchangeEnableMaterialXImport,
	TEXT("Whether MaterialX support is enabled."),
	ECVF_Default);

EInterchangeTranslatorType UInterchangeMaterialXTranslator::GetTranslatorType() const
{
	return EInterchangeTranslatorType::Assets;
}

EInterchangeTranslatorAssetType UInterchangeMaterialXTranslator::GetSupportedAssetTypes() const
{
	return EInterchangeTranslatorAssetType::Materials;
}

TArray<FString> UInterchangeMaterialXTranslator::GetSupportedFormats() const
{
	if (!GInterchangeEnableMaterialXImport && !GIsAutomationTesting)
	{
		return TArray<FString>{};
	}

	return TArray<FString>{ TEXT("mtlx;MaterialX File Format") };
}

bool UInterchangeMaterialXTranslator::Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const
{
	bool bIsDocumentValid = false;

#if WITH_EDITOR
	namespace mx = MaterialX;

	FString Filename = GetSourceData()->GetFilename();

	bIsDocumentValid = FMaterialXManager::GetInstance().Translate(Filename, BaseNodeContainer, this);

#endif // WITH_EDITOR

	if(bIsDocumentValid)
	{
		UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(&BaseNodeContainer);
		SourceNode->SetCustomImportUnusedMaterial(true);
	}

	return bIsDocumentValid;
}

TOptional<UE::Interchange::FImportImage> UInterchangeMaterialXTranslator::GetTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const
{
	return GetTexturePayloadData(PayloadKey, AlternateTexturePath, Results, AnalyticsHandler);
}

TOptional<UE::Interchange::FImportImage> UInterchangeMaterialXTranslator::GetTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath, UInterchangeResultsContainer* InResults, TObjectPtr<UInterchangeAnalyticsHandlerBase> InAnalyticsHandler)
{
	FString Filename = PayloadKey;
	TextureCompressionSettings CompressionSettings = TextureCompressionSettings::TC_Default;

#if WITH_EDITOR
	if (int32 IndexTextureCompression; PayloadKey.FindChar(FMaterialXManager::TexturePayloadSeparator, IndexTextureCompression))
	{
		Filename = PayloadKey.Mid(0, IndexTextureCompression);
		CompressionSettings = TextureCompressionSettings(FCString::Atoi(*PayloadKey.Mid(IndexTextureCompression + 1)));
	}
#endif

	if (FPaths::GetExtension(Filename).Equals(TEXT("exr"), ESearchCase::IgnoreCase))
	{
		CompressionSettings = TextureCompressionSettings::TC_HalfFloat;
	}

	UE::Interchange::Private::FScopedTranslator ScopedTranslator(Filename, InResults, InAnalyticsHandler);
	const IInterchangeTexturePayloadInterface* TextureTranslator = ScopedTranslator.GetPayLoadInterface<IInterchangeTexturePayloadInterface>();

	if (!TextureTranslator)
	{
		return TOptional<UE::Interchange::FImportImage>();
	}

	AlternateTexturePath = Filename;

	TOptional<UE::Interchange::FImportImage> TexturePayloadData = TextureTranslator->GetTexturePayloadData(PayloadKey, AlternateTexturePath);

	if (TexturePayloadData.IsSet())
	{
		TexturePayloadData.GetValue().CompressionSettings = CompressionSettings;
	}

	return TexturePayloadData;
}

TOptional<UE::Interchange::FImportBlockedImage> UInterchangeMaterialXTranslator::GetBlockedTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const
{
	return GetBlockedTexturePayloadData(PayloadKey, AlternateTexturePath, Results, AnalyticsHandler);
}

TOptional<UE::Interchange::FImportBlockedImage> UInterchangeMaterialXTranslator::GetBlockedTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath, UInterchangeResultsContainer* InResults, TObjectPtr<UInterchangeAnalyticsHandlerBase> InAnalyticsHandler)
{
	using namespace UE::Interchange;
	using namespace UE::TextureUtilitiesCommon;

	FString Filename = PayloadKey;
	TextureCompressionSettings CompressionSettings = TextureCompressionSettings::TC_Default;

#if WITH_EDITOR
	if (int32 IndexTextureCompression; PayloadKey.FindChar(FMaterialXManager::TexturePayloadSeparator, IndexTextureCompression))
	{
		Filename = PayloadKey.Mid(0, IndexTextureCompression);
		CompressionSettings = TextureCompressionSettings(FCString::Atoi(*PayloadKey.Mid(IndexTextureCompression + 1)));
	}
#endif

	if (FPaths::GetExtension(Filename).Equals(TEXT("exr"), ESearchCase::IgnoreCase))
	{
		CompressionSettings = TextureCompressionSettings::TC_HalfFloat;
	}

	FImportBlockedImage BlockedImage;
	BlockedImage.CompressionSettings = CompressionSettings;

	TMap<int32, FString> UDIMBlocks = GetUDIMBlocksFromSourceFile(Filename, DefaultUdimRegexPattern);

	TArray<FImportImage> Images;
	Images.Reserve(UDIMBlocks.Num());

	for (const TPair<int32, FString>& Pair : UDIMBlocks)
	{
		const int32 IndexUDIM = Pair.Get<0>();
		const FString& FilenameUDIM = Pair.Get<1>();

		UE::Interchange::Private::FScopedTranslator ScopedTranslator(FilenameUDIM, InResults, InAnalyticsHandler);
		const IInterchangeTexturePayloadInterface* TextureTranslator = ScopedTranslator.GetPayLoadInterface<IInterchangeTexturePayloadInterface>();

		// should we stop if one of the UDIM image is broken?
		if (!ensure(TextureTranslator))
		{
			continue;
		}

		AlternateTexturePath = FilenameUDIM;
		TOptional<UE::Interchange::FImportImage> TexturePayloadData = TextureTranslator->GetTexturePayloadData(FilenameUDIM, AlternateTexturePath);

		if (TexturePayloadData.IsSet())
		{
			const FImportImage& Image = Images.Emplace_GetRef(MoveTemp(*TexturePayloadData));
			BlockedImage.Format = Image.Format;

			int32 BlockX = INDEX_NONE, BlockY = INDEX_NONE;
			ExtractUDIMCoordinates(IndexUDIM, BlockX, BlockY);

			if (BlockX == INDEX_NONE || BlockY == INDEX_NONE)
			{
				continue;
			}

			BlockedImage.InitBlockFromImage(BlockX, BlockY, Image);
		}
	}

	BlockedImage.MigrateDataFromImagesToRawData(Images);

	return BlockedImage;
}
