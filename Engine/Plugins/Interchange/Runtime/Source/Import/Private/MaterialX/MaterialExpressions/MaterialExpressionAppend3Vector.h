// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"

#include "MaterialExpressionAppend3Vector.generated.h"

/**
 * A material expression that allows combining 3 channels together to create a vector with more channel than the original
 */
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, meta = (Private))
class UMaterialExpressionMaterialXAppend3Vector : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FExpressionInput A;

	UPROPERTY()
	FExpressionInput B;

	UPROPERTY()
	FExpressionInput C;

	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpressionMaterialX Interface
};

