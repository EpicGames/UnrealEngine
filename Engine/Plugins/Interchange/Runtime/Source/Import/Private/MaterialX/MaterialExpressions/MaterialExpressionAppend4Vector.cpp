// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialExpressionAppend4Vector.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionAppend4Vector)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXAppend4Vector"

UMaterialExpressionMaterialXAppend4Vector::UMaterialExpressionMaterialXAppend4Vector(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXAppend4Vector::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Append4 input A"));
	}
	else if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Append4 input B"));
	}
	else if(!C.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Append4 input C"));
	}
	else if(!D.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Append4 input D"));
	}
	else
	{
		int32 Arg1 = A.Compile(Compiler);
		int32 Arg2 = B.Compile(Compiler);
		int32 Arg3 = C.Compile(Compiler);
		int32 Arg4 = D.Compile(Compiler);
		return Compiler->AppendVector(Arg1,
			   Compiler->AppendVector(Arg2,
			   Compiler->AppendVector(Arg3, Arg4)));
	}
}

void UMaterialExpressionMaterialXAppend4Vector::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Append4"));
}

#endif

#undef LOCTEXT_NAMESPACE