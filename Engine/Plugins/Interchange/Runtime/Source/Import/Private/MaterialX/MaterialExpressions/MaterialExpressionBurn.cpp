// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionBurn.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionBurn)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXBurn"

UMaterialExpressionMaterialXBurn::UMaterialExpressionMaterialXBurn(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXBurn::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Burn input A"));
	}

	if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Burn input B"));
	}

	int32 IndexAlpha = Alpha.GetTracedInput().Expression ? Alpha.Compile(Compiler) : Compiler->Constant(ConstAlpha);

	int32 OneIndex = Compiler->Constant(1.f);
	auto OneMinus = [&](int32 index)
	{
		return Compiler->Sub(OneIndex, index);
	};

	int32 IndexB = B.Compile(Compiler);
	int32 Burn = OneMinus(Compiler->Div(OneMinus(IndexB), A.Compile(Compiler)));

	return Compiler->Lerp(IndexB, Burn, IndexAlpha);
}

void UMaterialExpressionMaterialXBurn::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Burn"));
}

#endif

#undef LOCTEXT_NAMESPACE 