// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialExpressionContrast.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionContrast)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXContrast"

UMaterialExpressionMaterialXContrast::UMaterialExpressionMaterialXContrast(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXContrast::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Contrast Input"));
	}

	int32 IndexAmount = Amount.GetTracedInput().Expression ? Amount.Compile(Compiler) : Compiler->Constant(ConstAmount);
	int32 IndexPivot = Pivot.GetTracedInput().Expression ? Pivot.Compile(Compiler) : Compiler->Constant(ConstPivot);

	int32 IndexSub = Compiler->Sub(Input.Compile(Compiler), IndexPivot);
	int32 IndexMul = Compiler->Mul(IndexSub, IndexAmount);
	int32 IndexAdd = Compiler->Add(IndexMul, IndexPivot);

	return IndexAdd;
}

void UMaterialExpressionMaterialXContrast::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Contrast"));
}

#endif

#undef LOCTEXT_NAMESPACE 