// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"

#include "MaterialExpressionContrast.generated.h"

/**
 * A material expression that increases or decreases contrast of a float/color value using a linear slope multiplier.
 */
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, meta = (Private))
class UMaterialExpressionMaterialXContrast : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FExpressionInput Input;

	UPROPERTY()
	FExpressionInput Amount;

	UPROPERTY()
	FExpressionInput Pivot;

	/** only used if Amount is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionContrast, meta = (OverridingInputProperty = "Amount"))
	float ConstAmount = 1.f;

	/** only used if Pivot is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionContrast, meta = (OverridingInputProperty = "Pivot"))
	float ConstPivot= 0.5f;

	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpressionMaterialX Interface
};

