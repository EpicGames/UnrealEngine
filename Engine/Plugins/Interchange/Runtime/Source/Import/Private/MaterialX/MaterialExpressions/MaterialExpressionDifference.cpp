// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionDifference.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionDifference)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXDifference"

UMaterialExpressionMaterialXDifference::UMaterialExpressionMaterialXDifference(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXDifference::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Difference input A"));
	}

	if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Difference input B"));
	}

	int32 IndexAlpha = Alpha.GetTracedInput().Expression ? Alpha.Compile(Compiler) : Compiler->Constant(ConstAlpha);

	int32 IndexB = B.Compile(Compiler);
	int32 Diff = Compiler->Abs(Compiler->Sub(IndexB, A.Compile(Compiler)));
	return Compiler->Lerp(IndexB, Diff, IndexAlpha);
}

void UMaterialExpressionMaterialXDifference::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Difference"));
}

#endif

#undef LOCTEXT_NAMESPACE 