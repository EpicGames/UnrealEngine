// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionDisjointOver.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionDisjointOver)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXDisjointOver"

UMaterialExpressionMaterialXDisjointOver::UMaterialExpressionMaterialXDisjointOver(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXDisjointOver::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX DisjointOver input A"));
	}

	if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX DisjointOver input B"));
	}

	int32 IndexAlpha = Alpha.GetTracedInput().Expression ? Alpha.Compile(Compiler) : Compiler->Constant(ConstAlpha);
	
	int32 IndexA = A.Compile(Compiler);
	int32 IndexB = B.Compile(Compiler);

	int32 IndexAlphaA = Compiler->ComponentMask(IndexA, false, false, false, true);
	int32 IndexAlphaB = Compiler->ComponentMask(IndexB, false, false, false, true);
	int32 IndexSumAlpha = Compiler->Add(IndexAlphaA, IndexAlphaB);

	int32 IndexRgbA = Compiler->ComponentMask(IndexA, true, true, true, false);
	int32 IndexRgbB = Compiler->ComponentMask(IndexB, true, true, true, false);
	int32 IndexSumRgb = Compiler->Add(IndexRgbA, IndexRgbB);
	
	int32 IndexOne = Compiler->Constant(1.f);
	int32 X = Compiler->Div(Compiler->Sub(IndexOne, IndexAlphaA), IndexAlphaB);
	int32 Disjoint = Compiler->Add(IndexRgbA, Compiler->Mul(IndexRgbB, X));

	int32 Result = Compiler->If(IndexSumAlpha, IndexOne, Disjoint, IndexSumRgb, IndexSumRgb, Compiler->Constant(0.00001f));
	int32 ResultAlpha = Compiler->Min(IndexSumAlpha, IndexOne);

	int32 IndexAppend = Compiler->AppendVector(Result, ResultAlpha);
	return Compiler->Lerp(IndexB, IndexAppend, IndexAlpha);
}

void UMaterialExpressionMaterialXDisjointOver::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX DisjointOver"));
}

#endif

#undef LOCTEXT_NAMESPACE 