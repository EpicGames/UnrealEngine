// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionDodge.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionDodge)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXDodge"

UMaterialExpressionMaterialXDodge::UMaterialExpressionMaterialXDodge(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXDodge::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Dodge input A"));
	}

	if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Dodge input B"));
	}

	int32 IndexAlpha = Alpha.GetTracedInput().Expression ? Alpha.Compile(Compiler) : Compiler->Constant(ConstAlpha);
	int32 IndexB = B.Compile(Compiler);
	int32 Sub = Compiler->Sub(Compiler->Constant(1.f), A.Compile(Compiler));

	int32 Zero = Compiler->Constant(0.f);
	int32 Dodge = Compiler->Div(IndexB, Sub);

	int32 Result = Compiler->Lerp(IndexB, Dodge, IndexAlpha);

	return Compiler->If(Compiler->Abs(Sub), Zero, Result, Result, Zero, Compiler->Constant(0.00001f));
}

void UMaterialExpressionMaterialXDodge::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Dodge"));
}

#endif

#undef LOCTEXT_NAMESPACE 