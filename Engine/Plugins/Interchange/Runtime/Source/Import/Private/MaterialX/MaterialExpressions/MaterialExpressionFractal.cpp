// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialExpressionFractal.h"
#include "MaterialCompiler.h"
#include "Materials/MaterialExpressionCustom.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionFractal)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXFractal"

UMaterialExpressionMaterialXFractal::UMaterialExpressionMaterialXFractal(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("R"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("RG"), 1, 1, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("RGB"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("RGBA"), 1, 1, 1, 1, 1));
	bShowOutputNameOnPin = true;
	bShowMaskColorsOnPin = false;

	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXFractal::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 IndexPosition = Position.GetTracedInput().Expression ? Position.Compile(Compiler) : Compiler->LocalPosition(EPositionIncludedOffsets::IncludeOffsets, ELocalPositionOrigin::Instance);

	if (IndexPosition == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("Failed to compile Position input."));
	}

	bFractal2dType = Compiler->GetType(IndexPosition) == EMaterialValueType::MCT_Float2;
	if (bFractal2dType)
	{
		IndexPosition = Compiler->AppendVector(IndexPosition, Compiler->Constant(0.));
	}

	UMaterialExpressionCustom* MaterialExpressionCustom = NewObject<UMaterialExpressionCustom>();
	MaterialExpressionCustom->Inputs[0].InputName = TEXT("Position");
	MaterialExpressionCustom->Inputs.Add({ TEXT("Octaves") });
	MaterialExpressionCustom->Inputs.Add({ TEXT("Lacunarity") });
	MaterialExpressionCustom->Inputs.Add({ TEXT("Diminish") });
	
	FString Result = TEXT("float result = 0.0;");
	FString Noise = TEXT("result.x += amplitude * GradientNoise3D_TEX(Position, bTiling, RepeatSize);\n");

	// Decorrelating offsets for multi-channel Perlin noise.
	// Chosen to avoid lattice alignment and visible correlation in grid-based noise.
	// Follows common practice used in MaterialX and Perlin-style implementations.
	switch (OutputIndex)
	{
	case 1:
		MaterialExpressionCustom->OutputType = ECustomMaterialOutputType::CMOT_Float2;
		Result = TEXT("float2 result = float2(0.0, 0.0);\n");
		Noise += TEXT("result.y += amplitude * GradientNoise3D_TEX(Position + float3(19,193,") + FString{ bFractal2dType ? TEXT("0") : TEXT("17") } + TEXT("), bTiling, RepeatSize);\n");
		break;
	case 2:
		MaterialExpressionCustom->OutputType = ECustomMaterialOutputType::CMOT_Float3;
		Result = TEXT("float3 result = float3(0.0, 0.0, 0.0);\n");
		Noise += TEXT("result.y += amplitude * GradientNoise3D_TEX(Position + float3(19,193,") + FString{ bFractal2dType ? TEXT("0") : TEXT("17") } + TEXT("), bTiling, RepeatSize);\n");
		Noise += TEXT("result.z += amplitude * GradientNoise3D_TEX(Position + float3(73,27,") + FString{ bFractal2dType ? TEXT("0") : TEXT("101") } + TEXT("), bTiling, RepeatSize);\n");
		break;
	case 3:
		MaterialExpressionCustom->OutputType = ECustomMaterialOutputType::CMOT_Float4;
		Result = TEXT("float4 result = float4(0.0, 0.0, 0.0, 0.0);\n");
		Noise += TEXT("result.y += amplitude * GradientNoise3D_TEX(Position + float3(19,193,") + FString{ bFractal2dType ? TEXT("0")  : TEXT("17") } + TEXT("), bTiling, RepeatSize);\n");
		Noise += TEXT("result.z += amplitude * GradientNoise3D_TEX(Position + float3(73,27,") + FString{ bFractal2dType ? TEXT("0") : TEXT("101") } + TEXT("), bTiling, RepeatSize);\n ");
		Noise += TEXT("result.w += amplitude * GradientNoise3D_TEX(Position + float3(151,89,") + FString{ bFractal2dType ? TEXT("0") : TEXT("7") } + TEXT("), bTiling, RepeatSize);\n");

		break;
	default:
		MaterialExpressionCustom->OutputType = ECustomMaterialOutputType::CMOT_Float1;
		break;
	}
	
	MaterialExpressionCustom->Code =
		TEXT(R"(const bool bTiling = false;
        const float RepeatSize = 512;
	    )")
		+ Result +
TEXT(R"(float amplitude = 1.0;
		for (int i = 0;  i < Octaves; ++i)
		{
         )") 
		 + Noise +
TEXT(R"(
			amplitude *= Diminish;
			Position *= Lacunarity;
		}
		return result;)");

	int32 IndexOctaves = Octaves.GetTracedInput().Expression ? Octaves.Compile(Compiler) : Compiler->Constant(ConstOctaves);
	int32 IndexLacunarity = Lacunarity.GetTracedInput().Expression ? Lacunarity.Compile(Compiler) : Compiler->Constant(ConstLacunarity);
	int32 IndexDiminish = Diminish.GetTracedInput().Expression ? Diminish.Compile(Compiler) : Compiler->Constant(ConstDiminish);

	TArray<int32> Inputs{ IndexPosition, IndexOctaves, IndexLacunarity, IndexDiminish };
	int32 IndexFractal = Compiler->CustomExpression(MaterialExpressionCustom, 0, Inputs);

	int32 IndexAmplitude = Amplitude.GetTracedInput().Expression ? Amplitude.Compile(Compiler) : Compiler->Constant(ConstAmplitude);
	
	return Compiler->Mul(IndexFractal, IndexAmplitude);
}

void UMaterialExpressionMaterialXFractal::GetCaption(TArray<FString>& OutCaptions) const
{
	if (bFractal2dType)
	{
		OutCaptions.Add(TEXT("MaterialX Fractal2D"));
	}
	else
	{
		OutCaptions.Add(TEXT("MaterialX Fractal3D"));
	}
}

FName UMaterialExpressionMaterialXFractal::GetInputName(int32 InputIndex) const
{
	// caveat : bFractal2dType is set during Compile, but the editor doesn't refresh the pins after it.
	// it will only work at creation-time or if the Material Editor is completely refreshed
	if (InputIndex == 0 && bFractal2dType)
	{
		return TEXT("Coordinates");
	}
	return Super::GetInputName(InputIndex);
}

void UMaterialExpressionMaterialXFractal::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip)
{
	if (InputIndex == 0 && bFractal2dType)
	{
		OutToolTip.Add(TEXT("The 2D texture coordinate at which the noise is evaluated."));
	}
	else
	{
		Super::GetConnectorToolTip(InputIndex, OutputIndex, OutToolTip);
	}
}
#endif

#undef LOCTEXT_NAMESPACE 