// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Materials/MaterialExpression.h"

#include "MaterialExpressionFractal.generated.h"

/**
 * Zero-centered 2D or 3D Fractal noise in 1, 2, 3 or 4 channels, created by summing several
 * octaves of 2D or 3D Perlin noise, increasing the frequency and decreasing the amplitude at each octave.
 * Defaulting to a fractal 3D. Inputting a 2D vector will create a fractal 2D.
 */

UCLASS(collapsecategories, hidecategories = Object, MinimalAPI, meta = (Private))
class UMaterialExpressionMaterialXFractal : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** The 3D position at which the noise is evaluated. By default the vector is in local space*/
	UPROPERTY()
	FExpressionInput Position;	

	/** Center-to-peak amplitude of the noise (peak-to-peak amplitude is 2x this value).*/
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstAmplitude' if not specified"))
	FExpressionInput Amplitude;

	/** only used if Amplitude is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionFractal, meta = (OverridingInputProperty = "Amplitude"))
	float ConstAmplitude = 1;

	/** The number of octaves of noise to be summed. */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstOctaves' if not specified"))
	FExpressionInput Octaves;

	/** only used if Octaves is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionFractal, meta = (OverridingInputProperty = "Octaves"))
	int32 ConstOctaves = 3;

	/** The exponential scale between successive octaves of noise. */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstLacunarity' if not specified"))
	FExpressionInput Lacunarity;

	/** only used if Lacunarity is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionFractal, meta = (OverridingInputProperty = "Lacunarity"))
	float ConstLacunarity = 2;

	/** The rate at which noise amplitude is diminished for each octave. Should be between 0.0 and 1.0 */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstDiminish' if not specified"))
	FExpressionInput Diminish;

	/** only used if Diminish is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionFractal, meta = (OverridingInputProperty = "Diminish"))
	float ConstDiminish = 0.5;

	UPROPERTY()
	float Scale_DEPRECATED = 1.f;

	UPROPERTY()
	bool bTurbulence_DEPRECATED = false;

	UPROPERTY()
	int32 Levels_DEPRECATED = 6;

	UPROPERTY()
	float OutputMin_DEPRECATED = 0.f;

	UPROPERTY()
	float OutputMax_DEPRECATED = 1.f;

	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) override;

private:
	bool bFractal2dType = false;
#endif
	//~ End UMaterialExpressionMaterialX Interface
};

