// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionIn.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionIn)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXIn"

UMaterialExpressionMaterialXIn::UMaterialExpressionMaterialXIn(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXIn::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX In input A"));
	}

	if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX In input B"));
	}

	int32 IndexAlpha = Alpha.GetTracedInput().Expression ? Alpha.Compile(Compiler) : Compiler->Constant(ConstAlpha);

	int32 IndexA = A.Compile(Compiler);
	int32 IndexB = B.Compile(Compiler);

	int32 IndexIn = Compiler->Mul(IndexA, Compiler->ComponentMask(IndexB, false, false, false, true));

	return Compiler->Lerp(IndexB, IndexIn, IndexAlpha);
}

void UMaterialExpressionMaterialXIn::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX In"));
}

#endif

#undef LOCTEXT_NAMESPACE 