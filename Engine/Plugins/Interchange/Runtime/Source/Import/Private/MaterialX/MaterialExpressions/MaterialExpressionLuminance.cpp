// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialExpressionLuminance.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionLuminance)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXLuminance"

UMaterialExpressionMaterialXLuminance::UMaterialExpressionMaterialXLuminance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer),
	LuminanceFactors(0.2722287f, 0.6740818f, 0.0536895f),
	LuminanceMode(EMaterialXLuminanceMode::ACEScg)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXLuminance::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Luminance input"));
	}

	const int32 LuminanceFactorsIndex = Compiler->Constant3(LuminanceFactors.R, LuminanceFactors.G, LuminanceFactors.B);
	const int32 InputIndex = Input.Compile(Compiler);
	if (InputIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	return Compiler->Dot(Compiler->ComponentMask(InputIndex, true, true, true, false), LuminanceFactorsIndex);
}

void UMaterialExpressionMaterialXLuminance::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Luminance"));
}

void UMaterialExpressionMaterialXLuminance::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.MemberProperty;
	const FString PropertyName = PropertyThatChanged ? PropertyThatChanged->GetName() : TEXT("");

	if(PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterialExpressionMaterialXLuminance, LuminanceMode))
	{
		switch(LuminanceMode)
		{
		case EMaterialXLuminanceMode::ACEScg:
			LuminanceFactors = FLinearColor(0.2722287f, 0.6740818f, 0.0536895f);
			break;

		case EMaterialXLuminanceMode::Rec709:
			LuminanceFactors = FLinearColor(0.2126f, 0.7152f, 0.0722f);
			break;

		case EMaterialXLuminanceMode::Rec2020:
			LuminanceFactors = FLinearColor(0.2627f, 0.6780f, 0.0593f);
			break;
		}
	}
	else if(PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterialExpressionMaterialXLuminance, LuminanceFactors))
	{
		LuminanceMode = EMaterialXLuminanceMode::Custom;
	}
		
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif

#undef LOCTEXT_NAMESPACE 