// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"

#include "MaterialExpressionLuminance.generated.h"

/**
 * A material expression that outputs a grayscale image containing the luminance of the incoming RGB color in all color channels;
    the alpha channel is left unchanged if present.
 */

UENUM()
enum class EMaterialXLuminanceMode : uint8
{
	ACEScg,
	Rec709,
	Rec2020,
	Rec2100 = Rec2020,
	Custom
};

UCLASS(collapsecategories, hidecategories = Object, MinimalAPI, meta = (Private))
class UMaterialExpressionMaterialXLuminance : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FExpressionInput Input;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionLuminance, Meta = (ShowAsInputPin = "Advanced"))
	FLinearColor LuminanceFactors;    // Color component factors for converting a color to greyscale.
	
	UPROPERTY(EditAnywhere, Category = MaterialExpressionLuminance, Meta = (ShowAsInputPin = "Advanced"))
	EMaterialXLuminanceMode LuminanceMode;
	
	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UMaterialExpressionMaterialX Interface
};

