// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionMask.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionMask)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXMask"

UMaterialExpressionMaterialXMask::UMaterialExpressionMaterialXMask(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXMask::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Mask input A"));
	}

	if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Mask input B"));
	}

	int32 IndexAlpha = Alpha.GetTracedInput().Expression ? Alpha.Compile(Compiler) : Compiler->Constant(ConstAlpha);

	int32 IndexA = A.Compile(Compiler);
	int32 IndexB = B.Compile(Compiler);

	int32 Mask = Compiler->Mul(IndexB, Compiler->ComponentMask(IndexA, false, false, false, true));

	return Compiler->Lerp(IndexB, Mask, IndexAlpha);
}

void UMaterialExpressionMaterialXMask::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Mask"));
}

#endif

#undef LOCTEXT_NAMESPACE 