// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionMatte.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionMatte)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXMatte"

UMaterialExpressionMaterialXMatte::UMaterialExpressionMaterialXMatte(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXMatte::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Matte input A"));
	}

	if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Matte input B"));
	}

	int32 IndexAlpha = Alpha.GetTracedInput().Expression ? Alpha.Compile(Compiler) : Compiler->Constant(ConstAlpha);

	int32 IndexA = A.Compile(Compiler);
	int32 IndexB = B.Compile(Compiler);

	int32 IndexAlphaA = Compiler->ComponentMask(IndexA, false, false, false, true);
	int32 IndexRgbA = Compiler->ComponentMask(IndexA, true, true, true, false);
	int32 IndexAlphaB = Compiler->ComponentMask(IndexB, false, false, false, true);
	int32 IndexRgbB = Compiler->ComponentMask(IndexB, true, true, true, false);

	int32 IndexOneMinusAlphaA = Compiler->Sub(Compiler->Constant(1.f), IndexAlphaA);
	int32 XYZ = Compiler->Add(Compiler->Mul(IndexRgbA, IndexAlphaA), Compiler->Mul(IndexRgbB, IndexOneMinusAlphaA));
	int32 W = Compiler->Add(IndexAlphaA, Compiler->Mul(IndexAlphaB, IndexOneMinusAlphaA));

	int32 Matte = Compiler->AppendVector(XYZ, W);

	return Compiler->Lerp(IndexB, Matte, IndexAlpha);
}

void UMaterialExpressionMaterialXMatte::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Matte"));
}

#endif

#undef LOCTEXT_NAMESPACE 