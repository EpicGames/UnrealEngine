// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"

#include "MaterialExpressionMinus.generated.h"

/**
 * Blend nodes take two 1-4 channel inputs and apply the same operator to all channels.
 * Blend nodes support an optional float input mix , which can be used
 * to mix the original B value with the result of the blend operation.
 * Operation: B - A
 * Result: Lerp(B, B - A, Alpha)
 */
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, meta = (Private))
class UMaterialExpressionMaterialXMinus : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FExpressionInput A;

	UPROPERTY()
	FExpressionInput B;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstAlpha' if not specified"))
	FExpressionInput Alpha;

	/** only used if Alpha is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionLinearInterpolate, meta = (OverridingInputProperty = "Alpha"))
	float ConstAlpha = 1.f;

	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpressionMaterialX Interface
};

