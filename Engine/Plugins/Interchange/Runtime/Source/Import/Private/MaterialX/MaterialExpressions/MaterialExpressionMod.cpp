// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionMod.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionMod)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXMod"

UMaterialExpressionMaterialXMod::UMaterialExpressionMaterialXMod(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXMod::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Mod input A"));
	}

	int32 X = A.Compile(Compiler);
	int32 Y = B.GetTracedInput().Expression ? B.Compile(Compiler) : Compiler->Constant(ConstB);

	return Compiler->Sub(X, Compiler->Mul(Y, Compiler->Floor(Compiler->Div(X, Y))));
}

void UMaterialExpressionMaterialXMod::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Mod"));
}

void UMaterialExpressionMaterialXMod::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT(R"(The remaining fraction after dividing an incoming input by a value and subtracting the integer portion.
Unlike UE FMod or Modulo expressions, Mod always returns a non - negative result, matching the interpretation of the GLSL and OSL mod() function(not fmod()).
This is computed as x - y * floor(x / y). )"), 40, OutToolTip);
}

#endif

#undef LOCTEXT_NAMESPACE 