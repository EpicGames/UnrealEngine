// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"

#include "MaterialExpressionMod.generated.h"

/**
 * The remaining fraction after dividing an incoming input by a value and subtracting the integer portion.
 * Unlike UE FMod or Modulo expressions, Mod always returns a non-negative result, matching the interpretation of the GLSL and OSL mod() function (not fmod()).
 * This is computed as x - y * floor(x/y). 
 */
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, meta = (Private))
class UMaterialExpressionMaterialXMod : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FExpressionInput A;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstB' if not specified"))
	FExpressionInput B;

	/** only used if B is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionLinearInterpolate, meta = (OverridingInputProperty = "B"))
	float ConstB = 1.f;

	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
#endif
	//~ End UMaterialExpressionMaterialX Interface
};

