// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionOut.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionOut)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXOut"

UMaterialExpressionMaterialXOut::UMaterialExpressionMaterialXOut(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXOut::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Out input A"));
	}

	if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Out input B"));
	}

	int32 IndexAlpha = Alpha.GetTracedInput().Expression ? Alpha.Compile(Compiler) : Compiler->Constant(ConstAlpha);

	int32 IndexA = A.Compile(Compiler);
	int32 IndexB = B.Compile(Compiler);

	int32 IndexAlphaB = Compiler->ComponentMask(IndexB, false, false, false, true);

	int32 IndexOneMinusAlphaB = Compiler->Sub(Compiler->Constant(1.f), IndexAlphaB);
	int32 Out = Compiler->Mul(IndexA, IndexOneMinusAlphaB);

	return Compiler->Lerp(IndexB, Out, IndexAlpha);
}

void UMaterialExpressionMaterialXOut::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Out"));
}

#endif

#undef LOCTEXT_NAMESPACE 