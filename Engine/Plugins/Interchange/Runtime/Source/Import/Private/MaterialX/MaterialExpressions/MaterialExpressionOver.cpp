// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionOver.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionOver)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXOver"

UMaterialExpressionMaterialXOver::UMaterialExpressionMaterialXOver(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXOver::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Over input A"));
	}

	if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Over input B"));
	}

	int32 IndexAlpha = Alpha.GetTracedInput().Expression ? Alpha.Compile(Compiler) : Compiler->Constant(ConstAlpha);

	int32 IndexA = A.Compile(Compiler);
	int32 IndexB = B.Compile(Compiler);

	int32 IndexAlphaA = Compiler->ComponentMask(IndexA, false, false, false, true);

	int32 IndexOneMinusAlphaA = Compiler->Sub(Compiler->Constant(1.f), IndexAlphaA);
	int32 Over = Compiler->Add(IndexA, Compiler->Mul(IndexB, IndexOneMinusAlphaA));

	return Compiler->Lerp(IndexB, Over, IndexAlpha);
}

void UMaterialExpressionMaterialXOver::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Over"));
}

#endif

#undef LOCTEXT_NAMESPACE 