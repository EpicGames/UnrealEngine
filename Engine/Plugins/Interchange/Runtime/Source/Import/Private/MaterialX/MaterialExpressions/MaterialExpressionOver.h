// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"

#include "MaterialExpressionOver.generated.h"

/**
 * Merge nodes take two 4-channel (color4) inputs and use the built-in alpha channel(s) to control the
 * compositing of the A and B inputs. "A" and "B" refer to the non-alpha channels of the A and B inputs respectively,
 * and "a" and "b" refer to the alpha channels of the A and B inputs.
 * Merge nodes are only defined for float4 inputs
 * Merge nodes support an optional float input Alpha , which can be used to mix the
 * original B value with the result of the blend operation.
 *
 * Operation: A + B(1-a)
 * Result: Lerp(B, A + B(1-a), Alpha)
 */
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, meta = (Private))
class UMaterialExpressionMaterialXOver : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FExpressionInput A;

	UPROPERTY()
	FExpressionInput B;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstAlpha' if not specified"))
	FExpressionInput Alpha;

	/** only used if Alpha is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionLinearInterpolate, meta = (OverridingInputProperty = "Alpha"))
	float ConstAlpha = 1.f;

	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpressionMaterialX Interface
};

