// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionOverlay.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionOverlay)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXOverlay"

UMaterialExpressionMaterialXOverlay::UMaterialExpressionMaterialXOverlay(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXOverlay::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Overlay input A"));
	}

	if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Overlay input B"));
	}

	int32 IndexAlpha = Alpha.GetTracedInput().Expression ? Alpha.Compile(Compiler) : Compiler->Constant(ConstAlpha);

	int32 OneIndex = Compiler->Constant(1.f);
	int32 TwoIndex = Compiler->Constant(2.f);
	auto OneMinus = [&](int32 index)
	{
		return Compiler->Sub(OneIndex, index);
	};

	int32 IndexA = A.Compile(Compiler);
	int32 IndexB = B.Compile(Compiler);
	int32 Screen = OneMinus(Compiler->Mul(TwoIndex, Compiler->Mul(OneMinus(IndexA), OneMinus(IndexB))));

	int32 Overlay = Compiler->If(IndexB,
								 Compiler->Constant(0.5f),
								 Screen,
								 Screen,
								 Compiler->Mul(Compiler->Mul(TwoIndex, IndexA), IndexB),
								 Compiler->Constant(0.00001f));

	return Compiler->Lerp(IndexB, Overlay, IndexAlpha);
}

void UMaterialExpressionMaterialXOverlay::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Overlay"));
}

#endif

#undef LOCTEXT_NAMESPACE 