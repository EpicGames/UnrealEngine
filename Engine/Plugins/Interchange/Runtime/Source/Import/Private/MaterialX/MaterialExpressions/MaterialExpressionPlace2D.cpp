// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialExpressionPlace2D.h"
#include "MaterialExpressionRotate2D.h"
#include "MaterialCompiler.h"
#include "Materials/MaterialExpressionAdd.h"
#include "Materials/MaterialExpressionAppendVector.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionDivide.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialExpressionOneMinus.h"
#include "Materials/MaterialExpressionSubtract.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"

#include "Materials/MaterialExpressionRotator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionPlace2D)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXPlace2D"

UMaterialExpressionMaterialXPlace2D::UMaterialExpressionMaterialXPlace2D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
namespace
{
	template<typename ExpressionT>
	FExpressionInput NewExpression(const FExpressionInput& A, const FExpressionInput& B)
	{
		FExpressionInput ExpressionInput;

		ExpressionT* Expression = NewObject<ExpressionT>();
		Expression->A = A;
		Expression->B = B;

		ExpressionInput.Connect(0, Expression);

		return ExpressionInput;
	}
}

int32 UMaterialExpressionMaterialXPlace2D::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	auto Constant2Expression = [](FExpressionInput& Input, float X, float Y)
	{
		if(Input.GetTracedInput().Expression == nullptr)
		{
			UMaterialExpressionConstant2Vector* Expression = NewObject<UMaterialExpressionConstant2Vector>();
			Expression->R = X;
			Expression->G = Y;
			Input.Connect(0, Expression);
		}
	};

	auto InputMask = [](const FExpressionInput& Input, uint32 R, uint32 G, uint32 B, uint32 A)
	{
		UMaterialExpressionComponentMask* Mask = NewObject<UMaterialExpressionComponentMask>();
		Mask->R = R;
		Mask->G = G;
		Mask->B = B;
		Mask->A = A;
		Mask->Input = Input;

		FExpressionInput ExpressionInput;
		ExpressionInput.Connect(0, Mask);
		return ExpressionInput;
	};

	if(Coordinates.GetTracedInput().Expression == nullptr)
	{
		UMaterialExpressionTextureCoordinate* TextureCoordinate = NewObject<UMaterialExpressionTextureCoordinate>();
		TextureCoordinate->CoordinateIndex = ConstCoordinate;
		Coordinates.Connect(0, TextureCoordinate);
	}

	if(RotationAngle.GetTracedInput().Expression == nullptr)
	{
		UMaterialExpressionConstant* Expression = NewObject<UMaterialExpressionConstant>();
		Expression->R = ConstRotationAngle;
		RotationAngle.Connect(0, Expression);
	}

	Constant2Expression(Pivot, 0, 0);
	Constant2Expression(Scale, 1, 1);
	Constant2Expression(Offset, 0, 0);

	// We need to adjust the UVs to MaterialX UVs, Y is inverted
	FExpressionInput ScaleX = InputMask(Scale, 1, 0, 0, 0);
	FExpressionInput ScaleY = InputMask(Scale, 0, 1, 0, 0);
	FExpressionInput NegateScaleY;
	{
		UMaterialExpressionMultiply* NegateScaleYExp = NewObject<UMaterialExpressionMultiply>();
		NegateScaleYExp->ConstA = -1.f;
		NegateScaleYExp->B = ScaleY;
		NegateScaleY.Connect(0, NegateScaleYExp);
	}

	FExpressionInput AppendVectorScale = NewExpression<UMaterialExpressionAppendVector>(ScaleX, NegateScaleY);

	FExpressionInput SubPivot = NewExpression<UMaterialExpressionSubtract>(Coordinates, Pivot);
	FExpressionInput ApplyScale = NewExpression<UMaterialExpressionDivide>(SubPivot, AppendVectorScale);

	FExpressionInput ApplyRot;

	FExpressionInput Applyoffset = NewExpression<UMaterialExpressionSubtract>(ApplyRot, Offset);
	FExpressionInput AddPivot = NewExpression<UMaterialExpressionAdd>(Applyoffset, Pivot);

	// Readjust the UVs back
	FExpressionInput AddPivotX = InputMask(AddPivot, 1, 0, 0, 0);
	FExpressionInput AddPivotY = InputMask(AddPivot, 0, 1, 0, 0);
	FExpressionInput NegateAddPivotY;
	{
		UMaterialExpressionOneMinus* NegateAddPivotYExp = NewObject<UMaterialExpressionOneMinus>();
		NegateAddPivotYExp->Input = AddPivotY;
		NegateAddPivotY.Connect(0, NegateAddPivotYExp);
	}

	FExpressionInput AppendVectorAddPivot = NewExpression<UMaterialExpressionAppendVector>(AddPivotX, NegateAddPivotY);

	return AppendVectorAddPivot.Compile(Compiler);
}

void UMaterialExpressionMaterialXPlace2D::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Place2D"));
}

void UMaterialExpressionMaterialXPlace2D::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Transform incoming UV texture coordinates from one 2D frame of reference to another."
								   "The order of operation is: -pivot, scale, rotate, translate, +pivot."),
							  75, OutToolTip);
}
#endif

#undef LOCTEXT_NAMESPACE 