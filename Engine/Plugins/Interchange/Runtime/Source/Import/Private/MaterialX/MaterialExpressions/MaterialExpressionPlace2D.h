// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"

#include "MaterialExpressionPlace2D.generated.h"

/**
 * Transform incoming UV texture coordinates from one 2D frame of reference to another. 
 * operationorder (integer enum): the order in which to perform the transform operations.
 * "0" or "SRT" performs -pivot, scale, rotate, translate, +pivot as per the original
 * implementation matching the behavior of certain DCC packages, and "1" or "TRS" performs
 * -pivot, translate, rotate, scale, +pivot which does not introduce texture shear.
 * Default is 0 "SRT" for backward compatibility.*/
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, meta = (Private))
class UE_DEPRECATED(5.5, "UMaterialExpressionMaterialXPlace2D is now deprecated, please use Material Function MX_Place2D") UMaterialExpressionMaterialXPlace2D : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstCoordinate' if not specified"))
	FExpressionInput Coordinates;

	/** The pivot coordinate for scale and rotate: this is subtracted from u,v before
	 *  applying scale/rotate, then added back after. Default is (0,0). */
	UPROPERTY()
	FExpressionInput Pivot;

	/** Divide the u,v coord (after subtracting pivot ) by this, so a scale (2,2)
	 *  makes the texture image appear twice as big. Negative values can be used to flip or flop the
	 *  texture space. Default is (1,1). */
	UPROPERTY()
	FExpressionInput Scale;

	/** Subtract this amount from the scaled/rotated/“pivot added back” UV
	 *  coordinate; since U0,V0 is typically the lower left corner, a positive offset moves the texture
	 *  image up and right. Default is (0,0).*/
	UPROPERTY()
	FExpressionInput Offset;

	/** Rotate u,v coord (after subtracting pivot) by this amount in degrees, so a
	 *  positive value rotates UV coords counter-clockwise, and the image clockwise. */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstRotationAngle' if not specified"))
	FExpressionInput RotationAngle;

	/** only used if RotationAngle is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpression, meta = (OverridingInputProperty = "RotationAngle"))
	float ConstRotationAngle = 0.f;

	/** only used if Coordinates is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpression, meta = (OverridingInputProperty = "Coordinates"))
	uint8 ConstCoordinate;

	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
#endif
	//~ End UMaterialExpressionMaterialX Interface
};

