// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionPremult.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionPremult)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXPremult"

UMaterialExpressionMaterialXPremult::UMaterialExpressionMaterialXPremult(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXPremult::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Premult input"));
	}

	int32 IndexInput = Input.Compile(Compiler);
	int32 RGB = Compiler->ComponentMask(IndexInput, true, true, true, false);
	int32 Alpha = Compiler->ComponentMask(IndexInput, false, false, false, true);
	return Compiler->AppendVector(Compiler->Mul(RGB, Alpha), Alpha);
}

void UMaterialExpressionMaterialXPremult::GetCaption(TArray<FString>& OutCaptions) const
{
		OutCaptions.Add(TEXT("MaterialX Premult"));
}

#endif

#undef LOCTEXT_NAMESPACE 