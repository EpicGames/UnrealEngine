// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"

#include "MaterialExpressionPremult.generated.h"

/**
 * Multiply the RGB channels of the input by the Alpha channel of the input.
 * Input must be of type float4
 */
UCLASS(collapsecategories, hidecategories = Object, MinimalAPI, meta = (Private))
class UMaterialExpressionMaterialXPremult : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FExpressionInput Input;

	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpressionMaterialX Interface
};

