// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionRamp4.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionRamp4)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXRamp4"

UMaterialExpressionMaterialXRamp4::UMaterialExpressionMaterialXRamp4(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXRamp4::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Ramp4 input A"));
	}

	if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Ramp4 input B"));
	}
	
	if(!C.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Ramp4 input C"));
	}

	if(!D.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX Ramp4 input D"));
	}

	int32 IndexCoordinates = Coordinates.GetTracedInput().Expression ? Coordinates.Compile(Compiler) : Compiler->TextureCoordinate(ConstCoordinate, false, false);
	int32 IndexValueTL = A.Compile(Compiler);
	int32 IndexValueTR = B.Compile(Compiler);
	int32 IndexValueBL = C.Compile(Compiler);
	int32 IndexValueBR = D.Compile(Compiler);

	int32 TexClamp = Compiler->Saturate(IndexCoordinates);

	int32 S = Compiler->ComponentMask(TexClamp, 1, 0, 0, 0);
	int32 T = Compiler->ComponentMask(TexClamp, 0, 1, 0, 0);
	int32 MixTop = Compiler->Lerp(IndexValueTL, IndexValueTR, S);
	int32 MixBot = Compiler->Lerp(IndexValueBL, IndexValueBR, S);

	return Compiler->Lerp(MixBot, MixTop, T);
}

void UMaterialExpressionMaterialXRamp4::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Ramp4"));
}

void UMaterialExpressionMaterialXRamp4::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("A 4-corner bilinear value ramp."), 40, OutToolTip);
}

#endif

#undef LOCTEXT_NAMESPACE 
