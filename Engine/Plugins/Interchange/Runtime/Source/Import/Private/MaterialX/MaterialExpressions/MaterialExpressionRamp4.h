// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"

#include "MaterialExpressionRamp4.generated.h"

/**
 * A material expression that computes a 4-corner bilinear value ramp..
 */
UCLASS(collapsecategories, hidecategories = Object, MinimalAPI, meta = (Private))
class UMaterialExpressionMaterialXRamp4: public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstCoordinate' if not specified"))
	FExpressionInput Coordinates;

	/** The value at the top-left (U0V1) corner */
	UPROPERTY()
	FExpressionInput A;

	/** The value at the top-right (U1V1) corner */
	UPROPERTY()
	FExpressionInput B;

	/** The value at the bottom-left (U0V0) corner */
	UPROPERTY()
	FExpressionInput C;

	/** The value at the bottom-right (U1V0) corner */
	UPROPERTY()
	FExpressionInput D;

	/** only used if Coordinates is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpression, meta = (OverridingInputProperty = "Coordinates"))
	uint8 ConstCoordinate;

	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
#endif
	//~ End UMaterialExpressionMaterialX Interface
};

