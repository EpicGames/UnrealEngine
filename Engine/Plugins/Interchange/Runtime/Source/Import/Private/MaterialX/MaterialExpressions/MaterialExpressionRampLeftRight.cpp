// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionRampLeftRight.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionRampLeftRight)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXRampLeftRight"

UMaterialExpressionMaterialXRampLeftRight::UMaterialExpressionMaterialXRampLeftRight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXRampLeftRight::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX RampLR input A"));
	}

	if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX RampLR input B"));
	}

	int32 CoordinateIndex = Coordinates.GetTracedInput().Expression ? Coordinates.Compile(Compiler) : Compiler->TextureCoordinate(ConstCoordinate, false, false);

	return Compiler->Lerp(A.Compile(Compiler), B.Compile(Compiler), Compiler->Saturate(Compiler->ComponentMask(CoordinateIndex, true, false, false, false)));
}

void UMaterialExpressionMaterialXRampLeftRight::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX RampLR"));
}

#endif

#undef LOCTEXT_NAMESPACE 