// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"

#include "MaterialExpressionRampLeftRight.generated.h"

/**
 * A material expression that computes a left-to-right bilinear value ramp.
 */
UCLASS(collapsecategories, hidecategories = Object, MinimalAPI, meta = (Private))
class UMaterialExpressionMaterialXRampLeftRight : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstCoordinate' if not specified"))
	FExpressionInput Coordinates;

	UPROPERTY()
	FExpressionInput A;

	UPROPERTY()
	FExpressionInput B;

	/** only used if Coordinates is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionTextureSample, meta = (OverridingInputProperty = "Coordinates"))
	uint8 ConstCoordinate;

	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpressionMaterialX Interface
};

