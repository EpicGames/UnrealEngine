// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionRampTopBottom.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionRampTopBottom)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXRampLeftRight"

UMaterialExpressionMaterialXRampTopBottom::UMaterialExpressionMaterialXRampTopBottom(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXRampTopBottom::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX RampTB input A"));
	}

	if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX RampTB input B"));
	}

	int32 CoordinateIndex = Coordinates.GetTracedInput().Expression ? Coordinates.Compile(Compiler) : Compiler->TextureCoordinate(ConstCoordinate, false, false);

	return Compiler->Lerp(B.Compile(Compiler), A.Compile(Compiler), Compiler->Saturate(Compiler->ComponentMask(CoordinateIndex, false, true, false, false)));
}

void UMaterialExpressionMaterialXRampTopBottom::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX RampTB"));
}

#endif

#undef LOCTEXT_NAMESPACE 