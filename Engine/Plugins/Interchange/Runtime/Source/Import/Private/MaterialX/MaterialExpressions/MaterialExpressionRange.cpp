// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionRange.h"
#include "MaterialExpressionRemap.h"
#include "MaterialCompiler.h"
#include "Materials/MaterialExpressionAbs.h"
#include "Materials/MaterialExpressionClamp.h"
#include "Materials/MaterialExpressionDivide.h"
#include "Materials/MaterialExpressionIf.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialExpressionPower.h"
#include "Materials/MaterialExpressionSign.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionRange)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXRange"

UMaterialExpressionMaterialXRange::UMaterialExpressionMaterialXRange(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

	ConstInputLow = 0.f;
	ConstInputHigh = 1.f;
	ConstTargetLow = 0.f;
	ConstTargetHigh = 1.f;
	ConstGamma = 1.f;
	bConstClamp = false;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXRange::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	UMaterialExpressionMaterialXRemap* Remap1 = NewObject<UMaterialExpressionMaterialXRemap>();
	
	Remap1->Input = Input;
	Remap1->InputLow = InputLow;
	Remap1->InputHigh = InputHigh;
	Remap1->InputLowDefault = ConstInputLow;
	Remap1->InputHighDefault = ConstInputHigh;
	
	//First remap, we only take the inputs, the target is (0,1), constants will be applied when we compile
	Remap1->TargetLowDefault = 0.f;
	Remap1->TargetHighDefault = 1.f;

	//Inverse Gamma
	UMaterialExpressionDivide* InvGamma = NewObject<UMaterialExpressionDivide>();
	InvGamma->ConstA = 1.f;
	InvGamma->B = Gamma;
	InvGamma->ConstB = ConstGamma;

	//Abs Remap
	UMaterialExpressionAbs* AbsRemap1 = NewObject<UMaterialExpressionAbs>();
	AbsRemap1->Input.Connect(0, Remap1);

	//Gamma correct
	UMaterialExpressionPower* Power = NewObject<UMaterialExpressionPower>();
	Power->Base.Connect(0, AbsRemap1);
	Power->Exponent.Connect(0, InvGamma);

	// Sign of remap
	UMaterialExpressionSign* Sign = NewObject<UMaterialExpressionSign>();
	Sign->Input.Connect(0, Remap1);

	//Sign Gamma Correct
	UMaterialExpressionMultiply* GammaSign = NewObject<UMaterialExpressionMultiply>();
	GammaSign->A.Connect(0, Power);
	GammaSign->B.Connect(0, Sign);

	UMaterialExpressionMaterialXRemap* Remap2 = NewObject<UMaterialExpressionMaterialXRemap>();
	Remap2->Input.Connect(0, GammaSign);
	Remap2->InputLowDefault = 0.f;
	Remap2->InputHighDefault = 1.f;
	Remap2->TargetLow = TargetLow;
	Remap2->TargetHigh = TargetHigh;
	Remap2->TargetLowDefault = ConstTargetLow;
	Remap2->TargetHighDefault = ConstTargetHigh;

	// Only Create a Clamp expression if its an input
	if (Clamp.GetTracedInput().Expression)
	{
		UMaterialExpressionClamp* ExpressionClamp = NewObject<UMaterialExpressionClamp>();
		ExpressionClamp->Input.Connect(0, Remap2);
		ExpressionClamp->Min = TargetLow;
		ExpressionClamp->MinDefault = ConstTargetLow;
		ExpressionClamp->Max = TargetHigh;
		ExpressionClamp->MaxDefault = ConstTargetHigh;

		UMaterialExpressionIf * If = NewObject<UMaterialExpressionIf>();
		If->A = Clamp;
		If->ConstB = 1;

		If->AEqualsB.Connect(0, ExpressionClamp);
		If->AGreaterThanB.Connect(0, Remap2);
		If->ALessThanB.Connect(0, Remap2);

		return If->Compile(Compiler, OutputIndex);
	}
	else
	{
		if (bConstClamp)
		{
			UMaterialExpressionClamp* ExpressionClamp = NewObject<UMaterialExpressionClamp>();
			ExpressionClamp->Input.Connect(0, Remap2);
			ExpressionClamp->Min = TargetLow;
			ExpressionClamp->MinDefault = ConstTargetLow;
			ExpressionClamp->Max = TargetHigh;
			ExpressionClamp->MaxDefault = ConstTargetHigh;

			return ExpressionClamp->Compile(Compiler, OutputIndex);
		}
		else
		{
			return Remap2->Compile(Compiler, OutputIndex);
		}
	}
}

void UMaterialExpressionMaterialXRange::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Range"));
}
#endif

#undef LOCTEXT_NAMESPACE 