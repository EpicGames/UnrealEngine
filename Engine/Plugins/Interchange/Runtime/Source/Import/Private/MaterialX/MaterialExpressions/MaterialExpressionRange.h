// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"

#include "MaterialExpressionRange.generated.h"

/**
 * A material expression that Remap a value from one range to another, optionally
 * applying a gamma correction in the middle, and optionally clamping output values.
 */
UCLASS(collapsecategories, hidecategories = Object, MinimalAPI, meta = (Private))
class UMaterialExpressionMaterialXRange : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FExpressionInput Input;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Low value for input range. Defaults to 'ConstInputLowDefault' if not specified"))
	FExpressionInput InputLow;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "High value for input range. Defaults to 'ConstInputHighDefault' if not specified"))
	FExpressionInput InputHigh;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Low value for target range. Defaults to 'ConstTargetLowDefault' if not specified"))
	FExpressionInput TargetLow;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "High value for target range. Defaults to 'ConstTargetHighDefault' if not specified"))
	FExpressionInput TargetHigh;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Inverse exponent applied to Input after first transforming from InputLow..InputHigh to 0..1, gamma values greater than 1.0 make midtones brighter. Defaults to 'ConstGamma' if not specified"))
	FExpressionInput Gamma;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "If true, the output is clamped to the range TargetLow..TargetHigh. Defaults to 'ConstClamp' if not specified"))
	FExpressionInput Clamp;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionRange, meta = (OverridingInputProperty = "InputLow"))
	float ConstInputLow;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionRange, meta = (OverridingInputProperty = "InputHigh"))
	float ConstInputHigh;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionRange, meta = (OverridingInputProperty = "TargetLow"))
	float ConstTargetLow;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionRange, meta = (OverridingInputProperty = "TargetHigh"))
	float ConstTargetHigh;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionRange, meta = (OverridingInputProperty = "Gamma"))
	float ConstGamma;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionRange, meta = (OverridingInputProperty = "Clamp"))
	bool bConstClamp;

	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpressionMaterialX Interface
};

