// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionRemap.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionRemap)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXRemap"

UMaterialExpressionMaterialXRemap::UMaterialExpressionMaterialXRemap(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

	InputLowDefault = 0.0f;
	InputHighDefault = 1.0f;
	TargetLowDefault = 0.0f;
	TargetHighDefault = 1.0f;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXRemap::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Remap input"));
	}

	// Min2 + (Input - Min1) x (Max2 - Min2) / (Max1 - Min1)
	const int32 InputLowIndex = InputLow.GetTracedInput().Expression ? InputLow.Compile(Compiler) : Compiler->Constant(InputLowDefault);
	const int32 InputHighIndex = InputHigh.GetTracedInput().Expression ? InputHigh.Compile(Compiler) : Compiler->Constant(InputHighDefault);

	const int32 TargetLowIndex = TargetLow.GetTracedInput().Expression ? TargetLow.Compile(Compiler) : Compiler->Constant(TargetLowDefault);
	const int32 TargetHighIndex = TargetHigh.GetTracedInput().Expression ? TargetHigh.Compile(Compiler) : Compiler->Constant(TargetHighDefault);

	const int32 InputDelta = Compiler->Sub(InputHighIndex, InputLowIndex);
	const int32 TargetDelta = Compiler->Sub(TargetHighIndex, TargetLowIndex);
	const int32 Delta = Compiler->Div(TargetDelta, InputDelta);

	return Compiler->Add(TargetLowIndex, Compiler->Mul(Compiler->Sub(Input.Compile(Compiler), InputLowIndex), Delta));
}

void UMaterialExpressionMaterialXRemap::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Remap"));
}
#endif

#undef LOCTEXT_NAMESPACE 