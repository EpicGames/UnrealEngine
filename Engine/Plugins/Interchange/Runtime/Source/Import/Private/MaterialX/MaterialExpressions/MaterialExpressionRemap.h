// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"

#include "MaterialExpressionRemap.generated.h"

/**
 * A material expression that Remap a value from one range to another.
 */
UCLASS(collapsecategories, hidecategories = Object, MinimalAPI, meta = (Private))
class UMaterialExpressionMaterialXRemap : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FExpressionInput Input;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'MinDefault1' if not specified"))
	FExpressionInput InputLow;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'MaxDefault1' if not specified"))
	FExpressionInput InputHigh;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'MinDefault2' if not specified"))
	FExpressionInput TargetLow;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'MaxDefault1' if not specified"))
	FExpressionInput TargetHigh;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionClamp, meta = (OverridingInputProperty = "InputLow"))
	float InputLowDefault;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionClamp, meta = (OverridingInputProperty = "InputHigh"))
	float InputHighDefault;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionClamp, meta = (OverridingInputProperty = "TargetLow"))
	float TargetLowDefault;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionClamp, meta = (OverridingInputProperty = "TargetHigh"))
	float TargetHighDefault;

	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpressionMaterialX Interface
};

