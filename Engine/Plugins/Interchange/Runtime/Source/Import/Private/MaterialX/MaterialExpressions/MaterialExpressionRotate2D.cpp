// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialExpressionRotate2D.h"
#include "MaterialCompiler.h"
#include "Materials/MaterialExpressionCustom.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionRotate2D)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXRotate2D"

UMaterialExpressionMaterialXRotate2D::UMaterialExpressionMaterialXRotate2D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXRotate2D::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Rotate2D input"));
	}

	int32 IndexInput = Input.Compile(Compiler);
	int32 IndexAngle = RotationAngle.GetTracedInput().Expression ? RotationAngle.Compile(Compiler) : Compiler->Constant(ConstRotationAngle);

	int32 RotationRadians = Compiler->Mul(IndexAngle, Compiler->Constant(float(UE_PI) / 180.f));
	int32 SA = Compiler->Sine(RotationRadians);
	int32 CA = Compiler->Cosine(RotationRadians);

	int32 InputX = Compiler->ComponentMask(IndexInput, true, false, false, false);
	int32 InputY = Compiler->ComponentMask(IndexInput, false, true, false, false);
	
	return Compiler->AppendVector(
		Compiler->Add(Compiler->Mul(CA, InputX),
					  Compiler->Mul(SA, InputY)),

		Compiler->Sub(Compiler->Mul(CA, InputY),
					  Compiler->Mul(SA, InputX))
	);
}

void UMaterialExpressionMaterialXRotate2D::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Rotate2D"));
}

void UMaterialExpressionMaterialXRotate2D::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	OutToolTip.Add(TEXT("rotates a vector2 value about the origin in 2D."));
}
#endif

#undef LOCTEXT_NAMESPACE 