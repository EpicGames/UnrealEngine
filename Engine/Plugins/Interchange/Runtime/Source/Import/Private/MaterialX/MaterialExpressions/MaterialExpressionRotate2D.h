// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Materials/MaterialExpression.h"

#include "MaterialExpressionRotate2D.generated.h"

/**
 * A material expression that rotates a vector2 value about the origin in 2D.
 */
UCLASS(collapsecategories, hidecategories = Object, MinimalAPI, meta = (Private))
class UE_DEPRECATED(5.5, "UMaterialExpressionMaterialXRotate2D is now deprecated, please use UMaterialExpressionRotator") UMaterialExpressionMaterialXRotate2D : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FExpressionInput Input;

	/** The amount to rotate, specified in degrees, with positive values rotating the
	 *  incoming vector counterclockwise.*/
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "RotationAngle in degrees. Defaults to 'ConstRotationAngle' if not specified"))
	FExpressionInput RotationAngle;

	/** only used if RotationAngle is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionAngle, meta = (OverridingInputProperty = "RotationAngle"))
	float ConstRotationAngle = 0.f;

	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
#endif
	//~ End UMaterialExpressionMaterialX Interface
};

