// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionSplitLeftRight.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionSplitLeftRight)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXSplitLeftRight"

UMaterialExpressionMaterialXSplitLeftRight::UMaterialExpressionMaterialXSplitLeftRight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer),
	ConstCenter(0.5f),
	ConstCoordinate(0)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXSplitLeftRight::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX SplitLR input A"));
	}

	if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX SplitLR input B"));
	}

	int32 CoordinateIndex = Coordinates.GetTracedInput().Expression ? Coordinates.Compile(Compiler) : Compiler->TextureCoordinate(ConstCoordinate, false, false);
	int32 CenterIndex = Center.GetTracedInput().Expression ? Center.Compile(Compiler) : Compiler->Constant(ConstCenter);

	int32 TexcoordXIndex = Compiler->ComponentMask(CoordinateIndex, true, false, false, false);
	int32 InvSqrt2 = Compiler->Constant(0.70710678118654757f);
	int32 AFWidth = Compiler->Mul(Compiler->Length(Compiler->AppendVector(Compiler->DDX(TexcoordXIndex), Compiler->DDY(TexcoordXIndex))), InvSqrt2);
	int32 AAStep = Compiler->SmoothStep(Compiler->Sub(CenterIndex, AFWidth), Compiler->Add(CenterIndex, AFWidth), TexcoordXIndex);

	return Compiler->Lerp(A.Compile(Compiler), B.Compile(Compiler), AAStep);
}

void UMaterialExpressionMaterialXSplitLeftRight::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX SplitLR"));
}

#endif

#undef LOCTEXT_NAMESPACE 