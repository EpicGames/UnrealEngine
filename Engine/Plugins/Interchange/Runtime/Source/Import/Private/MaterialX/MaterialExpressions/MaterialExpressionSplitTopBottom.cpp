// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionSplitTopBottom.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionSplitTopBottom)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXSplitLeftRight"

UMaterialExpressionMaterialXSplitTopBottom::UMaterialExpressionMaterialXSplitTopBottom(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer),
	ConstCenter(0.5f),
	ConstCoordinate(0)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXSplitTopBottom::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX SplitTB input A"));
	}

	if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialX SplitTB input B"));
	}

	int32 CoordinateIndex = Coordinates.GetTracedInput().Expression ? Coordinates.Compile(Compiler) : Compiler->TextureCoordinate(ConstCoordinate, false, false);
	int32 CenterIndex = Center.GetTracedInput().Expression ? Center.Compile(Compiler) : Compiler->Constant(ConstCenter);

	int32 TexcoordYIndex = Compiler->ComponentMask(CoordinateIndex, false, true, false, false);
	int32 InvSqrt2 = Compiler->Constant(0.70710678118654757f);
	int32 AFWidth = Compiler->Mul(Compiler->Length(Compiler->AppendVector(Compiler->DDX(TexcoordYIndex), Compiler->DDY(TexcoordYIndex))), InvSqrt2);
	int32 AAStep = Compiler->SmoothStep(Compiler->Sub(CenterIndex, AFWidth), Compiler->Add(CenterIndex, AFWidth), TexcoordYIndex);

	return Compiler->Lerp(B.Compile(Compiler), A.Compile(Compiler), AAStep);
}

void UMaterialExpressionMaterialXSplitTopBottom::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX SplitTB"));
}

#endif

#undef LOCTEXT_NAMESPACE 