// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"

#include "MaterialExpressionSplitTopBottom.generated.h"

/**
 * A material expression that computes a top-bottom split matte, split at a specified v value.
 */
UCLASS(collapsecategories, hidecategories = Object, MinimalAPI, meta = (Private))
class UMaterialExpressionMaterialXSplitTopBottom : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstCoordinate' if not specified"))
	FExpressionInput Coordinates;

	UPROPERTY()
	FExpressionInput A;

	UPROPERTY()
	FExpressionInput B;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstCenter' if not specified"))
	FExpressionInput Center;

	/** only used if A is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionMultiply, meta = (OverridingInputProperty = "Center"))
	float ConstCenter;

	/** only used if Coordinates is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionTextureSample, meta = (OverridingInputProperty = "Coordinates"))
	uint8 ConstCoordinate;

	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpressionMaterialX Interface
};

