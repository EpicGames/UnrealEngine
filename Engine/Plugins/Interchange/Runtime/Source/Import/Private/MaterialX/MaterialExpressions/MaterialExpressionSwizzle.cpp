// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialX/MaterialExpressions/MaterialExpressionSwizzle.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionSwizzle)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXSwizzle"

UMaterialExpressionMaterialXSwizzle::UMaterialExpressionMaterialXSwizzle(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXSwizzle::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Swizzle input"));
	}

	int32 Index = Input.Compile(Compiler);

	if(Channels.Len() > 4)
	{
		return Compiler->Errorf(TEXT("Too many channels"));
	}

	if(Channels.IsEmpty())
	{
		return Index;
	}

	//Don't allow mixing xyzw and rgba or invalid channels
	bool bHasRGBA = false;
	bool bHasXYZW = false;
	bool SwizzleX[4]{ false, false, false, false };
	bool SwizzleY[4]{ false, false, false, false };
	bool SwizzleZ[4]{ false, false, false, false };
	bool SwizzleW[4]{ false, false, false, false };
	
	for(int32 i = 0; i < Channels.Len(); ++i)
	{
		bool bIsRGBAorXYZW = false;
		const TCHAR Channel = Channels[i];
		if(Channel == TEXT('r') || Channel == TEXT('g') || Channel == TEXT('b') || Channel == TEXT('a'))
		{
			bHasRGBA = true;
			bIsRGBAorXYZW = true;
		}
		else if(Channel == TEXT('x') || Channel == TEXT('y') || Channel == TEXT('z') || Channel == TEXT('w'))
		{
			bHasXYZW = true;
			bIsRGBAorXYZW = true;
		}
		if(bHasRGBA && bHasXYZW)
		{
			return Compiler->Errorf(TEXT("Cannot mix rgba and xyzw channels"));
		}
		if(!bIsRGBAorXYZW)
		{
			return Compiler->Errorf(TEXT("%c is not recognized as a valid channel"), Channel);
		}

		switch(Channel)
		{
		case TEXT('x'):
		case TEXT('r'):
			SwizzleX[i] = true;
			break;

		case TEXT('y'):
		case TEXT('g'):
			SwizzleY[i] = true;
			break;

		case TEXT('z'):
		case TEXT('b'):
			SwizzleZ[i] = true;
			break;

		case TEXT('w'):
		case TEXT('a'):
			SwizzleW[i] = true;
			break;
		}
	}
	
	int32 Mask[4]{
		Channels.Len() >= 1 ? Compiler->ComponentMask(Index, SwizzleX[0], SwizzleY[0], SwizzleZ[0], SwizzleW[0]) : -1,
		Channels.Len() >= 2 ? Compiler->ComponentMask(Index, SwizzleX[1], SwizzleY[1], SwizzleZ[1], SwizzleW[1]) : -1,
		Channels.Len() >= 3 ? Compiler->ComponentMask(Index, SwizzleX[2], SwizzleY[2], SwizzleZ[2], SwizzleW[2]) : -1,
		Channels.Len() >= 4 ? Compiler->ComponentMask(Index, SwizzleX[3], SwizzleY[3], SwizzleZ[3], SwizzleW[3]) : -1,
	};

	Input.Expression->GetOutputValueType(Input.OutputIndex);
	int32 Res = INDEX_NONE;

	switch(Channels.Len())
	{
	case 1:
		Res = Mask[0];
		break;

	case 2:
		Res = Compiler->AppendVector(Mask[0], Mask[1]);
		break;

	case 3:
		Res = Compiler->AppendVector(Compiler->AppendVector(Mask[0], Mask[1]), Mask[2]);
		break;

	case 4:
		Res = Compiler->AppendVector(Compiler->AppendVector(Compiler->AppendVector(Mask[0], Mask[1]), Mask[2]), Mask[3]);
		break;
	}

	return Res;
}

void UMaterialExpressionMaterialXSwizzle::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Swizzle"));
}

void UMaterialExpressionMaterialXSwizzle::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("UMaterialExpressionMaterialXSwizzle is now deprecated since <swizzle> node has been removed from MaterialX 1.39.3")
							  TEXT("Please use UMaterialExpressionComponent instead."), 40, OutToolTip);
}

#endif

#undef LOCTEXT_NAMESPACE 