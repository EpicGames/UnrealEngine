// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialX/MaterialExpressions/MaterialExpressionTextureSampleParameterBlur.h"
#include "Materials/MaterialExpressionAdd.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionCustom.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionUtils.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionTextureSampleParameterBlur)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXTextureSampleBlur"

namespace Gauss
{
	namespace
	{
		static constexpr float KernelWeights3x3[9] =
		{
			0.077847f, 0.123317f, 0.077847f,
			0.123317f, 0.195346f, 0.123317f,
			0.077847f, 0.123317f, 0.077847f
		};

		static constexpr float KernelWeights5x5[25] =
		{
			0.003765f, 0.015019f, 0.023792f, 0.015019f, 0.003765f,
			0.015019f, 0.059912f, 0.094907f, 0.059912f, 0.015019f,
			0.023792f, 0.094907f, 0.150342f, 0.094907f, 0.023792f,
			0.015019f, 0.059912f, 0.094907f, 0.059912f, 0.015019f,
			0.003765f, 0.015019f, 0.023792f, 0.015019f, 0.003765f
		};

		static constexpr float KernelWeights7x7[49] =
		{
			0.000036f, 0.000363f, 0.001446f, 0.002291f, 0.001446f, 0.000363f, 0.000036f,
			0.000363f, 0.003676f, 0.014662f, 0.023226f, 0.014662f, 0.003676f, 0.000363f,
			0.001446f, 0.014662f, 0.058488f, 0.092651f, 0.058488f, 0.014662f, 0.001446f,
			0.002291f, 0.023226f, 0.092651f, 0.146768f, 0.092651f, 0.023226f, 0.002291f,
			0.001446f, 0.014662f, 0.058488f, 0.092651f, 0.058488f, 0.014662f, 0.001446f,
			0.000363f, 0.003676f, 0.014662f, 0.023226f, 0.014662f, 0.003676f, 0.000363f,
			0.000036f, 0.000363f, 0.001446f, 0.002291f, 0.001446f, 0.000363f, 0.000036f
		};
	}
}

namespace Box
{
	namespace
	{
		static constexpr float KernelWeights3x3[9] =
		{
			1.f / 9.f, 1.f / 9.f, 1.f / 9.f,
			1.f / 9.f, 1.f / 9.f, 1.f / 9.f,
			1.f / 9.f, 1.f / 9.f, 1.f / 9.f,
		};

		static constexpr float KernelWeights5x5[25] =
		{
			1.f / 25.f, 1.f / 25.f, 1.f / 25.f, 1.f / 25.f, 1.f / 25.f,
			1.f / 25.f, 1.f / 25.f, 1.f / 25.f, 1.f / 25.f, 1.f / 25.f,
			1.f / 25.f, 1.f / 25.f, 1.f / 25.f, 1.f / 25.f, 1.f / 25.f,
			1.f / 25.f, 1.f / 25.f, 1.f / 25.f, 1.f / 25.f, 1.f / 25.f,
			1.f / 25.f, 1.f / 25.f, 1.f / 25.f, 1.f / 25.f, 1.f / 25.f,
		};

		static constexpr float KernelWeights7x7[49] =
		{
			1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f,
			1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f,
			1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f,
			1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f,
			1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f,
			1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f,
			1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f, 1.f / 49.f,
		};
	}
}

UMaterialExpressionMaterialXTextureSampleParameterBlur::UMaterialExpressionMaterialXTextureSampleParameterBlur(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Empty();
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR

/**
 * The function is defined in MaterialExpressions.cpp, easier to declare it extern it here rather than redefining it.
 */
extern int32 CompileTextureSample(
	FMaterialCompiler* Compiler,
	UTexture* Texture,
	int32 TexCoordCodeIndex,
	const EMaterialSamplerType SamplerType,
	const TOptional<FName> ParameterName = TOptional<FName>(),
	const int32 MipValue0Index = INDEX_NONE,
	const int32 MipValue1Index = INDEX_NONE,
	const ETextureMipValueMode MipValueMode = TMVM_None,
	const ESamplerSourceMode SamplerSource = SSM_FromTextureAsset,
	const bool AutomaticViewMipBias = false,
	const ETextureGatherMode GatherMode = TGM_None
);

int32 UMaterialExpressionMaterialXTextureSampleParameterBlur::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(KernelSize == EMAterialXTextureSampleBlurKernel::Kernel1)
	{
		return Super::Compile(Compiler, OutputIndex);
	}

	if(FString ErrorMessage; !TextureIsValid(Texture, ErrorMessage))
	{
		return CompilerError(Compiler, *ErrorMessage);
	}

	if(FString SamplerTypeError; !MaterialExpressionUtils::VerifySamplerType(Compiler->GetShaderPlatform(), Compiler->GetTargetPlatform(), Texture, SamplerType, SamplerTypeError))
	{
		return Compiler->Errorf(TEXT("%s"), *SamplerTypeError);
	}

	int32 IndexCoordinates = Coordinates.GetTracedInput().Expression ? Coordinates.Compile(Compiler) : Compiler->TextureCoordinate(ConstCoordinate, false, false);
	int32 IndexHalf = Compiler->Constant(0.5f);

	int32 IndexDerivUVx = Compiler->Mul(Compiler->DDX(IndexCoordinates), IndexHalf);
	int32 IndexDerivUVy = Compiler->Mul(Compiler->DDY(IndexCoordinates), IndexHalf);

	int32 IndexDerivX = Compiler->Add(Compiler->Abs(Compiler->ComponentMask(IndexDerivUVx, true, false, false, false)),
									 Compiler->Abs(Compiler->ComponentMask(IndexDerivUVy, true, false, false, false)));

	int32 IndexDerivY = Compiler->Add(Compiler->Abs(Compiler->ComponentMask(IndexDerivUVx, false, true, false, false)),
									  Compiler->Abs(Compiler->ComponentMask(IndexDerivUVy, false, true, false, false)));

	int32 IndexEpsilon = Compiler->Constant(UE_SMALL_NUMBER);
	int32 IndexTwo = Compiler->Constant(2.f);
	int32 IndexFilterSize = Compiler->Constant(FilterSize);
	int32 IndexFilterOffset = Compiler->Constant(FilterOffset);

	int32 IndexComputeSampleSizeUV = Compiler->AppendVector(Compiler->Max(Compiler->Add(Compiler->Mul(Compiler->Mul(IndexTwo, IndexFilterSize), IndexDerivX), IndexFilterOffset), IndexEpsilon),
															Compiler->Max(Compiler->Add(Compiler->Mul(Compiler->Mul(IndexTwo, IndexFilterSize), IndexDerivY), IndexFilterOffset), IndexEpsilon));

	int32 FilterWidth, Index = 0;
	const float* Kernel = GetKernel(FilterWidth);

	int32 IndexResult = Compiler->Constant4(0.f, 0.f, 0.f, 0.f);

	for(int32 Row = -FilterWidth; Row <= FilterWidth; ++Row)
	{
		for(int32 Col = -FilterWidth; Col <= FilterWidth; ++Col)
		{
			IndexResult = Compiler->Add(IndexResult,
										Compiler->Mul(
											Compiler->Constant(Kernel[Index++]),
											CompileTextureSample(
												Compiler,
												Texture,
												Compiler->Add(Compiler->Mul(IndexComputeSampleSizeUV, Compiler->Constant2(Col, Row)), IndexCoordinates),
												SamplerType,
												ParameterName,
												CompileMipValue0(Compiler),
												CompileMipValue1(Compiler),
												MipValueMode,
												SamplerSource,
												AutomaticViewMipBias)));
		}
	}

	return IndexResult;
}

void UMaterialExpressionMaterialXTextureSampleParameterBlur::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX ParamBlur"));
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

const float* UMaterialExpressionMaterialXTextureSampleParameterBlur::GetKernel(int32& FilterWidth) const
{
	const float* Kernel;

	switch(KernelSize)
	{
	case EMAterialXTextureSampleBlurKernel::Kernel5:
		FilterWidth = 5 / 2;
		Kernel = Filter == EMaterialXTextureSampleBlurFilter::Box ? Box::KernelWeights5x5 : Gauss::KernelWeights5x5;
		break;

	case EMAterialXTextureSampleBlurKernel::Kernel7:
		FilterWidth = 7 / 2;
		Kernel = Filter == EMaterialXTextureSampleBlurFilter::Box ? Box::KernelWeights7x7 : Gauss::KernelWeights7x7;
		break;

	default:
		FilterWidth = 3 / 2;
		Kernel = Filter == EMaterialXTextureSampleBlurFilter::Box ? Box::KernelWeights3x3 : Gauss::KernelWeights3x3;
		break;
	}

	return Kernel;
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE