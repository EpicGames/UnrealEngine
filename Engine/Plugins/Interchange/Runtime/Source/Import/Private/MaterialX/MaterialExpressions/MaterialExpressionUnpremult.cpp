// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialExpressionUnpremult.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionUnpremult)

#define LOCTEXT_NAMESPACE "MaterialExpressionMaterialXUnpremult"

UMaterialExpressionMaterialXUnpremult::UMaterialExpressionMaterialXUnpremult(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_MaterialX;
		FConstructorStatics()
			: NAME_MaterialX(LOCTEXT("MaterialX", "MaterialX"))
		{}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MaterialX);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialXUnpremult::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Unpremult input"));
	}

	int32 IndexInput = Input.Compile(Compiler);
	int32 RGB = Compiler->ComponentMask(IndexInput, true, true, true, false);
	int32 Alpha = Compiler->ComponentMask(IndexInput, false, false, false, true);

	int32 Unpremult = Compiler->AppendVector(Compiler->Div(RGB, Alpha), Alpha);
	return Compiler->If(Alpha, Compiler->Constant(0.f), Unpremult, IndexInput, Unpremult, Compiler->Constant(0.00001f));
}

void UMaterialExpressionMaterialXUnpremult::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialX Unpremult"));
}

#endif

#undef LOCTEXT_NAMESPACE 