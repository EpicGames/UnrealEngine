// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "MaterialX/MaterialXUtils/MaterialXBase.h"
#include "MaterialX/MaterialXUtils/MaterialXManager.h"
#include "InterchangeImportLog.h"

namespace mx = MaterialX;

FMaterialXBase::FMaterialXBase(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: NodeContainer(BaseNodeContainer)
	, bIsSubstrateEnabled(FMaterialXManager::GetInstance().IsSubstrateEnabled())
	, bIsSubstrateAdaptiveGBufferEnabled(FMaterialXManager::GetInstance().IsSubstrateAdaptiveGBufferEnabled())
{}

void FMaterialXBase::UpdateDocumentRecursively(MaterialX::GraphElementPtr Graph)
{
	AddTexCoordToTiledImageNodes(Graph);

	//This function will replace all the nodes that are defined with a nodegraph with their equivalent node network
	//This allows us to handle nodes defined with a nodegraph, with nodes that we already support while traversing the current node graph
	//For example the node tiledimage is just a succession of the nodes image, multiply, add, etc.
	Graph->flattenSubgraphs(mx::EMPTY_STRING, [](mx::NodePtr Node){return FMaterialXManager::GetInstance().FilterNodeGraph(Node);} );

	ConvertGeomPropValueNodesToTexCoord(Graph);
	ConvertNeutralNodesToDot(Graph);

	mx::DocumentPtr Document = Graph->getDocument();
	//we also need need to flatten the subgraphs
	for(mx::NodeGraphPtr SubGraph : Graph->getChildrenOfType<mx::NodeGraph>())
	{
		if(SubGraph->getActiveSourceUri() == Document->getSourceUri())
		{
			UpdateDocumentRecursively(SubGraph);
		}
	}
}

void FMaterialXBase::SetNodeDefinition(const char* NodeDef)
{
	this->NodeDefinition = NodeDef;
}

void FMaterialXBase::SetTranslator(const UInterchangeTranslatorBase* InTranslator)
{
	Translator = InTranslator;
}

void FMaterialXBase::CollectAnalytics(TArray<FAnalyticsEventAttribute>& OutAnalyticsAttributes)
{
	OutAnalyticsAttributes.Append(AnalyticsAttributes);
}

void FMaterialXBase::AddTexCoordToTiledImageNodes(MaterialX::ElementPtr Graph)
{
	for(mx::NodePtr Node : Graph->getChildrenOfType<mx::Node>(mx::Category::TiledImage))
	{
		if(Node->getCategory() == mx::Category::TiledImage)
		{
			mx::InputPtr InputTexCoord = Node->getInput("texcoord");

			if(!InputTexCoord)
			{
				mx::NodePtr NodeTexCoord = CreateNode(Node->getParent(),
													  "node_ue_texcoord",
													  mx::Category::TexCoord,
													  {},
													  { {"index", FAttributeValueArray{{"type", "integer"}, {"value", "0"}}} });

				NodeTexCoord->setType(mx::Type::Vector2);
				InputTexCoord = Node->addInput(mx::NodeGroup::Texture2D::Inputs::TexCoord, mx::Type::Vector2);
				InputTexCoord->setNodeName(NodeTexCoord->getName());
			}
		}
	}
}

void FMaterialXBase::ConvertNeutralNodesToDot(MaterialX::ElementPtr NodeGraph)
{
	const std::vector<mx::NodePtr> Nodes = NodeGraph->getChildrenOfType<mx::Node>();
	for(mx::NodePtr Node : Nodes)
	{
		const std::string& Category = Node->getCategory();
		if(Category == mx::Category::Multiply || Category == mx::Category::Divide || Category == mx::Category::Add || Category == mx::Category::Sub)
		{
			mx::InputPtr Input2 = Node->getInput("in2");
			bool bIsNeutral = false;

			if(!Input2)
			{
				bIsNeutral = true;
				if(mx::InputPtr Input1 = Node->getInput("in1"))
				{
					Input1->setName("in");
				}
			}
			else
			{
				if(!Input2->hasValue() && !Input2->hasNodeName() && !Input2->hasInterfaceName())
				{
					bIsNeutral = true;
					Node->removeInput("in2");
					if(mx::InputPtr Input1 = Node->getInput("in1"))
					{
						Input1->setName("in");
					}
				}
				else if(Input2->hasValue())
				{
					using namespace mx::NodeGroup::Math;

					auto IsValueStringNeutral = [&bIsNeutral, &Category, Input2](auto T)
					{
						bIsNeutral = (Category == mx::Category::Multiply || Category == mx::Category::Divide) ?
							mx::fromValueString<decltype(T)>(Input2->getValueString()) == NeutralOne<decltype(T)> :
							mx::fromValueString<decltype(T)>(Input2->getValueString()) == NeutralZero<decltype(T)>;
					};

					if(Input2->getType() == mx::Type::Float)
					{
						IsValueStringNeutral(float());
					}
					else if(Input2->getType() == mx::Type::Vector2)
					{
						IsValueStringNeutral(mx::Vector2());
					}
					else if(Input2->getType() == mx::Type::Vector3)
					{
						IsValueStringNeutral(mx::Vector3());
					}
					else if(Input2->getType() == mx::Type::Vector4)
					{
						IsValueStringNeutral(mx::Vector4());
					}
					else if(Input2->getType() == mx::Type::Color3)
					{
						IsValueStringNeutral(mx::Color3());
					}
					else if(Input2->getType() == mx::Type::Color4)
					{
						IsValueStringNeutral(mx::Color4());
					}

					if(bIsNeutral)
					{
						Node->removeInput("in2");
						if(mx::InputPtr Input1 = Node->getInput("in1"))
						{
							Input1->setName("in");
						}
					}
				}
			}

			if(bIsNeutral)
			{
				Node->setCategory(mx::Category::Dot);
			}
		}
	}
}

void FMaterialXBase::ConvertGeomPropValueNodesToTexCoord(MaterialX::GraphElementPtr Graph)
{
	const std::vector<mx::NodePtr> Nodes = Graph->getChildrenOfType<mx::Node>(mx::Category::GeomPropValue);
	for (mx::NodePtr Node : Nodes)
	{
		if (Node->getType() != mx::Type::Vector2)
		{
			continue;
		}

		mx::InputPtr Input = Node->getInput("geomprop");
		if (!Input)
		{
			continue;
		}

		// check if the value has either a value string or an interfacename
		if (!Input->hasValueString())
		{
			Input = Input->getInterfaceInput();
		}

		if (!Input)
		{
			continue;
		}

		if (Input->getValueString() != "st")
		{
			continue;
		}

		// the default UV set is 0 (input "index" value=0)
		Graph->changeChildCategory(Node, mx::Category::TexCoord);
	}
}

MaterialX::NodePtr FMaterialXBase::CreateNode(MaterialX::ElementPtr NodeGraph, const char* NodeName, const char* Category, const TArray<FInputToCopy>& InputsToCopy, const TArray<FInputToCreate>& InputsToCreate)
{
	std::string UniqueNodeName{ NodeName + std::string{"_"} + Category };

	mx::NodePtr Node = NodeGraph->getChildOfType<mx::Node>(UniqueNodeName);

	if(!Node)
	{
		Node = NodeGraph->addChildOfCategory(Category, UniqueNodeName)->asA<mx::Node>();

		for(const FInputToCopy& Pair : InputsToCopy)
		{
			const char* NewInputName = Pair.Get<0>();
			mx::InputPtr Input = Pair.Get<1>();

			mx::InputPtr InputCopy = Node->addInput();
			InputCopy->copyContentFrom(Input);
			//remove the attribute NewName since when the input will be copied, the name will be the one from the spec (before a renaming)
			InputCopy->removeAttribute(mx::Attributes::NewName);

			if(NewInputName)
			{
				InputCopy->setName(NewInputName);
			}
		}

		for(const FInputToCreate& Input : InputsToCreate)
		{
			const char* InputName = Input.Get<0>();
			const FAttributeValueArray& Attributes = Input.Get<1>();

			mx::InputPtr NewInput = Node->addInput();
			NewInput->setName(InputName);
			for(const FAttributeValue& AttributeValue : Attributes)
			{
				const char* Attribute = AttributeValue.Get<0>();
				const char* Value = AttributeValue.Get<1>();
				NewInput->setAttribute(Attribute, Value);
			}
		}
	}

	return Node;
}

FString FMaterialXBase::GetColorSpace(MaterialX::ElementPtr Element)
{
	FString ColorSpace;

	if(Element)
	{
		if(Element->hasColorSpace())
		{
			return FString(Element->getColorSpace().c_str());
		}
		else
		{
			return GetColorSpace(Element->getParent());
		}
	}

	return ColorSpace;
}

MaterialX::InputPtr FMaterialXBase::GetInput(MaterialX::NodePtr Node, const char* InputName)
{
	mx::InputPtr Input = Node->getInput(InputName);

	if(!Input)
	{
		Input = Node->getDocument()->getNodeDef(NodeDefinition)->getActiveInput(InputName);
	}

	return Input;
}

FLinearColor FMaterialXBase::GetLinearColor(MaterialX::InputPtr Input) const
{
	//we assume that the default color space is linear
	FLinearColor LinearColor = FLinearColor::Black;

	if(Input->getType() == mx::Type::Color3)
	{
		mx::Color3 Color = mx::fromValueString<mx::Color3>(Input->getValueString());
		LinearColor = FLinearColor{ Color[0], Color[1], Color[2] };
	}
	else if(Input->getType() == mx::Type::Color4)
	{
		mx::Color4 Color = mx::fromValueString<mx::Color4>(Input->getValueString());
		LinearColor = FLinearColor{ Color[0], Color[1], Color[2], Color[3] };
	}
	else
	{
		ensureMsgf(false, TEXT("input type can only be color3 or color4"));
	}

	const FString ColorSpace = GetColorSpace(Input);

	if(ColorSpace.IsEmpty() || ColorSpace == TEXT("lin_rec709") || ColorSpace == TEXT("none"))
	{
		;//noop
	}
	else if(ColorSpace == TEXT("gamma22"))
	{
		LinearColor = FLinearColor::FromPow22Color(FColor(LinearColor.R / 255.f, LinearColor.G / 255.f, LinearColor.B / 255.f, LinearColor.A / 255.f));
	}
	else
	{
		MTLX_LOG("MaterialXGetLinearColor", "<{0}>-<{1}>: Colorspace {2} is not supported yet, falling back to linear space",
			   Input->getParent()->getName().c_str(),
			   Input->getName().c_str(),
			   *ColorSpace);
	}

	return LinearColor;
}
#endif
