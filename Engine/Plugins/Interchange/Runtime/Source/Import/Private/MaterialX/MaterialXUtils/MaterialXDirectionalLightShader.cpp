// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "MaterialXDirectionalLightShader.h"

namespace mx = MaterialX;

FMaterialXDirectionalLightShader::FMaterialXDirectionalLightShader(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: FMaterialXLightShaderAbstract(BaseNodeContainer)
{
	NodeDefinition = mx::NodeDefinition::DirectionalLight;
}

TSharedRef<FMaterialXBase> FMaterialXDirectionalLightShader::MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	return MakeShared<FMaterialXDirectionalLightShader>(BaseNodeContainer);
}

UInterchangeBaseNode* FMaterialXDirectionalLightShader::Translate(MaterialX::NodePtr DirectionalLightShaderNode)
{
	this->LightShaderNode = DirectionalLightShaderNode;

	PreTranslate();

	UInterchangeDirectionalLightNode* DirectionalLightNode = NewObject<UInterchangeDirectionalLightNode>(&NodeContainer);

	// Direction
	{
		mx::InputPtr DirectionInput = GetInput(DirectionalLightShaderNode, mx::Lights::DirectionalLight::Input::Direction);
		GetDirection(DirectionInput, Transform);
	}

	this->LightNode = DirectionalLightNode;

	PostTranslate();

	return DirectionalLightNode;
}

void FMaterialXDirectionalLightShader::GetDirection(MaterialX::InputPtr DirectionInput, FTransform& Transform)
{
	mx::Vector3 Direction = mx::fromValueString<mx::Vector3>(DirectionInput->getValueString());

	//Get rotation to go from UE's default direction of directional light and the direction of the MX directional light node
	const FVector LightDirection{ Direction[2], Direction[0], Direction[1] };
	const FVector TransformDirection = Transform.GetUnitAxis(EAxis::X); // it's the default direction of a UE directional light
	Transform.SetRotation(FQuat::FindBetween(LightDirection, TransformDirection));
}
#endif