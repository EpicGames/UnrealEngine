// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "MaterialX/MaterialXUtils/MaterialXLightShaderAbstract.h"

class FMaterialXDirectionalLightShader : public FMaterialXLightShaderAbstract
{
protected:

	template<typename T, ESPMode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;

	FMaterialXDirectionalLightShader(UInterchangeBaseNodeContainer& BaseNodeContainer);

public:

	static TSharedRef<FMaterialXBase> MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer);

	virtual UInterchangeBaseNode* Translate(MaterialX::NodePtr PointLightShaderNode) override;

	static void GetDirection(MaterialX::InputPtr DirectionInput, FTransform& Transform);
};
#endif
