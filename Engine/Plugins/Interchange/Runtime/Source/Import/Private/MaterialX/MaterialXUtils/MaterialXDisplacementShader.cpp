// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "MaterialXDisplacementShader.h"
#include "InterchangeImportLog.h"

namespace mx = MaterialX;

FMaterialXDisplacementShader::FMaterialXDisplacementShader(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: FMaterialXSurfaceShaderAbstract(BaseNodeContainer)
{
	NodeDefinition = mx::NodeDefinition::DisplacementFloat; // we set by default a displacement float but the SurfaceMaterial should set the correct nodedef
}

TSharedRef<FMaterialXBase> FMaterialXDisplacementShader::MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	TSharedRef<FMaterialXDisplacementShader> Result = MakeShared<FMaterialXDisplacementShader>(BaseNodeContainer);
	Result->RegisterConnectNodeOutputToInputDelegates();
	return Result;
}

UInterchangeBaseNode* FMaterialXDisplacementShader::Translate(MaterialX::NodePtr DisplacementNode)
{
	using namespace UE::Interchange::Materials;
	this->SurfaceShaderNode = DisplacementNode;

	float DisplacementCenter = 0.f;

	// if we set the bias and scale in the details panel then we should remove it from the shader graph
	if (mx::InputPtr Input = DisplacementNode->getInput("scale"))
	{
		if (Input->hasValue() && Input->getTypedAttribute<bool>("ue_displacement_attribute"))
		{
			ShaderGraphNode->SetCustomDisplacementMagnitude(Input->getValue()->asA<float>());
			DisplacementNode->removeInput(Input->getName());
		}
	}

	if (mx::InputPtr Input = DisplacementNode->getInput("bias"))
	{
		if (Input->hasValue() && Input->getTypedAttribute<bool>("ue_displacement_attribute"))
		{
			DisplacementCenter = Input->getValue()->asA<float>();
			DisplacementNode->removeInput(Input->getName());
		}
	}

	UInterchangeShaderNode* DisplacementShaderNode = FMaterialXSurfaceShaderAbstract::Translate(EInterchangeMaterialXShaders::Displacement);
	// by default the center is at 0.5, but in order to compute correctly the normals from the displacement we need it at 0 in MX_Displacement
	// only if we're creating a shadergraph for the input and we're recomputing the normals (see MaterialXPipeline)
	ShaderGraphNode->SetCustomDisplacementCenterMode(DisplacementCenter);

	// Outputs
	UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, Common::Parameters::Displacement.ToString(), DisplacementShaderNode->GetUniqueID(), Common::Parameters::Displacement.ToString());

	AnalyticsAttributes.Emplace(TEXT("ShaderType"), TEXT("Displacement Shader"));

	return DisplacementShaderNode;
}
#endif