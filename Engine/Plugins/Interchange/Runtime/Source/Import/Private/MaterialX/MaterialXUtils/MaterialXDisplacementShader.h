// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "MaterialX/MaterialXUtils/MaterialXSurfaceShaderAbstract.h"

/** Handles <displacement> node from the pbrlib, we derive the class from FMaterialXSurfaceShaderAbstract to avoid issue when we StaticCast in the SurfaceMaterial, and because we also need a shader graph */
class FMaterialXDisplacementShader : public FMaterialXSurfaceShaderAbstract
{
protected:

	template<typename T, ESPMode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;

	FMaterialXDisplacementShader(UInterchangeBaseNodeContainer& BaseNodeContainer);

public:

	static TSharedRef<FMaterialXBase> MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer);

	virtual UInterchangeBaseNode* Translate(MaterialX::NodePtr DisplacementNode) override;
};
#endif