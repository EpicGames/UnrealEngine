// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialXGltfPbr.h"

#if WITH_EDITOR

namespace mx = MaterialX;

FMaterialXGltfPbr::FMaterialXGltfPbr(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: FMaterialXSurfaceShaderAbstract(BaseNodeContainer)
{
	NodeDefinition = mx::NodeDefinition::GltfPbr;
}

TSharedRef<FMaterialXBase> FMaterialXGltfPbr::MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	TSharedRef<FMaterialXGltfPbr> Result = MakeShared<FMaterialXGltfPbr>(BaseNodeContainer);
	Result->RegisterConnectNodeOutputToInputDelegates();
	return Result;
}

UInterchangeBaseNode* FMaterialXGltfPbr::Translate(MaterialX::NodePtr GltfPbrNode)
{
	this->SurfaceShaderNode = GltfPbrNode;

	using namespace UE::Interchange::Materials;

	UInterchangeShaderNode* GltfPbrShaderNode = FMaterialXSurfaceShaderAbstract::Translate(EInterchangeMaterialXShaders::GltfPBR);

	UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::FrontMaterial.ToString(), GltfPbrShaderNode->GetUniqueID(), SubstrateMaterial::Parameters::FrontMaterial.ToString());
	UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, Common::Parameters::Refraction.ToString(), GltfPbrShaderNode->GetUniqueID(), Common::Parameters::Refraction.ToString());
	UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::Occlusion.ToString(), GltfPbrShaderNode->GetUniqueID(), SubstrateMaterial::Parameters::Occlusion.ToString());

	return GltfPbrShaderNode;
}

MaterialX::InputPtr FMaterialXGltfPbr::GetInputNormal(MaterialX::NodePtr GltfPbrNode, const char*& InputNormal) const
{
	InputNormal = mx::GltfPbr::Input::Normal;
	mx::InputPtr Input = GltfPbrNode->getActiveInput(InputNormal);

	if (!Input)
	{
		Input = GltfPbrNode->getNodeDef(mx::EMPTY_STRING, true)->getActiveInput(InputNormal);
	}

	return Input;
}

#endif // WITH_EDITOR