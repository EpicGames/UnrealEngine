// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "MaterialX/MaterialXUtils/MaterialXSurfaceShaderAbstract.h"

class FMaterialXGltfPbr : public FMaterialXSurfaceShaderAbstract
{
	template<typename T, ESPMode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;

	FMaterialXGltfPbr(UInterchangeBaseNodeContainer& BaseNodeContainer);

public:

	static TSharedRef<FMaterialXBase> MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer);

	virtual UInterchangeBaseNode* Translate(MaterialX::NodePtr GltfPbrNode) override;

	virtual MaterialX::InputPtr GetInputNormal(MaterialX::NodePtr GltfPbrNode, const char*& InputNormal) const override;
};

#endif // WITH_EDITOR