// Copyright Epic Games, Inc. All Rights Reserved. 

#if WITH_EDITOR

#include "MaterialX/MaterialXUtils/MaterialXLightShaderAbstract.h"
#include "Misc/Paths.h"

namespace mx = MaterialX;

FMaterialXLightShaderAbstract::FMaterialXLightShaderAbstract(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: FMaterialXBase(BaseNodeContainer)
{}

void FMaterialXLightShaderAbstract::PreTranslate()
{
	const FString FileName{ FPaths::GetBaseFilename(LightShaderNode->getActiveSourceUri().c_str()) };
	const FString LightNodeLabel = FString{ LightShaderNode->getName().c_str() };
	SceneNode = NewObject<UInterchangeSceneNode>(&NodeContainer);
	const FString SceneNodeUID = TEXT("\\Light\\") + FileName + TEXT("\\") + LightNodeLabel;
	NodeContainer.SetupNode(SceneNode, SceneNodeUID, LightNodeLabel, EInterchangeNodeContainerType::TranslatedScene);
}

void FMaterialXLightShaderAbstract::PostTranslate()
{
	const FString LightNodeLabel = FString{ LightShaderNode->getName().c_str() };
	const FString LightNodeUid = TEXT("\\Light\\") + LightNodeLabel;
	NodeContainer.SetupNode(LightNode, LightNodeUid, LightNodeLabel, EInterchangeNodeContainerType::TranslatedAsset);
	
	const FString Uid = LightNode->GetUniqueID();
	SceneNode->SetCustomAssetInstanceUid(Uid);

	// Color
	{
		mx::InputPtr LightColor = LightShaderNode->getInput(mx::Lights::Input::Color);
		const FLinearColor Color = GetLinearColor(LightColor);
		LightNode->SetCustomLightColor(Color);
	}

	// Intensity
	{
		mx::InputPtr LightIntensity = LightShaderNode->getInput(mx::Lights::Input::Intensity);
		LightNode->SetCustomIntensity(mx::fromValueString<float>(LightIntensity->getValueString()));
	}

	SceneNode->SetCustomLocalTransform(&NodeContainer, Transform);
}
#endif