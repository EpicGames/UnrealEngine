// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialX/MaterialXUtils/MaterialXManager.h"
#include "InterchangeImportModule.h"
#include "Nodes/InterchangeSourceNode.h"
#include "Misc/PackageName.h"
#include "UObject/SoftObjectPath.h"

#if WITH_EDITOR
#include "MaterialXFormat/Util.h"
#include "MaterialX/InterchangeMaterialXDefinitions.h"
#include "MaterialX/MaterialXUtils/MaterialXOpenPBRSurfaceShader.h"
#include "MaterialX/MaterialXUtils/MaterialXSurfaceShader.h"
#include "MaterialX/MaterialXUtils/MaterialXStandardSurfaceShader.h"
#include "MaterialX/MaterialXUtils/MaterialXGltfPbr.h"
#include "MaterialX/MaterialXUtils/MaterialXUsdPreviewSurfaceShader.h"
#include "MaterialX/MaterialXUtils/MaterialXSurfaceUnlitShader.h"
#include "MaterialX/MaterialXUtils/MaterialXPointLightShader.h"
#include "MaterialX/MaterialXUtils/MaterialXDirectionalLightShader.h"
#include "MaterialX/MaterialXUtils/MaterialXSpotLightShader.h"
#include "MaterialX/MaterialXUtils/MaterialXSurfaceMaterial.h"
#include "MaterialX/MaterialXUtils/MaterialXDisplacementShader.h"
#include "MaterialX/MaterialXUtils/MaterialXMixShader.h"
#include "MaterialX/MaterialXUtils/MaterialXVolumeShader.h"
#include "MaterialX/MaterialXUtils/MaterialXVolumeMaterial.h"

#include "InterchangeAnalyticsHandlerBase.h"
#include "InterchangeHelper.h"
#include "InterchangeImportLog.h"
#include "InterchangeMaterialDefinitions.h"

#define LOCTEXT_NAMESPACE "InterchangeMaterialXManager"

namespace mx = MaterialX;

//not a good solution to use semicolon because of drive disk on Windows
const TCHAR FMaterialXManager::TexturePayloadSeparator = TEXT('{');

FMaterialXManager::FMaterialXManager()
	: MatchingInputNames{
	   {{TEXT(""),                            TEXT("amplitude")},   ExpressionInput(UE::Expressions::Inputs::Amplitude)},
	   {{TEXT(""),                            TEXT("amount")},      ExpressionInput(UE::Expressions::Inputs::Amount)},
	   {{TEXT(""),                            TEXT("bg")},          ExpressionInput(UE::Expressions::Inputs::B)},
	   {{TEXT(""),                            TEXT("center")},      ExpressionInput(UE::Expressions::Inputs::Center)},
	   {{TEXT(""),                            TEXT("diminish")},    ExpressionInput(UE::Expressions::Inputs::Diminish)},
	   {{TEXT(""),                            TEXT("doclamp")},     ExpressionInput(UE::Expressions::Inputs::Clamp)},
	   {{TEXT(""),                            TEXT("fg")},          ExpressionInput(UE::Expressions::Inputs::A)},
	   {{TEXT(""),                            TEXT("gamma")},       ExpressionInput(UE::Expressions::Inputs::Gamma)},
	   {{TEXT(""),                            TEXT("high")},        ExpressionInput(UE::Expressions::Inputs::Max)},
	   {{TEXT(""),                            TEXT("in")},          ExpressionInput(UE::Expressions::Inputs::Input)},
	   {{TEXT(""),                            TEXT("in1")},         ExpressionInput(UE::Expressions::Inputs::A)},
	   {{TEXT(""),                            TEXT("in2")},         ExpressionInput(UE::Expressions::Inputs::B)},
	   {{TEXT(""),                            TEXT("in3")},         ExpressionInput(UE::Expressions::Inputs::C)},
	   {{TEXT(""),                            TEXT("in4")},         ExpressionInput(UE::Expressions::Inputs::D)},
	   {{TEXT(""),                            TEXT("inlow")},       ExpressionInput(UE::Expressions::Inputs::InputLow)},
	   {{TEXT(""),                            TEXT("inhigh")},      ExpressionInput(UE::Expressions::Inputs::InputHigh)},
	   {{TEXT(""),                            TEXT("lacunarity")},  ExpressionInput(UE::Expressions::Inputs::Lacunarity)},
	   {{TEXT(""),                            TEXT("low")},         ExpressionInput(UE::Expressions::Inputs::Min)},
	   {{TEXT(""),                            TEXT("lumacoeffs")},  ExpressionInput(UE::Expressions::Inputs::LuminanceFactors)},
	   {{TEXT(""),                            TEXT("mix")},         ExpressionInput(UE::Expressions::Inputs::Alpha)},
	   {{TEXT(""),                            TEXT("offset")},      ExpressionInput(UE::Expressions::Inputs::Offset)},
	   {{TEXT(""),                            TEXT("pivot")},       ExpressionInput(UE::Expressions::Inputs::Pivot)},
	   {{TEXT(""),                            TEXT("position")},    ExpressionInput(UE::Expressions::Inputs::Position)},
	   {{TEXT(""),                            TEXT("texcoord")},    ExpressionInput(UE::Expressions::Inputs::Coordinates)},
	   {{TEXT(""),                            TEXT("octaves")},     ExpressionInput(UE::Expressions::Inputs::Octaves)},
	   {{TEXT(""),                            TEXT("temperature")}, ExpressionInput(UE::Expressions::Inputs::Temp)},
	   {{TEXT(""),                            TEXT("outlow")},      ExpressionInput(UE::Expressions::Inputs::TargetLow)},
	   {{TEXT(""),                            TEXT("outhigh")},     ExpressionInput(UE::Expressions::Inputs::TargetHigh)},
	   {{TEXT(""),                            TEXT("valuel")},      ExpressionInput(UE::Expressions::Inputs::A)},
	   {{TEXT(""),                            TEXT("valuer")},      ExpressionInput(UE::Expressions::Inputs::B)},
	   {{TEXT(""),                            TEXT("valuet")},      ExpressionInput(UE::Expressions::Inputs::A)},
	   {{TEXT(""),                            TEXT("valueb")},      ExpressionInput(UE::Expressions::Inputs::B)},
	   {{TEXT(""),                            TEXT("valuetl")},     ExpressionInput(UE::Expressions::Inputs::A)},
	   {{TEXT(""),                            TEXT("valuetr")},     ExpressionInput(UE::Expressions::Inputs::B)},
	   {{TEXT(""),                            TEXT("valuebl")},     ExpressionInput(UE::Expressions::Inputs::C)},
	   {{TEXT(""),                            TEXT("valuebr")},     ExpressionInput(UE::Expressions::Inputs::D)},
	   {{TEXT(""),                            TEXT("value1")},      ExpressionInput(UE::Expressions::Inputs::A)},
	   {{TEXT(""),                            TEXT("value2")},      ExpressionInput(UE::Expressions::Inputs::B)},
	   {{mx::Category::Atan2,                 TEXT("iny")},         ExpressionInput(UE::Expressions::Inputs::Y)},
	   {{mx::Category::Atan2,                 TEXT("inx")},         ExpressionInput(UE::Expressions::Inputs::X)},
	   {{mx::Category::Fractal2D,             TEXT("texcoord")},    ExpressionInput(UE::Expressions::Inputs::Position)},
	   {{mx::Category::HeightToNormal,        TEXT("scale")},       ExpressionInput(*UE::Interchange::Materials::Standard::Nodes::NormalFromHeightMap::Inputs::Intensity.ToString())},
	   {{mx::Category::IfGreater,             TEXT("in1")},         ExpressionInput(UE::Expressions::Inputs::AGreaterThanB)},
	   {{mx::Category::IfGreater,             TEXT("in2")},         ExpressionInput(UE::Expressions::Inputs::ALessThanB)},     //another input is added for the case 'equal', see ConnectIfGreater
	   {{mx::Category::IfGreaterEq,           TEXT("in1")},         ExpressionInput(UE::Expressions::Inputs::AGreaterThanB)},  //another input is added for the case 'equal', see ConnectIfGreaterEq
	   {{mx::Category::IfGreaterEq,           TEXT("in2")},         ExpressionInput(UE::Expressions::Inputs::ALessThanB)},
	   {{mx::Category::IfEqual,               TEXT("in1")},         ExpressionInput(UE::Expressions::Inputs::AEqualsB)},
	   {{mx::Category::IfEqual,               TEXT("in2")},         ExpressionInput(UE::Expressions::Inputs::ALessThanB)},     // another input is added for the case 'greater', see ConnectIfEqual
	   {{mx::Category::Inside,                TEXT("in")},          ExpressionInput(UE::Expressions::Inputs::A)},			   // Inside is treated as a Multiply node
	   {{mx::Category::Inside,                TEXT("mask")},        ExpressionInput(UE::Expressions::Inputs::B)},			   // Inside is treated as a Multiply node
	   {{mx::Category::Invert,                TEXT("amount")},      ExpressionInput(UE::Expressions::Inputs::A)},
	   {{mx::Category::Invert,                TEXT("in")},          ExpressionInput(UE::Expressions::Inputs::B)},
	   {{mx::Category::Mix,                   TEXT("fg")},          ExpressionInput(UE::Expressions::Inputs::B)},
	   {{mx::Category::Mix,                   TEXT("bg")},          ExpressionInput(UE::Expressions::Inputs::A)},
	   {{mx::Category::Mix,                   TEXT("mix")},		    ExpressionInput(UE::Expressions::Inputs::Factor)},
	   {{mx::Category::Noise2D,               TEXT("amplitude")},   ExpressionInput(UE::Expressions::Inputs::B)},              // The amplitude of the noise is connected to a multiply node
	   {{mx::Category::Noise2D,               TEXT("pivot")},       ExpressionInput(UE::Expressions::Inputs::B)},              // The pivot of the noise is connected to a add node
	   {{mx::Category::Noise3D,               TEXT("amplitude")},   ExpressionInput(UE::Expressions::Inputs::B)},              // The amplitude of the noise is connected to a multiply node
	   {{mx::Category::Noise3D,               TEXT("pivot")},       ExpressionInput(UE::Expressions::Inputs::B)},              // The pivot of the noise is connected to a add node
	   {{mx::Category::Normalize,             TEXT("in")},          ExpressionInput(UE::Expressions::Inputs::VectorInput)},
	   {{mx::Category::NormalMap,             TEXT("in")},          ExpressionInput(UE::Expressions::Inputs::Normal)},
	   {{mx::Category::NormalMap,             TEXT("scale")},       ExpressionInput(UE::Expressions::Inputs::Flatness)},
	   {{mx::Category::Outside,               TEXT("in")},          ExpressionInput(UE::Expressions::Inputs::A)},			   // Outside is treated as Multiply node
	   {{mx::Category::Outside,               TEXT("mask")},        ExpressionInput(UE::Expressions::Inputs::B)},			   // Outside is treated as Multiply node
	   {{mx::Category::Power,                 TEXT("in1")},         ExpressionInput(UE::Expressions::Inputs::Base)},
	   {{mx::Category::Power,                 TEXT("in2")},         ExpressionInput(UE::Expressions::Inputs::Exponent)},
	   {{mx::Category::Refract,               TEXT("in")},          ExpressionInput(UE::Expressions::Inputs::RayDirection)},
	   {{mx::Category::Refract,               TEXT("normal")},      ExpressionInput(UE::Expressions::Inputs::SurfaceNormal)},
	   {{mx::Category::Refract,               TEXT("ior")},         ExpressionInput(UE::Expressions::Inputs::RefractiveIndexOrigin)},
	   {{mx::Category::Rotate2D,              TEXT("in")},          ExpressionInput(UE::Expressions::Inputs::Coordinate)},
	   {{mx::Category::Rotate2D,              TEXT("amount")},      ExpressionInput(UE::Expressions::Inputs::Time)},
	   {{mx::Category::Smoothstep,            TEXT("in")},          ExpressionInput(UE::Expressions::Inputs::Value)},
	   {{mx::Category::Separate2,             TEXT("in")},          ExpressionInput(UE::Expressions::Inputs::Float2)},
	   {{mx::Category::Separate3,             TEXT("in")},          ExpressionInput(UE::Expressions::Inputs::Float3)},
	   {{mx::Category::Separate4,             TEXT("in")},          ExpressionInput(UE::Expressions::Inputs::Float4)},
	   {{mx::Category::Switch,                TEXT("in1")},         ExpressionInput(UE::Expressions::Inputs::in1)},
	   {{mx::Category::Switch,                TEXT("in2")},         ExpressionInput(UE::Expressions::Inputs::in2)},
	   {{mx::Category::Switch,                TEXT("in3")},         ExpressionInput(UE::Expressions::Inputs::in3)},
	   {{mx::Category::Switch,                TEXT("in4")},         ExpressionInput(UE::Expressions::Inputs::in4)},
	   {{mx::Category::Switch,                TEXT("in5")},         ExpressionInput(UE::Expressions::Inputs::in5)},
	   {{mx::Category::Switch,                TEXT("in6")},         ExpressionInput(UE::Expressions::Inputs::in6)},
	   {{mx::Category::Switch,                TEXT("in7")},         ExpressionInput(UE::Expressions::Inputs::in7)},
	   {{mx::Category::Switch,                TEXT("in8")},         ExpressionInput(UE::Expressions::Inputs::in8)},
	   {{mx::Category::Switch,                TEXT("in9")},         ExpressionInput(UE::Expressions::Inputs::in9)},
	   {{mx::Category::Switch,                TEXT("in10")},        ExpressionInput(UE::Expressions::Inputs::in10)},
	   {{mx::Category::Switch,                TEXT("which")},       ExpressionInput(UE::Expressions::Inputs::SwitchValue)},
	}
	, MatchingMaterialExpressions {
		// Math nodes
		{mx::Category::Absval,       UE::Expressions::Names::Abs},
		{mx::Category::Add,          UE::Expressions::Names::Add},
		{mx::Category::Acos,         UE::Expressions::Names::Arccosine},
		{mx::Category::Asin,         UE::Expressions::Names::Arcsine},
		{mx::Category::Atan2,        UE::Expressions::Names::Arctangent2},
		{mx::Category::Ceil,         UE::Expressions::Names::Ceil},
		{mx::Category::Clamp,        UE::Expressions::Names::Clamp},
		{mx::Category::Cos,          UE::Expressions::Names::Cosine},
		{mx::Category::CrossProduct, UE::Expressions::Names::Crossproduct},
		{mx::Category::Divide,       UE::Expressions::Names::Divide},
		{mx::Category::Distance,     UE::Expressions::Names::Distance},
		{mx::Category::DotProduct,   UE::Expressions::Names::Dotproduct},
		{mx::Category::Exp,          UE::Expressions::Names::Exponential},
		{mx::Category::Floor,        UE::Expressions::Names::Floor},
		{mx::Category::Fract,        UE::Expressions::Names::Frac},
		{mx::Category::Invert,       UE::Expressions::Names::Subtract},
		{mx::Category::Ln,           UE::Expressions::Names::Logarithm},
		{mx::Category::Magnitude,    UE::Expressions::Names::Length},
		{mx::Category::Max,          UE::Expressions::Names::Max},
		{mx::Category::Min,          UE::Expressions::Names::Min},
		{mx::Category::Modulo,       UE::Expressions::Names::MaterialXMod},
		{mx::Category::Multiply,     UE::Expressions::Names::Multiply},
		{mx::Category::Normalize,    UE::Expressions::Names::Normalize},
		{mx::Category::Power,        UE::Expressions::Names::Power},
		{mx::Category::RampLR,       UE::Expressions::Names::MaterialXRampLeftRight},
		{mx::Category::RampTB,       UE::Expressions::Names::MaterialXRampTopBottom},
		{mx::Category::Round,        UE::Expressions::Names::Round},
		{mx::Category::Sign,         UE::Expressions::Names::Sign},
		{mx::Category::Sin,          UE::Expressions::Names::Sine},
		{mx::Category::SplitLR,      UE::Expressions::Names::MaterialXSplitLeftRight},
		{mx::Category::SplitTB,      UE::Expressions::Names::MaterialXSplitTopBottom},
		{mx::Category::Sqrt,         UE::Expressions::Names::SquareRoot},
		{mx::Category::Sub,          UE::Expressions::Names::Subtract},
		{mx::Category::Tan,          UE::Expressions::Names::Tangent},
		// Compositing nodes
		{mx::Category::Burn,         UE::Expressions::Names::MaterialXBurn},
		{mx::Category::Difference,   UE::Expressions::Names::MaterialXDifference},
		{mx::Category::Disjointover, UE::Expressions::Names::MaterialXDisjointover},
		{mx::Category::Dodge,        UE::Expressions::Names::MaterialXDodge},
		{mx::Category::In,           UE::Expressions::Names::MaterialXIn},
		{mx::Category::Inside,       UE::Expressions::Names::Multiply},
		{mx::Category::Mask,         UE::Expressions::Names::MaterialXMask},
		{mx::Category::Matte,        UE::Expressions::Names::MaterialXMatte},
		{mx::Category::Minus,        UE::Expressions::Names::MaterialXMinus},
		{mx::Category::Mix,          UE::Expressions::Names::Lerp},
		{mx::Category::Out,          UE::Expressions::Names::MaterialXOut},
		{mx::Category::Over,         UE::Expressions::Names::MaterialXOver},
		{mx::Category::Overlay,      UE::Expressions::Names::MaterialXOverlay},
		{mx::Category::Plus,         UE::Expressions::Names::MaterialXPlus},
		{mx::Category::Premult,      UE::Expressions::Names::MaterialXPremult},
		{mx::Category::Screen,       UE::Expressions::Names::MaterialXScreen},
		{mx::Category::Unpremult,    UE::Expressions::Names::MaterialXUnpremult},
		// Channel nodes
		{mx::Category::Combine2,     UE::Expressions::Names::AppendVector},
		{mx::Category::Combine3,     UE::Expressions::Names::MaterialXAppend3Vector},
		{mx::Category::Combine4,     UE::Expressions::Names::MaterialXAppend4Vector},
		// PBR
		{mx::Category::Blackbody,    UE::Expressions::Names::BlackBody},
		// Procedural2D nodes
		{mx::Category::Ramp4,        UE::Expressions::Names::MaterialXRamp4},
		{mx::Category::RampLR,       UE::Expressions::Names::MaterialXRampLeftRight},
		{mx::Category::RampTB,       UE::Expressions::Names::MaterialXRampTopBottom},
		{mx::Category::SplitLR,      UE::Expressions::Names::MaterialXSplitLeftRight},
		{mx::Category::SplitTB,      UE::Expressions::Names::MaterialXSplitTopBottom},
		// Geometric nodes 
		{mx::Category::GeomColor,    UE::Expressions::Names::VertexColor},
		// Adjustment nodes,
		{mx::Category::Contrast,     UE::Expressions::Names::Contrast},
		{mx::Category::HsvToRgb,     UE::Expressions::Names::HsvToRgb},
		{mx::Category::Range,        UE::Expressions::Names::MaterialXRange},
		{mx::Category::Remap,        UE::Expressions::Names::MaterialXRemap},
		{mx::Category::RgbToHsv,     UE::Expressions::Names::RgbToHsv},
		{mx::Category::Smoothstep,   UE::Expressions::Names::SmoothStep}
	}
	, MaterialXContainerDelegates {
		{mx::Category::OpenPBRSurface, FMaterialXManager::FOnGetMaterialXInstance::CreateStatic(&FMaterialXOpenPBRSurfaceShader::MakeInstance)},
		{mx::Category::Surface, FMaterialXManager::FOnGetMaterialXInstance::CreateStatic(&FMaterialXSurfaceShader::MakeInstance)},
		{mx::Category::SurfaceUnlit, FMaterialXManager::FOnGetMaterialXInstance::CreateStatic(&FMaterialXSurfaceUnlitShader::MakeInstance)},
		{mx::Category::StandardSurface, FMaterialXManager::FOnGetMaterialXInstance::CreateStatic(&FMaterialXStandardSurfaceShader::MakeInstance)},
		{mx::Category::GltfPbr, FMaterialXManager::FOnGetMaterialXInstance::CreateStatic(&FMaterialXGltfPbr::MakeInstance)},
		{mx::Category::UsdPreviewSurface, FMaterialXManager::FOnGetMaterialXInstance::CreateStatic(&FMaterialXUsdPreviewSurfaceShader::MakeInstance)},
		{mx::Category::Displacement, FMaterialXManager::FOnGetMaterialXInstance::CreateStatic(&FMaterialXDisplacementShader::MakeInstance)},
		{mx::Category::Mix, FMaterialXManager::FOnGetMaterialXInstance::CreateStatic(&FMaterialXMixShader::MakeInstance)},
		{mx::Category::PointLight, FMaterialXManager::FOnGetMaterialXInstance::CreateStatic(&FMaterialXPointLightShader::MakeInstance)},
		{mx::Category::DirectionalLight, FMaterialXManager::FOnGetMaterialXInstance::CreateStatic(&FMaterialXDirectionalLightShader::MakeInstance)},
		{mx::Category::SpotLight, FMaterialXManager::FOnGetMaterialXInstance::CreateStatic(&FMaterialXSpotLightShader::MakeInstance)},
		{mx::SURFACE_MATERIAL_NODE_STRING.c_str(), FMaterialXManager::FOnGetMaterialXInstance::CreateStatic(&FMaterialXSurfaceMaterial::MakeInstance)},
		{mx::VOLUME_MATERIAL_NODE_STRING.c_str(), FMaterialXManager::FOnGetMaterialXInstance::CreateStatic(&FMaterialXVolumeMaterial::MakeInstance)},
		{mx::Category::Volume, FMaterialXManager::FOnGetMaterialXInstance::CreateStatic(&FMaterialXVolumeShader::MakeInstance)},
	}
	, MatchingMaterialFunctions {

		// In case of StandardSurface and OpenPBR we set by default the value to the opaque shader, we decide later if we need to change for the transmission shader
		// Surface Shader nodes
		{mx::Category::StandardSurface,					FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXShaders>{}, EInterchangeMaterialXShaders::StandardSurface}},
		{mx::Category::OpenPBRSurface,					FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXShaders>{}, EInterchangeMaterialXShaders::OpenPBRSurface}},
		{mx::Category::UsdPreviewSurface,				FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXShaders>{}, EInterchangeMaterialXShaders::UsdPreviewSurface}},
		{mx::Category::Surface,							FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXShaders>{}, EInterchangeMaterialXShaders::Surface}},
		{mx::Category::SurfaceUnlit,					FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXShaders>{}, EInterchangeMaterialXShaders::SurfaceUnlit}},
		{mx::Category::Displacement,					FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXShaders>{}, EInterchangeMaterialXShaders::Displacement}},
		{mx::Category::Volume,							FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXShaders>{}, EInterchangeMaterialXShaders::Volume}},

		// BSDF Nodes
		{mx::Category::BurleyDiffuseBSDF,				FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXBSDF>{}, EInterchangeMaterialXBSDF::BurleyDiffuse}},
		{mx::Category::ConductorBSDF,					FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXBSDF>{}, EInterchangeMaterialXBSDF::Conductor}},
		{mx::Category::DielectricBSDF,					FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXBSDF>{}, EInterchangeMaterialXBSDF::Dielectric}},
		{mx::Category::GeneralizedSchlickBSDF,			FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXBSDF>{}, EInterchangeMaterialXBSDF::GeneralizedSchlick}},
		{mx::Category::OrenNayarDiffuseBSDF ,			FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXBSDF>{}, EInterchangeMaterialXBSDF::OrenNayarDiffuse}},
		{mx::Category::SheenBSDF,						FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXBSDF>{}, EInterchangeMaterialXBSDF::Sheen}},
		{mx::Category::SubsurfaceBSDF ,					FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXBSDF>{}, EInterchangeMaterialXBSDF::Subsurface}},
		{mx::Category::TranslucentBSDF,					FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXBSDF>{}, EInterchangeMaterialXBSDF::Translucent}},
		{mx::Category::ChiangHairBSDF,					FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXBSDF>{}, EInterchangeMaterialXBSDF::ChiangHair}},
		// EDF Nodes
		{mx::Category::ConicalEDF,						FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXEDF>{}, EInterchangeMaterialXEDF::Conical}},
		{mx::Category::MeasuredEDF,						FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXEDF>{}, EInterchangeMaterialXEDF::Measured}},
		{mx::Category::UniformEDF,						FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXEDF>{}, EInterchangeMaterialXEDF::Uniform}},
		// VDF Nodes
		{mx::Category::AbsorptionVDF,					FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXVDF>{}, EInterchangeMaterialXVDF::Absorption}},
		{mx::Category::AnisotropicVDF,					FMaterialXMaterialFunction{TInPlaceType<EInterchangeMaterialXVDF>{}, EInterchangeMaterialXVDF::Anisotropic}},
		// Utility nodes
		{mx::Category::ArtisticIOR,						FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxArtisticIOR}},
		{mx::Category::ChiangHairAbsorptionFromColor,	FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxChiangHairAbsorptionFromColor}},
		{mx::Category::ChiangHairRoughness,				FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxChiangHairRoughness}},
		{mx::Category::DeonHairAbsorptionFromMelanin,	FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxDeonHairAbsorptionFromMelanin}},
		{mx::Category::RoughnessAnisotropy,				FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxRoughnesAnisotropy}},
		{mx::Category::RoughnessDual,					FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxRoughnessDual}},
		// Math																								
		{mx::Category::Place2D,							FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxPlace2D}},
		{mx::Category::Refract,							FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::Refract}},
		{mx::Category::Rotate3D,						FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxRotate3D}},
		// ColorTransform																					
		{mx::Category::ACEScgToLinRec709,				FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxACEScgToRec709}},
		{mx::Category::AdobeRgbToLinRec709,				FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxAdobeRGBToRec709}},
		{mx::Category::LinAdobeRgbToLinRec709,			FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxLinearAdobeRGBToRec709}},
		{mx::Category::LinDisplayP3ToLinRec709,			FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxLinearDisplayP3ToRec709}},
		{mx::Category::SrgbDisplayP3ToLinRec709,		FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxSrgbDisplayP3ToRec709}},
		{mx::Category::SrgbTextureToLinRec709,			FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxSrgbToRec709}},
		// Procedural																						
		{mx::Category::Checkerboard,					FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxCheckerboard}},
		{mx::Category::Circle,							FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxCircle}},
		{mx::Category::Line,							FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxLine}},
		{mx::Category::Ramp,							FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxRamp}},
		{mx::Category::RampGradient,					FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxRampGradient}},
		{mx::Category::RandomColor,						FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxRandomColor}},
		{mx::Category::RandomFloat,						FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxRandomFloat}},
		{mx::Category::TiledCircles,					FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxTiledCircles}},
		{mx::Category::UnifiedNoise3D,					FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxUnifiedNoise3D}},
		// Adjustment																						
		{mx::Category::ColorCorrect,					FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxColorCorrect}},
		{mx::Category::HsvAdjust,						FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxHsvAdjust}},
		// NPR
		{mx::Category::GoochShade,						FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::MxGoochShade}},
		// Channel
		{mx::Category::Separate2,						FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::BreakOutFloat2Components}},
		{mx::Category::Separate3,						FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::BreakOutFloat3Components}},
		{mx::Category::Separate4,						FMaterialXMaterialFunction{TInPlaceType<FString>{}, UE::MaterialFunctions::Path::BreakOutFloat4Components}},
	}
	, CategoriesToSkip {
		mx::Category::ACEScgToLinRec709,
		mx::Category::AdobeRgbToLinRec709,
		mx::Category::Blur,
		mx::Category::Checkerboard,
		mx::Category::ChiangHairAbsorptionFromColor,
		mx::Category::ChiangHairRoughness,
		mx::Category::Circle,
		mx::Category::ColorCorrect,
		mx::Category::Contrast,
		mx::Category::Distance,
		mx::Category::DeonHairAbsorptionFromMelanin,
		mx::Category::Extract,
		mx::Category::Fractal2D,
		mx::Category::Fractal3D,
		mx::Category::GltfPbr,
		mx::Category::GoochShade,
		mx::Category::HsvAdjust,
		mx::Category::LatLongImage,
		mx::Category::Line,
		mx::Category::LinAdobeRgbToLinRec709,
		mx::Category::LinDisplayP3ToLinRec709,
		mx::Category::Noise2D,
		mx::Category::Noise3D,
		mx::Category::OpenPBRSurface,
		mx::Category::Overlay,
		mx::Category::Place2D,
		mx::Category::Ramp,
		mx::Category::RampGradient,
		mx::Category::Ramp4,
		mx::Category::RandomFloat,
		mx::Category::RandomColor,
		mx::Category::Range,
		mx::Category::Refract,
		mx::Category::Separate2,
		mx::Category::Separate3,
		mx::Category::Separate4,
		mx::Category::SrgbDisplayP3ToLinRec709,
		mx::Category::SrgbTextureToLinRec709,
		mx::Category::StandardSurface,
		mx::Category::Switch,
		mx::Category::TiledCircles,
		mx::Category::Time, // since 1.39.3, they added a nodegraph in stdlib/genglsl/stdlib_genglsl_impl.mtlx instead of stdlib_ng.mtlx
		mx::Category::UnifiedNoise3D,
		mx::Category::UsdPreviewSurface,
		mx::Category::Xor,
	}
	, NodeDefsCategories{
		mx::Category::Add,
		mx::Category::Combine2,
		mx::Category::Combine3,
		mx::Category::Combine4,
		mx::Category::Constant,
		mx::Category::Divide,
		mx::Category::IfEqual,
		mx::Category::IfGreater,
		mx::Category::IfGreaterEq,
		mx::Category::Max,
		mx::Category::Min,
		mx::Category::Sub,
	}
	, NodeInputsToRemove{
		{mx::Category::GeomColor, {"index"}},                          // There's only one set of VertexColor
		{mx::Category::NormalMap, {"normal", "tangent", "bitangent"}}, // FlattenNormalMap Material Function doesn't have normal/tangent inputs, we just remove them to avoid unnecessary connections
	}
	, bIsSubstrateEnabled{ IInterchangeImportModule::IsAvailable() ? IInterchangeImportModule::Get().IsSubstrateEnabled() : false }
	, bIsSubstrateAdaptiveGBufferEnabled{ IInterchangeImportModule::IsAvailable() ? IInterchangeImportModule::Get().IsSubstrateAdaptiveGBufferEnabled() : false }
{
	if (bIsSubstrateEnabled)
	{
		// Vertical Layering
		MatchingInputNames.Add({ TEXT(""), TEXT("top") }, ExpressionInput(UE::Expressions::Inputs::Top));
		MatchingInputNames.Add({ TEXT(""), TEXT("base") }, ExpressionInput(UE::Expressions::Inputs::Bottom));
		MatchingMaterialExpressions.Add({ mx::Category::Layer }, UE::Expressions::Names::SubstrateVerticalLayering);

		// Horizontal Layering, MaterialX names it mix so we have to be careful not to take the lerp expression
		MatchingInputNames.Add({ FKeyExpression{mx::Category::Mix, mx::NodeGroup::PBR, mx::Type::BSDF}, TEXT("bg") }, ExpressionInput(UE::Expressions::Inputs::Background));
		MatchingInputNames.Add({ FKeyExpression{mx::Category::Mix, mx::NodeGroup::PBR, mx::Type::BSDF}, TEXT("fg") }, ExpressionInput(UE::Expressions::Inputs::Foreground));
		MatchingInputNames.Add({ FKeyExpression{mx::Category::Mix, mx::NodeGroup::PBR, mx::Type::BSDF}, TEXT("mix") }, ExpressionInput(UE::Expressions::Inputs::Mix));
		MatchingMaterialExpressions.Add({ mx::Category::Mix, mx::NodeGroup::PBR, mx::Type::BSDF}, UE::Expressions::Names::SubstrateHorizontalMixing);

		// Mix surfaceshaders
		MatchingInputNames.Add({ FKeyExpression{mx::Category::Mix, mx::NodeGroup::Compositing, mx::Type::SurfaceShader}, TEXT("bg") }, ExpressionInput(UE::Expressions::Inputs::Background));
		MatchingInputNames.Add({ FKeyExpression{mx::Category::Mix, mx::NodeGroup::Compositing, mx::Type::SurfaceShader}, TEXT("fg") }, ExpressionInput(UE::Expressions::Inputs::Foreground));
		MatchingInputNames.Add({ FKeyExpression{mx::Category::Mix, mx::NodeGroup::Compositing, mx::Type::SurfaceShader}, TEXT("mix") }, ExpressionInput(UE::Expressions::Inputs::Mix));
		MatchingMaterialExpressions.Add({ mx::Category::Mix, mx::NodeGroup::Compositing, mx::Type::SurfaceShader }, UE::Expressions::Names::SubstrateHorizontalMixing);

		// Mix volumeshaders
		MatchingInputNames.Add({ FKeyExpression{mx::Category::Mix, mx::NodeGroup::Compositing, mx::Type::VolumeShader}, TEXT("bg") }, ExpressionInput(UE::Expressions::Inputs::Background));
		MatchingInputNames.Add({ FKeyExpression{mx::Category::Mix, mx::NodeGroup::Compositing, mx::Type::VolumeShader}, TEXT("fg") }, ExpressionInput(UE::Expressions::Inputs::Foreground));
		MatchingInputNames.Add({ FKeyExpression{mx::Category::Mix, mx::NodeGroup::Compositing, mx::Type::VolumeShader}, TEXT("mix") }, ExpressionInput(UE::Expressions::Inputs::Mix));
		MatchingMaterialExpressions.Add({ mx::Category::Mix, mx::NodeGroup::Compositing, mx::Type::VolumeShader }, UE::Expressions::Names::SubstrateHorizontalMixing);

		// Add
		MatchingMaterialExpressions.Add({ mx::Category::Add, mx::NodeGroup::PBR, mx::Type::BSDF}, UE::Expressions::Names::SubstrateAdd);

		// Multiply
		MatchingInputNames.Add({ FKeyExpression{mx::Category::Multiply, mx::NodeGroup::PBR, mx::Type::BSDF}, TEXT("in2") }, ExpressionInput(UE::Expressions::Inputs::Weight));
		MatchingMaterialExpressions.Add({ mx::Category::Multiply, mx::NodeGroup::PBR, mx::Type::BSDF}, UE::Expressions::Names::SubstrateWeight);
	}
}

FMaterialXManager& FMaterialXManager::GetInstance()
{
	static FMaterialXManager Instance;
	return Instance;
}



namespace
{
	bool ValidateDocument(MaterialX::DocumentPtr Document, const UInterchangeTranslatorBase* Translator = nullptr)
	{
		namespace mx = MaterialX;

		if (std::string MaterialXMessage; !Document->validate(&MaterialXMessage))
		{
			if(Translator)
			{
				UInterchangeResultError_Generic* Message = Translator->AddMessage<UInterchangeResultError_Generic>();
				Message->Text = FText::Format(LOCTEXT("MaterialXDocumentInvalid", "{0}"),
											  FText::FromString(MaterialXMessage.c_str()));
			}
			else
			{
				UE_LOGF(LogInterchangeImport, Error, "%ls", UTF8_TO_TCHAR(MaterialXMessage.c_str()));
			}
			return false;
		}

		for (mx::ElementPtr Elem : Document->traverseTree())
		{
			//make sure to read only the current file otherwise we'll process the entire library
			if (Elem->getActiveSourceUri() != Document->getActiveSourceUri())
			{
				continue;
			}

			mx::NodePtr Node = Elem->asA<mx::Node>();

			if (Node)
			{
				// Validate that all nodes in the file are strictly respecting their node definition
				if (!Node->getNodeDef())
				{
					if(Translator)
					{
						UInterchangeResultError_Generic* Message = Translator->AddMessage<UInterchangeResultError_Generic>();
						Message->Text = FText::Format(LOCTEXT("NodeDefNotFound", "<{0}> has no matching NodeDef, aborting import..."),
													  FText::FromString(Node->getName().c_str()));
					}
					else
					{
						UE_LOGF(LogInterchangeImport, Error, "<%ls> has no matching NodeDef, aborting import...", UTF8_TO_TCHAR(Node->getName().c_str()));
					}
					return false;
				}

				//verify first if it's not a multioutput node (weird that there's no typedef for it)
				if (!Node->getTypeDef() && Node->getType() != "multioutput")
				{
					if(Translator)
					{
						UInterchangeResultError_Generic* Message = Translator->AddMessage<UInterchangeResultError_Generic>();
						Message->Text = FText::Format(LOCTEXT("TypeDefNotFound", "<{0}> has no matching TypeDef, aborting import..."),
													  FText::FromString(Node->getName().c_str()));
					}
					else
					{
						UE_LOGF(LogInterchangeImport, Error, "<%ls> has no matching TypeDef, aborting import...", UTF8_TO_TCHAR(Node->getName().c_str()));
					}
					return false;
				}
			}
		}

		return true;
	}
}

bool FMaterialXManager::Translate(const FString& Filename, UInterchangeBaseNodeContainer& BaseNodeContainer, const UInterchangeTranslatorBase* Translator)
{
	using namespace UE::Interchange;

	bool bIsDocumentValid = false;

	if (!FPaths::FileExists(Filename))
	{
		return false;
	}

	try
	{
		mx::FileSearchPath MaterialXFolder{ TCHAR_TO_UTF8(*FPaths::Combine(
			FPaths::EngineDir(),
			TEXT("Binaries"),
			TEXT("ThirdParty"),
			TEXT("MaterialX"))) };

		mx::DocumentPtr MaterialXLibrary = mx::createDocument();

		mx::StringSet LoadedLibs = mx::loadLibraries({ mx::Library::Libraries }, MaterialXFolder, MaterialXLibrary);
		if (LoadedLibs.empty())
		{
			if (Translator)
			{
				UInterchangeResultError_Generic* Message = Translator->AddMessage<UInterchangeResultError_Generic>();
				Message->Text = FText::Format(LOCTEXT("MaterialXLibrariesNotFound", "Couldn't load MaterialX libraries from {0}"),
											  FText::FromString(MaterialXFolder.asString().c_str()));
			}
			else
			{
				UE_LOGF(LogInterchangeImport, Error, "Couldn't load MaterialX libraries from %ls", UTF8_TO_TCHAR(MaterialXFolder.asString().c_str()));
			}
			return false;
		}

		mx::DocumentPtr Document = mx::createDocument();
		mx::readFromXmlFile(Document, TCHAR_TO_UTF8(*Filename));
		Document->importLibrary(MaterialXLibrary);

		// The path to the folders containing the custom node nodedefs should also be indicated with the MATERIALX_SEARCH_PATH environment variable 
		mx::DocumentPtr EnvPathLibrary = mx::createDocument();
		mx::FileSearchPath EnvPath = mx::getEnvironmentPath();
		mx::StringSet EnvLibraries = mx::loadLibraries({}, EnvPath, EnvPathLibrary);
		Document->importLibrary(EnvPathLibrary);

		bIsDocumentValid = Translate(Document, BaseNodeContainer, Translator);

		if (Document->hasVersionString())
		{
			UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(&BaseNodeContainer);
			SourceNode->SetExtraInformation(FSourceNodeExtraInfoStaticData::GetApplicationVersionExtraInfoKey(), ANSI_TO_TCHAR(Document->getVersionString().c_str()));
		}
	}
	catch (std::exception& Exception)
	{
		bIsDocumentValid = false;
		if (Translator)
		{
			UInterchangeResultError_Generic* Message = Translator->AddMessage<UInterchangeResultError_Generic>();
			Message->Text = FText::Format(LOCTEXT("MaterialXException", "{0}"),
										  FText::FromString(Exception.what()));
		}
		else
		{
			UE_LOGF(LogInterchangeImport, Error, "%ls", UTF8_TO_TCHAR(Exception.what()));
		}
	}

	return bIsDocumentValid;
}

bool FMaterialXManager::Translate(MaterialX::DocumentPtr Document, UInterchangeBaseNodeContainer& BaseNodeContainer, const UInterchangeTranslatorBase* Translator)
{
	bool bIsDocumentValid;
	UE::Interchange::FScopedLambda ScopedLambda([this](){TextureNodeUids.Empty();});

	TArray<FAnalyticsEventAttribute> AnalyticsAttributes;

	try
	{
		// Read the document to be sure that the file is valid (meaning all nodes have their nodedef and typedef well-defined)
		bIsDocumentValid = ValidateDocument(Document, Translator);
		if (!bIsDocumentValid)
		{
			return false;
		}

		//Update the document by initializing and reorganizing the different nodes and subgraphs
		FMaterialXBase::UpdateDocumentRecursively(Document);

		// coming to this point we know for sure that the document is valid
		for (mx::ElementPtr Elem : Document->traverseTree())
		{
			//make sure to read only the current file otherwise we'll process the entire library
			if (Elem->getActiveSourceUri() != Document->getActiveSourceUri())
			{
				continue;
			}

			mx::NodePtr Node = Elem->asA<mx::Node>();

			if (Node)
			{
				bool bIsMaterialShader = Node->getType() == mx::Type::Material;
				bool bIsLightShader = Node->getType() == mx::Type::LightShader;

				//The entry point is only surfacematerial or lightshader
				if (bIsMaterialShader || bIsLightShader)
				{
					TSharedPtr<FMaterialXBase> ShaderTranslator = FMaterialXManager::GetInstance().GetShaderTranslator(Node->getCategory().c_str(), BaseNodeContainer);
					if (ShaderTranslator)
					{
						ShaderTranslator->SetTranslator(Translator);
						ShaderTranslator->Translate(Node);
						ShaderTranslator->CollectAnalytics(AnalyticsAttributes);
					}
				}
			}
		}
	}
	catch (std::exception& Exception)
	{
		bIsDocumentValid = false;
		if (Translator)
		{
			UInterchangeResultError_Generic* Message = Translator->AddMessage<UInterchangeResultError_Generic>();
			Message->Text = FText::Format(LOCTEXT("MaterialXException", "{0}"),
										  FText::FromString(Exception.what()));
			AnalyticsAttributes.Emplace(TEXT("MaterialXFail"), TEXT("[FAILED] MaterialX Document invalid"));
		}
		else
		{
			UE_LOGF(LogInterchangeImport, Error, "%ls", UTF8_TO_TCHAR(Exception.what()));
		}
	}

	if (Translator && Translator->AnalyticsHandler)
	{
		Translator->AnalyticsHandler->Append(TEXT("Interchange.Usage.Import.MaterialX"), AnalyticsAttributes);
	}

	return bIsDocumentValid;
}

const FString* FMaterialXManager::FindMatchingInput(const FString& CategoryKey, const FString& InputKey, const FString& NodeGroup, const FString& Type) const
{
	return MatchingInputNames.Find({ FKeyExpression{ CategoryKey, NodeGroup, Type}, InputKey });
}

const FString* FMaterialXManager::FindMaterialExpressionInput(const FString& InputKey) const
{
	return MaterialExpressionInputs.Find(InputKey);
}

const FString* FMaterialXManager::FindMatchingMaterialExpression(const FString& CategoryKey, const FString& NodeGroup, const FString& Type) const
{
	return MatchingMaterialExpressions.Find({ CategoryKey, NodeGroup, Type });
}

bool FMaterialXManager::FindMatchingMaterialFunction(const FString& CategoryKey, const FString*& MaterialFunctionPath, uint8& EnumType, uint8& EnumValue) const
{
	if(const FMaterialXMaterialFunction* MaterialFunction = MatchingMaterialFunctions.Find(CategoryKey))
	{
		MaterialFunctionPath = MaterialFunction->TryGet<FString>();
		if(const EInterchangeMaterialXShaders* Shader = MaterialFunction->TryGet<EInterchangeMaterialXShaders>())
		{
			EnumType = UE::Interchange::MaterialX::IndexSurfaceShaders;
			EnumValue = static_cast<uint8>(*Shader);
		}
		else if(const EInterchangeMaterialXBSDF* Bsdf = MaterialFunction->TryGet<EInterchangeMaterialXBSDF>())
		{
			EnumType = UE::Interchange::MaterialX::IndexBSDF;
			EnumValue = static_cast<uint8>(*Bsdf);
		}
		else if(const EInterchangeMaterialXEDF* Edf = MaterialFunction->TryGet<EInterchangeMaterialXEDF>())
		{
			EnumType = UE::Interchange::MaterialX::IndexEDF;
			EnumValue = static_cast<uint8>(*Edf);
		}
		else if(const EInterchangeMaterialXVDF* Vdf = MaterialFunction->TryGet<EInterchangeMaterialXVDF>())
		{
			EnumType = UE::Interchange::MaterialX::IndexVDF;
			EnumValue = static_cast<uint8>(*Vdf);
		}
		return true;
	}

	return false;
}

TSharedPtr<FMaterialXBase> FMaterialXManager::GetShaderTranslator(const FString& CategoryShader, UInterchangeBaseNodeContainer & NodeContainer)
{
	if(FOnGetMaterialXInstance* InstanceDelegate = MaterialXContainerDelegates.Find(CategoryShader))
	{
		if(InstanceDelegate->IsBound())
		{
			return InstanceDelegate->Execute(NodeContainer);
		}
	}

	return nullptr;
}

void FMaterialXManager::RegisterMaterialXInstance(const FString& Category, FOnGetMaterialXInstance MaterialXInstanceDelegate)
{
	if(!Category.IsEmpty())
	{
		MaterialXContainerDelegates.Add(Category, MaterialXInstanceDelegate);
	}
}

bool FMaterialXManager::IsSubstrateEnabled() const
{
	return bIsSubstrateEnabled;
}

bool FMaterialXManager::IsSubstrateAdaptiveGBufferEnabled() const
{
	return bIsSubstrateAdaptiveGBufferEnabled;
}

bool FMaterialXManager::FilterNodeGraph(MaterialX::NodePtr Node) const
{
	// the test seems counterintuitive, but MaterialX check is "!filter" in the flattenSubgraphs functions
	return CategoriesToSkip.find(Node->getCategory()) == CategoriesToSkip.cend();
}

void FMaterialXManager::RemoveInputs(MaterialX::NodePtr Node) const
{
	if (auto It = NodeInputsToRemove.find(Node->getCategory()); It != NodeInputsToRemove.cend())
	{
		for(const std::string& InputName: It->second)
		{
			Node->removeInput(InputName);
		}
	}
}

FString FMaterialXManager::FindOrAddTextureNodeUid(const FString& TexturePath)
{
	FString TextureNodeUid;
	if (const FString* TextureUidPtr = TextureNodeUids.Find(TexturePath))
	{
		TextureNodeUid =*TextureUidPtr;
	}
	else
	{
		FSHA1 SHA1;
		SHA1.UpdateWithString(*TexturePath, TexturePath.Len());
		FSHAHash Hash = SHA1.Finalize();
		TextureNodeUid = Hash.ToString();
		TextureNodeUid = TEXT("\\Texture\\") + TextureNodeUid + TEXT("\\") + FPaths::GetCleanFilename(TexturePath);
		TextureNodeUids.Emplace(TexturePath, TextureNodeUid);
	}
	return TextureNodeUid;
}

void FMaterialXManager::AddInputsFromNodeDef(MaterialX::NodePtr Node) const
{
	if (NodeDefsCategories.find(Node->getCategory()) != NodeDefsCategories.cend())
	{
		Node->addInputsFromNodeDef();
	}
}

const TCHAR* FMaterialXManager::ExpressionInput(const TCHAR* Input)
{
	MaterialExpressionInputs.FindOrAdd(Input);
	return Input;
}

INTERCHANGEIMPORT_API const TCHAR* FMaterialXManager::MaterialFunctionPackage(const TCHAR* Input)
{
	MaterialFunctionsToLoad.FindOrAdd(Input);
	return Input;
}

#endif

namespace UE::Interchange::MaterialX
{	
	bool AreMaterialFunctionPackagesLoaded()
	{
		return true;
	}
}

#undef LOCTEXT_NAMESPACE 