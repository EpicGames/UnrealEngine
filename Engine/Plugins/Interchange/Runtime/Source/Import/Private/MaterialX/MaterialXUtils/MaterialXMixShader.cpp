// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "MaterialXMixShader.h"
#include "InterchangeImportLog.h"

#include "Engine/EngineTypes.h"

namespace mx = MaterialX;

FMaterialXMixShader::FMaterialXMixShader(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: FMaterialXSurfaceShaderAbstract(BaseNodeContainer)
{
	// TODO: When we'll support volume materials we should also handle this case
	NodeDefinition = mx::NodeDefinition::MixSurfaceShader;
}

TSharedRef<FMaterialXBase> FMaterialXMixShader::MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	TSharedRef<FMaterialXMixShader> Result = MakeShared<FMaterialXMixShader>(BaseNodeContainer);
	Result->RegisterConnectNodeOutputToInputDelegates();
	return Result;
}

UInterchangeBaseNode* FMaterialXMixShader::Translate(mx::NodePtr MixNode)
{
	using namespace UE::Interchange::Materials;
	this->SurfaceShaderNode = MixNode;

	// we need to rename the input of this mix node to the ones used by Substrate Mix
	if (mx::InputPtr Input = MixNode->getInput("fg"))
	{
		SetAttributeNewName(Input, TCHAR_TO_ANSI(UE::Expressions::Inputs::Foreground));
	}
	if (mx::InputPtr Input = MixNode->getInput("bg"))
	{
		SetAttributeNewName(Input, TCHAR_TO_ANSI(UE::Expressions::Inputs::Background));
	}
	if (mx::InputPtr Input = MixNode->getInput("mix"))
	{
		SetAttributeNewName(Input, TCHAR_TO_ANSI(UE::Expressions::Inputs::Mix));
	}

	UInterchangeShaderNode* MixShaderNode = nullptr;
	if (bIsSubstrateEnabled)
    {
		MixShaderNode = CreateShaderNode(MixNode, MixNode->getName().c_str(), UE::Expressions::Names::SubstrateHorizontalMixing);
        MixShaderNode = FMaterialXSurfaceShaderAbstract::Translate(MixShaderNode);
        UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(ShaderGraphNode, SubstrateMaterial::Parameters::FrontMaterial.ToString(), MixShaderNode->GetUniqueID());

		AnalyticsAttributes.Emplace(TEXT("ShaderType"), TEXT("Mix Shader"));
    }
	else
	{
		MTLX_LOG("MaterialXMixShader", "\"{0}\": {1} of {2} is only available with Substrate.", MixNode->getName().c_str(), MixNode->getCategory().c_str(), MixNode->getType().c_str());
	}

	return MixShaderNode;
}
#endif