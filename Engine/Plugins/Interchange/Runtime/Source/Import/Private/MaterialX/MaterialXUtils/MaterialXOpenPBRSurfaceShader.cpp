// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "MaterialXOpenPBRSurfaceShader.h"

#include "Engine/EngineTypes.h"

namespace mx = MaterialX;

FMaterialXOpenPBRSurfaceShader::FMaterialXOpenPBRSurfaceShader(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: FMaterialXSurfaceShaderAbstract(BaseNodeContainer)
{
	NodeDefinition = mx::NodeDefinition::OpenPBRSurface;
}

TSharedRef<FMaterialXBase> FMaterialXOpenPBRSurfaceShader::MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	TSharedRef<FMaterialXOpenPBRSurfaceShader> Result = MakeShared<FMaterialXOpenPBRSurfaceShader>(BaseNodeContainer);
	Result->RegisterConnectNodeOutputToInputDelegates();
	return Result;
}

UInterchangeBaseNode* FMaterialXOpenPBRSurfaceShader::Translate(MaterialX::NodePtr OpenPBRSurfaceNode)
{
	using namespace UE::Interchange::Materials;
	using namespace UE::Interchange::MaterialX;
	using namespace UE::Interchange::Materials::Standard::Nodes;
	this->SurfaceShaderNode = OpenPBRSurfaceNode;

	EInterchangeMaterialXShaders OpenPBRShaderType = EInterchangeMaterialXShaders::OpenPBRSurface;
	UInterchangeShaderNode* OpenPBRSurfaceShaderNode = FMaterialXSurfaceShaderAbstract::Translate(OpenPBRShaderType);
	//Two sided
	if(MaterialX::InputPtr Input = GetInput(SurfaceShaderNode, mx::OpenPBRSurface::Input::GeometryThinWalled);
	   Input->hasValue() && mx::fromValueString<bool>(Input->getValueString()) == true)
	{
		// weird that we also have to enable that to have a two sided material (seems to only have meaning for Translucent material)
		ShaderGraphNode->SetCustomTwoSidedTransmission(true);
		ShaderGraphNode->SetCustomTwoSided(true);
	}

	if(UInterchangeShaderPortsAPI::HasInput(OpenPBRSurfaceShaderNode, OpenPBRSurface::Parameters::TransmissionWeight))
	{
		OpenPBRShaderType = EInterchangeMaterialXShaders::OpenPBRSurfaceTransmission;
		OpenPBRSurfaceShaderNode->AddInt32Attribute(Attributes::EnumType, IndexSurfaceShaders);
		OpenPBRSurfaceShaderNode->AddInt32Attribute(Attributes::EnumValue, int32(OpenPBRShaderType));
		ShaderGraphNode->SetCustomBlendMode(EBlendMode::BLEND_TranslucentColoredTransmittance);
	}

	bool bSubsurfaceWithTransmission = false;
	if(bIsSubstrateEnabled)
	{
		if (!bIsSubstrateAdaptiveGBufferEnabled)
		{
			MTLX_LOG("MaterialXOpenPBRSurfaceShader", "OpenPBR material rendering might be wrong. Please select Substrate Adaptive GBuffer format in the Project settings.");
		}

		bSubsurfaceWithTransmission = ConnectSubsurfaceTransmissionToSurfaceShader(OpenPBRSurfaceShaderNode, EInterchangeMaterialXShaders::OpenPBRSurface
			, OpenPBRSurface::Parameters::BaseWeight, OpenPBRSurface::Parameters::BaseColor
			, OpenPBRSurface::Parameters::TransmissionWeight, OpenPBRSurface::Parameters::TransmissionColor, OpenPBRSurface::Parameters::TransmissionDepth, OpenPBRSurface::Parameters::TransmissionDispersionAbbeNumber, OpenPBRSurface::Parameters::TransmissionDispersionScale, OpenPBRSurface::Parameters::TransmissionScatter, OpenPBRSurface::Parameters::TransmissionScatterAnisotropy
			, OpenPBRSurface::Parameters::SubsurfaceWeight, OpenPBRSurface::Parameters::SubsurfaceColor, OpenPBRSurface::Parameters::SubsurfaceRadiusScale
			, OpenPBRSurface::SubstrateMaterial::Outputs::FrontMaterial, OpenPBRSurface::SubstrateMaterial::Outputs::FrontMaterial);

		if(!bSubsurfaceWithTransmission)
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::FrontMaterial.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), OpenPBRSurface::SubstrateMaterial::Outputs::FrontMaterial.ToString());

			if (UInterchangeShaderPortsAPI::HasInput(OpenPBRSurfaceShaderNode, OpenPBRSurface::Parameters::GeometryOpacity))
			{
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::OpacityMask.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), OpenPBRSurface::SubstrateMaterial::Outputs::OpacityMask.ToString());
				ShaderGraphNode->SetCustomBlendMode(EBlendMode::BLEND_Masked);
			}
		}
	}
	else
	{	// Outputs
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::BaseColor.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::BaseColor.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Metallic.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Metallic.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Specular.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Specular.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Roughness.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Roughness.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::EmissiveColor.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::EmissiveColor.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Anisotropy.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Anisotropy.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Normal.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Normal.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Tangent.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Tangent.ToString());

		// We can't have all shading models at once, so we have to make a choice here
		if(UInterchangeShaderPortsAPI::HasInput(OpenPBRSurfaceShaderNode, OpenPBRSurface::Parameters::TransmissionWeight))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Opacity.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Opacity.ToString());
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, ThinTranslucent::Parameters::TransmissionColor.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), ThinTranslucent::Parameters::TransmissionColor.ToString());
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, Common::Parameters::Refraction.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), Common::Parameters::Refraction.ToString());
		}
		else if(UInterchangeShaderPortsAPI::HasInput(OpenPBRSurfaceShaderNode, OpenPBRSurface::Parameters::FuzzWeight))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, Sheen::Parameters::SheenColor.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), Sheen::Parameters::SheenColor.ToString());
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, Sheen::Parameters::SheenRoughness.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), Sheen::Parameters::SheenRoughness.ToString());
		}
		else if(UInterchangeShaderPortsAPI::HasInput(OpenPBRSurfaceShaderNode, OpenPBRSurface::Parameters::CoatWeight))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, ClearCoat::Parameters::ClearCoat.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), ClearCoat::Parameters::ClearCoat.ToString());
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, ClearCoat::Parameters::ClearCoatRoughness.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), ClearCoat::Parameters::ClearCoatRoughness.ToString());
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, ClearCoat::Parameters::ClearCoatNormal.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), ClearCoat::Parameters::ClearCoatNormal.ToString());
		}
		else if(UInterchangeShaderPortsAPI::HasInput(OpenPBRSurfaceShaderNode, OpenPBRSurface::Parameters::SubsurfaceWeight))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Opacity.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Opacity.ToString());
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, Subsurface::Parameters::SubsurfaceColor.ToString(), OpenPBRSurfaceShaderNode->GetUniqueID(), Subsurface::Parameters::SubsurfaceColor.ToString());
		}
	}

	AnalyticsAttributes.Emplace(TEXT("ShaderType"), TEXT("OpenPBR"));
	
	// this analytic can also be set in ConnectSubsurfaceTransmissionToSurfaceShader, we don't want to set it twice
	if (!bSubsurfaceWithTransmission)
	{
		switch (OpenPBRShaderType)
		{
		case EInterchangeMaterialXShaders::OpenPBRSurfaceTransmission:
			AnalyticsAttributes.Emplace(TEXT("OpenPBR"), TEXT("Transmission"));
			break;
		default:
			AnalyticsAttributes.Emplace(TEXT("OpenPBR"), TEXT("Opaque"));
			break;
		}
	}

	return OpenPBRSurfaceShaderNode;
}

mx::InputPtr FMaterialXOpenPBRSurfaceShader::GetInputNormal(mx::NodePtr OpenPbrSurfaceShaderNode, const char*& InputNormal) const
{
	InputNormal = mx::OpenPBRSurface::Input::GeometryNormal;
	mx::InputPtr Input = OpenPbrSurfaceShaderNode->getActiveInput(InputNormal);

	// if no input is connected take the one from the nodedef
	if (!Input)
	{
		Input = OpenPbrSurfaceShaderNode->getNodeDef(mx::EMPTY_STRING, true)->getActiveInput(InputNormal);
	}

	return Input;
}

#endif //WITH_EDITOR
