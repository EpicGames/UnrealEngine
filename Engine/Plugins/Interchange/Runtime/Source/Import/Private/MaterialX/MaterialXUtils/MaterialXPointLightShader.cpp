// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "MaterialXPointLightShader.h"

namespace mx = MaterialX;

FMaterialXPointLightShader::FMaterialXPointLightShader(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: FMaterialXLightShaderAbstract(BaseNodeContainer)
{
	NodeDefinition = mx::NodeDefinition::PointLight;
}

TSharedRef<FMaterialXBase> FMaterialXPointLightShader::MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	return MakeShared<FMaterialXPointLightShader>(BaseNodeContainer);
}

UInterchangeBaseNode* FMaterialXPointLightShader::Translate(MaterialX::NodePtr PointLightShaderNode)
{
	this->LightShaderNode = PointLightShaderNode;

	PreTranslate();

	LightNode = NewObject<UInterchangePointLightNode>(&NodeContainer);
	Cast<UInterchangePointLightNode>(LightNode)->SetCustomIntensityUnits(EInterchangeLightUnits::Candelas);

	// Decay rate
	SetDecayRate();

	// Position
	SetPosition();

	PostTranslate();

	return LightNode;
}

float FMaterialXPointLightShader::GetDecayRate(mx::InputPtr DecayRateInput)
{
	return 0.0f;
}

inline void FMaterialXPointLightShader::GetPosition(mx::InputPtr PositionInput, FTransform& Transform)
{
	const mx::Vector3 Position = mx::fromValueString<mx::Vector3>(PositionInput->getValueString());
	Transform.SetLocation(FVector{ Position[0] * 100, Position[1] * 100, Position[2] * 100 });
}

void FMaterialXPointLightShader::SetDecayRate()
{
	UInterchangePointLightNode* PointLightNode = Cast<UInterchangePointLightNode>(LightNode);
	mx::InputPtr DecayRateInput = GetInput(LightShaderNode, mx::Lights::PointLight::Input::DecayRate);
	const float DecayRate = mx::fromValueString<float>(DecayRateInput->getValueString());
	PointLightNode->SetCustomUseInverseSquaredFalloff(false);
	PointLightNode->SetCustomLightFalloffExponent(DecayRate);
}

void FMaterialXPointLightShader::SetPosition()
{
	mx::InputPtr PositionInput = GetInput(LightShaderNode, mx::Lights::PointLight::Input::Position);
	GetPosition(PositionInput, Transform);
}
#endif