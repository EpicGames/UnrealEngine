// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "MaterialX/MaterialXUtils/MaterialXLightShaderAbstract.h"

class FMaterialXPointLightShader : public FMaterialXLightShaderAbstract
{
protected:

	template<typename T, ESPMode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;

	FMaterialXPointLightShader(UInterchangeBaseNodeContainer & BaseNodeContainer);

public:

	static TSharedRef<FMaterialXBase> MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer);

	virtual UInterchangeBaseNode* Translate(MaterialX::NodePtr PointLightShaderNode) override;

	static float GetDecayRate(MaterialX::InputPtr DecayRateInput);

	static void GetPosition(MaterialX::InputPtr PositionInput, FTransform& Transform);

	void SetDecayRate();

	void SetPosition();
};
#endif