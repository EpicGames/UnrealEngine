// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "MaterialXSpotLightShader.h"
#include "MaterialXDirectionalLightShader.h"

namespace mx = MaterialX;

FMaterialXSpotLightShader::FMaterialXSpotLightShader(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: FMaterialXPointLightShader(BaseNodeContainer)
{
	NodeDefinition = mx::NodeDefinition::SpotLight;
}

TSharedRef<FMaterialXBase> FMaterialXSpotLightShader::MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	return MakeShared<FMaterialXSpotLightShader>(BaseNodeContainer);
}

UInterchangeBaseNode* FMaterialXSpotLightShader::Translate(MaterialX::NodePtr SpotLightShaderNode)
{
	this->LightShaderNode = SpotLightShaderNode;

	PreTranslate();

	LightNode = NewObject<UInterchangeSpotLightNode>(&NodeContainer);
	Cast<UInterchangeSpotLightNode>(LightNode)->SetCustomIntensityUnits(EInterchangeLightUnits::Candelas);

	// Decay rate
	SetDecayRate();

	// Decay rate
	SetDecayRate();

	// Position
	SetPosition();

	// Direction
	{
		mx::InputPtr DirectionInput = GetInput(SpotLightShaderNode, mx::Lights::DirectionalLight::Input::Direction);
		FMaterialXDirectionalLightShader::GetDirection(DirectionInput, Transform);
	}

	// Inner angle
	SetInnerAngle();

	// Outer angle
	SetOuterAngle();

	PostTranslate();

	return LightNode;
}

void FMaterialXSpotLightShader::SetInnerAngle()
{
	mx::InputPtr InnerAngleInput = GetInput(LightShaderNode, mx::Lights::SpotLight::Input::InnerAngle);
	const float InnerAngle = FMath::RadiansToDegrees(mx::fromValueString<float>(InnerAngleInput->getValueString()));
	Cast<UInterchangeSpotLightNode>(LightNode)->SetCustomInnerConeAngle(InnerAngle);
}

void FMaterialXSpotLightShader::SetOuterAngle()
{
	mx::InputPtr OuterAngleInput = GetInput(LightShaderNode, mx::Lights::SpotLight::Input::OuterAngle);
	const float OuterAngle = FMath::RadiansToDegrees(mx::fromValueString<float>(OuterAngleInput->getValueString()));
	Cast<UInterchangeSpotLightNode>(LightNode)->SetCustomOuterConeAngle(OuterAngle);
}
#endif