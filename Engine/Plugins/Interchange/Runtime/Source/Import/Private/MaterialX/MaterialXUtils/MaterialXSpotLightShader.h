// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "MaterialXPointLightShader.h"

class FMaterialXSpotLightShader : public FMaterialXPointLightShader
{
protected:

	template<typename T, ESPMode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;

	FMaterialXSpotLightShader(UInterchangeBaseNodeContainer & BaseNodeContainer);

public:

	static TSharedRef<FMaterialXBase> MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer);

	virtual UInterchangeBaseNode* Translate(MaterialX::NodePtr SpotLightShaderNode) override;

	void SetInnerAngle();

	void SetOuterAngle();
};

#endif