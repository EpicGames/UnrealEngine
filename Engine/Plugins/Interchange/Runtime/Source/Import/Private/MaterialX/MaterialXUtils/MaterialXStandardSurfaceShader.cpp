// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "MaterialXStandardSurfaceShader.h"
#include "Engine/EngineTypes.h"

namespace mx = MaterialX;

FMaterialXStandardSurfaceShader::FMaterialXStandardSurfaceShader(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: FMaterialXSurfaceShaderAbstract(BaseNodeContainer)
{
	NodeDefinition = mx::NodeDefinition::StandardSurface;
}

TSharedRef<FMaterialXBase> FMaterialXStandardSurfaceShader::MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	TSharedRef<FMaterialXStandardSurfaceShader> Result= MakeShared<FMaterialXStandardSurfaceShader>(BaseNodeContainer);
	Result->RegisterConnectNodeOutputToInputDelegates();
	return Result;
}

UInterchangeBaseNode* FMaterialXStandardSurfaceShader::Translate(mx::NodePtr StandardSurfaceNode)
{
	this->SurfaceShaderNode = StandardSurfaceNode;
	
	using namespace UE::Interchange::Materials;
	using namespace UE::Interchange::MaterialX;
	using namespace UE::Interchange::Materials::Standard::Nodes;

	EInterchangeMaterialXShaders StandardSurfaceShaderType = EInterchangeMaterialXShaders::StandardSurface;
	UInterchangeShaderNode* StandardSurfaceShaderNode = FMaterialXSurfaceShaderAbstract::Translate(StandardSurfaceShaderType);

	//Two sided
	if(MaterialX::InputPtr Input = GetInput(SurfaceShaderNode, mx::StandardSurface::Input::ThinWalled);
	   Input->hasValue() && mx::fromValueString<bool>(Input->getValueString()) == true)
	{
		// weird that we also have to enable that to have a two sided material (seems to only have meaning for Translucent material)
		ShaderGraphNode->SetCustomTwoSidedTransmission(true);
		ShaderGraphNode->SetCustomTwoSided(true);
	}

	if(UInterchangeShaderPortsAPI::HasInput(StandardSurfaceShaderNode, StandardSurface::Parameters::Transmission))
	{
		StandardSurfaceShaderType = EInterchangeMaterialXShaders::StandardSurfaceTransmission;
		StandardSurfaceShaderNode->AddInt32Attribute(Attributes::EnumType, IndexSurfaceShaders);
		StandardSurfaceShaderNode->AddInt32Attribute(Attributes::EnumValue, int32(EInterchangeMaterialXShaders::StandardSurfaceTransmission));
	}

	bool bSubsurfaceWithTransmission = false;
	if(bIsSubstrateEnabled)
	{
		if (!bIsSubstrateAdaptiveGBufferEnabled)
		{
			MTLX_LOG("MaterialXStandardSurfaceShader", "Standard Surface material rendering might be wrong. Please select Substrate Adaptive GBuffer format in the Project settings.");
		}

		bSubsurfaceWithTransmission = ConnectSubsurfaceTransmissionToSurfaceShader(StandardSurfaceShaderNode, EInterchangeMaterialXShaders::StandardSurface
			, StandardSurface::Parameters::Base, StandardSurface::Parameters::BaseColor
			, StandardSurface::Parameters::Transmission, StandardSurface::Parameters::TransmissionColor, StandardSurface::Parameters::TransmissionDepth, StandardSurface::Parameters::TransmissionExtraRoughness, StandardSurface::Parameters::TransmissionDispersion, StandardSurface::Parameters::TransmissionScatter, StandardSurface::Parameters::TransmissionScatterAnisotropy
			, StandardSurface::Parameters::Subsurface, StandardSurface::Parameters::SubsurfaceColor, StandardSurface::Parameters::SubsurfaceRadius
			, StandardSurface::SubstrateMaterial::Outputs::Opaque, StandardSurface::SubstrateMaterial::Outputs::Translucent);

		if (!bSubsurfaceWithTransmission)
		{
			if (UInterchangeShaderPortsAPI::HasInput(StandardSurfaceShaderNode, StandardSurface::Parameters::Transmission))
			{
				ShaderGraphNode->SetCustomBlendMode(EBlendMode::BLEND_TranslucentColoredTransmittance);
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::FrontMaterial.ToString(), StandardSurfaceShaderNode->GetUniqueID(), StandardSurface::SubstrateMaterial::Outputs::Translucent.ToString());
			}
			else
			{
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::FrontMaterial.ToString(), StandardSurfaceShaderNode->GetUniqueID(), StandardSurface::SubstrateMaterial::Outputs::Opaque.ToString());
				if (UInterchangeShaderPortsAPI::HasInput(StandardSurfaceShaderNode, StandardSurface::Parameters::Opacity))
				{
					UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::OpacityMask.ToString(), StandardSurfaceShaderNode->GetUniqueID(), StandardSurface::SubstrateMaterial::Outputs::Opacity.ToString());
					ShaderGraphNode->SetCustomBlendMode(EBlendMode::BLEND_Masked);
				}
			}
		}
	}
	else
	{
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::BaseColor.ToString(), StandardSurfaceShaderNode->GetUniqueID(), TEXT("Base Color"));
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Metallic.ToString(), StandardSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Metallic.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Specular.ToString(), StandardSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Specular.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Roughness.ToString(), StandardSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Roughness.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::EmissiveColor.ToString(), StandardSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::EmissiveColor.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Anisotropy.ToString(), StandardSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Anisotropy.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Normal.ToString(), StandardSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Normal.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Tangent.ToString(), StandardSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Tangent.ToString());

		// We can't have all shading models at once, so we have to make a choice here
		if(UInterchangeShaderPortsAPI::HasInput(StandardSurfaceShaderNode, StandardSurface::Parameters::Transmission))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Opacity.ToString(), StandardSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Opacity.ToString());
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, ThinTranslucent::Parameters::TransmissionColor.ToString(), StandardSurfaceShaderNode->GetUniqueID(), ThinTranslucent::Parameters::TransmissionColor.ToString());
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, Common::Parameters::Refraction.ToString(), StandardSurfaceShaderNode->GetUniqueID(), Common::Parameters::Refraction.ToString());

			// If we have have Transmission and Opacity let's use the Surface Coverage instead
			if(UInterchangeShaderPortsAPI::HasInput(StandardSurfaceShaderNode, StandardSurface::Parameters::Opacity))
			{
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, ThinTranslucent::Parameters::SurfaceCoverage.ToString(), StandardSurfaceShaderNode->GetUniqueID(), ThinTranslucent::Parameters::SurfaceCoverage.ToString());
			}
		}
		else if(UInterchangeShaderPortsAPI::HasInput(StandardSurfaceShaderNode, StandardSurface::Parameters::Sheen))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, Sheen::Parameters::SheenColor.ToString(), StandardSurfaceShaderNode->GetUniqueID(), Sheen::Parameters::SheenColor.ToString());
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, Sheen::Parameters::SheenRoughness.ToString(), StandardSurfaceShaderNode->GetUniqueID(), Sheen::Parameters::SheenRoughness.ToString());
		}
		else if(UInterchangeShaderPortsAPI::HasInput(StandardSurfaceShaderNode, StandardSurface::Parameters::Coat))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, ClearCoat::Parameters::ClearCoat.ToString(), StandardSurfaceShaderNode->GetUniqueID(), ClearCoat::Parameters::ClearCoat.ToString());
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, ClearCoat::Parameters::ClearCoatRoughness.ToString(), StandardSurfaceShaderNode->GetUniqueID(), ClearCoat::Parameters::ClearCoatRoughness.ToString());
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, ClearCoat::Parameters::ClearCoatNormal.ToString(), StandardSurfaceShaderNode->GetUniqueID(), ClearCoat::Parameters::ClearCoatNormal.ToString());
		}
		else if(UInterchangeShaderPortsAPI::HasInput(StandardSurfaceShaderNode, StandardSurface::Parameters::Subsurface))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Opacity.ToString(), StandardSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Opacity.ToString());
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, Subsurface::Parameters::SubsurfaceColor.ToString(), StandardSurfaceShaderNode->GetUniqueID(), Subsurface::Parameters::SubsurfaceColor.ToString());
		}
		else if(UInterchangeShaderPortsAPI::HasInput(StandardSurfaceShaderNode, StandardSurface::Parameters::Opacity))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, ThinTranslucent::Parameters::SurfaceCoverage.ToString(), StandardSurfaceShaderNode->GetUniqueID(), ThinTranslucent::Parameters::SurfaceCoverage.ToString());
		}
	}

	AnalyticsAttributes.Emplace(TEXT("ShaderType"), TEXT("Standard Surface"));

	// this analytic can also be set in ConnectSubsurfaceTransmissionToSurfaceShader, we don't want to set it twice
	if (!bSubsurfaceWithTransmission)
	{
		switch (StandardSurfaceShaderType)
		{
		case EInterchangeMaterialXShaders::StandardSurfaceTransmission:
			AnalyticsAttributes.Emplace(TEXT("StandardSurface"), TEXT("Transmission"));
			break;
		default:
			AnalyticsAttributes.Emplace(TEXT("StandardSurface"), TEXT("Opaque"));
			break;
		}
	}

	return StandardSurfaceShaderNode;
}

mx::InputPtr FMaterialXStandardSurfaceShader::GetInputNormal(mx::NodePtr StandardSurfaceNode, const char*& InputNormal) const
{
	InputNormal = mx::StandardSurface::Input::Normal;
	
	mx::InputPtr Input = StandardSurfaceNode->getActiveInput(InputNormal);
		
	if (!Input)
	{
		Input = StandardSurfaceNode->getNodeDef(mx::EMPTY_STRING, true)->getActiveInput(InputNormal);
	}

	return Input;
}

#endif
