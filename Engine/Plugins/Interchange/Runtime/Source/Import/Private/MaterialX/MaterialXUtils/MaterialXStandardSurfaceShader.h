// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "MaterialX/MaterialXUtils/MaterialXSurfaceShaderAbstract.h"

class FMaterialXStandardSurfaceShader :  public FMaterialXSurfaceShaderAbstract
{
protected:

	template<typename T, ESPMode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;

	FMaterialXStandardSurfaceShader(UInterchangeBaseNodeContainer& BaseNodeContainer);

public:

	static TSharedRef<FMaterialXBase> MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer);

	virtual UInterchangeBaseNode* Translate(MaterialX::NodePtr StandardSurfaceNode) override;

	virtual MaterialX::InputPtr GetInputNormal(MaterialX::NodePtr StandardSurfaceNode, const char*& InputNormal) const override;
};
#endif