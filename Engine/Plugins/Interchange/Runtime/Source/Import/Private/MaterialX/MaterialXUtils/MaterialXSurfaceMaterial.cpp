// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "MaterialXSurfaceMaterial.h"
#include "InterchangeImportLog.h"
#include "MaterialX/MaterialXUtils/MaterialXManager.h"
#include "MaterialX/MaterialXUtils/MaterialXStandardSurfaceShader.h"
#include "MaterialX/MaterialXUtils/MaterialXSurfaceUnlitShader.h"

namespace mx = MaterialX;

FMaterialXSurfaceMaterial::FMaterialXSurfaceMaterial(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: FMaterialXBase(BaseNodeContainer)
{}

TSharedRef<FMaterialXBase> FMaterialXSurfaceMaterial::MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	return MakeShared<FMaterialXSurfaceMaterial>(BaseNodeContainer);
}

UInterchangeBaseNode* FMaterialXSurfaceMaterial::Translate(MaterialX::NodePtr SurfaceMaterialNode)
{
	using namespace UE::Interchange::Materials;
	bool bHasSurfaceShader = false;
	bool bIsShaderGraphInContainer = true;

	 //We initialize the ShaderGraph outside of the loop, because a surfacematerial has only up to 2 inputs:
	 //- a surfaceshader that we handle
	 //- a displacementshader not yet supported
	const FString ShaderGraphNodeUID = UInterchangeShaderNode::MakeNodeUid(UTF8_TO_TCHAR(SurfaceMaterialNode->getName().c_str()), FStringView{});
	UInterchangeShaderGraphNode* ShaderGraphNode = const_cast<UInterchangeShaderGraphNode*>(Cast<UInterchangeShaderGraphNode>(NodeContainer.GetNode(ShaderGraphNodeUID)));

	if(ShaderGraphNode)
	{
		return ShaderGraphNode;
	}

	mx::InputPtr InputNormal;
	mx::NodePtr ConnectedSurfaceShaderNode, ConnectedDisplacementShaderNode;
	const char* InputNormalName = "normal";
	auto InitializeShaderTranslator = [&]
	(mx::InputPtr Input, bool bIsSurfaceShader = false) -> TSharedPtr<FMaterialXSurfaceShaderAbstract>
	{
		if (Input)
		{
			mx::NodePtr ConnectedNode = Input->getConnectedNode();
			if (!ConnectedNode)
			{
				return nullptr;
			}

			TSharedPtr<FMaterialXSurfaceShaderAbstract> ShaderTranslator = StaticCastSharedPtr<FMaterialXSurfaceShaderAbstract>(FMaterialXManager::GetInstance().GetShaderTranslator(ConnectedNode->getCategory().c_str(), NodeContainer));
			if (ShaderTranslator.IsValid())
			{
				// We only add the ShaderGraph to the container if we found a supported surfaceshader
				if (!ShaderGraphNode)
				{
					ShaderGraphNode = NewObject<UInterchangeShaderGraphNode>(&NodeContainer);
					NodeContainer.SetupNode(ShaderGraphNode, ShaderGraphNodeUID, SurfaceMaterialNode->getName().c_str(), EInterchangeNodeContainerType::TranslatedAsset);
					bIsShaderGraphInContainer = false;
				}

				ShaderTranslator->ShaderGraphNode = ShaderGraphNode;
				ShaderTranslator->SurfaceMaterialName = SurfaceMaterialNode->getName().c_str();
				ShaderTranslator->SetTranslator(Translator);
				// we set the nodedef here, because a displacement shader can either be a float or vector3, we want to be sure to retrieve the correct one
				ShaderTranslator->SetNodeDefinition(ConnectedNode->getNodeDef()->getName().c_str());
				bHasSurfaceShader |= true;

				if (bIsSurfaceShader)
				{
					ConnectedSurfaceShaderNode = ConnectedNode;
					InputNormal = ShaderTranslator->GetInputNormal(ConnectedNode, InputNormalName);
					if (ConnectedDisplacementShaderNode && InputNormal)
					{
						//add the input normal from the surfaceshader to the displacement
						ConnectedDisplacementShaderNode->addInput("normal", InputNormal->getType())->copyContentFrom(InputNormal);
						// now remove the normal input from the surfaceshader
						ConnectedNode->removeInput(InputNormal->getName());
					}
				}
				else
				{
					ConnectedDisplacementShaderNode = ConnectedNode;
				}
			}
			return ShaderTranslator;
		}
		return nullptr;
	};

	// if we have a displacement shader, the normals have to be blended in MX_Displacement, 
	// so we need to plug any shader graph linked to the normal input of the surfaceshader on the displacementshader
	// and then plug the normal output of the displacement to the surfaceshader	
	TSharedPtr<FMaterialXSurfaceShaderAbstract> DisplacementShaderTranslator = InitializeShaderTranslator(SurfaceMaterialNode->getInput("displacementshader"));
	TSharedPtr<FMaterialXSurfaceShaderAbstract> SurfaceShaderTranslator = InitializeShaderTranslator(SurfaceMaterialNode->getInput("surfaceshader"), true);

	UInterchangeShaderNode* SurfaceShaderNode = SurfaceShaderTranslator ? Cast<UInterchangeShaderNode>(SurfaceShaderTranslator->Translate(ConnectedSurfaceShaderNode)) : nullptr;
	UInterchangeShaderNode* DisplacementShaderNode = DisplacementShaderTranslator ? Cast<UInterchangeShaderNode>(DisplacementShaderTranslator->Translate(ConnectedDisplacementShaderNode)) : nullptr;

	//Now connect the displacementshader material function normal output to the input of the surfaceshader material function
	if(SurfaceShaderNode && DisplacementShaderNode && InputNormal)
	{
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(SurfaceShaderNode, InputNormalName, DisplacementShaderNode->GetUniqueID(), PBRMR::Parameters::Normal.ToString());
	}

	AnalyticsAttributes.Emplace(TEXT("MaterialType"), TEXT("Surface Material"));
	if(!bHasSurfaceShader)
	{
		MTLX_LOG("MaterialXSurfaceMaterial", "<{0}>: {1} \"{2}\" is not supported", SurfaceMaterialNode->getCategory().c_str(), SurfaceMaterialNode->getType().c_str(), SurfaceMaterialNode->getName().c_str());
		AnalyticsAttributes.Emplace(TEXT("ShaderType"), TEXT("Surface Shader unsupported"));
	}

	return ShaderGraphNode;
}

#endif