// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "MaterialXSurfaceShader.h"
#include "InterchangeImportLog.h"

namespace mx = MaterialX;

FMaterialXSurfaceShader::FMaterialXSurfaceShader(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: FMaterialXSurfaceShaderAbstract(BaseNodeContainer)
{
	NodeDefinition = mx::NodeDefinition::Surface;
}

TSharedRef<FMaterialXBase> FMaterialXSurfaceShader::MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	TSharedRef<FMaterialXSurfaceShader > Result = MakeShared<FMaterialXSurfaceShader >(BaseNodeContainer);
	Result->RegisterConnectNodeOutputToInputDelegates();
	return Result;
}

UInterchangeBaseNode* FMaterialXSurfaceShader::Translate(MaterialX::NodePtr SurfaceNode)
{
	this->SurfaceShaderNode = SurfaceNode;

	UInterchangeShaderNode* FunctionSurfaceShaderNode = FMaterialXSurfaceShaderAbstract::Translate(EInterchangeMaterialXShaders::Surface);

	using namespace UE::Interchange::Materials;

	// Outputs
	if(!bIsSubstrateEnabled)
	{
		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(ShaderGraphNode, Common::Parameters::BxDF.ToString(), FunctionSurfaceShaderNode->GetUniqueID());
	}
	else
	{
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::FrontMaterial.ToString(), FunctionSurfaceShaderNode->GetUniqueID(), Surface::Substrate::Outputs::Surface.ToString());
		if(UInterchangeShaderPortsAPI::HasInput(FunctionSurfaceShaderNode, Surface::Parameters::Opacity))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::OpacityMask.ToString(), FunctionSurfaceShaderNode->GetUniqueID(), Surface::Substrate::Outputs::Opacity.ToString());
		}
	}

	AnalyticsAttributes.Emplace(TEXT("ShaderType"), TEXT("Surface (Generic)"));
	return FunctionSurfaceShaderNode;
}
#endif