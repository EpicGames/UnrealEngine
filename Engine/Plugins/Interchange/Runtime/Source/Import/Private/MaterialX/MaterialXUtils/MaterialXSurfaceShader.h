// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "MaterialX/MaterialXUtils/MaterialXSurfaceShaderAbstract.h"

/** Handles <surface> node from the pbrlib, acts more or less as a generic surface shader */
class FMaterialXSurfaceShader : public FMaterialXSurfaceShaderAbstract
{
protected:

	template<typename T, ESPMode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;

	FMaterialXSurfaceShader(UInterchangeBaseNodeContainer& BaseNodeContainer);

public:

	static TSharedRef<FMaterialXBase> MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer);

	virtual UInterchangeBaseNode* Translate(MaterialX::NodePtr SurfaceNode) override;
};
#endif