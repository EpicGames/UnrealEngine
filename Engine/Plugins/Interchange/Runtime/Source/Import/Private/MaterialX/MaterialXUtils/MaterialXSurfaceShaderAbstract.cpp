// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "MaterialX/MaterialXUtils/MaterialXSurfaceShaderAbstract.h"

#include "InterchangeTexture2DNode.h"
#include "InterchangeTextureBlurNode.h"
#include "Materials/MaterialExpressionTransform.h"
#include "Materials/MaterialExpressionTextureProperty.h"
#include "Materials/MaterialExpressionVectorNoise.h"
#include "MaterialX/MaterialXUtils/MaterialXManager.h"
#include "MaterialX/MaterialExpressions/MaterialExpressionTextureSampleParameterBlur.h"
#include "Usd/InterchangeUsdDefinitions.h"

#define LOCTEXT_NAMESPACE "MaterialXSurfaceShaderAbstract"

namespace mx = MaterialX;

FString FMaterialXSurfaceShaderAbstract::EmptyString{};
FString FMaterialXSurfaceShaderAbstract::DefaultOutput{TEXT("out")};


/**
* Get the normal input of a surfaceshader, used to plug it in the displacementshader
* @param SurfaceShaderNode - The surfaceshader that we return the input from
* @return the input of the normal
*/

FMaterialXSurfaceShaderAbstract::FMaterialXSurfaceShaderAbstract(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: FMaterialXBase{ BaseNodeContainer }
	, ShaderGraphNode{ nullptr }
	, bTangentSpaceInput{ false }
{}

bool FMaterialXSurfaceShaderAbstract::AddAttribute(MaterialX::InputPtr Input, const FString& InputChannelName, UInterchangeShaderNode* ShaderNode, int32 OutputIndex, bool bBoolAsScalar)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	if(Input)
	{
		if(Input->getType() == mx::Type::Boolean)
		{
			return AddBooleanAttribute(Input, InputChannelName, ShaderNode, bBoolAsScalar);
		}
		else if(Input->getType() == mx::Type::Float)
		{
			return AddFloatAttribute(Input, InputChannelName, ShaderNode);
		}
		else if(Input->getType() == mx::Type::Integer) //Let's add Float attribute, because Interchange doesn't create a scalar if it's an int
		{
			return AddIntegerAttribute(Input, InputChannelName, ShaderNode);
		}
		else if(Input->getType() == mx::Type::Color3 || Input->getType() == mx::Type::Color4)
		{
			return AddLinearColorAttribute(Input, InputChannelName, ShaderNode, FLinearColor{ std::numeric_limits<float>::max(),std::numeric_limits<float>::max(),std::numeric_limits<float>::max() }, OutputIndex);
		}
		else if(Input->getType() == mx::Type::Vector2)
		{
			return AddVector2Attribute(Input, InputChannelName, ShaderNode, FVector2f{ std::numeric_limits<float>::max() ,std::numeric_limits<float>::max() }, OutputIndex);
		}
		else if(Input->getType() == mx::Type::Vector3 || Input->getType() == mx::Type::Vector4)
		{
			return AddVectorAttribute(Input, InputChannelName, ShaderNode, FVector4f{ std::numeric_limits<float>::max(),std::numeric_limits<float>::max(),std::numeric_limits<float>::max() }, OutputIndex);
		}
	}

	return false;
}

bool FMaterialXSurfaceShaderAbstract::AddAttributeFromValueOrInterface(MaterialX::InputPtr Input, const FString& InputChannelName, UInterchangeShaderNode* ShaderNode, int32 OutputIndex, bool bBoolAsScalar)
{
	bool bAttribute = false;

	if(Input)
	{
		UInterchangeShaderNode* ShaderNodeToConnectTo = ShaderNode;
		FString InputToConnectTo = InputChannelName;

		if(Input->hasValue())
		{
			bAttribute = AddAttribute(Input, InputToConnectTo, ShaderNodeToConnectTo, OutputIndex, bBoolAsScalar);
		}
		else if(Input->hasInterfaceName())
		{
			if(mx::InputPtr InputInterface = Input->getInterfaceInput(); InputInterface->hasValue())
			{
				bAttribute = AddAttribute(InputInterface, InputToConnectTo, ShaderNodeToConnectTo, OutputIndex, bBoolAsScalar);
			}
		}
	}

	return bAttribute;
}

bool FMaterialXSurfaceShaderAbstract::AddBooleanAttribute(MaterialX::InputPtr Input, const FString& InputChannelName, UInterchangeShaderNode* ShaderNode, bool bAsScalar)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	if(Input && Input->hasValue())
	{
		bool Value = mx::fromValueString<bool>(Input->getValueString());

		// The parent is either a node, or it's an interfacename and we just take the name of the input
		mx::NodePtr Node = Input->getParent()->asA<mx::Node>();
		FString NodeName = Node ? Node->getName().c_str() + FString{ TEXT("_") } : FString{};
		NodeName += Input->getName().c_str();

		UInterchangeShaderNode* StaticBoolParameterNode = CreateShaderNode(Input, NodeName, StaticBoolParameter::Name.ToString());
		if (bAsScalar)
		{
			StaticBoolParameterNode->SetCustomShaderType(ScalarParameter::Name.ToString());
			StaticBoolParameterNode->AddFloatAttribute(UInterchangeShaderPortsAPI::MakeInputParameterKey(ScalarParameter::Attributes::DefaultValue.ToString()), Value);
		}
		else
		{
			StaticBoolParameterNode->AddBooleanAttribute(UInterchangeShaderPortsAPI::MakeInputParameterKey(StaticBoolParameter::Attributes::DefaultValue.ToString()), Value);
		}

		return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(ShaderNode, InputChannelName, StaticBoolParameterNode->GetUniqueID());
	}

	return false;
}

bool FMaterialXSurfaceShaderAbstract::AddFloatAttribute(MaterialX::InputPtr Input, const FString& InputChannelName, UInterchangeShaderNode* ShaderNode, float DefaultValue)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	if(Input && Input->hasValue())
	{
		float Value = mx::fromValueString<float>(Input->getValueString());

		if(!FMath::IsNearlyEqual(Value, DefaultValue))
		{
			// The parent is either a node, or it's an interfacename and we just take the name of the input
			mx::NodePtr Node = Input->getParent()->asA<mx::Node>();
			FString NodeName = Node ? Node->getName().c_str() + FString{ TEXT("_") } : FString{};
			NodeName += Input->getName().c_str();
			

			UInterchangeShaderNode* ScalarParameterNode = CreateShaderNode(Input, NodeName, ScalarParameter::Name.ToString());
			UInterchangeShaderNode* NodeToConnectTo = ScalarParameterNode;
			
			const float AbsValue = FMath::Abs(Value);
			const int32 Exponent = AbsValue > 0.f ? -FMath::FloorToInt(FMath::LogX(10, AbsValue)) : 0;
			double Power = 1.;
			// In the Material Editor, Constant/Scalars are rounded to 0 with at least 7 digits after the comma
			if (Exponent >= 7)
			{
				Power = FMath::Pow(10., Exponent);

				UInterchangeShaderNode* DivideNode = CreateShaderNode(Input, NodeName + TEXT("_Divide"), UE::Expressions::Names::Divide);
				DivideNode->AddFloatAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(UE::Expressions::Inputs::B), Power);
				UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(DivideNode, UE::Expressions::Inputs::A, ScalarParameterNode->GetUniqueID());
				NodeToConnectTo = DivideNode;
			}

			ScalarParameterNode->AddFloatAttribute(UInterchangeShaderPortsAPI::MakeInputParameterKey(ScalarParameter::Attributes::DefaultValue.ToString()), Value * Power);
			return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(ShaderNode, InputChannelName, NodeToConnectTo->GetUniqueID());
		}
	}

	return false;
}

bool FMaterialXSurfaceShaderAbstract::AddIntegerAttribute(MaterialX::InputPtr Input, const FString& InputChannelName, UInterchangeShaderNode* ShaderNode, int DefaultValue)
{
	//We handle Integers as Scalars
	using namespace UE::Interchange::Materials::Standard::Nodes;
	if (Input && Input->hasValue())
	{
		int32 Value = mx::fromValueString<int32>(Input->getValueString());

		if (Value != DefaultValue)
		{
			// The parent is either a node, or it's an interfacename and we just take the name of the input
			mx::NodePtr Node = Input->getParent()->asA<mx::Node>();
			FString NodeName = Node ? Node->getName().c_str() + FString{ TEXT("_") } : FString{};
			NodeName += Input->getName().c_str();

			UInterchangeShaderNode* ScalarParameterNode = CreateShaderNode(Input, NodeName, ScalarParameter::Name.ToString());
			ScalarParameterNode->AddFloatAttribute(UInterchangeShaderPortsAPI::MakeInputParameterKey(ScalarParameter::Attributes::DefaultValue.ToString()), Value);
			return UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(ShaderNode, InputChannelName, ScalarParameterNode->GetUniqueID());
		}
	}

	return false;
}

bool FMaterialXSurfaceShaderAbstract::AddLinearColorAttribute(MaterialX::InputPtr Input, const FString& InputChannelName, UInterchangeShaderNode* ShaderNode, const FLinearColor& DefaultValue, int32 OutputIndex)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	if(Input && Input->hasValue())
	{
		const FLinearColor Value = GetLinearColor(Input);

		if(!Value.Equals(DefaultValue))
		{
			// The parent is either a node, or it's an interfacename and we just take the name of the input
			mx::NodePtr Node = Input->getParent()->asA<mx::Node>();
			FString NodeName = Node ? Node->getName().c_str() + FString{ TEXT("_") } : FString{};
			NodeName += Input->getName().c_str();

			UInterchangeShaderNode* VectorParameterNode = CreateShaderNode(Input, NodeName, VectorParameter::Name.ToString());
			VectorParameterNode->AddLinearColorAttribute(UInterchangeShaderPortsAPI::MakeInputParameterKey(VectorParameter::Attributes::DefaultValue.ToString()), Value);
			
			if (Input->getType() == mx::Type::Color4)
			{
				OutputIndex = 5; // RGBA output on a VectorParameter is the 6th output
			}
			
			return UInterchangeShaderPortsAPI::ConnectOuputToInputByIndex(ShaderNode, InputChannelName, VectorParameterNode->GetUniqueID(), OutputIndex);
		}
	}

	return false;
}

bool FMaterialXSurfaceShaderAbstract::AddVectorAttribute(MaterialX::InputPtr Input, const FString& InputChannelName, UInterchangeShaderNode* ShaderNode, const FVector4f& DefaultValue, int32 OutputIndex)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	if(Input && Input->hasValue())
	{
		const FLinearColor Value = GetVector(Input);

		if(!Value.Equals(DefaultValue))
		{
			// The parent is either a node, or it's an interfacename and we just take the name of the input
			mx::NodePtr Node = Input->getParent()->asA<mx::Node>();
			FString NodeName = Node ? Node->getName().c_str() + FString{ TEXT("_") } : FString{};
			NodeName += Input->getName().c_str();

			UInterchangeShaderNode* VectorParameterNode = CreateShaderNode(Input, NodeName, VectorParameter::Name.ToString());
			VectorParameterNode->AddLinearColorAttribute(UInterchangeShaderPortsAPI::MakeInputParameterKey(VectorParameter::Attributes::DefaultValue.ToString()), Value);
			
			if (Input->getType() == mx::Type::Vector4)
			{
				OutputIndex = 5; // RGBA output on a VectorParameter is the 6th output
			}
			
			return UInterchangeShaderPortsAPI::ConnectOuputToInputByIndex(ShaderNode, InputChannelName, VectorParameterNode->GetUniqueID(), OutputIndex);
		}
	}

	return false;
}

bool FMaterialXSurfaceShaderAbstract::AddVector2Attribute(MaterialX::InputPtr Input, const FString& InputChannelName, UInterchangeShaderNode* ShaderNode, const FVector2f& DefaultValue, int32 OutputIndex)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	if (Input && Input->hasValue())
	{
		const FLinearColor Value = GetVector(Input);
		
		if (!FVector2f{ Value.R,Value.B }.Equals(DefaultValue))
		{
			// The parent is either a node, or it's an interfacename and we just take the name of the input
			mx::NodePtr Node = Input->getParent()->asA<mx::Node>();
			FString NodeName = Node ? Node->getName().c_str() + FString{ TEXT("_") } : FString{};
			NodeName += Input->getName().c_str();

			UInterchangeShaderNode* VectorParameterNode = CreateShaderNode(Input, NodeName, VectorParameter::Name.ToString());
			VectorParameterNode->AddLinearColorAttribute(UInterchangeShaderPortsAPI::MakeInputParameterKey(VectorParameter::Attributes::DefaultValue.ToString()), Value);
			UInterchangeShaderNode* VectorMaskNode = CreateMaskShaderNode(0b1100, Input, NodeName + TEXT("_Vector2"));
			UInterchangeShaderPortsAPI::ConnectOuputToInputByIndex(VectorMaskNode, Mask::Inputs::Input.ToString(), VectorParameterNode->GetUniqueID(), OutputIndex);
			return UInterchangeShaderPortsAPI::ConnectOuputToInputByIndex(ShaderNode, InputChannelName, VectorMaskNode->GetUniqueID(), OutputIndex);
		}
	}

	return false;
}

bool FMaterialXSurfaceShaderAbstract::ConnectNodeGraphOutputToInput(MaterialX::InputPtr InputToNodeGraph, UInterchangeShaderNode* ShaderNode, const FString& ParentInputName)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	bool bHasNodeGraph = false;

	if(InputToNodeGraph->hasNodeGraphString())
	{
		bHasNodeGraph = true;

		mx::OutputPtr Output = InputToNodeGraph->getConnectedOutput();

		if(!Output)
		{
			MTLX_LOG("MaterialNodeGraphOutput", "Couldn't find a connected output to ({0}).", *GetInputName(InputToNodeGraph));
			return false;
		}

		// We may end up with an empty output string in case of a flattened nodegraph 
		FString OutputString = Output->hasOutputString() && !Output->getOutputString().empty() ? ANSI_TO_TCHAR(Output->getOutputString().c_str()) : TEXT("out");
		for(mx::Edge Edge : Output->traverseGraph())
		{
			ConnectNodeCategoryOutputToInput(Edge, ShaderNode, ParentInputName, OutputString);
		}
	}

	return bHasNodeGraph;
}

bool FMaterialXSurfaceShaderAbstract::ConnectMatchingNodeOutputToInput(const FConnectNode& Connect)
{
	FMaterialXManager& Manager = FMaterialXManager::GetInstance();

	auto GetIndexOutput = [&Connect]()
	{
		mx::NodeDefPtr NodeDef = Connect.UpstreamNode->getNodeDef(mx::EMPTY_STRING, true);
		int Index = NodeDef->getChildIndex(TCHAR_TO_UTF8(*Connect.OutputName));
		return Index -= NodeDef->getInputCount();
	};

	bool bIsConnected = false;

	auto ConnectOutputToInputInternal = [&](UInterchangeShaderNode* OperatorNode)
	{
		for(mx::InputPtr Input : Connect.UpstreamNode->getInputs())
		{
			if (const FString* InputNameFound = Manager.FindMaterialExpressionInput(GetInputName(Input)))
			{
				AddAttributeFromValueOrInterface(Input, *InputNameFound, OperatorNode);
			}
			else 
			{
				AddAttributeFromValueOrInterface(Input, Input->getName().c_str(), OperatorNode);
			}
		}

		int IndexOutput = GetIndexOutput();

		bIsConnected = IndexOutput < 0 ?
			UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, OperatorNode->GetUniqueID()) :
			UInterchangeShaderPortsAPI::ConnectOuputToInputByIndex(Connect.ParentShaderNode, Connect.InputChannelName, OperatorNode->GetUniqueID(), IndexOutput);
	};

	auto ConnectOutputToInput = [&](const FString* ShaderType, auto* (FMaterialXSurfaceShaderAbstract::* CreateFunctionCallOrShaderNode)(mx::ElementPtr, const FString&, const FString&, const FString&))
	{
		UInterchangeShaderNode* OperatorNode = nullptr;

		OperatorNode = (this->*CreateFunctionCallOrShaderNode)(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), *ShaderType, Connect.OutputName);

		ConnectOutputToInputInternal(OperatorNode);
	};

	auto ConnectFunctionShaderNodeOutputToInput = [&](uint8 EnumType, uint8 EnumValue, auto* (FMaterialXSurfaceShaderAbstract::* CreateFunctionCallOrShaderNode)(mx::ElementPtr, const FString&, uint8, uint8, const FString&))
	{
		UInterchangeShaderNode* OperatorNode = nullptr;

		OperatorNode = (this->*CreateFunctionCallOrShaderNode)(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), EnumType, EnumValue, Connect.OutputName);

		ConnectOutputToInputInternal(OperatorNode);
	};

	const FString* MaterialFunctionPath = nullptr;

	// First search a matching Material Expression
	// search for a Material Expression based on the node group (essentially used for Substrate Mix)
	if(const FString* ShaderType = Manager.FindMatchingMaterialExpression(Connect.UpstreamNode->getCategory().c_str(), Connect.UpstreamNode->getNodeDef(mx::EMPTY_STRING, true)->getNodeGroup().c_str(), Connect.UpstreamNode->getType().c_str()))
	{
		ConnectOutputToInput(ShaderType, &FMaterialXSurfaceShaderAbstract::CreateShaderNode);
	}
	else if((ShaderType = Manager.FindMatchingMaterialExpression(Connect.UpstreamNode->getCategory().c_str())))
	{
		ConnectOutputToInput(ShaderType, &FMaterialXSurfaceShaderAbstract::CreateShaderNode);
	}
	else if(FOnConnectNodeOutputToInput* Delegate = MatchingConnectNodeDelegates.Find(Connect.UpstreamNode->getCategory().c_str()))
	{
		bIsConnected = Delegate->ExecuteIfBound(Connect);
	}
	else if(uint8 EnumType, EnumValue; Manager.FindMatchingMaterialFunction(Connect.UpstreamNode->getCategory().c_str(), MaterialFunctionPath, EnumType, EnumValue))
	{
		// In case of a surfaceshader node, especialy openpbr and standard_surface we need to check if we have to take the transmission surface shader
		if (EnumType == UE::Interchange::MaterialX::IndexSurfaceShaders)
		{
			bool bIsTransmittance = false;
			//check if the surfaceshader has a transmission input, only for openpbr and standardsurface
			if (EnumValue == uint8(EInterchangeMaterialXShaders::OpenPBRSurface) && Connect.UpstreamNode->getInput(mx::OpenPBRSurface::Input::TransmissionWeight))
			{
				EnumValue = uint8(EInterchangeMaterialXShaders::OpenPBRSurfaceTransmission);
				bIsTransmittance = true;
			}
			else if (EnumValue == uint8(EInterchangeMaterialXShaders::StandardSurface) && Connect.UpstreamNode->getInput(mx::StandardSurface::Input::Transmission))
			{
				EnumValue = uint8(EInterchangeMaterialXShaders::StandardSurfaceTransmission);
				bIsTransmittance = true;
			}

			if (bIsTransmittance)
			{
				ShaderGraphNode->SetCustomBlendMode(EBlendMode::BLEND_TranslucentColoredTransmittance);
			}
		}

		MaterialFunctionPath ?
			ConnectOutputToInput(MaterialFunctionPath, &FMaterialXSurfaceShaderAbstract::CreateFunctionCallShaderNode) :
			ConnectFunctionShaderNodeOutputToInput(EnumType, EnumValue, &FMaterialXSurfaceShaderAbstract::CreateFunctionCallShaderNode);
	}

	return bIsConnected;
}

void FMaterialXSurfaceShaderAbstract::ConnectNodeCategoryOutputToInput(const MaterialX::Edge& Edge, UInterchangeShaderNode* ShaderNode, const FString& ParentInputName, const FString& OutputName)
{
	FMaterialXManager& Manager = FMaterialXManager::GetInstance();
	if(mx::NodePtr UpstreamNode = Edge.getUpstreamElement()->asA<mx::Node>())
	{
		// We need to connect the different descending nodes to all the outputs of a node
		// At least one output connected to the root shader node
		TArray<UInterchangeShaderNode*> ParentShaderNodeOutputs{ShaderNode};
		FString InputChannelName = ParentInputName;
				
		Manager.AddInputsFromNodeDef(UpstreamNode);
		Manager.RemoveInputs(UpstreamNode);

		// Replace the input's name by the one used in UE
		SetMatchingInputsNames(UpstreamNode);

		FString OutputChannelName = OutputName;

		if(mx::ElementPtr DownstreamElement = Edge.getDownstreamElement())
		{
			if(mx::NodePtr DownstreamNode = DownstreamElement->asA<mx::Node>())
			{
				mx::InputPtr ConnectedInput = Edge.getConnectingElement()->asA<mx::Input>();
				if(ConnectedInput)
				{
					InputChannelName = GetInputName(ConnectedInput);
					if (ConnectedInput->hasOutputString())
					{
						OutputChannelName = ConnectedInput->getOutputString().c_str();
					}
					// we take the output name from the nodedef, because some nodes, like <chiang_hair_absorption_from_color>, default output name is not called "out"
					// and we make sure we store the correct output name even for default outputs
					else if (std::vector<mx::OutputPtr> Outputs = UpstreamNode->getNodeDef(mx::EMPTY_STRING, true)->getActiveOutputs(); Outputs.size() == 1)
					{
						OutputChannelName = Outputs[0]->getName().c_str();
					}
				}

				std::vector<mx::OutputPtr> Outputs{ DownstreamNode->getActiveOutputs() };
				if (Outputs.empty())
				{
					Outputs = DownstreamNode->getNodeDef(mx::EMPTY_STRING, true)->getActiveOutputs();
				}

				ParentShaderNodeOutputs.Empty();
				for (std::size_t Index = 0; Index < Outputs.size(); ++Index)
				{
					if(UInterchangeShaderNode** FoundNode = ShaderNodes.Find({ GetAttributeParentName(DownstreamNode, ConnectedInput), Outputs[Index]->getName().c_str()}))
					{
						ParentShaderNodeOutputs.Emplace(*FoundNode);
					}
				}
			}
		}

		for (UInterchangeShaderNode* ParentShaderNode : ParentShaderNodeOutputs)
		{
			if (!ConnectMatchingNodeOutputToInput({ UpstreamNode, ParentShaderNode, InputChannelName, OutputChannelName }))
			{
				MTLX_LOG("MaterialXNodeCategoryOutput", "<{0}>: \"{1}\" is not supported. [{2}]", UpstreamNode->getCategory().c_str(), UpstreamNode->getName().c_str(), *SurfaceMaterialName);
			}
		}
	}
}

bool FMaterialXSurfaceShaderAbstract::ConnectNodeNameOutputToInput(MaterialX::InputPtr InputToConnectedNode, UInterchangeShaderNode* ShaderNode, const FString& ParentInputName)
{
	mx::NodePtr ConnectedNode = InputToConnectedNode->getConnectedNode();

	if(!ConnectedNode)
	{
		return false;
	}

	UInterchangeShaderNode* ParentShaderNode = ShaderNode;
	FString InputChannelName = ParentInputName;

	// for a direct node connection without nodegraph, we need the output of the first node connection
	// for example connecting directly a separate2/3/4 node on a surfaceshader input might miss an entire shader graph
	FString OutputString = !InputToConnectedNode->getOutputString().empty() ? ANSI_TO_TCHAR(InputToConnectedNode->getOutputString().c_str()) : TEXT("out");

	mx::Edge Edge(nullptr, InputToConnectedNode, ConnectedNode);

	TArray<mx::Edge> Stack{ Edge };

	while(!Stack.IsEmpty())
	{
		Edge = Stack.Pop();

		if(Edge.getUpstreamElement())
		{
			ConnectNodeCategoryOutputToInput(Edge, ShaderNode, ParentInputName, OutputString);
			ConnectedNode = Edge.getUpstreamElement()->asA<mx::Node>();
			for(mx::InputPtr Input : ConnectedNode->getInputs())
			{
				Stack.Emplace(ConnectedNode, Input, Input->getConnectedNode());
			}
		}
	}

	return true;
}

void FMaterialXSurfaceShaderAbstract::ConnectConstantInputToOutput(const FConnectNode& Connect)
{
	AddAttributeFromValueOrInterface(Connect.UpstreamNode->getInput("value"), Connect.InputChannelName, Connect.ParentShaderNode);
}

void FMaterialXSurfaceShaderAbstract::ConnectExtractInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	uint8 Index = 0;
	if(mx::InputPtr InputIndex = Connect.UpstreamNode->getInput("index"))
	{
		Index = mx::fromValueString<int>(InputIndex->getValueString());
	}

	if(mx::InputPtr Input = Connect.UpstreamNode->getInput("in"); Input && Input->hasValue())
	{
		// Output 0 means RGB, 1st channel starts at 1
		AddAttributeFromValueOrInterface(Input, Connect.InputChannelName, Connect.ParentShaderNode, Index + 1);
	}
	else
	{
		UInterchangeShaderNode* MaskShaderNode = CreateMaskShaderNode(1 << (3 - Index), Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str());
		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, MaskShaderNode->GetUniqueID());
	}
}

void FMaterialXSurfaceShaderAbstract::ConnectDotInputToOutput(const FConnectNode& Connect)
{
	if(mx::InputPtr Input = Connect.UpstreamNode->getInput("in"))
	{
		SetAttributeNewName(Input, TCHAR_TO_UTF8(*Connect.InputChannelName)); //let's take the parent node's input name
		ShaderNodes.Add({ Connect.UpstreamNode->getName().c_str(), Connect.OutputName}, Connect.ParentShaderNode);
	}
}

void FMaterialXSurfaceShaderAbstract::ConnectTransformPositionInputToOutput(const FConnectNode& Connect)
{
	UInterchangeShaderNode* TransformNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), UE::Expressions::Names::TransformPosition);
	AddAttributeFromValueOrInterface(Connect.UpstreamNode->getInput("in"), UE::Expressions::Inputs::Input, TransformNode);
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, TransformNode->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectTransformVectorInputToOutput(const FConnectNode& Connect)
{
	UInterchangeShaderNode* TransformNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), UE::Expressions::Names::Transform);
	AddAttributeFromValueOrInterface(Connect.UpstreamNode->getInput("in"), UE::Expressions::Inputs::Input, TransformNode);
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, TransformNode->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectRotate2DInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	UInterchangeShaderNode* Rotate2DNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), Rotator::Name.ToString());


	if(mx::InputPtr Input = Connect.UpstreamNode->getInput("in"))
	{
		AddAttributeFromValueOrInterface(Input, GetInputName(Input), Rotate2DNode);
	}

	// Amount is in degrees whereas Time (which in our case is the angle) input is in radians
	if(mx::InputPtr Input = Connect.UpstreamNode->getInput("amount"))
	{
		FString InputName = GetInputName(Input);
		UInterchangeShaderNode* DegreesToRadiansNode = CreateShaderNode(Connect.UpstreamNode, (Connect.UpstreamNode->getName() + "multiply").c_str(), Multiply::Name.ToString());
		constexpr float DegreesToRadians = UE_PI / 180.f;
		DegreesToRadiansNode->AddFloatAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(TEXT("B")), DegreesToRadians);

		//If it's a value we should always attach it on the A input of Divide node
		AddAttributeFromValueOrInterface(Input, UE::Expressions::Inputs::A, DegreesToRadiansNode);
		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Rotate2DNode, InputName, DegreesToRadiansNode->GetUniqueID());
		SetAttributeNewName(Input, TCHAR_TO_ANSI(UE::Expressions::Inputs::A));
	}

	Rotate2DNode->AddFloatAttribute(Rotator::Attributes::CenterX.ToString(), 0.f);
	Rotate2DNode->AddFloatAttribute(Rotator::Attributes::CenterY.ToString(), 0.f);
	Rotate2DNode->AddFloatAttribute(Rotator::Attributes::Speed.ToString(), 1.f);

	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, Rotate2DNode->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectImageInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	// Temporarily copy-pasted here so that we can submit this on a patch version
	constexpr const char* GeomPropNameAttribute = "UE:GeomPropName";

	if(UInterchangeTextureNode* TextureNode = CreateTextureNode<UInterchangeTexture2DNode>(Connect.UpstreamNode))
	{
		//By default set the output of a texture to RGB
		FString OutputChannel{ TEXT("RGB") };

		if(Connect.UpstreamNode->getType() == mx::Type::Vector4 || Connect.UpstreamNode->getType() == mx::Type::Color4)
		{
			OutputChannel = TEXT("RGBA");
		}
		else if(Connect.UpstreamNode->getType() == mx::Type::Float)
		{
			OutputChannel = TEXT("R");
		}

		UInterchangeShaderNode* TextureShaderNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), TextureSample::Name.ToString());
		TextureShaderNode->AddStringAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(TextureSample::Inputs::Texture.ToString()), TextureNode->GetUniqueID());

		if (Connect.UpstreamNode->getTypedAttribute<bool>(mx::Attributes::GeomPropImage))
		{
			// Add a custom attribute that this image node is in fact a geompropvalue converted to an image, in order to update the TextureSample material expression with the baked texture
			TextureShaderNode->AddBooleanAttribute(mx::Attributes::GeomPropImage, true);

			// Carry the geompropvalue (primvar) name through to the Interchange shader node, so the
			// USD pipeline can pair this TextureSample with the right entry in GeomPropValueNames
			// by name instead of by traversal order (see AddPrimvarBakingAttributes in
			// SchemaHandlerUtils.cpp).
			if (Connect.UpstreamNode->hasAttribute(GeomPropNameAttribute))
			{
				TextureShaderNode->AddStringAttribute(GeomPropNameAttribute, Connect.UpstreamNode->getAttribute(GeomPropNameAttribute).c_str());
			}
		}
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(Connect.ParentShaderNode, Connect.InputChannelName, TextureShaderNode->GetUniqueID(), OutputChannel);
	}
	else
	{
		AddAttributeFromValueOrInterface(Connect.UpstreamNode->getInput(mx::NodeGroup::Texture2D::Inputs::Default), Connect.InputChannelName, Connect.ParentShaderNode);
	}
}

void FMaterialXSurfaceShaderAbstract::ConnectIfGreaterInputToOutput(const FConnectNode& Connect)
{
	UInterchangeShaderNode* NodeIf = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), UE::Expressions::Names::If);
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, NodeIf->GetUniqueID());

	if(mx::InputPtr Input = Connect.UpstreamNode->getInput("value1"))
	{
		AddAttributeFromValueOrInterface(Input, GetInputName(Input), NodeIf);
	}

	if(mx::InputPtr Input = Connect.UpstreamNode->getInput("value2"))
	{
		AddAttributeFromValueOrInterface(Input, GetInputName(Input), NodeIf);
	}

	if(mx::InputPtr Input = Connect.UpstreamNode->getInput("in1"))
	{
		AddAttributeFromValueOrInterface(Input, GetInputName(Input), NodeIf);
	}

	//In that case we also need to add an attribute to AEqualsB
	mx::InputPtr Input = Connect.UpstreamNode->getInput("in2");
	if(Input)
	{
		AddAttributeFromValueOrInterface(Input, GetInputName(Input), NodeIf);
		AddAttributeFromValueOrInterface(Input, UE::Expressions::Inputs::AEqualsB, NodeIf);

		//Let's add a new input that is a copy of in2 to connect it to the equal input
		if (!Connect.UpstreamNode->getInput("in3"))
		{
			mx::InputPtr Input3 = Connect.UpstreamNode->addInput("in3");
			Input3->copyContentFrom(Input);
			SetAttributeNewName(Input3, TCHAR_TO_ANSI(UE::Expressions::Inputs::AEqualsB));
		}
	}
}

void FMaterialXSurfaceShaderAbstract::ConnectIfGreaterEqInputToOutput(const FConnectNode& Connect)
{
	UInterchangeShaderNode* NodeIf = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), UE::Expressions::Names::If);
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, NodeIf->GetUniqueID());

	if(mx::InputPtr Input = Connect.UpstreamNode->getInput("value1"))
	{
		AddAttributeFromValueOrInterface(Input, GetInputName(Input), NodeIf);
	}

	if(mx::InputPtr Input = Connect.UpstreamNode->getInput("value2"))
	{
		AddAttributeFromValueOrInterface(Input, GetInputName(Input), NodeIf);
	}

	if(mx::InputPtr Input = Connect.UpstreamNode->getInput("in2"))
	{
		AddAttributeFromValueOrInterface(Input, GetInputName(Input), NodeIf);
	}

	//In that case we also need to add an attribute to AEqualsB
	mx::InputPtr Input = Connect.UpstreamNode->getInput("in1");
	if(Input)
	{
		AddAttributeFromValueOrInterface(Input, GetInputName(Input), NodeIf);
		AddAttributeFromValueOrInterface(Input, UE::Expressions::Inputs::AEqualsB, NodeIf);

		//Let's add a new input that is a copy of in2 to connect it to the equal input
		if (!Connect.UpstreamNode->getInput("in3"))
		{
			mx::InputPtr Input3 = Connect.UpstreamNode->addInput("in3");
			Input3->copyContentFrom(Input);
			SetAttributeNewName(Input3, TCHAR_TO_ANSI(UE::Expressions::Inputs::AEqualsB));
		}
	}
}

void FMaterialXSurfaceShaderAbstract::ConnectIfEqualInputToOutput(const FConnectNode& Connect)
{
	UInterchangeShaderNode* NodeIf = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), UE::Expressions::Names::If);
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, NodeIf->GetUniqueID());

	if(mx::InputPtr Input = Connect.UpstreamNode->getInput("value1"))
	{
		AddAttributeFromValueOrInterface(Input, GetInputName(Input), NodeIf);
	}

	if(mx::InputPtr Input = Connect.UpstreamNode->getInput("value2"))
	{
		AddAttributeFromValueOrInterface(Input, GetInputName(Input), NodeIf);
	}

	if(mx::InputPtr Input = Connect.UpstreamNode->getInput("in1"))
	{
		AddAttributeFromValueOrInterface(Input, GetInputName(Input), NodeIf);
	}

	//In that case we also need to add an attribute to AGreaterThanB
	mx::InputPtr Input = Connect.UpstreamNode->getInput("in2");
	if(Input)
	{
		AddAttributeFromValueOrInterface(Input, GetInputName(Input), NodeIf);
		AddAttributeFromValueOrInterface(Input, UE::Expressions::Inputs::AGreaterThanB, NodeIf);

		//Let's add a new input that is a copy of in2 to connect it to the equal input
		if(!Connect.UpstreamNode->getInput("in3"))
		{
			mx::InputPtr Input3 = Connect.UpstreamNode->addInput("in3");
			Input3->copyContentFrom(Input);
			SetAttributeNewName(Input3, TCHAR_TO_ANSI(UE::Expressions::Inputs::AGreaterThanB));
		}
	}
}

void FMaterialXSurfaceShaderAbstract::ConnectOutsideInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	//in * (1 - mask)
	UInterchangeShaderNode* NodeMultiply = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), Multiply::Name.ToString());
	AddAttributeFromValueOrInterface(Connect.UpstreamNode->getInput("in"), Multiply::Inputs::A.ToString(), NodeMultiply);
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, NodeMultiply->GetUniqueID());

	UInterchangeShaderNode* NodeOneMinus = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str() + FString(TEXT("_OneMinus")), OneMinus::Name.ToString());
	AddAttributeFromValueOrInterface(Connect.UpstreamNode->getInput("mask"), OneMinus::Inputs::Input.ToString(), NodeOneMinus);
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(NodeMultiply, Multiply::Inputs::B.ToString(), NodeOneMinus->GetUniqueID());
}

UInterchangeShaderNode* FMaterialXSurfaceShaderAbstract::ConnectGeometryInputToOutput(const FConnectNode& Connect, const FString& ShaderType, const FString& TransformShaderType, const FString& TransformInput, const FString& TransformSourceType, int32 TransformSource, const FString& TransformType, int32 TransformSDestination, bool bIsVector)
{
	// MaterialX defines the space as: object, model, world
	// model: The local coordinate space of the geometry, before any local deformations or global transforms have been applied.
	// object: The local coordinate space of the geometry, after local deformations have been applied, but before any global transforms.
	// world : The global coordinate space of the geometry, after local deformationsand global transforms have been applied.

	// In case of model/object we need to add a TransformVector from world to local space
	UInterchangeShaderNode* GeometryNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), ShaderType);

	UInterchangeShaderNode* NodeToConnectTo = Connect.ParentShaderNode;
	FString InputToConnectTo = Connect.InputChannelName;

	mx::InputPtr InputSpace = Connect.UpstreamNode->getInput("space");

	//the default space defined by the nodedef is "object"
	bool bIsObjectSpace = (InputSpace && InputSpace->getValueString() != "world") || !InputSpace;

	// We transform to Tangent Space only for Vector nodes
	if(bTangentSpaceInput && bIsVector)
	{
		using namespace UE::Interchange::Materials::Standard::Nodes;
		UInterchangeShaderNode* TransformTSNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str() + FString(TEXT("_TransformTS")), TransformShaderType);
		EMaterialVectorCoordTransformSource SpaceSource = bIsObjectSpace ? EMaterialVectorCoordTransformSource::TRANSFORMSOURCE_Local : EMaterialVectorCoordTransformSource::TRANSFORMSOURCE_World;
		TransformTSNode->AddInt32Attribute(TransformSourceType, SpaceSource);
		TransformTSNode->AddInt32Attribute(TransformType, EMaterialVectorCoordTransform::TRANSFORM_Tangent);
		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(NodeToConnectTo, InputToConnectTo, TransformTSNode->GetUniqueID());
		NodeToConnectTo = TransformTSNode;
		InputToConnectTo = TransformInput; //Same a TransformVector
	}

	if(bIsObjectSpace)
	{
		UInterchangeShaderNode* TransformNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str() + FString(TEXT("_Transform")), TransformShaderType);
		TransformNode->AddInt32Attribute(TransformSourceType, TransformSource);
		TransformNode->AddInt32Attribute(TransformType, TransformSDestination);
		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(NodeToConnectTo, InputToConnectTo, TransformNode->GetUniqueID());
		NodeToConnectTo = TransformNode;
		InputToConnectTo = TransformInput;
	}

	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(NodeToConnectTo, InputToConnectTo, GeometryNode->GetUniqueID());

	return GeometryNode;
}

UInterchangeShaderNode* FMaterialXSurfaceShaderAbstract::ConnectNoise2DInputToOutput(const FConnectNode& Connect, const FString& ShaderType, EVectorNoiseFunction NoiseFunction, uint8 Mask)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	// Let's use a vector noise for this one, the only one that is close to MaterialX implementation
	UInterchangeShaderNode* NoiseNode = CreateShaderNode(Connect.UpstreamNode, (Connect.UpstreamNode->getName() + '_').c_str() + ShaderType, VectorNoise::Name.ToString());
	NoiseNode->AddInt32Attribute(VectorNoise::Attributes::Function.ToString(), NoiseFunction);
	//Quality only has a meaning with Voronoi (Worley), to be as close to MaterialX spec we want a 3x3x3 neighborhood
	NoiseNode->AddInt32Attribute(VectorNoise::Attributes::Quality.ToString(), 3);

	// These noise algorithms are meant to be used with 3D inputs, in order to use it with 2D input, let's put an arbitrary seed for 'z'
	// not sure 0 is a good candidate, we'll see when we'll have real use cases.
	UInterchangeShaderNode* AppendNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), UE::Expressions::Names::AppendVector);
	AppendNode->AddFloatAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(UE::Expressions::Inputs::B), 0.f);

	// The connecting node has to be plugged in the Append expression not the noise, since we are constructing a vector2 to a vector3
	if (mx::InputPtr Input = Connect.UpstreamNode->getInput("texcoord"))
	{
		AddAttributeFromValueOrInterface(Input, UE::Expressions::Inputs::A, AppendNode);
		SetAttributeNewName(Input, TCHAR_TO_ANSI(UE::Expressions::Inputs::A));
	}

	UInterchangeShaderNode* MaskNode = CreateMaskShaderNode(Mask, Connect.UpstreamNode, (Connect.UpstreamNode->getName() + "_ComponentMask").c_str());

	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(NoiseNode, VectorNoise::Inputs::Position.ToString(), AppendNode->GetUniqueID());
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(MaskNode, Mask::Inputs::Input.ToString(), NoiseNode->GetUniqueID());

	return MaskNode;
}

void FMaterialXSurfaceShaderAbstract::ConnectPositionInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	UInterchangeShaderNode* PositionNode = CreateShaderNode(Connect.UpstreamNode, (Connect.UpstreamNode->getName() + "_Position").c_str(), UE::Expressions::Names::LocalPosition);
	
	if (mx::InputPtr InputSpace = Connect.UpstreamNode->getInput("space"); InputSpace && InputSpace->getValueString() == "world")
	{
		PositionNode->SetCustomShaderType(UE::Expressions::Names::WorldPosition);
	}

	UInterchangeShaderNode* UnitNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), Multiply::Name.ToString());

	UnitNode->AddFloatAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(TEXT("B")), 0.01f);
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(UnitNode, TEXT("A"), PositionNode->GetUniqueID());

	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, UnitNode->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectNormalInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	ConnectGeometryInputToOutput(Connect, TEXT("VertexNormalWS"),
								 TransformVector::Name.ToString(),
								 TransformVector::Inputs::Input.ToString(),
								 TransformVector::Attributes::TransformSourceType.ToString(), EMaterialVectorCoordTransformSource::TRANSFORMSOURCE_World,
								 TransformVector::Attributes::TransformType.ToString(), EMaterialVectorCoordTransform::TRANSFORM_Local);
}

void FMaterialXSurfaceShaderAbstract::ConnectTangentInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	ConnectGeometryInputToOutput(Connect, TEXT("VertexTangentWS"),
								 TransformVector::Name.ToString(),
								 TransformVector::Inputs::Input.ToString(),
								 TransformVector::Attributes::TransformSourceType.ToString(), EMaterialVectorCoordTransformSource::TRANSFORMSOURCE_World,
								 TransformVector::Attributes::TransformType.ToString(), EMaterialVectorCoordTransform::TRANSFORM_Local);
}

void FMaterialXSurfaceShaderAbstract::ConnectBitangentInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	UInterchangeShaderNode* BitangentNode = ConnectGeometryInputToOutput(Connect, TEXT("CrossProduct"),
																		 TransformVector::Name.ToString(),
																		 TransformVector::Inputs::Input.ToString(),
																		 TransformVector::Attributes::TransformSourceType.ToString(), EMaterialVectorCoordTransformSource::TRANSFORMSOURCE_World,
																		 TransformVector::Attributes::TransformType.ToString(), EMaterialVectorCoordTransform::TRANSFORM_Local);

	UInterchangeShaderNode* NormalNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str() + FString(TEXT("_Normal")), UE::Expressions::Names::VertexNormalWS);
	UInterchangeShaderNode* TangentNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str() + FString(TEXT("_Tangent")), UE::Expressions::Names::VertexTangentWS);

	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(BitangentNode, TEXT("A"), NormalNode->GetUniqueID());
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(BitangentNode, TEXT("B"), TangentNode->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectTimeInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	UInterchangeShaderNode* TimeNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), UE::Expressions::Names::Time);
	TimeNode->AddBooleanAttribute(Time::Attributes::OverridePeriod.ToString(), true);

	float FPS;
	mx::InputPtr Input = Connect.UpstreamNode->getInput("fps");

	//Take the default value from the node definition
	if(!Input)
	{
		Input = Connect.UpstreamNode->getNodeDef(mx::EMPTY_STRING, true)->getInput("fps");
	}

	FPS = mx::fromValueString<float>(Input->getValueString());

	//UE is a period
	TimeNode->AddFloatAttribute(Time::Attributes::Period.ToString(), 1.f / FPS);

	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, TimeNode->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectNoise2DInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	
	const std::string& Type = Connect.UpstreamNode->getType();
	uint8 Mask = 
		Type == "float" ? 0b1000 :
		Type == "vector2" ? 0b1100 :
		0b1110;

	UInterchangeShaderNode* PerlinNoiseNode = ConnectNoise2DInputToOutput(Connect, VectorNoise::Name.ToString(), EVectorNoiseFunction::VNF_VectorALU, Mask);

	UInterchangeShaderNode* NodeToConnect = PerlinNoiseNode;

	// Amplitude * Noise + Pivot
	auto ConnectNodeToInput = [&](mx::InputPtr Input, UInterchangeShaderNode* NodeToConnectTo, const FString& ShaderType) -> UInterchangeShaderNode*
		{
			if (!Input)
			{
				return nullptr;
			}

			const FString ShaderNodeName = Connect.UpstreamNode->getName().c_str() + FString{ TEXT("_") } + ShaderType;
			UInterchangeShaderNode* ShaderNode = CreateShaderNode(Connect.UpstreamNode, ShaderNodeName, ShaderType);

			UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(ShaderNode, UE::Expressions::Inputs::A, NodeToConnectTo->GetUniqueID());

			// Connect the amplitude/pivot node to the shader node not the noise
			// it will be handle during the upstream-downstream connection phase
			Input->setAttribute(mx::Attributes::ParentName, TCHAR_TO_UTF8(*ShaderNodeName));
			AddAttributeFromValueOrInterface(Input, UE::Expressions::Inputs::B, ShaderNode);

			return ShaderNode;
		};

	if (UInterchangeShaderNode* MultiplyNode = ConnectNodeToInput(Connect.UpstreamNode->getInput("amplitude"), PerlinNoiseNode, Multiply::Name.ToString()))
	{
		NodeToConnect = MultiplyNode;
	}

	if (UInterchangeShaderNode* AddNode = ConnectNodeToInput(Connect.UpstreamNode->getInput("pivot"), NodeToConnect, Add::Name.ToString()))
	{
		NodeToConnect = AddNode;
	}

	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, NodeToConnect->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectCellNoise2DInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	// UE's Cellnoise algorithm returns a float3 but MaterialX returns only a float
	UInterchangeShaderNode* CellNoiseNode = ConnectNoise2DInputToOutput(Connect, VectorNoise::Name.ToString(), EVectorNoiseFunction::VNF_CellnoiseALU, 0b1000);
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, CellNoiseNode->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectWorleyNoise2DInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	// UE's Voronoi algorithm returns either XYZ (the position of a Cell in Voronoi) or W which is the distance (what we want in that case)
	// Sadly we have no way to return the distance to the closest seed in another dimensions, since the component is too deep in the algorithm (see VoronoiNoise3D_ALU in Random.ush)
	UInterchangeShaderNode* WorleyNoiseNode = ConnectNoise2DInputToOutput(Connect, VectorNoise::Name.ToString(), EVectorNoiseFunction::VNF_VoronoiALU, 0b0001);
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, WorleyNoiseNode->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectNoise3DInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	// MaterialX defines the Noise3d as Perlin Noise which is multiplied by the Amplitude then Added to Pivot
	UInterchangeShaderNode* NoiseNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), VectorNoise::Name.ToString());
	NoiseNode->AddInt32Attribute(VectorNoise::Attributes::Function.ToString(), EVectorNoiseFunction::VNF_VectorALU);

	UInterchangeShaderNode* NodeToConnect = NoiseNode;

	// Multiply Node
	auto ConnectNodeToInput = [&](mx::InputPtr Input, UInterchangeShaderNode* NodeToConnectTo, const FString& ShaderType) -> UInterchangeShaderNode*
	{
		if(!Input)
		{
			return nullptr;
		}

		const FString ShaderNodeName = Connect.UpstreamNode->getName().c_str() + FString{ TEXT("_") } + ShaderType;
		UInterchangeShaderNode* ShaderNode = CreateShaderNode(Connect.UpstreamNode, ShaderNodeName, ShaderType);

		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(ShaderNode, TEXT("A"), NodeToConnectTo->GetUniqueID());

		// Connect the amplitude node to the shader node not the noise
		// it will be handle during the upstream-downstream connection phase
		Input->setAttribute(mx::Attributes::ParentName, TCHAR_TO_UTF8(*ShaderNodeName));
		AddAttributeFromValueOrInterface(Input, UE::Expressions::Inputs::B, ShaderNode);

		return ShaderNode;
	};

	if(UInterchangeShaderNode* MultiplyNode = ConnectNodeToInput(Connect.UpstreamNode->getInput("amplitude"), NoiseNode, Multiply::Name.ToString()))
	{
		NodeToConnect = MultiplyNode;
	}

	if(UInterchangeShaderNode* AddNode = ConnectNodeToInput(Connect.UpstreamNode->getInput("pivot"), NodeToConnect, Add::Name.ToString()))
	{
		NodeToConnect = AddNode;
	}

	const std::string& Type = Connect.UpstreamNode->getType();
	uint8 Mask =
		Type == "float" ? 0b1000 :
		Type == "vector2" ? 0b1100 :
		0b1110;

	UInterchangeShaderNode* MaskNode = CreateMaskShaderNode(Mask, Connect.UpstreamNode, (Connect.UpstreamNode->getName() + "_ComponentMask").c_str());

	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(MaskNode, Mask::Inputs::Input.ToString(), NodeToConnect->GetUniqueID());
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, MaskNode->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectCellNoise3DInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	// Let's use a vector noise for this one, the only one that is close to MaterialX implementation
	UInterchangeShaderNode* NoiseNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), VectorNoise::Name.ToString());
	NoiseNode->AddInt32Attribute(VectorNoise::Attributes::Function.ToString(), EVectorNoiseFunction::VNF_CellnoiseALU);

	//cellnoise3d only supports float output
	UInterchangeShaderNode* MaskNode = CreateMaskShaderNode(0b1000, Connect.UpstreamNode, (Connect.UpstreamNode->getName() + "_ComponentMask").c_str());

	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(MaskNode, Mask::Inputs::Input.ToString(), NoiseNode->GetUniqueID());
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, MaskNode->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectWorleyNoise3DInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	//Also called Voronoi, the implementation is a bit different in UE, especially we don't have access to the jitter
	UInterchangeShaderNode* NoiseNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), VectorNoise::Name.ToString());
	NoiseNode->AddInt32Attribute(Noise::Attributes::Function.ToString(), EVectorNoiseFunction::VNF_VoronoiALU);
	// 3x3x3 neighborhood
	NoiseNode->AddInt32Attribute(Noise::Attributes::Quality.ToString(), 3);

	// Voronoi only supports float distance to the seed
	UInterchangeShaderNode* MaskNode = CreateMaskShaderNode(0b0001, Connect.UpstreamNode, (Connect.UpstreamNode->getName() + "_ComponentMask").c_str());

	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(MaskNode, Mask::Inputs::Input.ToString(), NoiseNode->GetUniqueID());
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, MaskNode->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectHeightToNormalInputToOutput(const FConnectNode& Connect)
{
	// We could always use the material function MX_HeightToNormalMap, 
	// though it has been proven that sampling textures to compute a normal map is cheaper and gives better results

	using namespace UE::Interchange::Materials::Standard::Nodes;

	if(mx::InputPtr Input = Connect.UpstreamNode->getInput("in"))
	{
		// Image node will become this node
		if(mx::NodePtr ConnectedNode = Input->getConnectedNode();
		   ConnectedNode && ConnectedNode->getCategory() == mx::Category::Image)
		{
			if(UInterchangeTextureNode* TextureNode = CreateTextureNode<UInterchangeTexture2DNode>(ConnectedNode))
			{
				UInterchangeShaderNode* HeightMapNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), NormalFromHeightMap::Name.ToString());
				UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, HeightMapNode->GetUniqueID());

				const FString TextureNodeName = Connect.UpstreamNode->getName().c_str() + FString{ "_texture" };
				UInterchangeShaderNode* TextureShaderNode = CreateShaderNode(Connect.UpstreamNode, TextureNodeName, TextureObject::Name.ToString());
				const FString TextureObjectInputKey = UInterchangeShaderPortsAPI::MakeInputParameterKey(TextureObject::Inputs::Texture.ToString());
				TextureShaderNode->AddStringAttribute(TextureObjectInputKey, TextureNode->GetUniqueID());
				UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(HeightMapNode, NormalFromHeightMap::Inputs::HeightMap.ToString(), TextureShaderNode->GetUniqueID());

				// Create a Texture Property with texel size to have a better quality when sampling the textures for the normalmap
				const FString TexturePropertyNodeName = Connect.UpstreamNode->getName().c_str() + FString{ "_textureproperty" };
				UInterchangeShaderNode* TexturePropertyNode = CreateShaderNode(Connect.UpstreamNode, TexturePropertyNodeName, TextureProperty::Name.ToString());
				TexturePropertyNode->AddInt32Attribute(TextureProperty::Attributes::Property.ToString(), EMaterialExposedTextureProperty::TMTM_TexelSize);
				UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(TexturePropertyNode, TextureProperty::Inputs::Texture.ToString(), TextureShaderNode->GetUniqueID());
				UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(HeightMapNode, NormalFromHeightMap::Inputs::Offset.ToString(), TexturePropertyNode->GetUniqueID());
				
				AddAttributeFromValueOrInterface(Connect.UpstreamNode->getInput("texcoord"), NormalFromHeightMap::Inputs::Coordinates.ToString(), HeightMapNode);

				// The default value in the material function is 8, but the default value for the node is 1
				if (!AddAttributeFromValueOrInterface(Connect.UpstreamNode->getInput("scale"), NormalFromHeightMap::Inputs::Intensity.ToString(), HeightMapNode))
				{
					const FString IntensityInputKey = UInterchangeShaderPortsAPI::MakeInputValueKey(NormalFromHeightMap::Inputs::Intensity.ToString());
					HeightMapNode->AddFloatAttribute(IntensityInputKey, 1.f);
				}
			}
			else
			{
				AddAttributeFromValueOrInterface(Connect.UpstreamNode->getInput(mx::NodeGroup::Texture2D::Inputs::Default), Connect.InputChannelName, Connect.ParentShaderNode);
			}
		}
		else
		{
			UInterchangeFunctionCallShaderNode* HeightToNormalMapNode = CreateFunctionCallShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), UE::MaterialFunctions::Path::MxHeightToNormalMap);
			
			for(mx::InputPtr InputHeightmap : Connect.UpstreamNode->getInputs())
			{
				AddAttributeFromValueOrInterface(InputHeightmap, GetInputName(InputHeightmap), HeightToNormalMapNode);
			}

			UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, HeightToNormalMapNode->GetUniqueID());
		}
	}
}

void FMaterialXSurfaceShaderAbstract::ConnectBlurInputToOutput(const FConnectNode& Connect)
{
	if(mx::InputPtr Input = Connect.UpstreamNode->getInput("in"))
	{
		// Image node will become this node
		if(mx::NodePtr ConnectedNode = Input->getConnectedNode();
		   ConnectedNode && ConnectedNode->getCategory() == mx::Category::Image)
		{
			std::string NodeType = Connect.UpstreamNode->getType();

			//we need to copy the content of the image node to this node
			Connect.UpstreamNode->copyContentFrom(ConnectedNode);

			//the copy overwrites every attribute of the node, so we need to get them back, essentially the type and the renaming
			Connect.UpstreamNode->setType(NodeType);

			SetMatchingInputsNames(Connect.UpstreamNode);

			mx::GraphElementPtr Graph = Connect.UpstreamNode->getParent()->asA<mx::GraphElement>();
			Graph->removeNode(ConnectedNode->getName());

			using namespace UE::Interchange::Materials::Standard::Nodes;

			if(UInterchangeTextureNode* TextureNode = CreateTextureNode<UInterchangeTextureBlurNode>(Connect.UpstreamNode))
			{
				FString OutputChannel{ TEXT("RGB") };

				if(NodeType == mx::Type::Vector4 || NodeType == mx::Type::Color4)
				{
					OutputChannel = TEXT("RGBA");
				}
				else if(NodeType == mx::Type::Float)
				{
					OutputChannel = TEXT("R");
				}

				UInterchangeShaderNode* TextureShaderNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), TextureSampleBlur::Name.ToString());
				TextureShaderNode->AddStringAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(TextureSampleBlur::Inputs::Texture.ToString()), TextureNode->GetUniqueID());
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(Connect.ParentShaderNode, Connect.InputChannelName, TextureShaderNode->GetUniqueID(), OutputChannel);

				if(mx::InputPtr InputKernel = Connect.UpstreamNode->getInput("filtertype"))
				{
					// By default TextureSampleBox uses gaussian filter
					if(InputKernel->getValueString() == "box")
					{
						TextureShaderNode->AddInt32Attribute(TextureSampleBlur::Attributes::Filter.ToString(), int32(EMaterialXTextureSampleBlurFilter::Box));
					}
				}

				if(mx::InputPtr InputKernel = Connect.UpstreamNode->getInput("size"))
				{
					if(InputKernel->hasValueString())
					{
						float KernelSize = mx::fromValueString<float>(InputKernel->getValueString());
						constexpr float Kernel1x1 = 0.f / 3.f;
						constexpr float Kernel3x3 = 1.f / 3.f;
						constexpr float Kernel5x5 = 2.f / 3.f;
						constexpr float Kernel7x7 = 3.f / 3.f;
						TextureShaderNode->AddInt32Attribute(TextureSampleBlur::Attributes::KernelSize.ToString(),
															 FMath::IsNearlyEqual(KernelSize, Kernel1x1) ? int32(EMAterialXTextureSampleBlurKernel::Kernel1) :
															 KernelSize <= Kernel3x3 ? int32(EMAterialXTextureSampleBlurKernel::Kernel3) :
															 KernelSize <= Kernel5x5 ? int32(EMAterialXTextureSampleBlurKernel::Kernel5) :
															 KernelSize <= Kernel7x7 ? int32(EMAterialXTextureSampleBlurKernel::Kernel7) :
															 int32(EMAterialXTextureSampleBlurKernel::Kernel1));
					}
					else
					{
						MTLX_LOG("MaterialXBlur", "<{0}>: input 'size' must have a value.", Connect.UpstreamNode->getName().c_str());
					}
				}
			}
			else
			{
				AddAttributeFromValueOrInterface(Connect.UpstreamNode->getInput(mx::NodeGroup::Texture2D::Inputs::Default), Connect.InputChannelName, Connect.ParentShaderNode);
			}
		}
		else
		{
			// For a blur it doesn't make sense if there's no image input
			SetAttributeNewName(Input, TCHAR_TO_UTF8(*Connect.InputChannelName)); //let's take the parent node's input name
			ShaderNodes.Add({ Connect.UpstreamNode->getName().c_str(), Connect.OutputName }, Connect.ParentShaderNode);
		}
	}
}

void FMaterialXSurfaceShaderAbstract::ConnectTexCoordInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	
	UInterchangeShaderNode* TexCoord;
	UInterchangeShaderNode* TexCoord2D;

	// UE only supports 2D UVs, let's just append a default 3rd channel
	if (Connect.UpstreamNode->getType() == mx::Type::Vector3)	
	{
		TexCoord2D = CreateShaderNode(Connect.UpstreamNode, (Connect.UpstreamNode->getName() + "_vector3").c_str(), TextureCoordinate::Name.ToString());
		TexCoord = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), UE::Expressions::Names::AppendVector);
		TexCoord->AddFloatAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(UE::Expressions::Inputs::B), 0.f);
		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(TexCoord, UE::Expressions::Inputs::A, TexCoord2D->GetUniqueID());
	}
	else
	{
		TexCoord = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), TextureCoordinate::Name.ToString());
		TexCoord2D = TexCoord;
	}

	if(mx::InputPtr Input = Connect.UpstreamNode->getInput("index"))
	{
		TexCoord2D->AddInt32Attribute(UInterchangeShaderPortsAPI::MakeInputValueKey(TextureCoordinate::Inputs::Index.ToString()), mx::fromValueString<int>(Input->getValueString()));
	}

	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, TexCoord->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectSwitchInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	UInterchangeShaderNode* SwitchNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), Switch::Name.ToString());

	int32 Index = 0;

	for (mx::InputPtr Input : Connect.UpstreamNode->getInputs())
	{
		if (Input->getName() != "which")
		{
			SwitchNode->AddStringAttribute(Switch::Inputs::InputName.ToString() + FString::FromInt(Index), Input->getName().c_str());
			AddAttributeFromValueOrInterface(Input, Input->getName().c_str(), SwitchNode);
			++Index;
		}
		else
		{
			AddAttributeFromValueOrInterface(Input, Switch::Inputs::Value.ToString(), SwitchNode);
		}
	}

	//In that case we also need to add an attribute to Default
	if (mx::InputPtr Input = Connect.UpstreamNode->getInput("in1"))
	{
		AddAttributeFromValueOrInterface(Input, Switch::Inputs::Default.ToString(), SwitchNode);

		//Let's add a new input that is a copy of in1 to connect it to the Default input of the MaterialExpressionSwitch
		if(!Connect.UpstreamNode->getInput("default"))
		{
			mx::InputPtr InputDefault = Connect.UpstreamNode->addInput("default");
			InputDefault->copyContentFrom(Input);
			SetAttributeNewName(InputDefault, TCHAR_TO_UTF8(*Switch::Inputs::Default.ToString()));
		}
	}

	SwitchNode->AddInt32Attribute(Switch::Attributes::InputCount.ToString(), Index);
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, SwitchNode->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectNormalMapInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	// Only create a FunctionCall if there's a scale and the value is not 1, otherwise just like dot
	mx::InputPtr Input = Connect.UpstreamNode->getInput("scale");
	bool bIsNotEqualOne = true;
	if(Input && Input->hasValueString())
	{
		bIsNotEqualOne = (mx::fromValueString<float>(Input->getValueString()) != 1.f);
	}
	else if(Input && Input->hasInterfaceName())
	{
		bIsNotEqualOne = (mx::fromValueString<float>(Input->getInterfaceInput()->getValueString()) != 1.f);
	}

	if(Input && bIsNotEqualOne)
	{
		UInterchangeShaderNode* FlattenNormalNode = CreateFunctionCallShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), TEXT("/Engine/Functions/Engine_MaterialFunctions01/Texturing/FlattenNormal.FlattenNormal"));

		UInterchangeShaderNode* OneMinusNode = CreateShaderNode(Connect.UpstreamNode, (Connect.UpstreamNode->getName() + "_OneMinusFlatness").c_str(), OneMinus::Name.ToString());
		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(FlattenNormalNode, FlattenNormal::Inputs::Flatness.ToString(), OneMinusNode->GetUniqueID());

		AddAttributeFromValueOrInterface(Input, OneMinus::Inputs::Input.ToString(), OneMinusNode);
		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, FlattenNormalNode->GetUniqueID());
	}
	else
	{
		SetAttributeNewName(Connect.UpstreamNode->getInput("in"), TCHAR_TO_UTF8(*Connect.InputChannelName)); //let's take the parent node's input name
		ShaderNodes.Add({ Connect.UpstreamNode->getName().c_str(), Connect.OutputName }, Connect.ParentShaderNode);
	}
}

void FMaterialXSurfaceShaderAbstract::ConnectRefractInputToOutput(const FConnectNode& Connect)
{
	UInterchangeShaderNode* RefractNode = CreateFunctionCallShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), TEXT("/Engine/Functions/Engine_MaterialFunctions01/Vectors/Refract.Refract"));
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, RefractNode->GetUniqueID());

	for (mx::InputPtr Input : Connect.UpstreamNode->getInputs())
	{
		AddAttributeFromValueOrInterface(Input, GetInputName(Input), RefractNode);
	}

	//Let's add a new input to map its value to the Refractive Index Target since its default value is 1.33 and we want it be 1
	// Because the Material Function does the operation IOR_SRC/IOR_TARGET
	mx::InputPtr InputIorTarget = Connect.UpstreamNode->getInput("ior_target");
	if(!InputIorTarget)
	{
		InputIorTarget = Connect.UpstreamNode->addInput("ior_target");
		InputIorTarget->setValue("1", "float");
		SetAttributeNewName(InputIorTarget, TCHAR_TO_ANSI(UE::Expressions::Inputs::RefractiveIndexTarget));
	}
	AddAttributeFromValueOrInterface(InputIorTarget, UE::Expressions::Inputs::RefractiveIndexTarget, RefractNode);
}

void FMaterialXSurfaceShaderAbstract::ConnectViewDirectionInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	UInterchangeShaderNode* ViewDirectionNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), UE::Expressions::Names::CameraVectorWS);
	UInterchangeShaderNode* NegateNode = CreateShaderNode(Connect.UpstreamNode, (Connect.UpstreamNode->getName() + "_Negate").c_str(), UE::Expressions::Names::Subtract);
	NegateNode->AddFloatAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(UE::Expressions::Inputs::A), 0.f);
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(NegateNode, UE::Expressions::Inputs::B, ViewDirectionNode->GetUniqueID());
	
	UInterchangeShaderNode* NodeToConnectTo = NegateNode;
	if (mx::InputPtr InputSpace = Connect.UpstreamNode->getInput("space"))
	{
		const bool bIsObjectSpace = InputSpace->getValueString() != "world";
		if (bIsObjectSpace)
		{
			UInterchangeShaderNode* TransformNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str() + FString(TEXT("_Transform")), TransformVector::Name.ToString());
			TransformNode->AddInt32Attribute(TransformVector::Attributes::TransformSourceType.ToString(), EMaterialVectorCoordTransformSource::TRANSFORMSOURCE_World);
			TransformNode->AddInt32Attribute(TransformVector::Attributes::TransformType.ToString(), EMaterialVectorCoordTransform::TRANSFORM_Local);
			UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(TransformNode, TransformVector::Inputs::Input.ToString(), NodeToConnectTo->GetUniqueID());
			NodeToConnectTo = TransformNode;
		}
	}
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, NodeToConnectTo->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectLogicalInputToOutput(const FConnectNode& Connect, const FString& LogicalMaterialFunction)
{
	UInterchangeShaderNode* LogicalNode = CreateFunctionCallShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), LogicalMaterialFunction);
	for(mx::InputPtr Input : Connect.UpstreamNode->getInputs())
	{
		AddAttributeFromValueOrInterface(Input, GetInputName(Input), LogicalNode, 0, true);
	}
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, LogicalNode->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectAndInputToOutput(const FConnectNode& Connect)
{
	ConnectLogicalInputToOutput(Connect, UE::MaterialFunctions::Path::MxAnd);
}

void FMaterialXSurfaceShaderAbstract::ConnectOrInputToOutput(const FConnectNode& Connect)
{
	ConnectLogicalInputToOutput(Connect, UE::MaterialFunctions::Path::MxOr);
}

void FMaterialXSurfaceShaderAbstract::ConnectXorInputToOutput(const FConnectNode & Connect)
{
	ConnectLogicalInputToOutput(Connect, UE::MaterialFunctions::Path::MxXor);
}

void FMaterialXSurfaceShaderAbstract::ConnectNotInputToOutput(const FConnectNode & Connect)
{
	ConnectLogicalInputToOutput(Connect, UE::MaterialFunctions::Path::MxNot);
}

void FMaterialXSurfaceShaderAbstract::ConnectSparseVolumeInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	// Temporarily copy-pasted here so that we can submit this on a patch version
	constexpr const char* GeomPropNameAttribute = "UE:GeomPropName";
	
	// the sparse_volume node is not standard, it's only here for USD, just like <geompropvalue> baked from primvars coming from a surface material
	// this one should only be created from a volume material in USD
	// we'll set the SparseVolumeTexture in the PostProcessFactory
	int32 OutputIndex = Connect.OutputName == "outA" ? 0 : 1;
	UInterchangeShaderNode* TextureShaderNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), TEXT("SparseVolumeTextureSampleParameter"));
	if (Connect.UpstreamNode->hasAttribute(GeomPropNameAttribute))
	{
		TextureShaderNode->AddStringAttribute(UE::Interchange::USD::Primvar::ShaderNodeSparseVolumeTextureSample, Connect.UpstreamNode->getAttribute(GeomPropNameAttribute).c_str());
	}
	const std::string& OutputTypeA = Connect.UpstreamNode->getNodeDef()->getOutput("outA")->getType();
	
	std::unordered_map<std::string, uint8> Masks{ 
		{mx::Type::Vector2, 0b1100},
		{mx::Type::Vector3, 0b1110},
		{mx::Type::Color3, 0b1100},
		{mx::Type::Float, 0b1000},
	};

	UInterchangeShaderNode* MaskNode = TextureShaderNode;
	if (OutputTypeA != mx::Type::Color4 && OutputTypeA != mx::Type::Vector4)
	{
		MaskNode = CreateMaskShaderNode(Masks[OutputTypeA], Connect.UpstreamNode, (Connect.UpstreamNode->getName() + "_Mask").c_str(), Connect.OutputName);
		UInterchangeShaderPortsAPI::ConnectOuputToInputByIndex(MaskNode, Mask::Inputs::Input.ToString(), TextureShaderNode->GetUniqueID(), OutputIndex);
		OutputIndex = 0;
	}

	UInterchangeShaderPortsAPI::ConnectOuputToInputByIndex(Connect.ParentShaderNode, Connect.InputChannelName, MaskNode->GetUniqueID(), OutputIndex);
}

void FMaterialXSurfaceShaderAbstract::ConnectLuminanceInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	UInterchangeShaderNode* LuminanceNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), Luminance::Name.ToString());
	AddAttributeFromValueOrInterface(Connect.UpstreamNode->getInput("in"), Luminance::Inputs::Input.ToString(), LuminanceNode);

	if (mx::InputPtr InputCoeffs = Connect.UpstreamNode->getInput("lumacoeffs"); InputCoeffs && InputCoeffs->hasValue())
	{
		mx::Color3 LumaCoeffs = InputCoeffs->getValue()->asA<mx::Color3>();
		LuminanceNode->AddLinearColorAttribute(Luminance::Attributes::LuminanceFactors.ToString(), FLinearColor{ LumaCoeffs[0], LumaCoeffs[1] ,LumaCoeffs[2] });
	}

	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, LuminanceNode->GetUniqueID());
}

void FMaterialXSurfaceShaderAbstract::ConnectHexTiledInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	const bool bNormalTexture = Connect.UpstreamNode->getCategory() == mx::Category::HexTiledNormalMap ? true : false;
	TGuardValue<bool>InputTypeBeingProcessedGuard(bTangentSpaceInput, bNormalTexture);

	if (UInterchangeTextureNode* TextureNode = CreateTextureNode<UInterchangeTexture2DNode>(Connect.UpstreamNode))
	{
		const FString HexTileFunctionPath = Connect.UpstreamNode->getCategory() == mx::Category::HexTiledImage ? UE::MaterialFunctions::Path::MxHexTiledImage : UE::MaterialFunctions::Path::MxHexTiledNormalMap;
		UInterchangeShaderNode* HexTileNode = CreateFunctionCallShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), HexTileFunctionPath);

		const FString TextureNodeName = Connect.UpstreamNode->getName().c_str() + FString{ "_texture" };
		UInterchangeShaderNode* TextureShaderNode = CreateShaderNode(Connect.UpstreamNode, TextureNodeName, TextureObject::Name.ToString());
		const FString TextureObjectInputKey = UInterchangeShaderPortsAPI::MakeInputParameterKey(TextureObject::Inputs::Texture.ToString());
		TextureShaderNode->AddStringAttribute(TextureObjectInputKey, TextureNode->GetUniqueID());

		for (mx::InputPtr Input : Connect.UpstreamNode->getInputs())
		{
			AddAttributeFromValueOrInterface(Input, GetInputName(Input), HexTileNode);
		}

		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(HexTileNode, TEXT("Texture"), TextureShaderNode->GetUniqueID());
		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, HexTileNode->GetUniqueID());
	}
	else
	{
		AddAttributeFromValueOrInterface(Connect.UpstreamNode->getInput("default"), Connect.InputChannelName, Connect.ParentShaderNode);
	}
}

void FMaterialXSurfaceShaderAbstract::ConnectLatLongImageInputToOutput(const FConnectNode& Connect)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	if (UInterchangeTexture2DNode* TextureNode = Cast<UInterchangeTexture2DNode>(CreateTextureNode<UInterchangeTexture2DNode>(Connect.UpstreamNode)))
	{
		TextureNode->SetForceLongLatCubemap(true);
		UInterchangeShaderNode* LatLongNode = CreateFunctionCallShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), UE::MaterialFunctions::Path::MxLatLongImage);

		const FString TextureNodeName = Connect.UpstreamNode->getName().c_str() + FString{ "_texture" };
		UInterchangeShaderNode* TextureShaderNode = CreateShaderNode(Connect.UpstreamNode, TextureNodeName, TextureObject::Name.ToString());
		const FString TextureObjectInputKey = UInterchangeShaderPortsAPI::MakeInputParameterKey(TextureObject::Inputs::Texture.ToString());
		TextureShaderNode->AddStringAttribute(TextureObjectInputKey, TextureNode->GetUniqueID());

		if (mx::InputPtr Input = Connect.UpstreamNode->getInput("viewdir"))
		{
			AddAttributeFromValueOrInterface(Input, GetInputName(Input), LatLongNode);
		}

		if (mx::InputPtr Input = Connect.UpstreamNode->getInput("rotation"))
		{
			AddAttributeFromValueOrInterface(Input, GetInputName(Input), LatLongNode);
		}

		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(LatLongNode, TEXT("Texture"), TextureShaderNode->GetUniqueID());
		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(Connect.ParentShaderNode, Connect.InputChannelName, LatLongNode->GetUniqueID());
	}
	else
	{
		AddAttributeFromValueOrInterface(Connect.UpstreamNode->getInput("default"), Connect.InputChannelName, Connect.ParentShaderNode);
	}
}

void FMaterialXSurfaceShaderAbstract::ConnectFractalInputToOutput(const FConnectNode& Connect)
{
	UInterchangeShaderNode* FractalNode = CreateShaderNode(Connect.UpstreamNode, Connect.UpstreamNode->getName().c_str(), UE::Expressions::Names::MaterialXFractal);
	for (mx::InputPtr Input : Connect.UpstreamNode->getInputs())
	{
		AddAttributeFromValueOrInterface(Input, GetInputName(Input), FractalNode);
	}

	const int32 OutputIndex =  Connect.UpstreamNode->getType() == mx::Type::Vector2 ? 1 
		: Connect.UpstreamNode->getType() == mx::Type::Vector3 || Connect.UpstreamNode->getType() == mx::Type::Color3 ? 2
		: Connect.UpstreamNode->getType() == mx::Type::Vector4 || Connect.UpstreamNode->getType() == mx::Type::Color4 ? 3
		: 0;

	UInterchangeShaderPortsAPI::ConnectOuputToInputByIndex(Connect.ParentShaderNode, Connect.InputChannelName, FractalNode->GetUniqueID(), OutputIndex);
}

bool FMaterialXSurfaceShaderAbstract::ConnectSubsurfaceTransmissionToSurfaceShader(UInterchangeShaderNode* OpenPbrStandarSurfaceShaderNode, EInterchangeMaterialXShaders SurfaceShaderType
																				   , const FName& BaseWeight, const FName& BaseColor
																				   , const FName& TransmissionWeight, const FName& TransmissionColor, const FName& TransmissionDepth, const FName& TransmissionDispersionAbbeNumber, const FName& TransmissionDispersionScale, const FName& TransmissionScatter, const FName& TransmissionScatterAnisotropy
																				   , const FName& SubsurfaceWeight, const FName& SubsurfaceColor, const FName& SubsurfaceRadiusScale
																				   , const FName& FrontMaterialOpaque, const FName& FrontMaterialTranslucent)
{
	using namespace UE::Interchange::Materials;
	using namespace UE::Interchange::MaterialX;
	using namespace UE::Interchange::Materials::Standard::Nodes;

	bool bSubsurfaceTransmission = false;

	UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, Common::Parameters::Refraction.ToString(), OpenPbrStandarSurfaceShaderNode->GetUniqueID(), TEXT("Refraction (IOR)"));

	if (UInterchangeShaderPortsAPI::HasInput(OpenPbrStandarSurfaceShaderNode, TransmissionWeight) &&
		UInterchangeShaderPortsAPI::HasInput(OpenPbrStandarSurfaceShaderNode, SubsurfaceWeight) &&
		UInterchangeShaderPortsAPI::HasInput(OpenPbrStandarSurfaceShaderNode, BaseColor))
	{
		FString SurfaceType = TEXT("Standard Surface");
		if (SurfaceShaderType == EInterchangeMaterialXShaders::OpenPBRSurface)
		{
			SurfaceType = TEXT("OpenPBR");
		}
		AnalyticsAttributes.Emplace(SurfaceType, "Opaque");

		bSubsurfaceTransmission = true;
		// 1. Set the Material as Opaque with Mask Mode
		OpenPbrStandarSurfaceShaderNode->AddInt32Attribute(Attributes::EnumType, IndexSurfaceShaders);
		OpenPbrStandarSurfaceShaderNode->AddInt32Attribute(Attributes::EnumValue, int32(SurfaceShaderType));
		ShaderGraphNode->SetCustomBlendMode(EBlendMode::BLEND_Masked);

		//2. Disconnect the transmission inputs
		FString TransmissionNodeUid, TransmissionOutputName;
		UInterchangeShaderPortsAPI::DisconnectInputFromOutputNode(OpenPbrStandarSurfaceShaderNode, TransmissionWeight.ToString(), TransmissionNodeUid, TransmissionOutputName);
		UInterchangeShaderPortsAPI::DisconnectInput(OpenPbrStandarSurfaceShaderNode, TransmissionColor.ToString());
		UInterchangeShaderPortsAPI::DisconnectInput(OpenPbrStandarSurfaceShaderNode, TransmissionDepth.ToString());
		UInterchangeShaderPortsAPI::DisconnectInput(OpenPbrStandarSurfaceShaderNode, TransmissionDispersionAbbeNumber.ToString());
		UInterchangeShaderPortsAPI::DisconnectInput(OpenPbrStandarSurfaceShaderNode, TransmissionDispersionScale.ToString());
		UInterchangeShaderPortsAPI::DisconnectInput(OpenPbrStandarSurfaceShaderNode, TransmissionScatterAnisotropy.ToString());
		UInterchangeShaderPortsAPI::DisconnectInput(OpenPbrStandarSurfaceShaderNode, TransmissionScatter.ToString());

		//3. Create DitherMask node			
		UInterchangeShaderNode* DitherMaskNode = CreateFunctionCallShaderNode(SurfaceShaderNode, TEXT("DitherMaskNode"), UE::MaterialFunctions::Path::DitherMask);

		//4. Connect the surface shader and the transmission to the MF_DitherMask
		if (const UInterchangeShaderNode* TransmissionNode = Cast<UInterchangeShaderNode>(NodeContainer.GetNode(TransmissionNodeUid)))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(DitherMaskNode, TEXT("BSDF"), OpenPbrStandarSurfaceShaderNode->GetUniqueID(), FrontMaterialOpaque.ToString());
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(DitherMaskNode, TEXT("transmission"), TransmissionNode->GetUniqueID(), TransmissionOutputName);
		}

		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::FrontMaterial.ToString(), DitherMaskNode->GetUniqueID(), TEXT("Dither"));
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::OpacityMask.ToString(), DitherMaskNode->GetUniqueID(), SubstrateMaterial::Parameters::OpacityMask.ToString());
	}
	else if (UInterchangeShaderPortsAPI::HasInput(OpenPbrStandarSurfaceShaderNode, TransmissionWeight) &&
			 UInterchangeShaderPortsAPI::HasInput(OpenPbrStandarSurfaceShaderNode, SubsurfaceWeight))
	{
		bSubsurfaceTransmission = true;
		ShaderGraphNode->SetCustomBlendMode(EBlendMode::BLEND_TranslucentColoredTransmittance);
		auto ReplaceInputConnection = [this, OpenPbrStandarSurfaceShaderNode](const FString& InputToRemove, const FString& InputName)
			{
				UInterchangeShaderPortsAPI::DisconnectInput(OpenPbrStandarSurfaceShaderNode, InputToRemove);

				FString NodeUid, NodeOutputName;
				UInterchangeShaderPortsAPI::GetInputConnection(OpenPbrStandarSurfaceShaderNode, InputName, NodeUid, NodeOutputName);
				if (const UInterchangeShaderNode* ShaderNode = Cast<UInterchangeShaderNode>(NodeContainer.GetNode(NodeUid)))
				{
					UInterchangeShaderPortsAPI::ConnectOuputToInputByName(OpenPbrStandarSurfaceShaderNode, InputToRemove, ShaderNode->GetUniqueID(), NodeOutputName);
				}
			};

		//1. subsurface_color input now replaces the base_color
		if (UInterchangeShaderPortsAPI::HasInput(OpenPbrStandarSurfaceShaderNode, SubsurfaceColor))
		{
			ReplaceInputConnection(BaseColor.ToString(), SubsurfaceColor.ToString());
		}

		//2. replace base by subsurface
		ReplaceInputConnection(BaseWeight.ToString(), SubsurfaceWeight.ToString());

		//3. set an empirical value to the TransmissionDepth if there isn't any input (otherwise just keep it as is).
		//   see egg_jade in https://github.com/stehrani3d/MaterialEggs
		//   and masonJarWater in https://github.com/DigitalProductionExampleLibrary/OpenPBRShaderPlayground
		if (!UInterchangeShaderPortsAPI::HasInput(OpenPbrStandarSurfaceShaderNode, TransmissionDepth))
		{
			constexpr float TransmissionDepthValue = 3.f;
			OpenPbrStandarSurfaceShaderNode->AddFloatAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(TransmissionDepth.ToString()), TransmissionDepthValue);
		}

		//4. Connect the SubsurfaceColor to the TransmissionScatter
		FString SubsurfaceColorNodeUid, SubsurfaceColorNodeOutputName;
		if (UInterchangeShaderPortsAPI::GetInputConnection(OpenPbrStandarSurfaceShaderNode, SubsurfaceColor.ToString(), SubsurfaceColorNodeUid, SubsurfaceColorNodeOutputName))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(OpenPbrStandarSurfaceShaderNode, TransmissionScatter.ToString(), SubsurfaceColorNodeUid, SubsurfaceColorNodeOutputName);
		}

		//5. Connect the SubsurfaceRadius for Standard Surface, SubsurfaceRadiusScale for OpenPBR to the TransmissionColor
		FString SubsurfaceRadiusNodeUid, SubsurfaceRadiusNodeOutputName;
		if (UInterchangeShaderPortsAPI::GetInputConnection(OpenPbrStandarSurfaceShaderNode, SubsurfaceRadiusScale.ToString(), SubsurfaceRadiusNodeUid, SubsurfaceRadiusNodeOutputName))
		{
			// 6. Multiply the SubsurfaceRadiusScale by the TransmissionColor if any otherwise just plug it to the TransmissionColor
			FString TransmissionColorNodeUid, TransmissionColorNodeOutputName;
			if (UInterchangeShaderPortsAPI::DisconnectInputFromOutputNode(OpenPbrStandarSurfaceShaderNode, TransmissionColor.ToString(), TransmissionColorNodeUid, TransmissionColorNodeOutputName))
			{
				UInterchangeShaderNode* SubsurfaceTransmissionColorNode = CreateShaderNode(SurfaceShaderNode, TEXT("SubsurfaceRadiusTransmissionColorMult"), Multiply::Name.ToString());
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(SubsurfaceTransmissionColorNode, Multiply::Inputs::A.ToString(), TransmissionColorNodeUid, TransmissionColorNodeOutputName);
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(SubsurfaceTransmissionColorNode, Multiply::Inputs::B.ToString(), SubsurfaceRadiusNodeUid, SubsurfaceRadiusNodeOutputName);
				UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(OpenPbrStandarSurfaceShaderNode, TransmissionColor.ToString(), SubsurfaceTransmissionColorNode->GetUniqueID());
			}
			else
			{
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(OpenPbrStandarSurfaceShaderNode, TransmissionColor.ToString(), SubsurfaceRadiusNodeUid, SubsurfaceRadiusNodeOutputName);
			}
		}

		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::FrontMaterial.ToString(), OpenPbrStandarSurfaceShaderNode->GetUniqueID(), FrontMaterialTranslucent.ToString());
	}

	return bSubsurfaceTransmission;
}

UInterchangeShaderNode* FMaterialXSurfaceShaderAbstract::CreateMaskShaderNode(uint8 RGBA, mx::ElementPtr Element, const FString& NodeName, const FString& OutputName)
{
	bool bR = (0b1000 & RGBA) >> 3;
	bool bG = (0b0100 & RGBA) >> 2;
	bool bB = (0b0010 & RGBA) >> 1;
	bool bA = (0b0001 & RGBA) >> 0;
	using namespace UE::Interchange::Materials::Standard::Nodes;
	UInterchangeShaderNode* MaskShaderNode = CreateShaderNode(Element, NodeName, Mask::Name.ToString(), OutputName);
	MaskShaderNode->AddBooleanAttribute(Mask::Attributes::R.ToString(), bR);
	MaskShaderNode->AddBooleanAttribute(Mask::Attributes::G.ToString(), bG);
	MaskShaderNode->AddBooleanAttribute(Mask::Attributes::B.ToString(), bB);
	MaskShaderNode->AddBooleanAttribute(Mask::Attributes::A.ToString(), bA);

	return MaskShaderNode;
}

UInterchangeShaderNode* FMaterialXSurfaceShaderAbstract::CreateShaderNode(mx::ElementPtr Element, const FString& NodeName, const FString& ShaderType, const FString& OutputName)
{
	UInterchangeShaderNode* Node;

	const FString NodeUID = UInterchangeShaderNode::MakeNodeUid(GetUniqueName(Element) + TEXT("_") + NodeName + TEXT('_'), FStringView{});

	//Test directly in the NodeContainer, because the ShaderNodes can be altered during the node graph either by the parent (dot/normalmap),
	//or by putting an intermediary node between the child and the parent (tiledimage)
	if(Node = const_cast<UInterchangeShaderNode*>(Cast<UInterchangeShaderNode>(NodeContainer.GetNode(NodeUID))); !Node)
	{
		Node = NewObject<UInterchangeShaderNode>(&NodeContainer);
		NodeContainer.SetupNode(Node, NodeUID, NodeName, EInterchangeNodeContainerType::TranslatedAsset);
		Node->SetCustomShaderType(ShaderType);
		ShaderNodes.FindOrAdd({ NodeName, OutputName }, Node);
	}

	return Node;
}

UInterchangeFunctionCallShaderNode* FMaterialXSurfaceShaderAbstract::CreateFunctionCallShaderNode(MaterialX::ElementPtr Element, const FString& NodeName, const FString& FunctionPath, const FString& OutputName)
{
	UInterchangeFunctionCallShaderNode* Node;

	const FString NodeUID = UInterchangeShaderNode::MakeNodeUid(GetUniqueName(Element) + TEXT("_") + NodeName  + TEXT('_'), FStringView{});

	if(Node = const_cast<UInterchangeFunctionCallShaderNode*>(Cast<UInterchangeFunctionCallShaderNode>(NodeContainer.GetNode(NodeUID))); !Node)
	{
		Node = NewObject<UInterchangeFunctionCallShaderNode>(&NodeContainer);
		NodeContainer.SetupNode(Node, NodeUID, NodeName, EInterchangeNodeContainerType::TranslatedAsset);
		Node->SetCustomMaterialFunction(FunctionPath);
		ShaderNodes.FindOrAdd({ NodeName, OutputName }, Node);
	}

	return Node;
}

UInterchangeFunctionCallShaderNode* FMaterialXSurfaceShaderAbstract::CreateFunctionCallShaderNode(MaterialX::ElementPtr Element, const FString& NodeName, uint8 EnumType, uint8 EnumValue, const FString& OutputName)
{
	UInterchangeFunctionCallShaderNode* Node;

	const FString NodeUID = UInterchangeShaderNode::MakeNodeUid(GetUniqueName(Element) + TEXT("_") + NodeName + TEXT('_'), FStringView{});

	if(Node = const_cast<UInterchangeFunctionCallShaderNode*>(Cast<UInterchangeFunctionCallShaderNode>(NodeContainer.GetNode(NodeUID))); !Node)
	{
		Node = NewObject<UInterchangeFunctionCallShaderNode>(&NodeContainer);
		NodeContainer.SetupNode(Node, NodeUID, NodeName, EInterchangeNodeContainerType::TranslatedAsset);
		//this is just a dummy path name so the Generic Material Pipeline consider it as a FunctionCallShader but where in fact the path is given by an enum
		Node->SetCustomMaterialFunction(TEXT("/Game/Default.Default"));
		Node->AddInt32Attribute(UE::Interchange::MaterialX::Attributes::EnumType, EnumType);
		Node->AddInt32Attribute(UE::Interchange::MaterialX::Attributes::EnumValue, EnumValue);
		ShaderNodes.FindOrAdd({ NodeName, OutputName }, Node);
	}

	return Node;
}

const FString& FMaterialXSurfaceShaderAbstract::GetMatchedInputName(MaterialX::NodePtr Node, MaterialX::InputPtr Input) const
{
	FMaterialXManager& Manager = FMaterialXManager::GetInstance();

	if(Input)
	{
		const FString NodeCategory{ Node->getCategory().c_str() };
		const FString InputName{ GetInputName(Input) };

		if(const FString* Result = Manager.FindMatchingInput(NodeCategory, InputName, Node->getNodeDef(mx::EMPTY_STRING, true)->getNodeGroup().c_str(), Node->getType().c_str()))
		{
			return *Result;
		}
		else if((Result = Manager.FindMatchingInput(NodeCategory, InputName)))
		{
			return *Result;
		}
		else if((Result = Manager.FindMatchingInput(EmptyString, InputName)))
		{
			return *Result;
		}
	}

	return EmptyString;
}

FString FMaterialXSurfaceShaderAbstract::GetInputName(MaterialX::InputPtr Input) const
{
	if(Input)
	{
		if (Input->hasAttribute(mx::Attributes::NewName))
		{
			return Input->getAttribute(mx::Attributes::NewName).c_str();
		}
		else
		{
			return Input->getName().c_str();
		}
	}

	return {};
}

FString FMaterialXSurfaceShaderAbstract::GetFilePrefix(MaterialX::ElementPtr Element) const
{
	return UTF8_TO_TCHAR(Element->getActiveFilePrefix().c_str());
}

FLinearColor FMaterialXSurfaceShaderAbstract::GetVector(MaterialX::InputPtr Input) const
{
	FLinearColor LinearColor = FLinearColor::Black;

	if(Input->getType() == mx::Type::Vector2)
	{
		mx::Vector2 Color = mx::fromValueString<mx::Vector2>(Input->getValueString());
		LinearColor = FLinearColor{ Color[0], Color[1], 0 };
	}
	else if(Input->getType() == mx::Type::Vector3)
	{
		mx::Vector3 Color = mx::fromValueString<mx::Vector3>(Input->getValueString());
		LinearColor = FLinearColor{ Color[0], Color[1], Color[2] };
	}
	else if(Input->getType() == mx::Type::Vector4)
	{
		mx::Vector4 Color = mx::fromValueString<mx::Vector4>(Input->getValueString());
		LinearColor = FLinearColor{ Color[0], Color[1], Color[2], Color[3] };
	}
	else
	{
		ensureMsgf(false, TEXT("Input type can only be a vectorN."));
	}

	return LinearColor;
}

FString FMaterialXSurfaceShaderAbstract::GetAttributeParentName(MaterialX::NodePtr Node, MaterialX::InputPtr ConnectedInput) const
{
	if (ConnectedInput && ConnectedInput->hasAttribute(mx::Attributes::ParentName))
	{
		return ConnectedInput->getAttribute(mx::Attributes::ParentName).c_str();
	}

	return Node->getName().c_str();
}

namespace
{
	void GetUniqueName(mx::ElementPtr Element, TStringBuilder<256>& Buffer, const FString& SurfaceMaterialName)
	{
		//Write the hierarchy name as A_B_C
		if (Element)
		{
			if (mx::ElementPtr Parent = Element->getParent())
			{
				GetUniqueName(Parent, Buffer, SurfaceMaterialName);
			}
			else
			{
				Buffer += SurfaceMaterialName + TEXT("_");
			}

			Buffer += Element->getName().c_str();
			Buffer += TEXT("_");
			Element->setAttribute(mx::Attributes::UniqueName, TCHAR_TO_UTF8(*FString{ Buffer.ToString() }.TrimChar(TEXT('_'))));
		}
	}
}

FString FMaterialXSurfaceShaderAbstract::GetUniqueName(MaterialX::ElementPtr Element) const
{
	if (Element->hasAttribute(mx::Attributes::UniqueName))
	{
		return Element->getAttribute(mx::Attributes::UniqueName).c_str();
	}
	else
	{
		TStringBuilder<256> Buffer;
		::GetUniqueName(Element, Buffer, SurfaceMaterialName);
		return FString{ Buffer.ToString() }.TrimChar(TEXT('_')); // remove any trailing '_'
	}
}

void FMaterialXSurfaceShaderAbstract::RegisterConnectNodeOutputToInputDelegates()
{
	MatchingConnectNodeDelegates.Add(mx::Category::Constant,		  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectConstantInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Extract,			  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectExtractInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Dot,				  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectDotInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::NormalMap,		  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectNormalMapInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::TransformPoint,	  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectTransformPositionInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::TransformVector,   FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectTransformVectorInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::TransformNormal,   FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectTransformVectorInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Rotate2D,		  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectRotate2DInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Image,			  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectImageInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::IfGreater,		  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectIfGreaterInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::IfGreaterEq,		  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectIfGreaterEqInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::IfEqual,			  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectIfEqualInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Outside,			  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectOutsideInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Position,		  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectPositionInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Normal,			  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectNormalInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Tangent,			  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectTangentInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Bitangent,		  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectBitangentInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Time,			  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectTimeInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Noise2D,			  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectNoise2DInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Noise3D,			  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectNoise3DInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::CellNoise2D,       FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectCellNoise2DInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::CellNoise3D,		  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectCellNoise3DInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::WorleyNoise2D,	  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectWorleyNoise2DInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::WorleyNoise3D,	  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectWorleyNoise3DInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Blur,			  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectBlurInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::HeightToNormal,	  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectHeightToNormalInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::TexCoord,		  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectTexCoordInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Switch,			  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectSwitchInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Refract,			  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectRefractInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::ViewDirection,	  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectViewDirectionInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::And,				  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectAndInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Or,				  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectOrInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Xor,				  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectXorInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Not,				  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectNotInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::SparseVolume,	  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectSparseVolumeInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Luminance,		  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectLuminanceInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::HexTiledImage,	  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectHexTiledInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::HexTiledNormalMap, FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectHexTiledInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::LatLongImage,	  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectLatLongImageInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Fractal2D,		  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectFractalInputToOutput));
	MatchingConnectNodeDelegates.Add(mx::Category::Fractal3D,		  FOnConnectNodeOutputToInput::CreateSP(this, &FMaterialXSurfaceShaderAbstract::ConnectFractalInputToOutput));
}

void FMaterialXSurfaceShaderAbstract::SetMatchingInputsNames(MaterialX::NodePtr Node) const
{
	if(Node)
	{
		if(const std::string& IsVisited = Node->getAttribute(mx::Attributes::IsVisited); IsVisited.empty())
		{
			Node->setAttribute(mx::Attributes::IsVisited, "true");

			for(mx::InputPtr Input : Node->getInputs())
			{
				if(const FString& Name = GetMatchedInputName(Node, Input); !Name.IsEmpty())
				{
					SetAttributeNewName(Input, TCHAR_TO_UTF8(*Name));
				}
			}
		}
	}
}

void FMaterialXSurfaceShaderAbstract::SetAttributeNewName(MaterialX::InputPtr Input, const char* NewName) const
{
	Input->setAttribute(mx::Attributes::NewName, NewName);
}

namespace
{
	template<typename T>
	constexpr T DefaultMaxValue = std::numeric_limits<T>::max();

	template<typename T>
	std::string ValueToString = TCHAR_TO_UTF8(*FString::SanitizeFloat(DefaultMaxValue<T>));

	template<typename T>
	T GetValue(const auto& Value)
	{
		using Type = std::remove_cvref_t<decltype(Value)>;

		if constexpr(std::is_same_v<mx::Color3, Type>)
		{
			return FLinearColor{ Value[0], Value[1], Value[2] };
		}
		else if constexpr(std::is_same_v<mx::Color4, Type>)
		{
			return FLinearColor{ Value[0], Value[1], Value[2], Value[3] };
		}
		else if constexpr(std::is_same_v<mx::Vector2, Type>)
		{
			return FVector2f{ Value[0], Value[1] };
		}
		else if constexpr(std::is_same_v<mx::Vector3, Type>)
		{
			return FVector3f{ Value[0], Value[1], Value[2] };
		}
		else if constexpr(std::is_same_v<mx::Vector4, Type>)
		{
			return FVector4f{ Value[0], Value[1], Value[2], Value[3] };
		}
	}
}

UInterchangeShaderNode* FMaterialXSurfaceShaderAbstract::Translate(EInterchangeMaterialXShaders ShaderType)
{
	UInterchangeFunctionCallShaderNode* FunctionCallShaderNode = CreateFunctionCallShaderNode(SurfaceShaderNode, SurfaceMaterialName + TEXT("_") + SurfaceShaderNode->getName().c_str(), UE::Interchange::MaterialX::IndexSurfaceShaders, uint8(ShaderType));

	return Translate(FunctionCallShaderNode);
}

UInterchangeShaderNode* FMaterialXSurfaceShaderAbstract::Translate(UInterchangeShaderNode* ShaderNode)
{
	constexpr bool bInputInTangentSpace = true;

	for (mx::InputPtr Input : SurfaceShaderNode->getInputs())
	{
		mx::ValuePtr DefaultValue = Input->getDefaultValue();

		//Let's create default values in case there is none
		if (!DefaultValue)
		{
			if (Input->getType() == mx::Type::Float)
			{
				DefaultValue = mx::Value::createValueFromStrings(ValueToString<float>, mx::Type::Float);
			}
			else if (Input->getType() == mx::Type::Color3)
			{
				DefaultValue = mx::Value::createValueFromStrings(ValueToString<float> +"," + ValueToString<float> +"," + ValueToString<float>, mx::Type::Color3);
			}
			else if (Input->getType() == mx::Type::Color4)
			{
				DefaultValue = mx::Value::createValueFromStrings(ValueToString<float> +"," + ValueToString<float> +"," + ValueToString<float> +"," + ValueToString<float>, mx::Type::Color4);
			}
			else if (Input->getType() == mx::Type::Boolean)
			{
				DefaultValue = mx::Value::createValueFromStrings("false", mx::Type::Boolean);
			}
			else if (Input->getType() == mx::Type::Integer)
			{
				DefaultValue = mx::Value::createValueFromStrings(ValueToString<int32>, mx::Type::Integer);
			}
			else if (Input->getType() == mx::Type::Vector2)
			{
				DefaultValue = mx::Value::createValueFromStrings(ValueToString<float> +"," + ValueToString<float>, mx::Type::Vector2);
			}
			else if (Input->getType() == mx::Type::Vector3)
			{
				DefaultValue = mx::Value::createValueFromStrings(ValueToString<float> +"," + ValueToString<float> +"," + ValueToString<float>, mx::Type::Vector3);
			}
			else if (Input->getType() == mx::Type::Vector4)
			{
				DefaultValue = mx::Value::createValueFromStrings(ValueToString<float> +"," + ValueToString<float> +"," + ValueToString<float> +"," + ValueToString<float>, mx::Type::Vector4);
			}
		}

		if (DefaultValue->getTypeString() == mx::Type::Float)
		{
			ConnectNodeOutputToInput(Input->getName().c_str(), ShaderNode, Input->getName().c_str(), mx::fromValueString<float>(DefaultValue->getValueString()));
		}
		else if (DefaultValue->getTypeString() == mx::Type::Color3)
		{
			ConnectNodeOutputToInput(Input->getName().c_str(), ShaderNode, Input->getName().c_str(), GetValue<FLinearColor>(mx::fromValueString<mx::Color3>(DefaultValue->getValueString())));
		}
		else if (DefaultValue->getTypeString() == mx::Type::Color4)
		{
			ConnectNodeOutputToInput(Input->getName().c_str(), ShaderNode, Input->getName().c_str(), GetValue<FLinearColor>(mx::fromValueString<mx::Color4>(DefaultValue->getValueString())));
		}
		else if (DefaultValue->getTypeString() == mx::Type::Boolean)
		{
			ConnectNodeOutputToInput(Input->getName().c_str(), ShaderNode, Input->getName().c_str(), mx::fromValueString<bool>(DefaultValue->getValueString()));
		}
		else if (DefaultValue->getTypeString() == mx::Type::Vector2)
		{
			ConnectNodeOutputToInput(Input->getName().c_str(), ShaderNode, Input->getName().c_str(), FVector4f{ GetValue<FVector2f>(mx::fromValueString<mx::Vector2>(DefaultValue->getValueString())), FVector2f{0,0} }, bInputInTangentSpace);
		}
		else if (DefaultValue->getTypeString() == mx::Type::Vector3)
		{
			ConnectNodeOutputToInput(Input->getName().c_str(), ShaderNode, Input->getName().c_str(), GetValue<FVector3f>(mx::fromValueString<mx::Vector3>(DefaultValue->getValueString())), bInputInTangentSpace);
		}
		else if (DefaultValue->getTypeString() == mx::Type::Vector4)
		{
			ConnectNodeOutputToInput(Input->getName().c_str(), ShaderNode, Input->getName().c_str(), GetValue<FVector4f>(mx::fromValueString<mx::Vector4>(DefaultValue->getValueString())), bInputInTangentSpace);
		}
		else if (DefaultValue->getTypeString() == mx::Type::Integer)
		{
			ConnectNodeOutputToInput(Input->getName().c_str(), ShaderNode, Input->getName().c_str(), mx::fromValueString<int32>(DefaultValue->getValueString()));
		}
		else // this is most likely a BSDF
		{
			ConnectNodeOutputToInput(Input->getName().c_str(), ShaderNode, GetInputName(Input), nullptr); // for BSDF/Surface shaders, we need to take a matching input just in case
		}
	}

	return ShaderNode;
}

mx::InputPtr FMaterialXSurfaceShaderAbstract::GetInputNormal(mx::NodePtr Node, const char*& InputNormal) const
{
	InputNormal = "normal";
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
#endif