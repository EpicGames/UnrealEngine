// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "MaterialXSurfaceUnlitShader.h"

#include "Engine/EngineTypes.h"

namespace mx = MaterialX;

FMaterialXSurfaceUnlitShader::FMaterialXSurfaceUnlitShader(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: FMaterialXSurfaceShaderAbstract(BaseNodeContainer)
{
	NodeDefinition = mx::NodeDefinition::SurfaceUnlit;
}

TSharedRef<FMaterialXBase> FMaterialXSurfaceUnlitShader::MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	TSharedRef<FMaterialXSurfaceUnlitShader> Result = MakeShared<FMaterialXSurfaceUnlitShader>(BaseNodeContainer);
	Result->RegisterConnectNodeOutputToInputDelegates();
	return Result;
}

UInterchangeBaseNode* FMaterialXSurfaceUnlitShader::Translate(mx::NodePtr SurfaceUnlitNode)
{
	using namespace UE::Interchange::Materials;
	this->SurfaceShaderNode = SurfaceUnlitNode;

	UInterchangeShaderNode* SurfaceUnlitShaderNode = FMaterialXSurfaceShaderAbstract::Translate(EInterchangeMaterialXShaders::SurfaceUnlit);

	if(!bIsSubstrateEnabled)
	{
		// Outputs
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::EmissiveColor.ToString(), SurfaceUnlitShaderNode->GetUniqueID(), PBRMR::Parameters::EmissiveColor.ToString());

		//We can't have both Opacity and Opacity Mask we need to make a choice	
		if(UInterchangeShaderPortsAPI::HasInput(SurfaceUnlitShaderNode, SurfaceUnlit::Parameters::Transmission))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Opacity.ToString(), SurfaceUnlitShaderNode->GetUniqueID(), PBRMR::Parameters::Opacity.ToString());
		}
		else if(UInterchangeShaderPortsAPI::HasInput(SurfaceUnlitShaderNode, SurfaceUnlit::Parameters::Opacity))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Opacity.ToString(), SurfaceUnlitShaderNode->GetUniqueID(), SurfaceUnlit::Outputs::OpacityMask.ToString());
			ShaderGraphNode->SetCustomOpacityMaskClipValue(1.f); // just to connect to the opacity mask
		}
	}
	else
	{
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::FrontMaterial.ToString(), SurfaceUnlitShaderNode->GetUniqueID(), SurfaceUnlit::Substrate::Outputs::SurfaceUnlit.ToString());
		if(UInterchangeShaderPortsAPI::HasInput(SurfaceUnlitShaderNode, SurfaceUnlit::Parameters::Transmission))
		{
			ShaderGraphNode->SetCustomBlendMode(EBlendMode::BLEND_TranslucentColoredTransmittance);
		}
		else if(UInterchangeShaderPortsAPI::HasInput(SurfaceUnlitShaderNode, SurfaceUnlit::Parameters::Opacity))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::OpacityMask.ToString(), SurfaceUnlitShaderNode->GetUniqueID(), SurfaceUnlit::Substrate::Outputs::OpacityMask.ToString());
			ShaderGraphNode->SetCustomBlendMode(EBlendMode::BLEND_Masked);
		}
	}

	AnalyticsAttributes.Emplace(TEXT("ShaderType"), TEXT("Surface Unlit"));
	return SurfaceUnlitShaderNode;
}
#endif