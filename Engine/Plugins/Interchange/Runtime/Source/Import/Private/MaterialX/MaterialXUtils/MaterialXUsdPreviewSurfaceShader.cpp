// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "MaterialXUsdPreviewSurfaceShader.h"
#include "Engine/EngineTypes.h"

namespace mx = MaterialX;

FMaterialXUsdPreviewSurfaceShader::FMaterialXUsdPreviewSurfaceShader(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: FMaterialXSurfaceShaderAbstract(BaseNodeContainer)
{
	NodeDefinition = mx::NodeDefinition::UsdPreviewSurface;
}

TSharedRef<FMaterialXBase> FMaterialXUsdPreviewSurfaceShader::MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	TSharedRef<FMaterialXUsdPreviewSurfaceShader> Result = MakeShared<FMaterialXUsdPreviewSurfaceShader>(BaseNodeContainer);
	Result->RegisterConnectNodeOutputToInputDelegates();
	return Result;
}

UInterchangeBaseNode* FMaterialXUsdPreviewSurfaceShader::Translate(MaterialX::NodePtr UsdPreviewSurfaceNode)
{
	this->SurfaceShaderNode = UsdPreviewSurfaceNode;

	using namespace UE::Interchange::Materials;

	UInterchangeShaderNode* UsdPreviewSurfaceShaderNode = FMaterialXSurfaceShaderAbstract::Translate(EInterchangeMaterialXShaders::UsdPreviewSurface);

	// Outputs
	if(!bIsSubstrateEnabled)
	{
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::BaseColor.ToString(), UsdPreviewSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::BaseColor.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Metallic.ToString(), UsdPreviewSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Metallic.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Specular.ToString(), UsdPreviewSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Specular.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Roughness.ToString(), UsdPreviewSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Roughness.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::EmissiveColor.ToString(), UsdPreviewSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::EmissiveColor.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Normal.ToString(), UsdPreviewSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Normal.ToString());

		if(UInterchangeShaderPortsAPI::HasInput(UsdPreviewSurfaceShaderNode, UsdPreviewSurface::Parameters::Opacity))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Opacity.ToString(), UsdPreviewSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Opacity.ToString());
		}

		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Occlusion.ToString(), UsdPreviewSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Occlusion.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, PBRMR::Parameters::Refraction.ToString(), UsdPreviewSurfaceShaderNode->GetUniqueID(), PBRMR::Parameters::Refraction.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, ClearCoat::Parameters::ClearCoat.ToString(), UsdPreviewSurfaceShaderNode->GetUniqueID(), ClearCoat::Parameters::ClearCoat.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, ClearCoat::Parameters::ClearCoatRoughness.ToString(), UsdPreviewSurfaceShaderNode->GetUniqueID(), ClearCoat::Parameters::ClearCoatRoughness.ToString());

		if(UInterchangeShaderPortsAPI::HasInput(UsdPreviewSurfaceShaderNode, UsdPreviewSurface::Parameters::Displacement))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, Common::Parameters::Displacement.ToString(), UsdPreviewSurfaceShaderNode->GetUniqueID(), Common::Parameters::Displacement.ToString());
		}
	}
	else
	{
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::FrontMaterial.ToString(), UsdPreviewSurfaceShaderNode->GetUniqueID(), UsdPreviewSurface::SubstrateMaterial::Outputs::FrontMaterial.ToString());
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::Occlusion.ToString(), UsdPreviewSurfaceShaderNode->GetUniqueID(), SubstrateMaterial::Parameters::Occlusion.ToString());

		if(UInterchangeShaderPortsAPI::HasInput(UsdPreviewSurfaceShaderNode, UsdPreviewSurface::Parameters::Displacement))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(ShaderGraphNode, SubstrateMaterial::Parameters::Displacement.ToString(), UsdPreviewSurfaceShaderNode->GetUniqueID(), SubstrateMaterial::Parameters::Displacement.ToString());
		}

		if(UInterchangeShaderPortsAPI::HasInput(UsdPreviewSurfaceShaderNode, UsdPreviewSurface::Parameters::Opacity))
		{
			ShaderGraphNode->SetCustomBlendMode(EBlendMode::BLEND_TranslucentColoredTransmittance);
		}
	}

	AnalyticsAttributes.Emplace(TEXT("ShaderType"), TEXT("USD Preview Surface"));

	return UsdPreviewSurfaceShaderNode;
}

mx::InputPtr FMaterialXUsdPreviewSurfaceShader::GetInputNormal(mx::NodePtr UsdPreviewSurfaceNode, const char*& InputNormal) const
{
	InputNormal = mx::UsdPreviewSurface::Input::Normal;
	mx::InputPtr Input = UsdPreviewSurfaceNode->getActiveInput(InputNormal);
	
	if (!Input)
	{
		Input = UsdPreviewSurfaceNode->getNodeDef(mx::EMPTY_STRING, true)->getActiveInput(InputNormal);
	}

	return Input;
}
#endif