// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "MaterialXVolumeMaterial.h"
#include "InterchangeImportLog.h"
#include "MaterialX/MaterialXUtils/MaterialXManager.h"
#include "MaterialX/MaterialXUtils/MaterialXSurfaceShaderAbstract.h"

#define LOCTEXT_NAMESPACE "InterchangeMaterialXVolumeMaterial"

namespace mx = MaterialX;

FMaterialXVolumeMaterial::FMaterialXVolumeMaterial(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: FMaterialXBase(BaseNodeContainer)
{}

TSharedRef<FMaterialXBase> FMaterialXVolumeMaterial::MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	return MakeShared<FMaterialXVolumeMaterial>(BaseNodeContainer);
}

UInterchangeBaseNode* FMaterialXVolumeMaterial::Translate(MaterialX::NodePtr VolumeMaterialNode)
{
	using namespace UE::Interchange::Materials;

	if (!FMaterialXManager::GetInstance().IsSubstrateEnabled())
	{
		MTLX_LOG("MaterialXVolumeMaterial", "\"{0}\": {1} is only supported with Substrate enabled.", VolumeMaterialNode->getName().c_str(), VolumeMaterialNode->getCategory().c_str());
		return nullptr;
	}

	const FString ShaderGraphNodeUID = UInterchangeShaderNode::MakeNodeUid(UTF8_TO_TCHAR(VolumeMaterialNode->getName().c_str()), FStringView{});
	UInterchangeShaderGraphNode* ShaderGraphNode = const_cast<UInterchangeShaderGraphNode*>(Cast<UInterchangeShaderGraphNode>(NodeContainer.GetNode(ShaderGraphNodeUID)));

	if(ShaderGraphNode)
	{
		return ShaderGraphNode;
	}

	if (mx::InputPtr Input = VolumeMaterialNode->getInput("volumeshader"))
	{
		if (mx::NodePtr VolumeShaderNode = Input->getConnectedNode())
		{
			if(TSharedPtr<FMaterialXSurfaceShaderAbstract> ShaderTranslator = StaticCastSharedPtr<FMaterialXSurfaceShaderAbstract>(FMaterialXManager::GetInstance().GetShaderTranslator(VolumeShaderNode->getCategory().c_str(), NodeContainer)))
			{
				ShaderGraphNode = NewObject<UInterchangeShaderGraphNode>(&NodeContainer);
				NodeContainer.SetupNode(ShaderGraphNode, ShaderGraphNodeUID, VolumeMaterialNode->getName().c_str(), EInterchangeNodeContainerType::TranslatedAsset);
				ShaderTranslator->ShaderGraphNode = ShaderGraphNode;
				ShaderTranslator->SurfaceMaterialName = VolumeShaderNode->getName().c_str();
				ShaderTranslator->SetTranslator(Translator);
				ShaderTranslator->Translate(VolumeShaderNode);

				AnalyticsAttributes.Emplace(TEXT("MaterialType"), TEXT("Volume Material"));
			}
			else
			{
				MTLX_LOG("MaterialXVolumeMaterial", "<{0}>: {1} \"{2}\" is not supported", VolumeShaderNode->getCategory().c_str(), VolumeShaderNode->getType().c_str(), VolumeShaderNode->getName().c_str());
				AnalyticsAttributes.Emplace(TEXT("ShaderType"), TEXT("Volume Shader unsupported"));
			}
		}
	}

	return ShaderGraphNode;
}

#undef LOCTEXT_NAMESPACE 
#endif