// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "MaterialXVolumeShader.h"
#include "InterchangeImportLog.h"

namespace mx = MaterialX;

FMaterialXVolumeShader::FMaterialXVolumeShader(UInterchangeBaseNodeContainer& BaseNodeContainer)
	: FMaterialXSurfaceShaderAbstract(BaseNodeContainer)
{
	NodeDefinition = mx::NodeDefinition::Volume;
}

TSharedRef<FMaterialXBase> FMaterialXVolumeShader::MakeInstance(UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	TSharedRef<FMaterialXVolumeShader> Result = MakeShared<FMaterialXVolumeShader>(BaseNodeContainer);
	Result->RegisterConnectNodeOutputToInputDelegates();
	return Result;
}

UInterchangeBaseNode* FMaterialXVolumeShader::Translate(MaterialX::NodePtr VolumeNode)
{
	this->SurfaceShaderNode = VolumeNode;

	UInterchangeShaderNode* FunctionVolumeShaderNode = FMaterialXSurfaceShaderAbstract::Translate(EInterchangeMaterialXShaders::Volume);

	using namespace UE::Interchange::Materials;

	// Outputs
	UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(ShaderGraphNode, SubstrateMaterial::Parameters::FrontMaterial.ToString(), FunctionVolumeShaderNode->GetUniqueID());

	AnalyticsAttributes.Emplace(TEXT("ShaderType"), TEXT("Volume (Generic)"));

	return FunctionVolumeShaderNode;
}
#endif