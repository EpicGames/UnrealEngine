// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Mesh/InterchangeGeometryCacheFactory.h"

#include "Animation/InterchangeAnimationPayloadInterface.h"
#include "Async/ParallelFor.h"
#include "GeometryCache.h"
#include "GeometryCacheCodecV1.h"
#include "GeometryCacheComponent.h"
#include "GeometryCacheHelpers.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrackStreamable.h"
#include "InterchangeCommonPipelineDataFactoryNode.h"
#include "InterchangeGeometryCacheFactoryNode.h"
#include "InterchangeImportCommon.h"
#include "InterchangeImportLog.h"
#include "InterchangeManager.h"
#include "InterchangeMeshDefinitions.h"
#include "InterchangeSceneNode.h"
#include "InterchangeSourceData.h"
#include "Mesh/InterchangeMeshHelper.h"
#include "Mesh/InterchangeMeshPayloadInterface.h"
#include "Misc/ScopedSlowTask.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGeometryCacheFactory)

#if WITH_EDITORONLY_DATA
#include "EditorFramework/AssetImportData.h"
#endif //WITH_EDITORONLY_DATA

static int32 GGeometryCacheParallelFrameReads = 16;
static FAutoConsoleVariableRef CVarGeometryCacheParallelFrameReads(
	TEXT("Interchange.GeometryCache.ParallelFrameReads"),
	GGeometryCacheParallelFrameReads,
	TEXT("Maximum number of frames to read in parallel")
);

class FGeometryCacheComponentResetAsset
{
public:
	/* 
	 * Clear the given geometry cache from any geometry cache component using it.
	 * This stops the geometry cache from playing while it is re-imported.
	 */
	FGeometryCacheComponentResetAsset(UGeometryCache* InGeometryCache)
	: GeometryCache(InGeometryCache)
	{
		for (TObjectIterator<UGeometryCacheComponent> GeometryCacheComponentIt; GeometryCacheComponentIt; ++GeometryCacheComponentIt)
		{
			if (GeometryCacheComponentIt->GeometryCache == InGeometryCache)
			{
				GeometryCacheComponentIt->SetGeometryCache(nullptr);
				GeometryCacheComponents.Add(*GeometryCacheComponentIt);
			}
		}
	}

	/** Restore the geometry cache on the geometry cache components that were previously using it */
	~FGeometryCacheComponentResetAsset()
	{
		const int32 NumComponents = GeometryCacheComponents.Num();
		for (UGeometryCacheComponent* GeometryCacheComponent : GeometryCacheComponents)
		{
			GeometryCacheComponent->SetGeometryCache(GeometryCache);
		}
	}

	FGeometryCacheComponentResetAsset(const FGeometryCacheComponentResetAsset& Other) = delete;
	FGeometryCacheComponentResetAsset(const FGeometryCacheComponentResetAsset&& Other) = delete;
	FGeometryCacheComponentResetAsset& operator=(const FGeometryCacheComponentResetAsset& Other) = delete;

private:
	UGeometryCache* GeometryCache;
	TArray<UGeometryCacheComponent*> GeometryCacheComponents;
};

UClass* UInterchangeGeometryCacheFactory::GetFactoryClass() const
{
	return UGeometryCache::StaticClass();
}

void UInterchangeGeometryCacheFactory::CreatePayloadTasks(const FImportAssetObjectParams& Arguments, bool bAsync, TArray<TSharedPtr<UE::Interchange::FInterchangeTaskBase>>& PayloadTasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeGeometryCacheFactory::CreatePayloadTasks);

	const IInterchangeAnimationPayloadInterface* AnimSequenceTranslatorPayloadInterface = Cast<IInterchangeAnimationPayloadInterface>(Arguments.Translator);
	if (!AnimSequenceTranslatorPayloadInterface)
	{
		return;
	}

	const UInterchangeGeometryCacheFactoryNode* GeometryCacheFactoryNode = Cast<UInterchangeGeometryCacheFactoryNode>(Arguments.AssetNode);
	if (!GeometryCacheFactoryNode)
	{
		return;
	}

	TMap<FString, FString> PayloadKeys;
	GeometryCacheFactoryNode->GetSceneNodeAnimationPayloadKeys(PayloadKeys);
	TransformAnimationPayloadResults.Reserve(PayloadKeys.Num());

	// Create one payload task per query
	for (const TPair<FString, FString>& SceneNodeUidAndPayloadKey : PayloadKeys)
	{
		const FString& SceneNodeUid = SceneNodeUidAndPayloadKey.Key;

		// Force the payload key type to TRANSFORM since we need transforms, not curves
		FInterchangeAnimationPayLoadKey PayloadKey(SceneNodeUidAndPayloadKey.Value, EInterchangeAnimationPayLoadType::GEOMETRY_CACHE_TRANSFORM);
		UE::Interchange::FAnimationPayloadQuery AnimationPayloadQuery(SceneNodeUid, PayloadKey);

		// Allocate the results so we do not need any mutex in the callback
		TransformAnimationPayloadResults.Add(SceneNodeUid, UE::Interchange::FAnimationPayloadData(SceneNodeUid, PayloadKey));

		TArray<UE::Interchange::FAnimationPayloadQuery> SingleQuery{AnimationPayloadQuery};
		TSharedPtr<UE::Interchange::FInterchangeTaskLambda, ESPMode::ThreadSafe> GetAnimationPayloadTask = MakeShared<UE::Interchange::FInterchangeTaskLambda, ESPMode::ThreadSafe>(
			bAsync ? UE::Interchange::EInterchangeTaskThread::AsyncThread : UE::Interchange::EInterchangeTaskThread::GameThread,
			[SingleQuery, &TransformAnimationPayloadResults = TransformAnimationPayloadResults, AnimSequenceTranslatorPayloadInterface]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeGeometryCacheFactory::GetAnimationPayloadSingleQuery);

				TArray<UE::Interchange::FAnimationPayloadData> AnimationPayloads = AnimSequenceTranslatorPayloadInterface->GetAnimationPayloadData(SingleQuery);
				for (const UE::Interchange::FAnimationPayloadData& PayloadData : AnimationPayloads)
				{
					TransformAnimationPayloadResults[PayloadData.SceneNodeUniqueID] = PayloadData;
				}
			});
		PayloadTasks.Add(GetAnimationPayloadTask);
	}
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeGeometryCacheFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeGeometryCacheFactory::BeginImportAsset_GameThread);

	FImportAssetResult ImportAssetResult;

	const IInterchangeMeshPayloadInterface* MeshTranslatorPayloadInterface = Cast<IInterchangeMeshPayloadInterface>(Arguments.Translator);
	if (!MeshTranslatorPayloadInterface)
	{
		UE_LOGF(LogInterchangeImport, Error, "Cannot import geometry cache. The translator does not implement IInterchangeMeshPayloadInterface.");
		return ImportAssetResult;
	}

	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	const UInterchangeGeometryCacheFactoryNode* GeometryCacheFactoryNode = Cast<UInterchangeGeometryCacheFactoryNode>(Arguments.AssetNode);
	if (GeometryCacheFactoryNode == nullptr)
	{
		return ImportAssetResult;
	}

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (GeometryCacheFactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}
	}

	// Query the build settings
	bool bFlattenTracks = true;
	GeometryCacheFactoryNode->GetCustomFlattenTracks(bFlattenTracks);

	float PositionPrecision = 0.01f;
	GeometryCacheFactoryNode->GetCustomPositionPrecision(PositionPrecision);

	int32 NumBitsForUVs = 10;
	GeometryCacheFactoryNode->GetCustomNumBitsForUVs(NumBitsForUVs);

	TOptional<int32> FrameStartOverride;
	if (int32 FrameStart = 0; GeometryCacheFactoryNode->GetCustomStartFrame(FrameStart))
	{
		FrameStartOverride = FrameStart;
	}

	TOptional<int32> FrameEndOverride;
	if (int32 FrameEnd = 0; GeometryCacheFactoryNode->GetCustomEndFrame(FrameEnd))
	{
		FrameEndOverride = FrameEnd;
	}

	EInterchangeMotionVectorsHandling MotionVectors = EInterchangeMotionVectorsHandling::NoMotionVectors;
	GeometryCacheFactoryNode->GetCustomMotionVectorsImport(MotionVectors);

	bool bApplyConstantTopologyOptimizations = false;
	GeometryCacheFactoryNode->GetCustomApplyConstantTopologyOptimization(bApplyConstantTopologyOptimizations);

	bool bStoreImportedVertexNumbers = false;
	GeometryCacheFactoryNode->GetCustomStoreImportedVertexNumbers(bStoreImportedVertexNumbers);

	bool bOptimizeIndexBuffers = false;
	GeometryCacheFactoryNode->GetCustomOptimizeIndexBuffers(bOptimizeIndexBuffers);

	FTransform GlobalOffsetTransform = FTransform::Identity;
	if (UInterchangeCommonPipelineDataFactoryNode* CommonPipelineDataFactoryNode = UInterchangeCommonPipelineDataFactoryNode::GetUniqueInstance(Arguments.NodeContainer))
	{
		CommonPipelineDataFactoryNode->GetCustomGlobalOffsetTransform(GlobalOffsetTransform);
	}

	// Helper class to get the transform for a mesh at a given frame number
	// The animated transforms from the payloads are local so they must be
	// composed together from leaf to root to get the world transform
	class FAnimatedTransformStack
	{
	public:
		FAnimatedTransformStack() = default;

		FAnimatedTransformStack(const FTransform& InDefaultTransform)
		: DefaultTransform(InDefaultTransform)
		{
		}

		FTransform GetTransform(int32 FrameNumber) const
		{
			if (TransformStack.Num() == 0)
			{
				return DefaultTransform;
			}
			else
			{
				FTransform ComposedTransform;
				for (int32 Index = 0; Index < TransformStack.Num(); ++Index)
				{
					int32 NormalizedFrameIndex = FrameNumber - FrameIndexOffsets[Index];
					const TArray<FTransform>& CurrentTransforms = *TransformStack[Index];
					if (CurrentTransforms.IsValidIndex(NormalizedFrameIndex))
					{
						ComposedTransform *= CurrentTransforms[NormalizedFrameIndex];
					}
					else
					{
						ComposedTransform *= NormalizedFrameIndex >= CurrentTransforms.Num() ? CurrentTransforms.Last() : CurrentTransforms[0];
					}
				}
				return ComposedTransform;
			}
		}

		// Add static transform
		void AddTransform(const FTransform& InTransform)
		{
			// Create a transform array with a single transform that it owns to free on destruction
			TUniquePtr<TArray<FTransform>>& Transform = OwnedTransforms.Emplace_GetRef(MakeUnique<TArray<FTransform>>(TArray<FTransform>({InTransform})));
			TransformStack.Add(Transform.Get());
			FrameIndexOffsets.Add(0);
		}

		// Add animated transform
		void AddTransform(const TArray<FTransform>& InTransforms, int32 IndexOffset)
		{
			// Keep a reference to an array of transforms
			TransformStack.Add(&InTransforms);
			FrameIndexOffsets.Add(IndexOffset);
		}

	private:
		TArray<const TArray<FTransform>*> TransformStack;
		TArray<int32> FrameIndexOffsets;
		TArray<TUniquePtr<TArray<FTransform>>> OwnedTransforms;
		FTransform DefaultTransform;
	};

	struct FNodeInfo
	{
		FString Uid;
		const UInterchangeMeshNode* Node = nullptr;
		FAnimatedTransformStack TransformStack;
		double FrameRate = 24.0;
		bool bConstantTopology = true;
	};

	TArray<FNodeInfo> ValidatedNodes;
	int32 FrameStart = MIN_int32;
	int32 FrameEnd = MAX_int32;
	bool bGloballyConstantTopology = true;

	TArray<FString> MeshUids;
	GeometryCacheFactoryNode->GetTargetNodeUids(MeshUids);
	const FString& AssetName = Arguments.AssetName;
	for (int32 MeshIndex = 0; MeshIndex < MeshUids.Num(); ++MeshIndex)
	{
		const FString& MeshUid = MeshUids[MeshIndex];

		FTransform GlobalMeshTransform;
		const UInterchangeBaseNode* Node = Arguments.NodeContainer->GetNode(MeshUid);
		const UInterchangeGeometryCacheNode* GeometryCacheNode = Cast<const UInterchangeGeometryCacheNode>(Node);
		const UInterchangeMeshNode* MeshNode = nullptr;
		const UInterchangeSceneNode* SceneNode = nullptr;
		if (GeometryCacheNode == nullptr)
		{
			// MeshUid must refer to a scene node
			SceneNode = Cast<const UInterchangeSceneNode>(Node);
			if (!ensure(SceneNode))
			{
				UE_LOGF(LogInterchangeImport, Warning, "Invalid mesh reference when importing GeometryCache asset %ls.", *AssetName);
				continue;
			}

			// Ignore invisible meshes
			bool bVisible = true;
			if (SceneNode->GetCustomActorVisibility(bVisible) && !bVisible)
			{
				continue;
			}

			// Get the transform from the scene node
			FTransform SceneNodeGlobalTransform;
			if (SceneNode->GetCustomGlobalTransform(Arguments.NodeContainer, GlobalOffsetTransform, SceneNodeGlobalTransform))
			{
				GlobalMeshTransform = SceneNodeGlobalTransform;
			}

			constexpr bool bBakeMeshes = true;
			constexpr bool bBakePivotMeshes = false;
			UE::Interchange::Private::MeshHelper::AddSceneNodeGeometricAndPivotToGlobalTransform(GlobalMeshTransform, SceneNode, bBakeMeshes, bBakePivotMeshes);

			// And get the mesh node which it references
			FString MeshDependencyUid;
			SceneNode->GetCustomAssetInstanceUid(MeshDependencyUid);
			GeometryCacheNode = Cast<UInterchangeGeometryCacheNode>(Arguments.NodeContainer->GetNode(MeshDependencyUid));
			MeshNode = Cast<UInterchangeMeshNode>(Arguments.NodeContainer->GetNode(MeshDependencyUid));
		}
		else
		{
			// If we have a mesh that is not reference by a scene node, we must apply the global offset.
			GlobalMeshTransform = GlobalOffsetTransform;
		}

		if (!ensure(MeshNode))
		{
			UE_LOGF(LogInterchangeImport, Warning, "Invalid mesh reference when importing GeometryCache asset %ls.", *AssetName);
			continue;
		}

		// Build the transform stack for the mesh
		// The global mesh transform will be used as the default if there's no animated transforms
		FAnimatedTransformStack TransformStack(GlobalMeshTransform);
		if (TransformAnimationPayloadResults.Num() > 0)
		{
			// Add the local transform to the stack from leaf to root
			const UInterchangeSceneNode* LoopNode = SceneNode;
			while (LoopNode)
			{
				// Use the animated transforms associated with the scene node, otherwise use its local transform
				UE::Interchange::FAnimationPayloadData* Payload = TransformAnimationPayloadResults.Find(LoopNode->GetUniqueID());
				if (Payload)
				{
					TransformStack.AddTransform(Payload->Transforms, Payload->RangeStartTime);
				}
				else if (FTransform LocalTransform; LoopNode->GetCustomLocalTransform(LocalTransform))
				{
					TransformStack.AddTransform(LocalTransform);
				}
				LoopNode = Cast<UInterchangeSceneNode>(Arguments.NodeContainer->GetNode(LoopNode->GetParentUid()));
			}
		}

		// Cache the node info for the validated node
		FNodeInfo& NodeInfo = ValidatedNodes.Emplace_GetRef();
		NodeInfo.Uid = MeshUid;
		NodeInfo.Node = MeshNode;
		NodeInfo.TransformStack = MoveTemp(TransformStack);

		if (GeometryCacheNode)
		{
			GeometryCacheNode->GetCustomHasConstantTopology(NodeInfo.bConstantTopology);
			bGloballyConstantTopology &= NodeInfo.bConstantTopology;

			int32 MeshFrameStart = 0;
			GeometryCacheNode->GetCustomStartFrame(MeshFrameStart);
			FrameStart = FMath::Max(FrameStart, MeshFrameStart);

			int32 MeshFrameEnd = 0;
			GeometryCacheNode->GetCustomEndFrame(MeshFrameEnd);
			FrameEnd = FMath::Min(FrameEnd, MeshFrameEnd);

			GeometryCacheNode->GetCustomFrameRate(NodeInfo.FrameRate);
		}
	}

	// Apply the time range overrides if enabled
	if (FrameStartOverride.IsSet())
	{
		FrameStart = FrameStartOverride.GetValue();
	}

	if (FrameEndOverride.IsSet())
	{
		FrameEnd = FrameEndOverride.GetValue();
	}

	// Make sure there's at least one frame in the animation
	if (FrameEnd <= FrameStart)
	{
		FrameEnd = FrameStart + 1;
	}

	if (ValidatedNodes.Num() == 0)
	{
		UE_LOGF(LogInterchangeImport, Error, "Cannot import geometry cache. No valid mesh to import.");
		return ImportAssetResult;
	}

	// Create a new geometry cache or overwrite existing asset, if possible
	UGeometryCache* GeometryCache = nullptr;
	if (!ExistingAsset)
	{
		GeometryCache = NewObject<UGeometryCache>(Arguments.Parent, *Arguments.AssetName, RF_Public | RF_Standalone);
	}
	else
	{
		// This is a reimport, we are just re-updating the source data
		GeometryCache = Cast<UGeometryCache>(ExistingAsset);
		if (GeometryCache)
		{
			ResetAssetOnReimport = MakePimpl<FGeometryCacheComponentResetAsset>(GeometryCache);

			// Backup the material assignment since it will be reset with the call to ClearForReimporting
			TArray<TObjectPtr<UMaterialInterface>> ExistingMaterials = GeometryCache->Materials;
			TArray<FName> ExistingMaterialSlotNames = GeometryCache->MaterialSlotNames;

			GeometryCache->ClearForReimporting();

			// Re-apply the material assignment
			GeometryCache->Materials = ExistingMaterials;
			GeometryCache->MaterialSlotNames = ExistingMaterialSlotNames;
		}
	}

	if (!GeometryCache)
	{
		if (!Arguments.ReimportObject)
		{
			UE_LOGF(LogInterchangeImport, Warning, "Could not create GeometryCache asset %ls.", *AssetName);
		}
		return ImportAssetResult;
	}

#if WITH_EDITOR
	GeometryCache->PreEditChange(nullptr);
#endif // WITH_EDITOR

	const bool bCalculateMotionVectors = MotionVectors == EInterchangeMotionVectorsHandling::CalculateMotionVectorsDuringImport;
	auto CreateTrack = [GeometryCache, PositionPrecision, NumBitsForUVs, bApplyConstantTopologyOptimizations, bCalculateMotionVectors, bOptimizeIndexBuffers](const FString& Name, bool bConstantTopology)
	{
		FName CodecName = MakeUniqueObjectName(GeometryCache, UGeometryCacheCodecV1::StaticClass(), FName(Name + FString(TEXT("_Codec"))));
		UGeometryCacheCodecV1* Codec = NewObject<UGeometryCacheCodecV1>(GeometryCache, CodecName, RF_Public);

#if WITH_EDITORONLY_DATA
		Codec->InitializeEncoder(PositionPrecision, NumBitsForUVs);
#endif

		FName TrackName = MakeUniqueObjectName(GeometryCache, UGeometryCacheTrackStreamable::StaticClass(), FName(Name));
		UGeometryCacheTrackStreamable* Track = NewObject<UGeometryCacheTrackStreamable>(GeometryCache, TrackName, RF_Public);

#if WITH_EDITORONLY_DATA
		const bool bCanApplyConstantTopologyOptimizations = bApplyConstantTopologyOptimizations && bConstantTopology;
		Track->BeginCoding(Codec, bCanApplyConstantTopologyOptimizations, bCalculateMotionVectors, bOptimizeIndexBuffers);
		// EndCoding has to be called from the main thread once all the frame data have been added to the track
#endif
		return Track;
	};

	// No need to flatten tracks if there's only one mesh
	bFlattenTracks &= ValidatedNodes.Num() > 1;
	if (bFlattenTracks)
	{
		Tracks.Add(CreateTrack(AssetName, bGloballyConstantTopology));
	}
	else
	{
		// Create a track for each mesh to be processed
		for (const FNodeInfo& NodeInfo : ValidatedNodes)
		{
			const FString& MeshUid = NodeInfo.Uid;

			// Extract the mesh name
			// #ueent_todo: Make this more generic as this is currently based on the USD uids
			TArray<FString> MeshUidTokens;
			MeshUid.ParseIntoArray(MeshUidTokens, TEXT("/"));
			FString MeshName = MeshUidTokens[MeshUidTokens.Num() - 1];

			Tracks.Add(CreateTrack(MeshName, NodeInfo.bConstantTopology));
		}
	}

	const bool bReimport = Arguments.ReimportObject && GeometryCache;

	//Call the mesh helper to create the missing material and to use the unmatched existing slot with the unmatched import slot
	{
		using namespace UE::Interchange::Private::MeshHelper;
		TMap<FString, FString> SlotMaterialDependencies;
		GeometryCacheFactoryNode->GetSlotMaterialDependencies(SlotMaterialDependencies);
		GeometryCacheFactorySetupAssetMaterialArray(GeometryCache->Materials, GeometryCache->MaterialSlotNames, SlotMaterialDependencies, Arguments.NodeContainer, bReimport);
	}

	int32 NumFrames = FrameEnd - FrameStart;
	FString Title = FString::Format(TEXT("Importing frames for {0} ({1}/{2})"), { AssetName, 0,  NumFrames });
	FScopedSlowTask SlowTask(static_cast<float>(FrameEnd - FrameStart), FText::FromString(Title));
	SlowTask.MakeDialog(true);

	const int32 MaxWorkerThreads = FApp::ShouldUseThreadingForPerformance() ? FTaskGraphInterface::Get().GetNumWorkerThreads() : 1;
	const int32 NumFrameThreads = FMath::Clamp(MaxWorkerThreads, 1, GGeometryCacheParallelFrameReads);

	// The GeometryCache data is processed in the game thread since the payloads are queried in
	// parallel but need to be processed in order. This allows to minimize memory usage since
	// only NumFrameThreads amount of data will be in memory at a time.
	// 
	// ProcessFrames takes a delegate to handle the conversion to FGeometryCacheMeshData, which
	// can be merged into a single mesh (flattened) or kept separate (one per track)
	UE::Interchange::FAttributeStorage PayloadAttributes;
	UInterchangeMeshFactoryNode::CopyPayloadKeyStorageAttributes(GeometryCacheFactoryNode, PayloadAttributes);
	auto ProcessFrames = [MeshTranslatorPayloadInterface, NumFrameThreads, FrameStart, FrameEnd, &SlowTask, &Tracks = Tracks, &ValidatedNodes, &AssetName, NumFrames, &PayloadAttributes](auto ConvertMeshes)
	{
		// Fetch the payloads for all meshes at time FrameIndex and return their processed MeshDescriptions
		auto ReadFrame = [&ValidatedNodes, &AssetName, MeshTranslatorPayloadInterface, &PayloadAttributes](int32 FrameIndex)
		{
			TArray<FMeshDescription> MeshDescriptions;
			for (const FNodeInfo& NodeInfo : ValidatedNodes)
			{
				const UInterchangeMeshNode* MeshNode = NodeInfo.Node;

				TOptional<FInterchangeMeshPayLoadKey> OptionalPayLoadKey = MeshNode->GetPayLoadKey();
				if (!ensure(OptionalPayLoadKey.IsSet()))
				{
					MeshDescriptions.Add({});
					UE_LOGF(LogInterchangeImport, Warning, "Empty mesh reference payload when importing GeometryCache asset %ls.", *AssetName);
					continue;
				}

				FInterchangeMeshPayLoadKey PayLoadKey(OptionalPayLoadKey->UniqueId, FrameIndex);

				// Use a copy of the payload attributes since the transform will vary for each mesh and frame
				UE::Interchange::FAttributeStorage PayloadAttributesCopy(PayloadAttributes);
				PayloadAttributesCopy.RegisterAttribute(UE::Interchange::FAttributeKey{ UE::Interchange::MeshPayload::Attributes::MeshGlobalTransform }, NodeInfo.TransformStack.GetTransform(FrameIndex));
				TOptional<UE::Interchange::FMeshPayloadData> PayloadData = MeshTranslatorPayloadInterface->GetMeshPayloadData(PayLoadKey, PayloadAttributesCopy);

				if (PayloadData.IsSet() && !PayloadData->MeshDescription.IsEmpty())
				{
					// Compute the normals and tangents for the mesh
					const float ComparisonThreshold = THRESH_POINTS_ARE_SAME;

					// This function make sure the Polygon Normals Tangents Binormals are computed and also remove degenerated triangle from the render mesh
					// description.
					FStaticMeshOperations::ComputeTriangleTangentsAndNormals(PayloadData->MeshDescription, ComparisonThreshold);

					// Compute any missing normals or tangents.
					// Static meshes always blend normals of overlapping corners.
					EComputeNTBsFlags ComputeNTBsOptions = EComputeNTBsFlags::BlendOverlappingNormals;
					ComputeNTBsOptions |= EComputeNTBsFlags::IgnoreDegenerateTriangles;
					ComputeNTBsOptions |= EComputeNTBsFlags::UseMikkTSpace;

					FStaticMeshOperations::ComputeTangentsAndNormals(PayloadData->MeshDescription, ComputeNTBsOptions);

					MeshDescriptions.Emplace(PayloadData->MeshDescription);
				}
				else
				{
					MeshDescriptions.Add({});
				}
			}
			return MeshDescriptions;
		};

		// Frame data can be read concurrently but will be processed sequentially.
		std::atomic<int32> WriteFrameIndex = FrameStart;
		FCriticalSection Mutex;
		FEvent* FrameWrittenEvent = FPlatformProcess::GetSynchEventFromPool();
		int32 TotalFrames = 0;
		float CompletedFrames = 0.0f;
		bool bIsCancelled = false;

		ParallelFor(NumFrameThreads, 
			[FrameStart, 
			FrameEnd,
			&bIsCancelled,
			&SlowTask,
			ReadFrame,
			ConvertMeshes,
			&WriteFrameIndex,
			&Mutex,
			&FrameWrittenEvent,
			&Tracks, 
			&ValidatedNodes, 
			&AssetName,
			&TotalFrames, 
			&CompletedFrames, 
			NumFrames,
			NumFrameThreads](int32 ThreadIndex)
		{
			int32 FrameIndex = FrameStart + ThreadIndex;

			while (FrameIndex < FrameEnd && !bIsCancelled)
			{
				if (IsInGameThread() && SlowTask.ShouldCancel())
				{
					bIsCancelled = true;
					break;
				}

				// Read frame data into memory
				TArray<FMeshDescription> MeshDescriptions = ReadFrame(FrameIndex);

				// And convert it to flattened mesh or separate meshes
				TArray<FGeometryCacheMeshData> MeshData = ConvertMeshes(FrameIndex, MeshDescriptions);

				// Wait until it's our turn to process this frame.
				while (WriteFrameIndex < FrameIndex)
				{
					const uint32 WaitTimeInMs = 10;
					FrameWrittenEvent->Wait(WaitTimeInMs);
				}

				{
					FScopeLock WriteLock(&Mutex);

#if WITH_EDITORONLY_DATA
					// Add the mesh data to the tracks
					for (int32 Index = 0; Index < MeshData.Num(); ++Index)
					{
						if (UGeometryCacheTrackStreamable* Track = Cast<UGeometryCacheTrackStreamable>(Tracks[Index].Get()))
						{
							const FNodeInfo& NodeInfo = ValidatedNodes[Index];
							Track->AddMeshSample(MeshData[Index], (FrameIndex - FrameStart) / NodeInfo.FrameRate, NodeInfo.bConstantTopology);
						}
					}
#endif

					// Mark the next frame index as ready for processing.
					++WriteFrameIndex;

					++CompletedFrames;
					if (IsInGameThread())
					{
						TotalFrames += int32(CompletedFrames);
						FString Title = FString::Format(TEXT("Importing frames for {0} ({1}/{2})"), { AssetName, TotalFrames,  NumFrames });

						SlowTask.EnterProgressFrame(CompletedFrames, FText::FromString(Title));
						CompletedFrames = 0.0f;
					}

					FrameWrittenEvent->Trigger();
				}

				// Get new frame index to read for next run cycle
				FrameIndex += NumFrameThreads;
			}
		});

		FPlatformProcess::ReturnSynchEventToPool(FrameWrittenEvent);

		return !bIsCancelled;
	};

	bool bSuccess = false;
	const bool bUseVelocitiesAsMotionVectors = MotionVectors == EInterchangeMotionVectorsHandling::ImportVelocitiesAsMotionVectors;
	if (bFlattenTracks)
	{
		auto MergeMeshes = [FrameRate = ValidatedNodes[0].FrameRate, bUseVelocitiesAsMotionVectors, bStoreImportedVertexNumbers](int32 FrameIndex, TArray<FMeshDescription>& MeshDescriptions)
		{
			// Take the first MeshDescription as the base on which to append the others
			FMeshDescription MergedMesh = MoveTemp(MeshDescriptions[0]);

			TArray<const FMeshDescription*> MeshDescriptionPtrs;
			MeshDescriptionPtrs.Reserve(MeshDescriptions.Num());
			for (int32 Index = 1; Index < MeshDescriptions.Num(); ++Index)
			{
				MeshDescriptionPtrs.Add(&MeshDescriptions[Index]);
			}

			FStaticMeshOperations::FAppendSettings AppendSettings;
			FStaticMeshOperations::AppendMeshDescriptions(MeshDescriptionPtrs, MergedMesh, AppendSettings);

			UE::GeometryCache::Utils::FMeshDataConversionArguments ConversionArgs;
			ConversionArgs.MaterialOffset = 0;
			ConversionArgs.FramesPerSecond = FrameRate;
			ConversionArgs.bUseVelocitiesAsMotionVectors = bUseVelocitiesAsMotionVectors;
			ConversionArgs.bStoreImportedVertexNumbers = bStoreImportedVertexNumbers;

			FGeometryCacheMeshData MeshData;
			UE::GeometryCache::Utils::GetGeometryCacheMeshDataFromMeshDescription(MeshData, MergedMesh, ConversionArgs);

			return TArray<FGeometryCacheMeshData>{MeshData};
		};

		bSuccess = ProcessFrames(MergeMeshes);
	}
	else
	{
		auto OneMeshPerTrack = [&ValidatedNodes, bUseVelocitiesAsMotionVectors, bStoreImportedVertexNumbers](int32 FrameIndex, TArray<FMeshDescription>& MeshDescriptions)
		{
			TArray<FGeometryCacheMeshData> AllMeshData;
			AllMeshData.Reserve(ValidatedNodes.Num());
			int32 MaterialOffset = 0;
			for (int32 Index = 0; Index < ValidatedNodes.Num(); ++Index)
			{
				if (MeshDescriptions[Index].IsEmpty())
				{
					AllMeshData.Add({});
					continue;
				}

				// Convert the MeshDescription to MeshData
				UE::GeometryCache::Utils::FMeshDataConversionArguments ConversionArgs;
				ConversionArgs.MaterialOffset = MaterialOffset;
				ConversionArgs.FramesPerSecond = ValidatedNodes[Index].FrameRate;
				ConversionArgs.bUseVelocitiesAsMotionVectors = bUseVelocitiesAsMotionVectors;
				ConversionArgs.bStoreImportedVertexNumbers = bStoreImportedVertexNumbers;

				FGeometryCacheMeshData MeshData;
				UE::GeometryCache::Utils::GetGeometryCacheMeshDataFromMeshDescription(MeshData, MeshDescriptions[Index], ConversionArgs);
				AllMeshData.Emplace(MeshData);

				++MaterialOffset;
			}
			return AllMeshData;
		};

		bSuccess = ProcessFrames(OneMeshPerTrack);
	}

	if (!bSuccess)
	{
		GeometryCache->MarkAsGarbage();
		return ImportAssetResult;
	}

	GeometryCache->SetFrameStartEnd(FrameStart, FrameEnd);

	ImportAssetResult.ImportedObject = GeometryCache;
	return ImportAssetResult;
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeGeometryCacheFactory::ImportAsset_Async(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeGeometryCacheFactory::ImportAsset_Async);

	FImportAssetResult ImportAssetResult;
	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	UInterchangeGeometryCacheFactoryNode* GeometryCacheFactoryNode = Cast<UInterchangeGeometryCacheFactoryNode>(Arguments.AssetNode);
	if (GeometryCacheFactoryNode == nullptr)
	{
		return ImportAssetResult;
	}

	UObject* GeometryCacheObject = UE::Interchange::FFactoryCommon::AsyncFindObject(GeometryCacheFactoryNode, GetFactoryClass(), Arguments.Parent, Arguments.AssetName);
	if (!GeometryCacheObject)
	{
		UE_LOGF(LogInterchangeImport, Error, "Could not import the GeometryCache asset %ls because the asset does not exist.", *Arguments.AssetName);
		return ImportAssetResult;
	}

	UGeometryCache* GeometryCache = Cast<UGeometryCache>(GeometryCacheObject);
	if (!ensure(GeometryCache))
	{
		UE_LOGF(LogInterchangeImport, Error, "Could not cast to GeometryCache asset %ls.", *Arguments.AssetName);
		return ImportAssetResult;
	}

	// Getting the file Hash will cache it into the source data
	Arguments.SourceData->GetFileContentHash();

	ImportAssetResult.ImportedObject = GeometryCacheObject;
	return ImportAssetResult;
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeGeometryCacheFactory::EndImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeGeometryCacheFactory::EndImportAsset_GameThread);

	FImportAssetResult ImportAssetResult;
	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	UInterchangeGeometryCacheFactoryNode* GeometryCacheFactoryNode = Cast<UInterchangeGeometryCacheFactoryNode>(Arguments.AssetNode);
	if (GeometryCacheFactoryNode == nullptr)
	{
		return ImportAssetResult;
	}

	const UClass* GeometryCacheClass = GeometryCacheFactoryNode->GetObjectClass();
	check(GeometryCacheClass && GeometryCacheClass->IsChildOf(GetFactoryClass()));

	UObject* ExistingAsset = StaticFindObject(nullptr, Arguments.Parent, *Arguments.AssetName);

	UGeometryCache* GeometryCache = Cast<UGeometryCache>(ExistingAsset);
	if (!ensure(GeometryCache))
	{
		UE_LOGF(LogInterchangeImport, Error, "Could not create GeometryCache asset %ls.", *Arguments.AssetName);
		return ImportAssetResult;
	}

#if WITH_EDITORONLY_DATA
	// Finalize the coding for all tracks
	for (UGeometryCacheTrack* Track : Tracks)
	{
		if (UGeometryCacheTrackStreamable* StreamableTrack = Cast<UGeometryCacheTrackStreamable>(Track))
		{
			if (StreamableTrack->EndCoding())
			{
				TArray<FMatrix> Mats;
				Mats.Add(FMatrix::Identity);
				Mats.Add(FMatrix::Identity);

				TArray<float> MatTimes;
				MatTimes.Add(0.0f);
				MatTimes.Add(0.0f);
				Track->SetMatrixSamples(Mats, MatTimes);

				GeometryCache->AddTrack(StreamableTrack);
			}
		}
	}
#endif

	ImportAssetResult.ImportedObject = GeometryCache;

	ResetAssetOnReimport.Reset();

	return ImportAssetResult;
}

void UInterchangeGeometryCacheFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeGeometryCacheFactory::SetupObject_GameThread);

	Super::SetupObject_GameThread(Arguments);

#if WITH_EDITORONLY_DATA
	if (ensure(Arguments.ImportedObject && Arguments.SourceData))
	{
		// We must call the Update of the asset source file in the main thread because UAssetImportData::Update execute some delegate we do not control
		UGeometryCache* GeometryCache = CastChecked<UGeometryCache>(Arguments.ImportedObject);

		UE::Interchange::FFactoryCommon::FUpdateImportAssetDataParameters UpdateImportAssetDataParameters(GeometryCache, GeometryCache->AssetImportData, Arguments.SourceData, Arguments.NodeUniqueID, Arguments.NodeContainer, Arguments.OriginalPipelines, Arguments.Translator);
		GeometryCache->AssetImportData = UE::Interchange::FFactoryCommon::UpdateImportAssetData(UpdateImportAssetDataParameters);
	}
#endif
}

bool UInterchangeGeometryCacheFactory::GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const
{
#if WITH_EDITORONLY_DATA
	if (const UGeometryCache* GeometryCache = Cast<UGeometryCache>(Object))
	{
		return UE::Interchange::FFactoryCommon::GetSourceFilenames(GeometryCache->AssetImportData, OutSourceFilenames);
	}
#endif

	return false;
}

bool UInterchangeGeometryCacheFactory::SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const
{
#if WITH_EDITORONLY_DATA
	if (const UGeometryCache* GeometryCache = Cast<UGeometryCache>(Object))
	{
		return UE::Interchange::FFactoryCommon::SetSourceFilename(GeometryCache->AssetImportData, SourceFilename, SourceIndex);
	}
#endif

	return false;
}

void UInterchangeGeometryCacheFactory::BackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UGeometryCache* GeometryCache = Cast<UGeometryCache>(Object))
	{
		UE::Interchange::FFactoryCommon::BackupSourceData(GeometryCache->AssetImportData);
	}
#endif
}

void UInterchangeGeometryCacheFactory::ReinstateSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UGeometryCache* GeometryCache = Cast<UGeometryCache>(Object))
	{
		UE::Interchange::FFactoryCommon::ReinstateSourceData(GeometryCache->AssetImportData);
	}
#endif
}

void UInterchangeGeometryCacheFactory::ClearBackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UGeometryCache* GeometryCache = Cast<UGeometryCache>(Object))
	{
		UE::Interchange::FFactoryCommon::ClearBackupSourceData(GeometryCache->AssetImportData);
	}
#endif
}
