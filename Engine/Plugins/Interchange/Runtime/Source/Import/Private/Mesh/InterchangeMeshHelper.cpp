// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/InterchangeMeshHelper.h"

#if WITH_EDITOR
#include "BSPOps.h"
#endif
#include "Engine/NaniteAssemblyData.h"
#include "Engine/Polys.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshSocket.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Math/GenericOctree.h"
#include "Mesh/InterchangeMeshPayload.h"
#include "MeshDescription.h"
#include "MeshUVChannelInfo.h"
#include "Misc/PackageName.h"
#include "Model.h"
#include "NaniteAssemblyDataBuilder.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/ConvexElem.h"
#include "Rendering/SkeletalMeshModel.h"
#include "SkeletalMeshAttributes.h"
#include "SkeletalMeshTypes.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"

#include "InterchangeAssetImportData.h"
#include "InterchangeCommonPipelineDataFactoryNode.h"
#include "InterchangeHelper.h"
#include "InterchangeImportLog.h"
#include "InterchangeJointNode.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeMeshDefinitions.h"
#include "InterchangeMeshNode.h"
#include "InterchangeSceneNode.h"
#include "InterchangeStaticMeshFactoryNode.h"
#include "InterchangeStaticMeshLodDataNode.h"
#include "Nodes/InterchangeSourceNode.h"

#include <type_traits>

#if WITH_EDITORONLY_DATA

#include "EditorFramework/AssetImportData.h"

#endif //WITH_EDITORONLY_DATA

namespace UE::Interchange::Private::MeshHelper
{
	bool AddConvexGeomFromVertices(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		const FMeshDescription& MeshDescription,
		FKAggregateGeom& AggGeom
	)
	{
		FStaticMeshConstAttributes Attributes(MeshDescription);
		TVertexAttributesRef<const FVector3f> VertexPositions = Attributes.GetVertexPositions();

		if (VertexPositions.GetNumElements() == 0)
		{
			return false;
		}

		FKConvexElem& ConvexElem = AggGeom.ConvexElems.Emplace_GetRef();
		ConvexElem.VertexData.AddUninitialized(VertexPositions.GetNumElements());

		for (int32 Index = 0; Index < VertexPositions.GetNumElements(); Index++)
		{
			ConvexElem.VertexData[Index] = FVector(VertexPositions[Index]);
		}

		ConvexElem.UpdateElemBox();

		return true;
	}

	bool DecomposeConvexMesh(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		const FMeshDescription& MeshDescription,
		UBodySetup* BodySetup
	)
	{
#if WITH_EDITOR

		// Construct a bit array containing a bit for each triangle ID in the mesh description.
		// We are assuming the mesh description is compact, i.e. it has no holes, and so the number of triangles is equal to the array size.
		// The aim is to identify 'islands' of adjacent triangles which will form separate convex hulls

		check(MeshDescription.Triangles().Num() == MeshDescription.Triangles().GetArraySize());
		TBitArray<> BitArray(0, MeshDescription.Triangles().Num());

		// Here we build the groups of triangle IDs

		TArray<TArray<FTriangleID>> TriangleGroups;

		int32 FirstIndex = BitArray.FindAndSetFirstZeroBit();
		while (FirstIndex != INDEX_NONE)
		{
			// Find the first index we haven't used yet, and use it as the beginning of a new triangle group

			TArray<FTriangleID>& TriangleGroup = TriangleGroups.Emplace_GetRef();
			TriangleGroup.Emplace(FirstIndex);

			// Now iterate through the TriangleGroup array, finding unused adjacent triangles to each index, and appending them
			// to the end of the array.  Note we deliberately check the array size each time round the loop, as each iteration
			// can cause it to grow.

			for (int32 CheckIndex = 0; CheckIndex < TriangleGroup.Num(); ++CheckIndex)
			{
				for (FTriangleID AdjacentTriangleID : MeshDescription.GetTriangleAdjacentTriangles(TriangleGroup[CheckIndex]))
				{
					if (BitArray[AdjacentTriangleID] == 0)
					{
						// Append unused adjacent triangles to the TriangleGroup, to be considered for adjacency later
						TriangleGroup.Emplace(AdjacentTriangleID);
						BitArray[AdjacentTriangleID] = 1;
					}
				}
			}

			// When we exhaust the triangle group array, there are no more triangles in this island.
			// Now find the start of the next group.

			FirstIndex = BitArray.FindAndSetFirstZeroBit();
		}

		// Now iterate through the triangle groups, adding each as a convex hull to the AggGeom

		UModel* TempModel = NewObject<UModel>();
		TempModel->RootOutside = true;
		TempModel->EmptyModel(true, true);
		TempModel->Polys->ClearFlags(RF_Transactional);

		FStaticMeshConstAttributes Attributes(MeshDescription);
		TTriangleAttributesRef<TArrayView<const FVertexID>> TriangleVertices = Attributes.GetTriangleVertexIndices();
		TVertexAttributesRef<const FVector3f> VertexPositions = Attributes.GetVertexPositions();

		bool bSuccess = true;

		for (const TArray<FTriangleID>& TriangleGroup : TriangleGroups)
		{
			// Initialize a new brush
			TempModel->Polys->Element.Empty();

			// Add each triangle to the brush
			int32 Index = 0;
			for (FTriangleID TriangleID : TriangleGroup)
			{
				FPoly& Poly = TempModel->Polys->Element.Emplace_GetRef();
				Poly.Init();
				Poly.iLink = Index++;

				// For reasons lost in time, BSP poly vertices have the opposite winding order to regular mesh vertices.
				// So add them backwards (sigh)
				Poly.Vertices.Emplace(FVector(VertexPositions[TriangleVertices[TriangleID][2]]));
				Poly.Vertices.Emplace(FVector(VertexPositions[TriangleVertices[TriangleID][1]]));
				Poly.Vertices.Emplace(FVector(VertexPositions[TriangleVertices[TriangleID][0]]));

				Poly.CalcNormal(true);
			}

			// Build bounding box
			TempModel->BuildBound();

			// Build BSP for the brush
			FBSPOps::bspBuild(TempModel, FBSPOps::BSP_Good, 15, 70, 1, 0);
			FBSPOps::bspRefresh(TempModel, true);
			FBSPOps::bspBuildBounds(TempModel);

			bSuccess &= BodySetup->CreateFromModel(TempModel, false);
		}

		TempModel->ClearInternalFlags(EInternalObjectFlags::Async);
		TempModel->Polys->ClearInternalFlags(EInternalObjectFlags::Async);

		return bSuccess;

#else	 // #if WITH_EDITOR

		return false;

#endif
	}

	static bool AreEqual(float A, float B)
	{
		constexpr float MeshToPrimTolerance = 0.001f;
		return FMath::Abs(A - B) < MeshToPrimTolerance;
	}

	static bool AreParallel(const FVector3f& A, const FVector3f& B)
	{
		float Dot = FVector3f::DotProduct(A, B);

		return (AreEqual(FMath::Abs(Dot), 1.0f));
	}

	static FVector3f GetTriangleNormal(
		TVertexAttributesRef<const FVector3f> VertexPositions,
		TArrayView<const FVertexID> VertexIndices
	)
	{
		const FVector3f& V0 = VertexPositions[VertexIndices[0]];
		const FVector3f& V1 = VertexPositions[VertexIndices[1]];
		const FVector3f& V2 = VertexPositions[VertexIndices[2]];
		// @todo: LWC conversions everywhere here; surely this can be more elegant?
		return FVector3f(FVector(FVector3f::CrossProduct(V1 - V0, V2 - V0).GetSafeNormal()));
	}

	bool AddBoxGeomFromTris(
		const FMeshDescription& MeshDescription,
		FKAggregateGeom& AggGeom,
		bool bForcePrimitiveGeneration
	)
	{
		FStaticMeshConstAttributes Attributes{MeshDescription};
		TTriangleAttributesRef<TArrayView<const FVertexID>> TriangleVertices = Attributes.GetTriangleVertexIndices();
		TVertexAttributesRef<const FVector3f> VertexPositions = Attributes.GetVertexPositions();
		FBox Box{ForceInit};

		// Maintain an array of the planes we have encountered so far.
		// We are expecting two instances of three unique plane orientations, one for each side of the box.

		struct FPlaneInfo
		{
			FPlaneInfo(const FVector3f InNormal, float InFirstDistance)
				: Normal(InNormal)
				, DistCount(1)
				, PlaneDist{InFirstDistance, 0.0f}
			{
			}

			FVector3f Normal;
			int32 DistCount;
			float PlaneDist[2];
		};

		TArray<FPlaneInfo> Planes;		

		for (FTriangleID TriangleID : MeshDescription.Triangles().GetElementIDs())
		{
			TArrayView<const FVertexID> VertexIndices = TriangleVertices[TriangleID];

			// Maintain an AABB, adding points from each triangle.
			Box += FVector(VertexPositions[VertexIndices[0]]);
			Box += FVector(VertexPositions[VertexIndices[1]]);
			Box += FVector(VertexPositions[VertexIndices[2]]);

			FVector3f TriangleNormal = GetTriangleNormal(VertexPositions, VertexIndices);
			if (TriangleNormal.IsNearlyZero())
			{
				continue;
			}

			bool bFoundPlane = false;
			bool bFailedPlanes = false;
			for (int32 PlaneIndex = 0; PlaneIndex < Planes.Num() && !bFoundPlane; PlaneIndex++)
			{
				// if this triangle plane is already known...
				if (AreParallel(TriangleNormal, Planes[PlaneIndex].Normal))
				{
					// Always use the same normal when comparing distances, to ensure consistent sign.
					float Dist = FVector3f::DotProduct(VertexPositions[VertexIndices[0]], Planes[PlaneIndex].Normal);

					// we only have one distance, and its not that one, add it.
					if (Planes[PlaneIndex].DistCount == 1 && !AreEqual(Dist, Planes[PlaneIndex].PlaneDist[0]))
					{
						Planes[PlaneIndex].PlaneDist[1] = Dist;
						Planes[PlaneIndex].DistCount = 2;
					}
					// if we have a second distance, and its not that either, something is wrong.
					else if (Planes[PlaneIndex].DistCount == 2 
						&& !AreEqual(Dist, Planes[PlaneIndex].PlaneDist[0])
						&& !AreEqual(Dist, Planes[PlaneIndex].PlaneDist[1])
					)
					{
						bFailedPlanes = true;
						break;
					}

					bFoundPlane = true;
				}
			}

			// If this triangle does not match an existing plane, add to list.
			if (!bFailedPlanes && !bFoundPlane)
			{
				Planes.Emplace(TriangleNormal, FVector3f::DotProduct(VertexPositions[VertexIndices[0]], TriangleNormal));
			}
		}

		// Now we have our candidate planes, see if we can match all the requirements

		// Check for the right number of planes.
		if (Planes.Num() == 3)
		{
			// Check if we have the 3 pairs we need
			if ((Planes[0].DistCount == 2) && (Planes[1].DistCount == 2) && (Planes[2].DistCount == 2))
			{
				// ensure valid TM by cross-product
				if (AreParallel(FVector3f::CrossProduct(Planes[0].Normal, Planes[1].Normal), Planes[2].Normal))
				{
					// Allocate box in array
					FKBoxElem BoxElem;
			
					// In case we have a box oriented with the world axis system we want to reorder the plane to not introduce axis swap.
					// If the box was turned, the order of the planes will be arbitrary and the box rotation will make the collision
					// not playing well if the asset is built or place in a level with a non uniform scale.
					FVector3f Axis[3] = {FVector3f::XAxisVector, FVector3f::YAxisVector, FVector3f::ZAxisVector};
					int32 Reorder[3] = {INDEX_NONE, INDEX_NONE, INDEX_NONE};
					for (int32 PlaneIndex = 0; PlaneIndex < 3; ++PlaneIndex)
					{
						for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
						{
							if (AreParallel(Planes[PlaneIndex].Normal, Axis[AxisIndex]))
							{
								Reorder[PlaneIndex] = AxisIndex;
								break;
							}
						}
					}
			
					if (Reorder[0] == INDEX_NONE || Reorder[1] == INDEX_NONE || Reorder[2] == INDEX_NONE)
					{
						Reorder[0] = 0;
						Reorder[1] = 1;
						Reorder[2] = 2;
					}
			
					BoxElem.SetTransform(
						FTransform(FVector(Planes[Reorder[0]].Normal), FVector(Planes[Reorder[1]].Normal), FVector(Planes[Reorder[2]].Normal), Box.GetCenter())
					);
			
					// distance between parallel planes is box edge lengths.
					BoxElem.X = FMath::Abs(Planes[Reorder[0]].PlaneDist[0] - Planes[Reorder[0]].PlaneDist[1]);
					BoxElem.Y = FMath::Abs(Planes[Reorder[1]].PlaneDist[0] - Planes[Reorder[1]].PlaneDist[1]);
					BoxElem.Z = FMath::Abs(Planes[Reorder[2]].PlaneDist[0] - Planes[Reorder[2]].PlaneDist[1]);
			
					AggGeom.BoxElems.Add(BoxElem);
					
					return true;
				}
			}
		}

		// We couldn't produce the box we desired, so fallback to a simple AABB if we must produce anything
		if (bForcePrimitiveGeneration && Box.IsValid)
		{
			FKBoxElem BoxElem;			
			BoxElem.Center = Box.GetCenter();

			FVector Extents = Box.GetExtent();
			BoxElem.X = 2.0 * Extents.X;
			BoxElem.Y = 2.0 * Extents.Y;
			BoxElem.Z = 2.0 * Extents.Z;

			AggGeom.BoxElems.Add(BoxElem);
			return true;
		}

		return false;
	}

	bool AddSphereGeomFromVertices(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		const FMeshDescription& MeshDescription,
		FKAggregateGeom& AggGeom,
		bool bForcePrimitiveGeneration
	)
	{
		FStaticMeshConstAttributes Attributes(MeshDescription);
		TVertexAttributesRef<const FVector3f> VertexPositions = Attributes.GetVertexPositions();

		if (VertexPositions.GetNumElements() == 0)
		{
			return false;
		}

		FBox Box(ForceInit);

		for (const FVector3f& VertexPosition : VertexPositions.GetRawArray())
		{
			Box += FVector(VertexPosition);
		}

		FVector Center;
		FVector Extents;
		Box.GetCenterAndExtents(Center, Extents);
		float Longest = 2.0f * Extents.GetMax();
		float Radius = 0.5f * Longest;
		
		// Validation
		if (!bForcePrimitiveGeneration)
		{
			float Shortest = 2.0f * Extents.GetMin();

			// check that the AABB is roughly a square (5% tolerance)
			if ((Longest - Shortest) / Longest > 0.05f)
			{
				return false;
			}

			// Test that all vertices are a similar radius (5%) from the sphere centre.
			float MaxR = 0;
			float MinR = BIG_NUMBER;

			for (const FVector3f& VertexPosition : VertexPositions.GetRawArray())
			{
				FVector CToV = FVector(VertexPosition) - Center;
				float RSqr = CToV.SizeSquared();

				MaxR = FMath::Max(RSqr, MaxR);

				// Sometimes vertex at centre, so reject it.
				if (RSqr > KINDA_SMALL_NUMBER)
				{
					MinR = FMath::Min(RSqr, MinR);
				}
			}

			MaxR = FMath::Sqrt(MaxR);
			MinR = FMath::Sqrt(MinR);

			if ((MaxR - MinR) / Radius > 0.05f)
			{
				return false;
			}
		}

		// Allocate sphere in array
		FKSphereElem SphereElem;
		SphereElem.Center = Center;
		SphereElem.Radius = Radius;
		AggGeom.SphereElems.Add(SphereElem);

		return true;
	}

	bool AddCapsuleGeomFromVertices(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		const FMeshDescription& MeshDescription,
		FKAggregateGeom& AggGeom
	)
	{
		FStaticMeshConstAttributes Attributes(MeshDescription);
		TVertexAttributesRef<const FVector3f> VertexPositions = Attributes.GetVertexPositions();

		if (VertexPositions.GetNumElements() == 0)
		{
			return false;
		}

		FVector AxisStart = FVector::ZeroVector;
		FVector AxisEnd = FVector::ZeroVector;
		float MaxDistSqr = 0.f;

		for (int32 IndexA = 0; IndexA < VertexPositions.GetNumElements() - 1; IndexA++)
		{
			for (int32 IndexB = IndexA + 1; IndexB < VertexPositions.GetNumElements(); IndexB++)
			{
				FVector TransformedA = FVector(VertexPositions[IndexA]);
				FVector TransformedB = FVector(VertexPositions[IndexB]);

				float DistSqr = (TransformedA - TransformedB).SizeSquared();
				if (DistSqr > MaxDistSqr)
				{
					AxisStart = TransformedA;
					AxisEnd = TransformedB;
					MaxDistSqr = DistSqr;
				}
			}
		}

		// If we got a valid axis, find vertex furthest from it
		if (MaxDistSqr > SMALL_NUMBER)
		{
			float MaxRadius = 0.0f;

			const FVector LineOrigin = AxisStart;
			const FVector LineDir = (AxisEnd - AxisStart).GetSafeNormal();

			for (int32 IndexA = 0; IndexA < VertexPositions.GetNumElements(); IndexA++)
			{
				FVector TransformedA = FVector(VertexPositions[IndexA]);

				float DistToAxis = FMath::PointDistToLine(TransformedA, LineDir, LineOrigin);
				if (DistToAxis > MaxRadius)
				{
					MaxRadius = DistToAxis;
				}
			}

			if (MaxRadius > SMALL_NUMBER)
			{
				// Allocate capsule in array
				FKSphylElem SphylElem;
				SphylElem.Center = 0.5f * (AxisStart + AxisEnd);
				SphylElem.Rotation = FQuat::FindBetweenVectors(FVector::ZAxisVector, LineDir).Rotator();	// Get quat that takes you from z axis to
																											// desired axis
				SphylElem.Radius = MaxRadius;
				SphylElem.Length = FMath::Max(FMath::Sqrt(MaxDistSqr) - (2.0f * MaxRadius), 0.0f);	  // subtract two radii from total length to get
																									  // segment length (ensure > 0)
				AggGeom.SphylElems.Add(SphylElem);
				return true;
			}
		}

		return false;
	}

	bool ImportBoxCollision(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		const TMap<FInterchangeMeshPayLoadKey, FMeshPayload>& BoxCollisionPayloads,
		UStaticMesh* StaticMesh,
		bool bForcePrimitiveGeneration
	)
	{
		using namespace UE::Interchange;

		bool bResult = false;

		FKAggregateGeom& AggGeo = StaticMesh->GetBodySetup()->AggGeom;

		for (const TPair<FInterchangeMeshPayLoadKey, FMeshPayload>& BoxCollisionPayload : BoxCollisionPayloads)
		{
			const TOptional<FMeshPayloadData>& PayloadData = BoxCollisionPayload.Value.PayloadData;
			if (!PayloadData.IsSet())
			{
				continue;
			}

			if (AddBoxGeomFromTris(PayloadData->MeshDescription, AggGeo, bForcePrimitiveGeneration))
			{
				bResult = true;
				FKBoxElem& NewElem = AggGeo.BoxElems.Last();

				// Now test the last element in the AggGeo list and remove it if its a duplicate
				// @TODO: determine why we have to do this. Was it to prevent duplicate boxes accumulating when reimporting?
				for (int32 ElementIndex = 0; ElementIndex < AggGeo.BoxElems.Num() - 1; ++ElementIndex)
				{
					FKBoxElem& CurrentElem = AggGeo.BoxElems[ElementIndex];

					if (CurrentElem == NewElem)
					{
						// The new element is a duplicate, remove it
						AggGeo.BoxElems.RemoveAt(AggGeo.BoxElems.Num() - 1);
						break;
					}
				}
			}
		}

		return bResult;
	}

	bool ImportCapsuleCollision(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		const TMap<FInterchangeMeshPayLoadKey, FMeshPayload>& CapsuleCollisionPayloads,
		UStaticMesh* StaticMesh
	)
	{
		using namespace UE::Interchange;

		bool bResult = false;

		FKAggregateGeom& AggGeo = StaticMesh->GetBodySetup()->AggGeom;

		for (const TPair<FInterchangeMeshPayLoadKey, FMeshPayload>& CapsuleCollisionPayload : CapsuleCollisionPayloads)
		{
			const TOptional<FMeshPayloadData>& PayloadData = CapsuleCollisionPayload.Value.PayloadData;

			if (!PayloadData.IsSet())
			{
				continue;
			}

			if (AddCapsuleGeomFromVertices(Arguments, PayloadData->MeshDescription, AggGeo))
			{
				bResult = true;

				FKSphylElem& NewElem = AggGeo.SphylElems.Last();

				// Now test the late element in the AggGeo list and remove it if its a duplicate
				// @TODO: determine why we have to do this. Was it to prevent duplicate boxes accumulating when reimporting?
				for (int32 ElementIndex = 0; ElementIndex < AggGeo.SphylElems.Num() - 1; ++ElementIndex)
				{
					FKSphylElem& CurrentElem = AggGeo.SphylElems[ElementIndex];
					if (CurrentElem == NewElem)
					{
						// The new element is a duplicate, remove it
						AggGeo.SphylElems.RemoveAt(AggGeo.SphylElems.Num() - 1);
						break;
					}
				}
			}
		}

		return bResult;
	}

	bool ImportSphereCollision(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		const TMap<FInterchangeMeshPayLoadKey, FMeshPayload>& SphereCollisionPayloads,
		UStaticMesh* StaticMesh,
		bool bForcePrimitiveGeneration
	)
	{
		using namespace UE::Interchange;

		bool bResult = false;

		FKAggregateGeom& AggGeo = StaticMesh->GetBodySetup()->AggGeom;

		for (const TPair<FInterchangeMeshPayLoadKey, FMeshPayload>& SphereCollisionPayload : SphereCollisionPayloads)
		{
			const TOptional<FMeshPayloadData>& PayloadData = SphereCollisionPayload.Value.PayloadData;

			if (!PayloadData.IsSet())
			{
				continue;
			}

			if (AddSphereGeomFromVertices(Arguments, PayloadData->MeshDescription, AggGeo, bForcePrimitiveGeneration))
			{
				bResult = true;

				FKSphereElem& NewElem = AggGeo.SphereElems.Last();

				// Now test the last element in the AggGeo list and remove it if its a duplicate
				// @TODO: determine why we have to do this. Was it to prevent duplicate boxes accumulating when reimporting?
				for (int32 ElementIndex = 0; ElementIndex < AggGeo.SphereElems.Num() - 1; ++ElementIndex)
				{
					FKSphereElem& CurrentElem = AggGeo.SphereElems[ElementIndex];

					if (CurrentElem == NewElem)
					{
						// The new element is a duplicate, remove it
						AggGeo.SphereElems.RemoveAt(AggGeo.SphereElems.Num() - 1);
						break;
					}
				}
			}
		}

		return bResult;
	}

	bool ImportConvexCollision(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		const TMap<FInterchangeMeshPayLoadKey, FMeshPayload>& ConvexCollisionPayloads,
		UStaticMesh* StaticMesh,
		const UInterchangeStaticMeshLodDataNode* LodDataNode
	)
	{
		using namespace UE::Interchange;

		bool bResult = false;

		bool bOneConvexHullPerUCX;
		if (!LodDataNode->GetOneConvexHullPerUCX(bOneConvexHullPerUCX) || !bOneConvexHullPerUCX)
		{
			for (const TPair<FInterchangeMeshPayLoadKey, FMeshPayload>& ConvexCollisionPayload : ConvexCollisionPayloads)
			{
				const TOptional<FMeshPayloadData>& PayloadData = ConvexCollisionPayload.Value.PayloadData;

				if (!PayloadData.IsSet())
				{
					continue;
				}

				if (DecomposeConvexMesh(Arguments, PayloadData->MeshDescription, StaticMesh->GetBodySetup()))
				{
					bResult = true;
				}
			}
		}
		else
		{
			FKAggregateGeom& AggGeo = StaticMesh->GetBodySetup()->AggGeom;

			for (const TPair<FInterchangeMeshPayLoadKey, FMeshPayload>& ConvexCollisionPayload : ConvexCollisionPayloads)
			{
				TOptional<FMeshPayloadData> PayloadData = ConvexCollisionPayload.Value.PayloadData;

				if (!PayloadData.IsSet())
				{
					continue;
				}

				if (AddConvexGeomFromVertices(Arguments, PayloadData->MeshDescription, AggGeo))
				{
					bResult = true;

					FKConvexElem& NewElem = AggGeo.ConvexElems.Last();

					// Now test the late element in the AggGeo list and remove it if its a duplicate
					// @TODO: determine why the importer used to do this. Was it something to do with reimport not adding extra collision or
					// something?
					for (int32 ElementIndex = 0; ElementIndex < AggGeo.ConvexElems.Num() - 1; ++ElementIndex)
					{
						FKConvexElem& CurrentElem = AggGeo.ConvexElems[ElementIndex];

						if (CurrentElem.VertexData.Num() == NewElem.VertexData.Num())
						{
							bool bFoundDifference = false;
							for (int32 VertexIndex = 0; VertexIndex < NewElem.VertexData.Num(); ++VertexIndex)
							{
								if (CurrentElem.VertexData[VertexIndex] != NewElem.VertexData[VertexIndex])
								{
									bFoundDifference = true;
									break;
								}
							}

							if (!bFoundDifference)
							{
								// The new collision geo is a duplicate, delete it
								AggGeo.ConvexElems.RemoveAt(AggGeo.ConvexElems.Num() - 1);
								break;
							}
						}
					}
				}
			}
		}

		return bResult;
	}

	bool ImportSockets(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		UStaticMesh* StaticMesh,
		const UInterchangeStaticMeshFactoryNode* FactoryNode
	)
	{
		bool bImportSockets = false;
		FactoryNode->GetCustomImportSockets(bImportSockets);
		if (!bImportSockets)
		{
			// Skip Import sockets
			return true;
		}

		TArray<FString> SocketUids;
		FactoryNode->GetSocketUids(SocketUids);

		TSet<FName> ImportedSocketNames;

		FTransform GlobalOffsetTransform = FTransform::Identity;

		bool bBakeMeshes = false;
		bool bBakePivotMeshes = false;
		if (UInterchangeCommonPipelineDataFactoryNode* CommonPipelineDataFactoryNode = UInterchangeCommonPipelineDataFactoryNode::GetUniqueInstance(
				Arguments.NodeContainer
			))
		{
			CommonPipelineDataFactoryNode->GetCustomGlobalOffsetTransform(GlobalOffsetTransform);
			CommonPipelineDataFactoryNode->GetBakeMeshes(bBakeMeshes);
			if (!bBakeMeshes)
			{
				CommonPipelineDataFactoryNode->GetBakePivotMeshes(bBakePivotMeshes);
			}
		}

		const UInterchangeSceneNode* MeshSceneNode = nullptr;
		FTransform MeshGlobalTransform = FTransform::Identity;

		for (const FString& SocketUid : SocketUids)
		{
			if (const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(Arguments.NodeContainer->GetNode(SocketUid)))
			{
				FString NodeDisplayName = SceneNode->GetDisplayLabel();
				if (NodeDisplayName.StartsWith(UInterchangeMeshFactoryNode::GetMeshSocketPrefix()))
				{
					NodeDisplayName.RightChopInline(UInterchangeMeshFactoryNode::GetMeshSocketPrefix().Len(), EAllowShrinking::No);
				}
				FName SocketName = FName(NodeDisplayName);
				ImportedSocketNames.Add(SocketName);

				FTransform Transform;
				if (bBakeMeshes)
				{
					SceneNode->GetCustomGlobalTransform(Arguments.NodeContainer, GlobalOffsetTransform, Transform);
				}
				else
				{
					// In case we are not baking, we need to find the relative transform of the socket from the mesh non-baked origin.
					if (!MeshSceneNode)
					{
						// For this we need the current global transform for the mesh holding those sockets.
						// Since all sockets share the same mesh, we can do a search once and use the computed MeshGlobalTransform for every other sockets.
						TArray<FString> MeshNodeUids;
						FactoryNode->GetTargetNodeUids(MeshNodeUids);

						if (ensure(MeshNodeUids.Num() > 0))
						{
							const FString MeshNodeUid = MeshNodeUids[0];

							MeshSceneNode = Cast<UInterchangeSceneNode>(Arguments.NodeContainer->GetNode(SceneNode->GetParentUid()));
							while (MeshSceneNode)
							{
								FString MeshUid;
								if (MeshSceneNode->GetCustomAssetInstanceUid(MeshUid) && MeshUid == MeshNodeUid)
								{
									break;
								}

								MeshSceneNode = Cast<UInterchangeSceneNode>(Arguments.NodeContainer->GetNode(MeshSceneNode->GetParentUid()));
							}

							if (ensure(MeshSceneNode))
							{
								MeshSceneNode->GetCustomGlobalTransform(Arguments.NodeContainer, FTransform::Identity, MeshGlobalTransform);
							}
						}
					}

					if (MeshSceneNode)
					{
						FTransform SocketGlobalTransform;
						SceneNode->GetCustomGlobalTransform(Arguments.NodeContainer, FTransform::Identity, SocketGlobalTransform);
						Transform = SocketGlobalTransform.GetRelativeTransform(MeshGlobalTransform);
					}
				}

				UE::Interchange::Private::MeshHelper::AddSceneNodeGeometricAndPivotToGlobalTransform(
					Transform,
					SceneNode,
					bBakeMeshes,
					bBakePivotMeshes
				);

				// Apply axis transformation inverse to get correct Socket Transform:
				const UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::GetUniqueInstance(Arguments.NodeContainer);
				FTransform AxisConversionInverseTransform;
				if (SourceNode->GetCustomAxisConversionInverseTransform(AxisConversionInverseTransform))
				{
					Transform = AxisConversionInverseTransform * Transform;
				}

				UStaticMeshSocket* Socket = StaticMesh->FindSocket(SocketName);
				if (!Socket)
				{
					// If the socket didn't exist create a new one now
					Socket = NewObject<UStaticMeshSocket>(StaticMesh);
#if WITH_EDITORONLY_DATA
					Socket->bSocketCreatedAtImport = true;
#endif
					Socket->SocketName = SocketName;
					StaticMesh->AddSocket(Socket);
				}

				Socket->RelativeLocation = Transform.GetLocation();
				Socket->RelativeRotation = Transform.GetRotation().Rotator();
				Socket->RelativeScale = Transform.GetScale3D();
			}
		}

		// Delete any sockets which were previously imported but which no longer exist in the imported scene
		for (TArray<TObjectPtr<UStaticMeshSocket>>::TIterator It = StaticMesh->Sockets.CreateIterator(); It; ++It)
		{
			UStaticMeshSocket* Socket = *It;
			if (
#if WITH_EDITORONLY_DATA
			Socket->bSocketCreatedAtImport &&
#endif
			!ImportedSocketNames.Contains(Socket->SocketName))
			{
				It.RemoveCurrent();
			}
		}

		return true;
	}

	void RemapPolygonGroups(const FMeshDescription& SourceMesh, FMeshDescription& TargetMesh, PolygonGroupMap& RemapPolygonGroup)
	{
		FStaticMeshConstAttributes SourceAttributes(SourceMesh);
		TPolygonGroupAttributesConstRef<FName> SourceImportedMaterialSlotNames = SourceAttributes.GetPolygonGroupMaterialSlotNames();

		FStaticMeshAttributes TargetAttributes(TargetMesh);
		TPolygonGroupAttributesRef<FName> TargetImportedMaterialSlotNames = TargetAttributes.GetPolygonGroupMaterialSlotNames();

		for (FPolygonGroupID SourcePolygonGroupID : SourceMesh.PolygonGroups().GetElementIDs())
		{
			FPolygonGroupID TargetMatchingID = INDEX_NONE;
			for (FPolygonGroupID TargetPolygonGroupID : TargetMesh.PolygonGroups().GetElementIDs())
			{
				if (SourceImportedMaterialSlotNames[SourcePolygonGroupID] == TargetImportedMaterialSlotNames[TargetPolygonGroupID])
				{
					TargetMatchingID = TargetPolygonGroupID;
					break;
				}
			}
			if (TargetMatchingID == INDEX_NONE)
			{
				TargetMatchingID = TargetMesh.CreatePolygonGroup();
				TargetImportedMaterialSlotNames[TargetMatchingID] = SourceImportedMaterialSlotNames[SourcePolygonGroupID];
			}
			else
			{
				//Since we want to keep the sections separate we need to create a new polygongroup
				TargetMatchingID = TargetMesh.CreatePolygonGroup();
				FString NewSlotName = SourceImportedMaterialSlotNames[SourcePolygonGroupID].ToString() + TEXT("_Section") + FString::FromInt(TargetMatchingID.GetValue());
				TargetImportedMaterialSlotNames[TargetMatchingID] = FName(NewSlotName);
			}
			RemapPolygonGroup.Add(SourcePolygonGroupID, TargetMatchingID);
		}
	}

	void AddSceneNodeGeometricAndPivotToGlobalTransform(FTransform& GlobalTransform, const UInterchangeSceneNode* SceneNode, const bool bBakeMeshes, const bool bBakePivotMeshes)
	{
		FTransform SceneNodeGeometricTransform;
		SceneNode->GetCustomGeometricTransform(SceneNodeGeometricTransform);

		if (!bBakeMeshes)
		{
			if (bBakePivotMeshes)
			{
				FTransform SceneNodePivotNodeTransform;
				if (SceneNode->GetCustomPivotNodeTransform(SceneNodePivotNodeTransform))
				{
					SceneNodeGeometricTransform = SceneNodePivotNodeTransform * SceneNodeGeometricTransform;
				}
			}
			else
			{
				SceneNodeGeometricTransform.SetIdentity();
			}
		}
		GlobalTransform = SceneNodeGeometricTransform * GlobalTransform;
	}

	//Resolves the optional RootJointParentGlobalInverseMatrix, or in case it's not set, 
	// it traverses the parent hierarchy of the SceneNode until it finds the RootJoint's Parent (RootJoint's Parent, is the first SceneNode that is not a JointNode) and acquire the GlobalInverseMatrix from found node.
	FMatrix ResolveRootJointParentGlobalInverseMatrix(UInterchangeBaseNodeContainer& BaseNodeContainer, const UInterchangeSceneNode* SceneNode, const FTransform& GlobalOffsetTransform, const TOptional<TPair<FString, FMatrix>>& RootJointParentUidAndGlobalInverseMatrixOptional)
	{
		FMatrix RootJointParentGlobalInverseMatrix = FMatrix::Identity;
		if (RootJointParentUidAndGlobalInverseMatrixOptional.IsSet())
		{
			RootJointParentGlobalInverseMatrix = RootJointParentUidAndGlobalInverseMatrixOptional.GetValue().Value;
		}
		else
		{
			const UInterchangeSceneNode* Current = SceneNode;
			const UInterchangeSceneNode* Candidate = Cast<UInterchangeSceneNode>(BaseNodeContainer.GetNode(Current->GetParentUid()));

			//In this case we will try to identify the root joint node via iterating through Parents until we hit the top or a non-UInterchangeJointNode.
			static const uint32 MaxDepthCheckLimit = UINT16_MAX;
			uint32 MaxDepthChecker = 0;
			while (Candidate && Candidate->IsA<UInterchangeJointNode>())
			{
				Current = Candidate;
				Candidate = Cast<UInterchangeSceneNode>(BaseNodeContainer.GetNode(Current->GetParentUid()));
				MaxDepthChecker++;
				if (MaxDepthChecker > MaxDepthCheckLimit)
				{
					Candidate = nullptr;
					break;
				}
			}

			if (Candidate)
			{
				FTransform RootJointParentGlobalTransform = FTransform::Identity;
				Candidate->GetCustomGlobalTransform(&BaseNodeContainer, GlobalOffsetTransform, RootJointParentGlobalTransform);

				RootJointParentGlobalInverseMatrix = RootJointParentGlobalTransform.ToMatrixWithScale().Inverse();
			}
		}

		return RootJointParentGlobalInverseMatrix;
	}

	/**
	* When we are trying to acquire the wanted/needed transform for a MeshNode we have to combine/calculate with the following transforms:
	*		- GlobalTransform: only if bBakeMeshes is true. (Global transform of the instantiating SceneNode)
	*		- GeometricTransform
	*		- PivotTransform: only if !bBakeMeshes && bBakePivotMeshes.
	* To do this we use the following function: UE::Interchange::Private::MeshHelper::AddSceneNodeGeometricAndPivotToGlobalTransform
	* The complete setup should be as follows:
	*		Transform:= MeshTransform * PivotNodeTransform * GeometricTransform * GlobalSceneNodeTransform
	*			MeshTransform is specific to LODMeshes (UInterchangeMeshLODContainerNode->FInterchangeLODMeshData.Transform)
	*				and it is not handled directly within this function.
	* In case the input SceneNode is a JointNode we are going to calculate the SkeletonTransform of the given Node (for bBakeMeshes=false)
	*/
	FTransform CalculateCompleteSceneNodeTransform(
		UInterchangeBaseNodeContainer* BaseNodeContainer, const UInterchangeSceneNode* SceneNode, 
		const bool bBakeMeshes, const bool bBakePivotMeshes, 
		const FTransform& GlobalOffsetTransform,
		const bool bUseTimeZeroAsBindPose,
		const TOptional<TPair<FString, FMatrix>>& RootJointParentUidAndGlobalInverseMatrixOptional,
		const bool bProcessingSkeletalMesh,
		const bool bAlignSkeletalMeshInScene)
	{
		if (!BaseNodeContainer || !SceneNode)
		{
			return FTransform::Identity;
		}

		FTransform OutGlobalTransform = FTransform::Identity;
		
		if (const UInterchangeJointNode* JointNode = Cast<UInterchangeJointNode>(SceneNode))
		{
			FTransform JointNodeSkeletonTransform = FTransform::Identity;

			if (!bUseTimeZeroAsBindPose || !JointNode->GetTimeZeroGlobalTransform(BaseNodeContainer, GlobalOffsetTransform, JointNodeSkeletonTransform))
			{
				//Even if BindPose transform is not present it will fallback to Local Transform for the GlobalTransform generation
				//in case that fails, there is no other fallback mechanism to be applied.
				//Hence the reason why we can ignore the success check returned from GetBindPoseGlobalTransform.
				JointNode->GetBindPoseGlobalTransform(BaseNodeContainer, GlobalOffsetTransform, JointNodeSkeletonTransform);
			}

			if (bBakeMeshes)
			{
				OutGlobalTransform = JointNodeSkeletonTransform;
			}
			else
			{
				FMatrix RootJointParentGlobalInverseMatrix = ResolveRootJointParentGlobalInverseMatrix(*BaseNodeContainer, SceneNode, GlobalOffsetTransform, RootJointParentUidAndGlobalInverseMatrixOptional);

				//We calculate the 'Skeleton Transform' here:
				// In other words the Global/Relative transform to the RootJointNode's Parent.

				FMatrix JointNodeGlobalMatrix = JointNodeSkeletonTransform.ToMatrixWithScale();
				FTransform SkeletonTransform(JointNodeGlobalMatrix * RootJointParentGlobalInverseMatrix);

				OutGlobalTransform = SkeletonTransform;
			}
		}
		else if (bBakeMeshes)
		{
			FTransform SceneNodeGlobalTransform;
			if (SceneNode->GetCustomGlobalTransform(BaseNodeContainer, GlobalOffsetTransform, SceneNodeGlobalTransform))
			{
				OutGlobalTransform = SceneNodeGlobalTransform;
			}

			if (!bAlignSkeletalMeshInScene && 
				bProcessingSkeletalMesh && 
				RootJointParentUidAndGlobalInverseMatrixOptional.IsSet())
			{
				//**bAlignSkeletalMeshInScene==FALSE**
				//If the SK and SKM was aligned prior to the Scene Composition,
				//in that case we have to make sure we are baking the final SKM Instantiating Node's transform.
				// Which is the SkeletonRoot's Parent Global transform.
				//This is already part of the Mesh's SceneNode's GlobalTransform, IF the SceneNode is a child of the RootJoint's ParentNode,
				if (!BaseNodeContainer->GetIsAncestor(SceneNode->GetUniqueID(), RootJointParentUidAndGlobalInverseMatrixOptional.GetValue().Key))
				{
					//But if the SceneNode is not part of the RootJointParent children tree (aka GetIsAncestor returns false), 
					// then we need to apply the RootJointParentGlobalMatrix:
					//So for example, this scenario:
					//		SceneRoot
					//			RootJointParent
					//				RootJoint (aka Skeleton)
					//					BoneXYZ
					//					...
					//			TransformNodeABC
					//				Mesh (Mesh referenced)
					//	As the bAlignSkeletalMeshInScene==false, which means the alignment is happening outside of the Scene Composition (prior to it).
					//
					//In this case the RootJointParent's Global Matrix needs to be additionally baked.
					// Test file for this: glTF 2.0 example file RiggedFigure.
					FMatrix RootJointParentGlobalMatrix = ResolveRootJointParentGlobalInverseMatrix(*BaseNodeContainer, SceneNode, GlobalOffsetTransform, RootJointParentUidAndGlobalInverseMatrixOptional).Inverse();
					FTransform SkeletonTransform(OutGlobalTransform.ToMatrixWithScale() * RootJointParentGlobalMatrix);

					OutGlobalTransform = SkeletonTransform;
				}
			}
		}
		else if (bAlignSkeletalMeshInScene && bProcessingSkeletalMesh && RootJointParentUidAndGlobalInverseMatrixOptional.IsSet())
		{
			//**bAlignSkeletalMeshInScene==TRUE**
			//If it is expected that SKM and SK is aligned in the Scene (via the Scene Composition).
			//Then we have to bake the complete transform for the mesh (SceneNode), regardless of the bBakeMesh setting.
			// However we do have to remove the final SKM Instantiating Node's global transform 
			// (aka the RootJointParentGlobalTransfrom needs to be inverse applied, so that we keep SKM and SK allignment.)
			// So for example, for this scenario:
			//		SceneRoot
			//			RootJointParent
			//				RootJoint (aka Skeleton)
			//					BoneXYZ
			//					...
			//				TransformNodeABC
			//					Mesh (Mesh referenced)
			//	We have to remove the RootJointParent Global Transform in case bBakeMesh==false (which is this else if branch) 
			// And we also have to remove the RootJointParent Global Transform (aka apply the RootJointParentGlobalTransform.Inverse())
			//		for this scenario:
			//			SceneRoot
			//				RootJointParent
			//					RootJoint (aka Skeleton)
			//						BoneXYZ
			//						...
			//				TransformNodeABC
			//					Mesh (Mesh referenced)


			FTransform SceneNodeGlobalTransform;
			if (SceneNode->GetCustomGlobalTransform(BaseNodeContainer, GlobalOffsetTransform, SceneNodeGlobalTransform))
			{
				OutGlobalTransform = SceneNodeGlobalTransform;
			}

			FMatrix RootJointParentGlobalInverseMatrix = ResolveRootJointParentGlobalInverseMatrix(*BaseNodeContainer, SceneNode, GlobalOffsetTransform, RootJointParentUidAndGlobalInverseMatrixOptional);
			FTransform SkeletonTransform(OutGlobalTransform.ToMatrixWithScale() * RootJointParentGlobalInverseMatrix);

			OutGlobalTransform = SkeletonTransform;
		}

		UE::Interchange::Private::MeshHelper::AddSceneNodeGeometricAndPivotToGlobalTransform(OutGlobalTransform, SceneNode, bBakeMeshes, bBakePivotMeshes);
		return OutGlobalTransform;
	}

	template<typename MaterialType>
	class INTERCHANGEIMPORT_API FMeshMaterialViewer
	{
	public:
		FMeshMaterialViewer(TArray<MaterialType>& InMaterials, int32 InMaterialIndex)
			: Materials(InMaterials)
			, MaterialIndex(InMaterialIndex)
		{ }

		FName GetMaterialSlotName() const
		{
			if (Materials.IsValidIndex(MaterialIndex))
			{
				return Materials[MaterialIndex].MaterialSlotName;
			}
			return NAME_None;
		}

		FName GetImportedMaterialSlotName() const
		{
#if WITH_EDITOR
			if (Materials.IsValidIndex(MaterialIndex))
			{
				return Materials[MaterialIndex].ImportedMaterialSlotName;
			}
			return NAME_None;
#else
			return GetMaterialSlotName();
#endif
		}

		void SetMaterialSlotName(const FName MaterialSlotName)
		{
			if (Materials.IsValidIndex(MaterialIndex))
			{
				Materials[MaterialIndex].MaterialSlotName = MaterialSlotName;
			}
		}

		void SetImportedMaterialSlotName(const FName ImportedMaterialSlotName)
		{
#if WITH_EDITOR
			if (Materials.IsValidIndex(MaterialIndex))
			{
				Materials[MaterialIndex].ImportedMaterialSlotName = ImportedMaterialSlotName;
			}
#endif
		}

		UMaterialInterface* GetMaterialInterface() const
		{
			if (Materials.IsValidIndex(MaterialIndex))
			{
				return Materials[MaterialIndex].MaterialInterface;
			}
			return nullptr;
		}

		void SetMaterialInterface(UMaterialInterface* MaterialInterface)
		{
			if (Materials.IsValidIndex(MaterialIndex))
			{
				Materials[MaterialIndex].MaterialInterface = MaterialInterface;
			}
		}
	private:
		TArray<MaterialType>& Materials;
		int32 MaterialIndex = INDEX_NONE;
	};

	template<>
	class FMeshMaterialViewer<TObjectPtr<UMaterialInterface>>
	{
	public:
		FMeshMaterialViewer(TArray<TObjectPtr<UMaterialInterface>>& InMaterials, TArray<FName>& InMaterialSlotNames, int32 InMaterialIndex)
			: Materials(InMaterials)
			, MaterialSlotNames(InMaterialSlotNames)
			, MaterialIndex(InMaterialIndex)
		{ }

		FName GetMaterialSlotName() const
		{
			if (Materials.IsValidIndex(MaterialIndex))
			{
				return MaterialSlotNames[MaterialIndex];
			}
			return NAME_None;
		}

		FName GetImportedMaterialSlotName() const
		{
			return GetMaterialSlotName();
		}

		void SetMaterialSlotName(const FName MaterialSlotName)
		{
			if (Materials.IsValidIndex(MaterialIndex))
			{
				MaterialSlotNames[MaterialIndex] = MaterialSlotName;
			}
		}

		void SetImportedMaterialSlotName(const FName ImportedMaterialSlotName)
		{
		}

		UMaterialInterface* GetMaterialInterface() const
		{
			if (Materials.IsValidIndex(MaterialIndex))
			{
				return Materials[MaterialIndex];
			}
			return nullptr;
		}

		void SetMaterialInterface(UMaterialInterface* MaterialInterface)
		{
			if (Materials.IsValidIndex(MaterialIndex))
			{
				Materials[MaterialIndex] = MaterialInterface;
			}
		}
	private:
		TArray<TObjectPtr<UMaterialInterface>>& Materials;
		TArray<FName>& MaterialSlotNames;
		int32 MaterialIndex = INDEX_NONE;
	};

	template<typename MaterialType>
	class INTERCHANGEIMPORT_API FMeshMaterialArrayViewer
	{
	public:
		FMeshMaterialArrayViewer(TArray<MaterialType>& InMaterials, TFunction<void(MaterialType& Material)>& InEmplaceMaterialFunctor)
			: Materials(InMaterials)
			, EmplaceMaterialFunctor(InEmplaceMaterialFunctor)
		{
			RebuildViewer();
		}

		void RebuildViewer()
		{
			MeshMaterialArrayViewer.Reset(Materials.Num());
			for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
			{
				MeshMaterialArrayViewer.Emplace(Materials, MaterialIndex);
			}
		}

		int32 Num() const
		{
			return MeshMaterialArrayViewer.Num();
		}

		FMeshMaterialViewer<MaterialType>& operator[](int32 MaterialIndex)
		{
			check(MeshMaterialArrayViewer.IsValidIndex(MaterialIndex));
			return MeshMaterialArrayViewer[MaterialIndex];
		}

		FMeshMaterialViewer<MaterialType>* FindByPredicate(TFunction<bool(const FMeshMaterialViewer<MaterialType>& MaterialViewer)> Predicate)
		{
			return MeshMaterialArrayViewer.FindByPredicate(Predicate);
		}

		void Emplace(UMaterialInterface* NewMaterial, FName MaterialSlotName, FName ImportedMaterialSlotName)
		{
			MaterialType& Material = Materials.AddDefaulted_GetRef();
			Material.MaterialInterface = NewMaterial;
			Material.MaterialSlotName = MaterialSlotName;
#if WITH_EDITOR
			Material.ImportedMaterialSlotName = ImportedMaterialSlotName;
#endif

			EmplaceMaterialFunctor(Material);

			RebuildViewer();
		}

		void Reserve(int32 Count)
		{
			Materials.Reserve(Count);

			RebuildViewer();
		}

	private:
		TArray<MaterialType>& Materials;
		TFunction<void(MaterialType& Material)>& EmplaceMaterialFunctor;
		TArray<FMeshMaterialViewer<MaterialType>> MeshMaterialArrayViewer;
	};

	template<>
	class FMeshMaterialArrayViewer<TObjectPtr<UMaterialInterface>>
	{
	public:
		FMeshMaterialArrayViewer(TArray<TObjectPtr<UMaterialInterface>>& InMaterials, TArray<FName>& InMaterialSlotNames, TFunction<void(TObjectPtr<UMaterialInterface>& Material)>& InEmplaceMaterialFunctor)
			: Materials(InMaterials)
			, MaterialSlotNames(InMaterialSlotNames)
			, EmplaceMaterialFunctor(InEmplaceMaterialFunctor)
		{
			RebuildViewer();
		}

		void RebuildViewer()
		{
			MeshMaterialArrayViewer.Reset(Materials.Num());
			for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
			{
				MeshMaterialArrayViewer.Emplace(Materials, MaterialSlotNames, MaterialIndex);
			}
		}

		int32 Num() const
		{
			return MeshMaterialArrayViewer.Num();
		}

		FMeshMaterialViewer<TObjectPtr<UMaterialInterface>>& operator[](int32 MaterialIndex)
		{
			check(MeshMaterialArrayViewer.IsValidIndex(MaterialIndex));
			return MeshMaterialArrayViewer[MaterialIndex];
		}

		FMeshMaterialViewer<TObjectPtr<UMaterialInterface>>* FindByPredicate(TFunction<bool(const FMeshMaterialViewer<TObjectPtr<UMaterialInterface>>& MaterialViewer)> Predicate)
		{
			return MeshMaterialArrayViewer.FindByPredicate(Predicate);
		}

		void Emplace(UMaterialInterface* NewMaterial, FName InMaterialSlotName, FName ImportedMaterialSlotName)
		{
			TObjectPtr<UMaterialInterface>& Material = Materials.AddDefaulted_GetRef();
			Material = NewMaterial;

			FName& MaterialSlotName = MaterialSlotNames.AddDefaulted_GetRef();
			MaterialSlotName = InMaterialSlotName;

			EmplaceMaterialFunctor(Material);

			RebuildViewer();
		}

		void Reserve(int32 Count)
		{
			Materials.Reserve(Count);
			MaterialSlotNames.Reserve(Count);

			RebuildViewer();
		}

	private:
		TArray<TObjectPtr<UMaterialInterface>>& Materials;
		TArray<FName>& MaterialSlotNames;
		TFunction<void(TObjectPtr<UMaterialInterface>& Material)>& EmplaceMaterialFunctor;
		TArray<FMeshMaterialViewer<TObjectPtr<UMaterialInterface>>> MeshMaterialArrayViewer;
	};

	template<typename MaterialType>
	void InternalMeshFactorySetupAssetMaterialArray(FMeshMaterialArrayViewer<MaterialType>& ExistMaterialArrayViewer
		, TMap<FString, FString> ImportedSlotMaterialDependencies
		, const UInterchangeBaseNodeContainer* NodeContainer
		, const bool bIsReimport)
	{
		//Store the existing material index that match with the imported material
		TArray<int32> ImportedMaterialMatchExistingMaterialIndex;
		//Store the imported material index that match with the existing material
		TArray<int32> ExistingMaterialMatchImportedMaterialIndex;

		const int32 ImportedMaterialCount = ImportedSlotMaterialDependencies.Num();
		const int32 ExistingMaterialCount = ExistMaterialArrayViewer.Num();

		//Find which imported slot material match with existing slot material
		{
			ImportedMaterialMatchExistingMaterialIndex.SetNum(ImportedMaterialCount);
			for (int32 ImportedMaterialIndex = 0; ImportedMaterialIndex < ImportedMaterialCount; ++ImportedMaterialIndex)
			{
				ImportedMaterialMatchExistingMaterialIndex[ImportedMaterialIndex] = INDEX_NONE;
			}

			ExistingMaterialMatchImportedMaterialIndex.SetNum(ExistingMaterialCount);
			for (int32 ExistingMaterialIndex = 0; ExistingMaterialIndex < ExistingMaterialCount; ++ExistingMaterialIndex)
			{
				ExistingMaterialMatchImportedMaterialIndex[ExistingMaterialIndex] = INDEX_NONE;
			}

			int32 ImportedMaterialIndex = 0;
			for (TPair<FString, FString>& SlotMaterialDependency : ImportedSlotMaterialDependencies)
			{
				FName MaterialSlotName = *SlotMaterialDependency.Key;
				for (int32 ExistingMaterialIndex = 0; ExistingMaterialIndex < ExistingMaterialCount; ++ExistingMaterialIndex)
				{
					if (ExistingMaterialMatchImportedMaterialIndex[ExistingMaterialIndex] != INDEX_NONE)
					{
						continue;
					}

					const FMeshMaterialViewer<MaterialType>& Material = ExistMaterialArrayViewer[ExistingMaterialIndex];
					if (Material.GetMaterialSlotName() == MaterialSlotName)
					{
						ExistingMaterialMatchImportedMaterialIndex[ExistingMaterialIndex] = ImportedMaterialIndex;
						ImportedMaterialMatchExistingMaterialIndex[ImportedMaterialIndex] = ExistingMaterialIndex;
						break;
					}
				}
				ImportedMaterialIndex++;
			}
		}


		auto UpdateOrAddMaterial = [&ExistMaterialArrayViewer
			, bIsReimport
			, &ImportedMaterialMatchExistingMaterialIndex
			, &ExistingMaterialMatchImportedMaterialIndex
			, &ExistingMaterialCount]
			(const FName& MaterialSlotName, UMaterialInterface* MaterialInterface, const int32 ImportedMaterialIndex)
			{
				UMaterialInterface* NewMaterial = MaterialInterface ? MaterialInterface : UMaterial::GetDefaultMaterial(MD_Surface);

				FMeshMaterialViewer<MaterialType>* MeshMaterialViewer = ExistMaterialArrayViewer.FindByPredicate([&MaterialSlotName](const FMeshMaterialViewer<MaterialType>& Material) { return Material.GetMaterialSlotName() == MaterialSlotName; });
				if (MeshMaterialViewer)
				{
					//When we are not re-importing, we always force update the material, we should see this case when importing LODs is on since its an import.
					//When we do a re-import we update the material interface only if the current asset matching material is null and is not the default material.
					if (!bIsReimport || (MaterialInterface && (!MeshMaterialViewer->GetMaterialInterface() || MeshMaterialViewer->GetMaterialInterface() == UMaterial::GetDefaultMaterial(MD_Surface))))
					{
						MeshMaterialViewer->SetMaterialInterface(NewMaterial);
					}
				}
				else
				{
					//See if we can pick and unmatched existing material slot before creating one
					bool bCreateNewMaterialSlot = true;
					for (int32 ExistingMaterialIndex = 0; ExistingMaterialIndex < ExistingMaterialCount; ++ExistingMaterialIndex)
					{
						//Find the next available unmatched existing material slot and pick it up instead of creating a new material
						if (ExistingMaterialMatchImportedMaterialIndex[ExistingMaterialIndex] == INDEX_NONE)
						{
							bCreateNewMaterialSlot = false;
							FMeshMaterialViewer<MaterialType>& ExistingSkeletalMaterial = ExistMaterialArrayViewer[ExistingMaterialIndex];
							ExistingSkeletalMaterial.SetMaterialSlotName(MaterialSlotName);
							ExistingSkeletalMaterial.SetImportedMaterialSlotName(MaterialSlotName);
							ExistingSkeletalMaterial.SetMaterialInterface(NewMaterial);
							ExistingMaterialMatchImportedMaterialIndex[ExistingMaterialIndex] = ImportedMaterialIndex;
							ImportedMaterialMatchExistingMaterialIndex[ImportedMaterialIndex] = ExistingMaterialIndex;
							break;
						}
					}
					if (bCreateNewMaterialSlot)
					{
						ExistMaterialArrayViewer.Emplace(NewMaterial, MaterialSlotName, MaterialSlotName);
					}
				}
			};

		//Preallocate the extra memory if needed
		if (ImportedMaterialCount > ExistingMaterialCount)
		{
			ExistMaterialArrayViewer.Reserve(ImportedMaterialCount);
		}

		int32 ImportedMaterialIndex = 0;
		for (TPair<FString, FString>& SlotMaterialDependency : ImportedSlotMaterialDependencies)
		{
			UE::Interchange::FScopedLambda ScopedLambda([&ImportedMaterialIndex]()
				{
					++ImportedMaterialIndex;
				});
			FName MaterialSlotName = *SlotMaterialDependency.Key;

			const UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode = Cast<UInterchangeBaseMaterialFactoryNode>(NodeContainer->GetNode(SlotMaterialDependency.Value));
			if (!MaterialFactoryNode)
			{
				// Fallback: the dependency value may be a content path to an existing material rather than a node UID. 
				// Use ResolveObject instead of TryLoad as this can run on a background thread
				FSoftObjectPath ContentPath(SlotMaterialDependency.Value);
				if (ContentPath.IsValid())
				{
					if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(ContentPath.ResolveObject()))
					{
						UpdateOrAddMaterial(MaterialSlotName, MaterialInterface, ImportedMaterialIndex);
						continue;
					}
				}

				UpdateOrAddMaterial(MaterialSlotName, nullptr, ImportedMaterialIndex);
				continue;
			}

			FSoftObjectPath MaterialFactoryNodeReferenceObject;
			MaterialFactoryNode->GetCustomReferenceObject(MaterialFactoryNodeReferenceObject);
			if (!MaterialFactoryNodeReferenceObject.IsValid())
			{
				UpdateOrAddMaterial(MaterialSlotName, nullptr, ImportedMaterialIndex);
				continue;
			}

			UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(MaterialFactoryNodeReferenceObject.ResolveObject());
			UpdateOrAddMaterial(MaterialSlotName, MaterialInterface ? MaterialInterface : nullptr, ImportedMaterialIndex);
		}
	}

	void SkeletalMeshFactorySetupAssetMaterialArray(TArray<FSkeletalMaterial>& ExistMaterials
		, TMap<FString, FString> ImportedSlotMaterialDependencies
		, const UInterchangeBaseNodeContainer* NodeContainer
		, const bool bIsReimport)
	{
		TFunction<void(FSkeletalMaterial&)> EmplaceMaterialFunctor = [](FSkeletalMaterial& Material) { };
		FMeshMaterialArrayViewer<FSkeletalMaterial> MeshMaterialArrayViewer(ExistMaterials, EmplaceMaterialFunctor);
		InternalMeshFactorySetupAssetMaterialArray<FSkeletalMaterial>(MeshMaterialArrayViewer, ImportedSlotMaterialDependencies, NodeContainer, bIsReimport);
	}

	void StaticMeshFactorySetupAssetMaterialArray(TArray<FStaticMaterial>& ExistMaterials
		, TMap<FString, FString> ImportedSlotMaterialDependencies
		, const UInterchangeBaseNodeContainer* NodeContainer
		, const bool bIsReimport)
	{
		TFunction<void(FStaticMaterial&)> EmplaceMaterialFunctor = [](FStaticMaterial& Material)
			{
#if !WITH_EDITOR
				// UV density is not supported to be generated at runtime for now. We fake that it has been initialized so that we don't trigger ensures.
				Material.UVChannelData = FMeshUVChannelInfo(1.f);
#endif
			};
		FMeshMaterialArrayViewer<FStaticMaterial> MeshMaterialArrayViewer(ExistMaterials, EmplaceMaterialFunctor);
		InternalMeshFactorySetupAssetMaterialArray<FStaticMaterial>(MeshMaterialArrayViewer, ImportedSlotMaterialDependencies, NodeContainer, bIsReimport);
	}

	void GeometryCacheFactorySetupAssetMaterialArray(TArray<TObjectPtr<UMaterialInterface>>& ExistMaterials
		, TArray<FName>& InMaterialSlotNames
		, TMap<FString, FString> ImportedSlotMaterialDependencies
		, const UInterchangeBaseNodeContainer* NodeContainer
		, const bool bIsReimport)
	{
		TFunction<void(TObjectPtr<UMaterialInterface>&)> EmplaceMaterialFunctor = [](TObjectPtr<UMaterialInterface>& Material) {};
		FMeshMaterialArrayViewer<TObjectPtr<UMaterialInterface>> MeshMaterialArrayViewer(ExistMaterials, InMaterialSlotNames, EmplaceMaterialFunctor);
		InternalMeshFactorySetupAssetMaterialArray<TObjectPtr<UMaterialInterface>>(MeshMaterialArrayViewer, ImportedSlotMaterialDependencies, NodeContainer, bIsReimport);
	}

	void CopyMorphTargetsMeshDescriptionToSkeletalMeshDescription(TArray<FString>& SkeletonMorphCurveMetadataNames
		, const TMap<FString, TOptional<UE::Interchange::FMeshPayloadData>>& LodMorphTargetMeshDescriptions
		, FMeshDescription& DestinationMeshDescription
		, const bool bMergeMorphTargetWithSameName)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CopyMorphTargetsMeshDescriptionToSkeletalMeshDescription)
			const int32 OriginalMorphTargetCount = LodMorphTargetMeshDescriptions.Num();
		TArray < TPair<FString, TArray<FString>>> KeysPerName;
		auto FindOrAdd = [&KeysPerName](const FString& MorphTargetName)->TPair<FString, TArray<FString>>&
			{
				for (TPair<FString, TArray<FString>>& Pair : KeysPerName)
				{
					if (Pair.Key.Equals(MorphTargetName))
					{
						return Pair;
					}
				}

				TPair<FString, TArray<FString>>& NewPair = KeysPerName.AddDefaulted_GetRef();
				NewPair.Key = MorphTargetName;
				return NewPair;
			};
		for (const TPair<FString, TOptional<UE::Interchange::FMeshPayloadData>>& Pair : LodMorphTargetMeshDescriptions)
		{
			const FString MorphTargetUniqueId(Pair.Key);
			const TOptional<UE::Interchange::FMeshPayloadData>& MorphTargetPayloadData = Pair.Value;
			if (!MorphTargetPayloadData.IsSet())
			{
				UE_LOGF(LogInterchangeImport, Error, "Empty morph target optional payload data [%ls].", *MorphTargetUniqueId);
				continue;
			}

			const FMeshDescription& SourceMeshDescription = MorphTargetPayloadData.GetValue().MeshDescription;
			const int32 VertexOffset = MorphTargetPayloadData->VertexOffset;
			const int32 SourceMeshVertexCount = SourceMeshDescription.Vertices().Num();
			const int32 DestinationVertexIndexMax = VertexOffset + SourceMeshVertexCount;
			if (DestinationMeshDescription.Vertices().Num() <= (DestinationVertexIndexMax - 1))
			{
				UE_LOGF(LogInterchangeImport, Error, "Corrupted morph target optional payload data [%ls].", *MorphTargetUniqueId);
				continue;
			}

			if (bMergeMorphTargetWithSameName)
			{
				TPair<FString, TArray<FString>>& PairMorphNameWithKeys = FindOrAdd(MorphTargetPayloadData->MorphTargetName);
				PairMorphNameWithKeys.Value.Add(MorphTargetUniqueId);
			}
			else
			{
				TPair<FString, TArray<FString>>& NewPair = KeysPerName.AddDefaulted_GetRef();
				NewPair.Key = MorphTargetPayloadData->MorphTargetName;
				NewPair.Value.Add(MorphTargetUniqueId);
			}
		}

		//Adjust the count from the merge context
		const int32 MorphTargetCount = KeysPerName.Num();

		//No morph target to import
		if (MorphTargetCount == 0)
		{
			return;
		}

		SkeletonMorphCurveMetadataNames.Reserve(MorphTargetCount);
		FSkeletalMeshAttributes DestinationMeshAttributes(DestinationMeshDescription);

		TVertexAttributesConstRef<FVector3f> DestinationMeshVertexPositions = DestinationMeshAttributes.GetVertexPositions();
		TVertexInstanceAttributesConstRef<FVector3f> DestinationMeshVertexInstanceNormals = DestinationMeshAttributes.GetVertexInstanceNormals();

		TSet<FName> UsedMorphTargetNames;
		UsedMorphTargetNames.Append(DestinationMeshAttributes.GetMorphTargetNames());
		UsedMorphTargetNames.Reserve(UsedMorphTargetNames.Num() + MorphTargetCount);

		for (TPair<FString, TArray<FString>>& NamePerUIDs : KeysPerName)
		{
			const FString& MorphTargetNameBase = NamePerUIDs.Key;
			FName MorphTargetName(UE::Interchange::MakeSanitizedName(MorphTargetNameBase));

			for (int32 Suffix = 1; UsedMorphTargetNames.Contains(MorphTargetName); Suffix++)
			{
				MorphTargetName = FName(MorphTargetName, Suffix);
			}
			UsedMorphTargetNames.Add(MorphTargetName);

			if (MorphTargetName != MorphTargetNameBase)
			{
				UE_LOGF(LogInterchangeImport, Warning, "Duplicate morph target '%ls' found, renamed to '%ls'.", *MorphTargetNameBase, *MorphTargetName.ToString());
			}

			//Systemic morph target normals registration can be memory expensive for large mesh description (UE-348667).
			//If they are needed, they will be registered later when processing NDeltas, but for now, do not register them.
			DestinationMeshAttributes.RegisterMorphTargetAttribute(MorphTargetName, false);

			TVertexAttributesRef<FVector3f> DestinationMeshMorphPosDeltas = DestinationMeshAttributes.GetVertexMorphPositionDelta(MorphTargetName);
			TOptional<TVertexInstanceAttributesRef<FVector3f>> DestinationMeshMorphNormals;
			ensure(!DestinationMeshMorphNormals.IsSet());

			for (const FString& MorphTargetKey : NamePerUIDs.Value)
			{
				const TOptional<UE::Interchange::FMeshPayloadData>& MorphTargetPayloadData = LodMorphTargetMeshDescriptions.FindChecked(MorphTargetKey);
				if (!ensure(MorphTargetPayloadData.IsSet()))
				{
					continue;
				}

				const FMeshDescription& MorphTargetMeshDescription = MorphTargetPayloadData->MeshDescription;
				const int32 VertexOffset = MorphTargetPayloadData->VertexOffset;

				FStaticMeshConstAttributes MorphTargetMeshAttributes(MorphTargetMeshDescription);

				TVertexAttributesConstRef<FVector3f> MorphTargetMeshVertexPositions = MorphTargetMeshAttributes.GetVertexPositions();
				TVertexInstanceAttributesConstRef<FVector3f> MorphTargetMeshVertexInstanceNormals = MorphTargetMeshAttributes.GetVertexInstanceNormals();

				const bool bHasVertexInstanceNormals = MorphTargetMeshVertexInstanceNormals.IsValid() && (MorphTargetMeshVertexInstanceNormals.GetNumElements() > 0);

				//Populate the deltas in the target mesh description
				//Note: We don't have to apply GlobalTransform to MorphTarget Position/Normal here anymore, as it is now passed to PayloadRequest.
				//VertexInstanceNormals will always be present as the Attribute registration will create it 
				// and it will have the elements inside it due to VertexInstance creation automatically adds empty elements. (only exception if we don't have VertexInstances set to begin with)

				for (FVertexID MorphTargetVertexID : MorphTargetMeshDescription.Vertices().GetElementIDs())
				{
					FVertexID DestinationVertexID = MorphTargetVertexID + VertexOffset;

					if ((DestinationMeshMorphPosDeltas.GetNumElements() > DestinationVertexID) &&
						(DestinationMeshVertexPositions.GetNumElements() > DestinationVertexID) &&
						(MorphTargetMeshVertexPositions.GetNumElements() > MorphTargetVertexID))
					{
						FVector3f PositionDelta = MorphTargetMeshVertexPositions[MorphTargetVertexID] - DestinationMeshVertexPositions[DestinationVertexID];

						DestinationMeshMorphPosDeltas[DestinationVertexID] = PositionDelta;
					}

					if (bHasVertexInstanceNormals)
					{
						const TArrayView<const FVertexInstanceID> DestinationVertexInstanceIDs = DestinationMeshDescription.GetVertexVertexInstanceIDs(DestinationVertexID);
						const TArrayView<const FVertexInstanceID> MorphTargetVertexInstanceIDs = MorphTargetMeshDescription.GetVertexVertexInstanceIDs(MorphTargetVertexID);

						if (DestinationVertexInstanceIDs.Num() == MorphTargetVertexInstanceIDs.Num())
						{
							for (size_t VertexInstanceIndex = 0; VertexInstanceIndex < DestinationVertexInstanceIDs.Num(); VertexInstanceIndex++)
							{
								const FVertexInstanceID DestinationVertexInstanceID = DestinationVertexInstanceIDs[VertexInstanceIndex];
								const FVertexInstanceID MorphTargetVertexInstanceID = MorphTargetVertexInstanceIDs[VertexInstanceIndex];

								const FVector3f TargetVertexInstanceNormal = DestinationMeshVertexInstanceNormals.Get(DestinationVertexInstanceID);
								const FVector3f SourceVertexInstanceNormal = MorphTargetMeshVertexInstanceNormals.Get(MorphTargetVertexInstanceID);
								const FVector3f NDelta(SourceVertexInstanceNormal - TargetVertexInstanceNormal);

								if (!NDelta.IsNearlyZero())
								{
									if (!DestinationMeshMorphNormals.IsSet())
									{
										//We only register morph target normals when finding at least one non-zero NDelta.
										DestinationMeshAttributes.RegisterMorphTargetAttribute(MorphTargetName, true);
										DestinationMeshMorphNormals = DestinationMeshAttributes.GetVertexInstanceMorphNormalDelta(MorphTargetName);
									}

									ensure(DestinationMeshMorphNormals.IsSet());
									DestinationMeshMorphNormals->Set(DestinationVertexInstanceID, NDelta);
								}
							}
						}
					}
				}
			}

			SkeletonMorphCurveMetadataNames.Add(MorphTargetName.ToString());
		}
	}

	/** Helper struct for the mesh component vert position octree */
	struct FSkeletalMeshVertPosOctreeSemantics
	{
		enum { MaxElementsPerLeaf = 16 };
		enum { MinInclusiveElementsPerNode = 7 };
		enum { MaxNodeDepth = 12 };

		typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

		/**
		 * Get the bounding box of the provided octree element. In this case, the box
		 * is merely the point specified by the element.
		 *
		 * @param	Element	Octree element to get the bounding box for
		 *
		 * @return	Bounding box of the provided octree element
		 */
		FORCEINLINE static FBoxCenterAndExtent GetBoundingBox(const FSoftSkinVertex& Element)
		{
			return FBoxCenterAndExtent(FVector(Element.Position), FVector::ZeroVector);
		}

		/**
		 * Determine if two octree elements are equal
		 *
		 * @param	A	First octree element to check
		 * @param	B	Second octree element to check
		 *
		 * @return	true if both octree elements are equal, false if they are not
		 */
		FORCEINLINE static bool AreElementsEqual(const FSoftSkinVertex& A, const FSoftSkinVertex& B)
		{
			return (A.Position == B.Position && A.UVs[0] == B.UVs[0]);
		}

		/** Ignored for this implementation */
		FORCEINLINE static void SetElementId(const FSoftSkinVertex& Element, FOctreeElementId2 Id)
		{
		}
	};
	typedef TOctree2<FSoftSkinVertex, FSkeletalMeshVertPosOctreeSemantics> TSKCVertPosOctree;

#if WITH_EDITOR
	void RemapSkeletalMeshVertexColorToMeshDescription(const USkeletalMesh* SkeletalMesh, const int32 LODIndex, FMeshDescription& MeshDescription)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RemapSkeletalMeshVertexColorToMeshDescription)
		//Make sure we have all the source data we need to do the remap
		if (!SkeletalMesh->GetImportedModel() || !SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex) || !SkeletalMesh->GetHasVertexColors())
		{
			return;
		}

		// Find the extents formed by the cached vertex positions in order to optimize the octree used later
		FBox3f Bounds(ForceInitToZero);

		FSkeletalMeshAttributes MeshAttributes(MeshDescription);

		TVertexAttributesConstRef<FVector3f> VertexPositions = MeshAttributes.GetVertexPositions();
		TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormals = MeshAttributes.GetVertexInstanceNormals();
		TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = MeshAttributes.GetVertexInstanceUVs();
		TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = MeshAttributes.GetVertexInstanceColors();

		for (const FVertexID VertexID : MeshDescription.Vertices().GetElementIDs())
		{
			const FVector3f& Position = VertexPositions[VertexID];
			Bounds += Position;
		}

		TArray<FSoftSkinVertex> Vertices;
		SkeletalMesh->GetImportedModel()->LODModels[LODIndex].GetVertices(Vertices);
		for (int32 SkinVertexIndex = 0; SkinVertexIndex < Vertices.Num(); ++SkinVertexIndex)
		{
			const FSoftSkinVertex& SkinVertex = Vertices[SkinVertexIndex];
			Bounds += SkinVertex.Position;
		}

		TSKCVertPosOctree VertPosOctree(FVector(Bounds.GetCenter()), Bounds.GetExtent().GetMax());

		// Add each old vertex to the octree
		for (int32 SkinVertexIndex = 0; SkinVertexIndex < Vertices.Num(); ++SkinVertexIndex)
		{
			const FSoftSkinVertex& SkinVertex = Vertices[SkinVertexIndex];
			VertPosOctree.AddElement(SkinVertex);
		}

		// Iterate over each new vertex position, attempting to find the old vertex it is closest to, applying
		// the color of the old vertex to the new position if possible.
		for (const FVertexID VertexID : MeshDescription.Vertices().GetElementIDs())
		{
			FVector Position = FVector(VertexPositions[VertexID]);

			TArray<FSoftSkinVertex> PointsToConsider;
			VertPosOctree.FindNearbyElements(Position, [&PointsToConsider](const FSoftSkinVertex& Vertex)
				{
					PointsToConsider.Add(Vertex);
				});

			if (PointsToConsider.Num() > 0)
			{
				//Get the closest position
				float MaxNormalDot = -MAX_FLT;
				float MinUVDistance = MAX_FLT;
				int32 MatchIndex = INDEX_NONE;
				for (int32 ConsiderationIndex = 0; ConsiderationIndex < PointsToConsider.Num(); ++ConsiderationIndex)
				{
					const FSoftSkinVertex& SkinVertex = PointsToConsider[ConsiderationIndex];
					const FVector2f& SkinVertexUV = SkinVertex.UVs[0];

					for (const FVertexInstanceID VertexInstanceID : MeshDescription.GetVertexVertexInstanceIDs(VertexID))
					{
						FVector3f Normal = VertexInstanceNormals[VertexInstanceID];
						FVector2f UV = VertexInstanceUVs[VertexInstanceID];

						const float UVDistanceSqr = FVector2f::DistSquared(UV, SkinVertexUV);
						if (UVDistanceSqr < MinUVDistance)
						{
							MinUVDistance = FMath::Min(MinUVDistance, UVDistanceSqr);
							MatchIndex = ConsiderationIndex;
							MaxNormalDot = Normal | SkinVertex.TangentZ;
						}
						else if (FMath::IsNearlyEqual(UVDistanceSqr, MinUVDistance, KINDA_SMALL_NUMBER))
						{
							//This case is useful when we have hard edge that shared vertice, somtime not all the shared wedge have the same paint color
							//Think about a cube where each face have different vertex color.
							float NormalDot = Normal | SkinVertex.TangentZ;
							if (NormalDot > MaxNormalDot)
							{
								MaxNormalDot = NormalDot;
								MatchIndex = ConsiderationIndex;
							}
						}

						if (PointsToConsider.IsValidIndex(MatchIndex))
						{
							VertexInstanceColors[VertexInstanceID] = PointsToConsider[MatchIndex].Color.ReinterpretAsLinear();
						}
					}
				}
			}
		}
	}
#endif

	EInterchangeMeshCollision GetMeshCollisionFromName(const FString& Name)
	{
		EInterchangeMeshCollision CollisionType = EInterchangeMeshCollision::None;

		if (Name.StartsWith(TEXT("UBX_")))
		{
			CollisionType = EInterchangeMeshCollision::Box;
		}
		else if (Name.StartsWith(TEXT("UCX_")) || Name.StartsWith(TEXT("MCDCX_")))
		{
			CollisionType = EInterchangeMeshCollision::Convex18DOP;
		}
		else if (Name.StartsWith(TEXT("USP_")))
		{
			CollisionType = EInterchangeMeshCollision::Sphere;
		}
		else if (Name.StartsWith(TEXT("UCP_")))
		{
			CollisionType = EInterchangeMeshCollision::Capsule;
		}

		return CollisionType;
	}

	bool HasPotentialMeshCollisionPrefix(const FString& Name)
	{
		return Name.StartsWith(TEXT("UBX")) ||
			Name.StartsWith(TEXT("UCX")) ||
			Name.StartsWith(TEXT("USP")) ||
			Name.StartsWith(TEXT("UCP")) ||
			Name.StartsWith(TEXT("MCDCX"));
	}

#if WITH_EDITOR
	/** Nanite assembly builder abstract base class (to hide underlying USkeletalMesh/UStaticMesh templated type) */

	class FInterchangeNaniteAssemblyBuilder::ImplBase
	{
	public:

		ImplBase() = default;
		virtual ~ImplBase() = default;
		UE_NONCOPYABLE(ImplBase);

		virtual void AddDescription(const TOptional<UE::Interchange::FNaniteAssemblyDescription>& Description) = 0;
	};

	/** Nanite assembly builder implementation templated by either USkeletalMesh or UStaticMesh. The main responsibility of this class
	  * is to translate instance data from one or more interchange FNaniteAssemblyDescription items, to node data on the underlying
	  * FNaniteAssemblyDataBuilder (nanite assembly API refers to an instance as a 'node', and a prototype mesh as a 'part').
	  * 
	  * Note that in addition to the typical transform + prototype/part id per-instance data, skeletal mesh nanite assembly nodes also
	  * require one or more FNaniteAssemblyBoneInfluence bindings (basically joint index and a weight). This data tells nanite where/how
	  * to attach the root bone from the skeletal mesh part being instanced onto the main skeletal mesh skeleton.
	  */

	template<typename MeshType>
	class FInternalNaniteAssemblyBuilder : public FInterchangeNaniteAssemblyBuilder::ImplBase
	{
		static_assert(
			std::is_same_v<UStaticMesh, MeshType> || std::is_same_v<USkeletalMesh, MeshType>,
			"MeshType must be either UStaticMesh or USkeletalMesh"
		);

		friend class FInterchangeNaniteAssemblyBuilder;

		using FNaniteAssemblyDescription = UE::Interchange::FNaniteAssemblyDescription;

		// List of USkeletalMesh/UStaticMesh objects per part/prototype index (most commonly just a single object)  
		using FPartObjectList = TArray<TArray<MeshType*, TInlineAllocator<1>>>;

		// The assembly builder instance used to create the assembly nodes and parts.
		FNaniteAssemblyDataBuilder Builder;
		
		// The base USkeletalMesh/UStaticMesh object we're enabling nanite assembly on.
		MeshType* Mesh = nullptr;

		// The UInterchangeMeshFactoryNode for this mesh that holds the part dependency information.
		const UInterchangeMeshFactoryNode* MeshFactoryNode = nullptr;

		// The base node container
		const UInterchangeBaseNodeContainer* NodeContainer = nullptr;

		// Maps to help reassociate existing part meshes and slot bindings (intended for use during Interchange reimport)
		TMap<FString, TWeakObjectPtr<MeshType>> CurrentNodeUidToPartMesh;
		TMap<TWeakObjectPtr<MeshType>, TArray<int32>> PartMeshToMaterialRemaps;

		bool bCtorIsValid = false;

		FInternalNaniteAssemblyBuilder(MeshType* InMesh
			, const UInterchangeMeshFactoryNode* InMeshFactoryNode
			, const UInterchangeBaseNodeContainer* InNodeContainer)
		  : Mesh(InMesh)
		  , MeshFactoryNode(InMeshFactoryNode)
		  , NodeContainer(InNodeContainer)
		  , bCtorIsValid(IsInGameThread() && Mesh && MeshFactoryNode && NodeContainer)
		{
			if (!ensure(bCtorIsValid))
			{
				return;
			}

			Mesh->ClearCachedNaniteAssemblyReferences();

			ExtractPreexistingAssemblyPartMeshes();

			// Add the initial base mesh materials to the builder.
			ForEachMeshMaterial(
				Mesh,
				[this](int32 LocalMaterialIndex, const FName& SlotName, const TObjectPtr<UMaterialInterface>& Material)
				{
					Builder.AddMaterialSlot({ Material, SlotName });
				}
			);
		}

		static constexpr bool IsSkeletalMesh()
		{
			return std::is_same_v<USkeletalMesh, MeshType>;
		}

		static constexpr bool IsStaticMesh()
		{
			return std::is_same_v<UStaticMesh, MeshType>;
		}

		~FInternalNaniteAssemblyBuilder()
		{
			EnableNaniteAndFinalize();
		}

		/** Main entry point to add an interchange FNaniteAssemblyDescription to the builder */
		void AddDescription(const TOptional<FNaniteAssemblyDescription>& InDescription) override
		{
			if (!bCtorIsValid || !InDescription.IsSet())
			{
				return;
			}

			const FNaniteAssemblyDescription& Description = InDescription.GetValue();

			// Verify the description data is well-formed. This check will also ensure that iterating over the
			// bone influence data (based on the description's supplied stride data) later on is safe. 
			if (FString Reason; !Description.IsValid(&Reason))
			{
				UE_LOGF(LogInterchangeImport, Warning 
					,"Invalid description while building nanite assembly data for mesh '%ls' - %ls"
					,*MeshFactoryNode->GetDisplayLabel()
					,*Reason
				);
				return;
			}

			// Skeletal mesh assemblies must have binding data.
			if (IsSkeletalMesh() && Description.BoneInfluences.IsEmpty())
			{
				UE_LOGF(LogInterchangeImport, Warning
					,"Invalid description while building nanite assembly data for mesh '%ls' - Skeletal "
						  "meshes must have at least one bone influence per instance."
					,*MeshFactoryNode->GetDisplayLabel()
				);
				return;
			}

			// Get valid UStaticMesh/USkeletal objects from the descriptions part list.
			FPartObjectList PartObjects;
			if (!GetPartObjects(Description, PartObjects))
			{
				return;
			}

			// Setup our binding data. Each instance can have a variable number of bindings as specified by the
			// NumInfluencesPerInstance field on the description, and so the pointer must be advanced accordingly.
			// Note that the call to Description.IsValid() above will have checked that the binding array is 
			// appropriately sized. Note also that when adding static mesh nodes we must still supply an empty binding.

			FNaniteAssemblyBoneInfluence* BoneInfluenceData = nullptr;
			TArray<FNaniteAssemblyBoneInfluence> NaniteBoneInfluences;
			NaniteBoneInfluences.Reserve(Description.BoneInfluences.Num());
			if constexpr (IsSkeletalMesh())
			{
				for (const FNaniteAssemblyDescription::FBoneInfluence& Influence : Description.BoneInfluences)
				{
					NaniteBoneInfluences.Emplace(Influence.BoneIndex, Influence.BoneWeight);
				}
				BoneInfluenceData = NaniteBoneInfluences.GetData();
			}

			TArrayView<const FNaniteAssemblyBoneInfluence> Binding{};

			// Loop over every instance and add a node for each part object.
			const int32 NumInstances = Description.Transforms.Num();
			int32 NumInstancesWithInvalidPartIndex = 0;
			for (int32 Index = 0; Index < NumInstances; ++Index)
			{
				// Update the skeletal mesh binding view
				if constexpr (IsSkeletalMesh())
				{
					const int32 NumInfluences = Description.NumInfluencesPerInstance[Index];
					Binding = MakeArrayView(BoneInfluenceData, NumInfluences);
					BoneInfluenceData += NumInfluences;
				}

				const int32 PartIndex = Description.PartIndices[Index];
				if (PartObjects.IsValidIndex(PartIndex))
				{
					for (MeshType* PartObject : PartObjects[PartIndex])
					{
						AddNodeAndBindMaterials(PartObject, Description.Transforms[Index], Binding);
					}
				}
				else
				{
					NumInstancesWithInvalidPartIndex++;
				}
			}

			// Report invalid instance indices
			if (NumInstancesWithInvalidPartIndex > 0)
			{
				UE_LOGF(LogInterchangeImport, Warning
					, "Nanite assembly description part index data for mesh '%ls' contains one or more invalid indices."
					, *MeshFactoryNode->GetDisplayLabel()
				);
			}
		}

		/** Enable nanite and finalize/apply the FNaniteAssemblyDataBuilder data to the mesh. If the finalize process 
		  * fails for any reason, revert the nanite enabled flag to the original value.
		  */
		void EnableNaniteAndFinalize()
		{
			if (!bCtorIsValid)
			{
				return;
			}

			if (!Builder.GetData().IsValid())
			{
				UE_LOGF(LogInterchangeImport, Warning
					,"Nanite assembly builder data for mesh '%ls' is invalid, skipping."
					, *MeshFactoryNode->GetDisplayLabel()
				);
				return;
			}

			// Enable nanite, apply the builder data and re-cache references ...

			const bool bOriginalNaniteEnabledValue = SetNaniteEnabled(true);

			bool ApplyResult = false;

			if constexpr (IsSkeletalMesh())
			{
				ApplyResult = Builder.ApplyToSkeletalMesh(*Mesh);
			}
			else if constexpr (IsStaticMesh())
			{
				ApplyResult = Builder.ApplyToStaticMesh(*Mesh);
			}

			if (!ApplyResult)
			{
				SetNaniteEnabled(bOriginalNaniteEnabledValue);

				UE_LOGF(LogInterchangeImport, Warning
					, "Failed to apply assembly builder data to mesh '%ls', skipping."
					, * MeshFactoryNode->GetDisplayLabel()
				);
				return;
			}

			Mesh->RecacheNaniteAssemblyReferences();

			return;
		}

		/** Add a node to the FNaniteAssemblyDataBuilder. Also add a new part if this is the first time the part
		  * mesh has been encountered.
		  * 
		  * Note that the bool return value is not success/failure, but whether or not a new part was created.
		  */
		bool AddNode(const MeshType* PartMesh
			, const FTransform& LocalTransform
			, const TArrayView<const FNaniteAssemblyBoneInfluence>& Bindings
			, int32& OutPartIndex)
		{
			const FSoftObjectPath MeshPath(PartMesh);
			OutPartIndex = Builder.FindPart(MeshPath);

			const bool bNewPart = (OutPartIndex == INDEX_NONE);
			if (bNewPart)
			{
				OutPartIndex = Builder.AddPart(MeshPath);
			}

			Builder.AddNode(OutPartIndex, FTransform3f(LocalTransform), ENaniteAssemblyNodeTransformSpace::Local, Bindings);

			return bNewPart;
		}

		/** Add a node and materials from the given part mesh to the builder */
		void AddNodeAndBindMaterials(const MeshType* PartMesh
			, const FTransform& LocalTransform
			, const TArrayView<const FNaniteAssemblyBoneInfluence>& Bindings)
		{
			int32 PartIndex;
			if (AddNode(PartMesh, LocalTransform, Bindings, PartIndex))
			{
				ForEachMeshMaterial(
					PartMesh,
					[this, PartIndex, PartMesh](int32 LocalMaterialIndex, const FName& SlotName, const TObjectPtr<UMaterialInterface>& PartMaterial)
					{
						int32 MaterialIndex = INDEX_NONE;
						
						// First attempt to find an existing material slot that was already present on the base mesh.
						if (const TArray<int32>* RemappedIndices = PartMeshToMaterialRemaps.Find(PartMesh))
						{
							if (RemappedIndices->IsValidIndex(LocalMaterialIndex))
							{
								const int32 RemappedIndex = (*RemappedIndices)[LocalMaterialIndex];
								if (Builder.GetMaterialSlots().IsValidIndex(RemappedIndex))
								{
									Builder.RemapPartMaterial(PartIndex, LocalMaterialIndex, RemappedIndex);
									return;
								}
							}
						}

						// For now we always merge identical materials. There are however more advanced workflows 
						// that require exact control over material and slot management apparently, and so more
						// flexibility may be required in the future.
						constexpr bool bMergeIdenticalMaterialSlots = true;
						if (bMergeIdenticalMaterialSlots)
						{
							MaterialIndex = Builder.GetMaterialSlots().IndexOfByPredicate(
								[&PartMaterial](const FNaniteAssemblyDataBuilder::FMaterialSlot& ExistingMaterial)
								{
									return ExistingMaterial.Material == PartMaterial;
								}
							);
						}

						if (MaterialIndex == INDEX_NONE)
						{
							MaterialIndex = Builder.AddMaterialSlot({ PartMaterial, SlotName });
						}

						Builder.RemapPartMaterial(PartIndex, LocalMaterialIndex, MaterialIndex);
					}
				);
			}
		}

		/* Resolve the string mesh uids and/or top-level asset references from the given description to valid
         * USkeletalMesh/UStaticMesh objects. Note that invalid (null) objects will be omitted from the inner array.
		 * The outer array length is guaranteed to match the number of parts in the input description however, so that 
		 * each per-instance part index indexes into the correct list of part objects. 
		 * 
		 * Returns false if every uid was invalid and so we effectively have no parts to work with.
		 * 
		 */
		bool GetPartObjects(const UE::Interchange::FNaniteAssemblyDescription& Description, FPartObjectList& OutPartObjects)
		{
			TSet<FString> MissingPartUids;
			bool bFoundAtLeastOnePartObject = false;

			TMap<FString, FString> PartUidToFactoryNodeUidTable;
			MeshFactoryNode->GetAssemblyPartDependencies(PartUidToFactoryNodeUidTable);

			// Note the length of output part array must match the length of the incoming part uids from the description.
			OutPartObjects.SetNum(Description.PartUids.Num());

			for (int Index = 0; Index < Description.PartUids.Num(); ++Index)
			{
				const TArray<FString>& PartUids = Description.PartUids[Index];
				for (const FString& PartUid : PartUids)
				{
					if (PartUid.IsEmpty())
					{
						MissingPartUids.Add(TEXT(""));
						continue;
					}

					// First check if this part relates to a factory node (most common case) and if so get the UObject from
					// from the node. If that fails, try loading the part as an already existing asset.

					bool bFoundPartObject = false;
					if (const FString* FactoryNodeDependencyUid = PartUidToFactoryNodeUidTable.Find(PartUid))
					{
						if (const UInterchangeMeshFactoryNode* PartMeshFactoryNode = Cast<const UInterchangeMeshFactoryNode>(NodeContainer->GetFactoryNode(*FactoryNodeDependencyUid)))
						{
							FSoftObjectPath PartReferenceObject;
							// Attempt to get the dependent object from the factory node. NOTE this will return false during a reimport
							if (PartMeshFactoryNode->GetCustomReferenceObject(PartReferenceObject))
							{
								if (MeshType* MeshObject = Cast<MeshType>(PartReferenceObject.TryLoad()))
								{
									OutPartObjects[Index].Add(MeshObject);
									bFoundPartObject = true;
								}
							}
							// Otherwise try and find a matching part that was already attached to the base mesh we received
							else if (TWeakObjectPtr<MeshType>* MeshObject = CurrentNodeUidToPartMesh.Find(*FactoryNodeDependencyUid))
							{
								if (MeshObject->IsValid())
								{
									OutPartObjects[Index].Add(MeshObject->Get());
									bFoundPartObject = true;
								}
							}
						}
					}
					else if (FPackageName::IsValidLongPackageName(PartUid) || FPackageName::IsValidObjectPath(PartUid))
					{
						FTopLevelAssetPath AssetPath(PartUid);
						if (AssetPath.IsValid())
						{
							if (UObject* MeshObject = StaticLoadAsset(MeshType::StaticClass(), AssetPath))
							{
								OutPartObjects[Index].Add(Cast<MeshType>(MeshObject));
								bFoundPartObject = true;
							}
						}
					}

					if (bFoundPartObject)
					{
						bFoundAtLeastOnePartObject = true;
					}
					else
					{
						MissingPartUids.Add(PartUid);
					}
				}
			}

			if (!MissingPartUids.IsEmpty())
			{
				FString Message = FString::Printf(
					TEXT("Failed to resolve (%d) parts to valid objects while processing nanite assembly description for mesh '%s' : \n")
					, MissingPartUids.Num()
					, *MeshFactoryNode->GetDisplayLabel()
				);
				for (const FString& PartUid : MissingPartUids)
				{
					Message.Append(FString::Printf(TEXT("... %s\n"), *PartUid));
				}
				UE_LOGF(LogInterchangeImport, Warning, "%ls", *Message);
			}

			return bFoundAtLeastOnePartObject;
		}

		/** Extract the current nanite assembly parts (if any) from the base mesh. This enables us to map the
		  * incoming factory node UIDs to usable parts during reimport, which is the best we can do until
		  * Interchange provides valid dependencies on reimports.
		  */
		void ExtractPreexistingAssemblyPartMeshes()
		{
#if WITH_EDITORONLY_DATA
			FMeshNaniteSettings Settings = Mesh->GetNaniteSettings();
			if (!Settings.bEnabled)
			{
				return;
			}

			const FNaniteAssemblyData& AssemblyData = Settings.NaniteAssemblyData;
			for (const FNaniteAssemblyPart& Part : AssemblyData.Parts)
			{
				const FSoftObjectPath& PartObjectPath = Part.MeshObjectPath;
				if (!PartObjectPath.IsValid() || !PartObjectPath.IsAsset())
				{
					continue;
				}

				if (MeshType* PartObject = Cast<MeshType>(PartObjectPath.TryLoad()))
				{
					if (UAssetImportData* ImportDataPtr = PartObject->GetAssetImportData())
					{
						if (UInterchangeAssetImportData* AssetImportData = Cast<UInterchangeAssetImportData>(ImportDataPtr))
						{
							const FString& NodeUid = AssetImportData->NodeUniqueID;
							if (!NodeUid.IsEmpty())
							{
								CurrentNodeUidToPartMesh.Add(NodeUid, PartObject);
								PartMeshToMaterialRemaps.Add(PartObject, Part.MaterialRemap);
							}
						}
					}
				}
			}
#endif //WITH_EDITORONLY_DATA
		}

		/** Set the nanite enabled flag to the given value. Note the bool return value here is the previous enabled 
		  * on/off state (since we may need to revert it), not whether the new setting succeeded/failed. 
		  */
		bool SetNaniteEnabled(bool Value)
		{
			FMeshNaniteSettings Settings = Mesh->GetNaniteSettings();
			const bool bOriginalValue = Settings.bEnabled;
			if (Value != bOriginalValue)
			{
				Settings.bEnabled = Value;
				Mesh->SetNaniteSettings(Settings);
			}
			return bOriginalValue;
		}

		/** Loop over the materials for the given mesh object, and call Func for each MaterialInterface */
		template <typename TFunc>
		static void ForEachMeshMaterial(const UObject* MeshObject, TFunc&& Func)
		{
			const MeshType* MeshTypeObject = Cast<MeshType>(MeshObject);
			if (!ensure(MeshTypeObject))
			{
				return;
			}

			int32 LocalMaterialIndex = 0;
			if constexpr (IsSkeletalMesh())
			{
				for (const FSkeletalMaterial& SkeletalMaterial : MeshTypeObject->GetMaterials())
				{
					Func(LocalMaterialIndex++, SkeletalMaterial.MaterialSlotName, SkeletalMaterial.MaterialInterface);
				}
			}
			else if constexpr (IsStaticMesh())
			{
				for (const FStaticMaterial& StaticMaterial : MeshTypeObject->GetStaticMaterials())
				{
					Func(LocalMaterialIndex++, StaticMaterial.MaterialSlotName, StaticMaterial.MaterialInterface);
				}
			}
		}
	};

	/** Nanite assembly builder public interface */

	FInterchangeNaniteAssemblyBuilder FInterchangeNaniteAssemblyBuilder::Create(USkeletalMesh* SkeletalMesh
		, const UInterchangeMeshFactoryNode* MeshFactoryNode
		, const UInterchangeBaseNodeContainer* NodeContainer)
	{
		return FInterchangeNaniteAssemblyBuilder(TUniquePtr<ImplBase>(new FInternalNaniteAssemblyBuilder<USkeletalMesh>(SkeletalMesh, MeshFactoryNode, NodeContainer)));
	}

	FInterchangeNaniteAssemblyBuilder FInterchangeNaniteAssemblyBuilder::Create(UStaticMesh* StaticMesh
		, const UInterchangeMeshFactoryNode* MeshFactoryNode
		, const UInterchangeBaseNodeContainer* NodeContainer)
	{
		return FInterchangeNaniteAssemblyBuilder(TUniquePtr<ImplBase>(new FInternalNaniteAssemblyBuilder<UStaticMesh>(StaticMesh, MeshFactoryNode, NodeContainer)));
	}

	void FInterchangeNaniteAssemblyBuilder::AddDescription(const TOptional<UE::Interchange::FNaniteAssemblyDescription>& InDescription)
	{
		Impl->AddDescription(InDescription);
	}

	/** Nanite assembly builder ctor/dtor */

	FInterchangeNaniteAssemblyBuilder::FInterchangeNaniteAssemblyBuilder(TUniquePtr<ImplBase> InImpl)
		: Impl(MoveTemp(InImpl))
	{

	}

	FInterchangeNaniteAssemblyBuilder::~FInterchangeNaniteAssemblyBuilder() = default;
#endif // WITH_EDITOR
} //ns UE::Interchange::Private::MeshHelper