// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/InterchangeMeshPayload.h"

#include "Algo/Accumulate.h" 

namespace UE::Interchange
{
	bool FNaniteAssemblyDescription::IsValid(FString* OutReason) const
	{
		// Transforms and PartIndices must not be empty and should be of equal length. There must also be at least one part. 
		// Note we don't check for invalid PartIndices indexing into PartUids here - those will be skipped by the assembly builder.
		if (Transforms.IsEmpty() || PartIndices.IsEmpty() || (Transforms.Num() != PartIndices.Num()))
		{
			if (OutReason)
			{
				*OutReason = FString::Printf(TEXT("Transforms and part-indices arrays are either empty or different lengths (%d vs %d)"),
					Transforms.Num(), PartIndices.Num());
			}
			return false;
		}

		if (!BoneInfluences.IsEmpty())
		{
			// The stride data should match the number of transforms.
			if (NumInfluencesPerInstance.Num() != Transforms.Num())
			{
				if (OutReason)
				{
					*OutReason = FString::Printf(TEXT("Transforms and influences-per-instance arrays are different lengths (%d vs %d)"),
						Transforms.Num(), NumInfluencesPerInstance.Num());
				}
				return false;
			}

			// Lastly, sum the stride data and make sure it agrees with the number of influences.
			const int32 NumTotalInflueces = Algo::Accumulate(NumInfluencesPerInstance, 0);
			if (NumTotalInflueces != BoneInfluences.Num())
			{
				if (OutReason)
				{
					*OutReason = FString::Printf(TEXT("Bone influences array length (%d) does not match influences-per-instance total (%d)"),
						BoneInfluences.Num(), NumTotalInflueces);
				}
				return false;
			}
		}

		return true;
	}
} //ns UE::Interchange