// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Mesh/InterchangeOBJTranslator.h"

#include "Algo/BinarySearch.h"
#include "Algo/Find.h"
#include "InterchangeMaterialDefinitions.h"
#include "InterchangeMeshDefinitions.h"
#include "InterchangeMeshNode.h"
#include "InterchangeSceneNode.h"
#include "InterchangeMaterialInstanceNode.h"
#include "InterchangeTexture2DNode.h"
#include "InterchangeTextureNode.h"
#include "InterchangeTranslatorHelper.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeSourceNode.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "UVMapSettings.h"
#include "Material/InterchangeMaterialNodesHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeOBJTranslator)

DEFINE_LOG_CATEGORY_STATIC(LogInterchangeObjImport, Log, All);

#define LOCTEXT_NAMESPACE "InterchangeOBJTranslator"

static bool GInterchangeEnableOBJImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnableOBJImport(
	TEXT("Interchange.FeatureFlags.Import.OBJ"),
	GInterchangeEnableOBJImport,
	TEXT("Whether OBJ support is enabled."),
	ECVF_Default);

static bool GInterchangeEnableOBJLevelImport = false;
static FAutoConsoleVariableRef CCvarInterchangeEnableOBJLevelImport(
	TEXT("Interchange.FeatureFlags.Import.OBJ.ToLevel"),
	GInterchangeEnableOBJLevelImport,
	TEXT("Whether support for OBJ level import is enabled."),
	ECVF_Default);


#define INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Verbosity, Format, ...) \
{ \
	static bool bLogged = false; \
	UE_CLOG(!bLogged, LogInterchangeObjImport, Verbosity, Format, ##__VA_ARGS__); \
	bLogged = true; \
}
/**
 * This is a binary representation of the .obj file 
 */
struct FObjData
{
	/** Filename of the .obj file being parsed */
	FString ObjFilename;

	/** .obj element arrays */
	TArray<FVector3f> Positions;
	TArray<FVector2f> UVs;
	TArray<FVector3f> Normals;

	/** Defines the data for a single face vertex */
	struct FVertexData
	{
		FVertexData(int32 InVertexIndex, int32 InUVIndex, int32 InNormalIndex)
			: VertexIndex(InVertexIndex), UVIndex(InUVIndex), NormalIndex(InNormalIndex)
		{}

		/** .obj face vertices have three attributes: position index, UV index, normal index */
		int32 VertexIndex;
		int32 UVIndex;
		int32 NormalIndex;
	};

	/** Defines the data for a single face */
	struct FFaceData
	{
		/** .obj faces consist of an array of face vertices */
		TArray<FVertexData, TInlineAllocator<4>> Vertices;

		/** Return the FVertexData of this face which contains the given vertex index */
		const FVertexData& GetVertexDataContainingVertexIndex(int32 VertexIndex) const
		{
			static FVertexData InvalidVertexData { INDEX_NONE, INDEX_NONE, INDEX_NONE };

			const FVertexData* VertexData = Algo::FindBy(Vertices, VertexIndex, &FVertexData::VertexIndex);
			return ensure(VertexData) ? *VertexData : InvalidVertexData;
		}
	};

	struct FFaceGroupData
	{
		FString MaterialName;
		TArray<FFaceData> Faces;
	};

	/** Defines the data for a group */
	struct FGroupData
	{
		TArray<FFaceGroupData> FaceGroups;
	};

	/** All the groups defined by the .obj */
	TMap<FString, FGroupData> Groups;

	struct FTextureParameter
	{
		FString Path;
		float BumpMultiplier = 1.0f;  // Default to 1.0f, can be negative
		bool IsEmpty() const
		{
			return Path.IsEmpty();
		}
	};

	/** Defines the data for a material */
	struct FMaterialData
	{
		// Unspecified properties designated by negative values
		int32 IlluminationModel = 2;  // Default illumination model to Phong for non-Pbr materials
		FVector3f DiffuseColor = -FVector3f::One();
		FVector3f AmbientColor = -FVector3f::One();
		FVector3f SpecularColor = -FVector3f::One();
		FVector3f EmissiveColor = -FVector3f::One();
		FTextureParameter DiffuseTexture;
		FTextureParameter AmbientTexture;
		FTextureParameter SpecularTexture;
		FTextureParameter EmissiveTexture;
		FTextureParameter OpacityTexture;
		FTextureParameter TransparencyTexture;
		FTextureParameter SpecularExponentTexture;
		FVector3f TransmissionFilter = -FVector3f::One();
		float IndexOfRefraction = -1.f;
		float Opacity = -1.f;
		float Transparency = -1.f;

		float SpecularExponent = -1.f;

		float Roughness = -1.f;
		FTextureParameter RoughnessTexture;

		float Metallic = -1.f;
		FTextureParameter MetallicTexture;
		FTextureParameter BumpmapTexture;
		FTextureParameter NormalmapTexture;

		float ClearCoatThickness = -1.f;
		float ClearCoatRoughness = -1.f;
		float Anisotropy = -1.f;
	};

	/** All the materials defined by the .obj */
	TMap<FString, FMaterialData> Materials;

	/** This is the current group name being parsed */
	FString CurrentGroup = TEXT("Default");

	/** This is the current material name being parsed */
	FString MaterialBeingDefined;

	/** This is the current material name in use */
	FString CurrentMaterial;

	/** Generates a MeshDescription for the named group */
	FMeshDescription MakeMeshDescriptionForGroup(const FString& GroupName, const FTransform& MeshGlobalTransform) const;

	/** Returns a bounding box fitting the vertices in the named group */
	FBox GetGroupBoundingBox(const FString& GroupName) const;

	/** Returns the vertex count for the named group */
	int32 GetGroupVertexCount(const FString& GroupName) const;

	/** Returns the poly count for the named group */
	int32 GetGroupPolygonCount(const FString& GroupName) const;

private:
	TArray<int32> GetVertexIndicesUsedByGroup(const FGroupData& GroupData) const;
	TArray<int32> GetUVIndicesUsedByGroup(const FGroupData& GroupData) const;
};


static FVector3f PositionToUEBasis(const FVector3f& InVector)
{
	return FVector3f(InVector.X, -InVector.Y, InVector.Z);
}


static FVector2f UVToUEBasis(const FVector2f& InVector)
{
	return FVector2f(InVector.X, 1.0f - InVector.Y);
}


TArray<int32> FObjData::GetVertexIndicesUsedByGroup(const FGroupData& GroupData) const
{
	TSet<int32> VertexIndexSet;

	for (const FFaceGroupData& FaceGroup : GroupData.FaceGroups)
	{
		for (const FFaceData& FaceData : FaceGroup.Faces)
		{
			for (const FVertexData& VertexData : FaceData.Vertices)
			{
				VertexIndexSet.Add(VertexData.VertexIndex);
			}
		}
	}

	return VertexIndexSet.Array();
}


TArray<int32> FObjData::GetUVIndicesUsedByGroup(const FGroupData& GroupData) const
{
	TSet<int32> UVIndexSet;

	for (const FFaceGroupData& FaceGroup : GroupData.FaceGroups)
	{
		for (const FFaceData& FaceData : FaceGroup.Faces)
		{
			for (const FVertexData& VertexData : FaceData.Vertices)
			{
				if (ensure(UVs.IsValidIndex(VertexData.UVIndex)))
				{
					UVIndexSet.Add(VertexData.UVIndex);
				}
			}
		}
	}

	return UVIndexSet.Array();
}


FBox FObjData::GetGroupBoundingBox(const FString& GroupName) const
{
	FBox Box(ForceInit);

	const FGroupData* GroupDataPtr = Groups.Find(GroupName);
	if (!GroupDataPtr)
	{
		// If group name not found, return an empty bounding box
		return Box;
	}

	const FGroupData& GroupData = *GroupDataPtr;

	for (int32 PositionIndex : GetVertexIndicesUsedByGroup(GroupData))
	{
		if(Positions.IsValidIndex(PositionIndex))
		{
			Box += FVector(PositionToUEBasis(Positions[PositionIndex]));
		}
		else
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("FObjData::GetGroupBoundingBox: One or more Position index were invalid, skipping those position to compute the bounding box"));
		}
	}

	return Box;
}


int32 FObjData::GetGroupVertexCount(const FString& GroupName) const
{
	const FGroupData* GroupDataPtr = Groups.Find(GroupName);
	if (!GroupDataPtr)
	{
		return 0;
	}

	const FGroupData& GroupData = *GroupDataPtr;
	return GetVertexIndicesUsedByGroup(GroupData).Num();
}


int32 FObjData::GetGroupPolygonCount(const FString& GroupName) const
{
	const FGroupData* GroupDataPtr = Groups.Find(GroupName);
	if (!GroupDataPtr)
	{
		return 0;
	}

	const FGroupData& GroupData = *GroupDataPtr;
	return Algo::Accumulate(GroupData.FaceGroups, 0, [](int32 Accum, const FFaceGroupData& FaceGroup)
	{
		return Accum + FaceGroup.Faces.Num();
	});
}


FMeshDescription FObjData::MakeMeshDescriptionForGroup(const FString& GroupName, const FTransform& MeshGlobalTransform) const
{
	static FString DefaultMaterialName = UMaterial::GetDefaultMaterial(MD_Surface)->GetName();

	FMeshDescription MeshDescription;

	const FGroupData* GroupDataPtr = Groups.Find(GroupName);
	if (!GroupDataPtr)
	{
		// If group name not found, return an empty mesh description
		return MeshDescription;
	}

	FMatrix TotalMatrix = MeshGlobalTransform.ToMatrixWithScale();
	FMatrix TotalMatrixForNormal;
	TotalMatrixForNormal = TotalMatrix.Inverse();
	TotalMatrixForNormal = TotalMatrixForNormal.GetTransposed();

	auto TransformPosition = [](const FMatrix& Matrix, FVector3f& Position)
	{
		const FVector TransformedPosition = Matrix.TransformPosition(FVector(Position));
		Position = static_cast<FVector3f>(TransformedPosition);
	};

	FStaticMeshAttributes Attributes(MeshDescription);
	Attributes.Register();
	Attributes.RegisterPolygonObjectNameAttribute();

	MeshDescription.SuspendVertexInstanceIndexing();
	MeshDescription.SuspendEdgeIndexing();
	MeshDescription.SuspendPolygonIndexing();
	MeshDescription.SuspendPolygonGroupIndexing();
	MeshDescription.SuspendUVIndexing();

	const FGroupData& GroupData = *GroupDataPtr;

	// We prefer that the MeshDescription vertex and UV element buffers be in the same order as the .obj ones.
	// (there are certain use cases for which this is convenient, in order to be able to match imported data to the source)
	// So get the list of vertex and UV indices used by the faces in this group, and sort and compact them to contiguous indices,
	// preserving their original ordering.

	TArray<int32> VertexIndexMapping = GetVertexIndicesUsedByGroup(GroupData);
	TArray<int32> UVIndexMapping = GetUVIndicesUsedByGroup(GroupData);
	VertexIndexMapping.Sort();
	UVIndexMapping.Sort();
	
	// Create vertices and initialize positions
	// Note that we create a contiguous list of vertex indices from 0...n-1, referencing potentially sparse obj vertices

	TVertexAttributesRef<FVector3f> MeshPositions = Attributes.GetVertexPositions();
	MeshDescription.ReserveNewVertices(VertexIndexMapping.Num());
	for (int32 ObjVertexIndex : VertexIndexMapping)
	{
		FVertexID VertexIndex = MeshDescription.CreateVertex();
		if (MeshPositions.GetRawArray().IsValidIndex(VertexIndex) && Positions.IsValidIndex(ObjVertexIndex))
		{
			FVector3f& Position = Attributes.GetVertexPositions()[VertexIndex];
			Position = PositionToUEBasis(Positions[ObjVertexIndex]);
			TransformPosition(TotalMatrix, Position);
		}
		else
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("FObjData::MakeMeshDescriptionForGroup: One or more vertex position index are not valid, skipping those vertex position"));
		}
		
	}

	// Create UVs and initialize values
	MeshDescription.SetNumUVChannels(1);
	if (UVIndexMapping.Num() > 0)
	{
		const int32 UVChannel = 0;
		MeshDescription.ReserveNewUVs(UVIndexMapping.Num());
		for (int32 ObjUVIndex : UVIndexMapping)
		{
			FUVID UVIndex = MeshDescription.CreateUV(UVChannel);
			Attributes.GetUVCoordinates(UVChannel)[UVIndex] = UVs[ObjUVIndex];
		}
	}

	TMap<FPolygonID, const FFaceData*> Polygons;
	for (const FFaceGroupData& FaceGroup : GroupData.FaceGroups)
	{
		
		// Create polygon group
		FPolygonGroupID PolygonGroupIndex = MeshDescription.CreatePolygonGroup();
		const FString MaterialName = FaceGroup.MaterialName.IsEmpty() ? DefaultMaterialName : FaceGroup.MaterialName;
		Attributes.GetPolygonGroupMaterialSlotNames()[PolygonGroupIndex] = FName(MaterialName);

		// Create faces.
		// n-gons are preserved

		MeshDescription.ReserveNewTriangles(FaceGroup.Faces.Num());
		MeshDescription.ReserveNewPolygons(FaceGroup.Faces.Num());

		TArray<FVertexInstanceID, TInlineAllocator<8>> VertexInstanceIDs;
		for (const FFaceData& FaceData : FaceGroup.Faces)
		{
			VertexInstanceIDs.Reset();
			MeshDescription.ReserveNewVertexInstances(FaceData.Vertices.Num());

			for (const FVertexData& VertexData : FaceData.Vertices)
			{
				FVertexID VertexID = Algo::BinarySearch(VertexIndexMapping, VertexData.VertexIndex);
				if (ensure(VertexID != INDEX_NONE))
				{
					FVertexInstanceID VertexInstanceID = MeshDescription.CreateVertexInstance(VertexID);
					VertexInstanceIDs.Add(VertexInstanceID);

					if (Normals.IsValidIndex(VertexData.NormalIndex))
					{
						FVector3f& Normal = Attributes.GetVertexInstanceNormals()[VertexInstanceID];
						Normal = PositionToUEBasis(Normals[VertexData.NormalIndex]);
						TransformPosition(TotalMatrixForNormal, Normal);
					}

					if (UVs.IsValidIndex(VertexData.UVIndex))
					{
						Attributes.GetVertexInstanceUVs()[VertexInstanceID] = UVToUEBasis(UVs[VertexData.UVIndex]);
					}
				}
			}

			if (ensure(VertexInstanceIDs.Num() > 2))
			{
				Polygons.Add(MeshDescription.CreatePolygon(PolygonGroupIndex, VertexInstanceIDs), &FaceData);
			}
		}
	}

	// Determine edge hardnesses
	// Iterate through all the edges, looking at the adjacent faces.
	// If the face vertex normals aren't all the same, we consider the edge to be hard.
	// Internal edges within polygons are always counted as soft.

	// Allow triangles adjacent to edges to be indexed
	// We need this when getting edge connected polygons.
	MeshDescription.ResumeEdgeIndexing();

	for (const FEdgeID& EdgeID : MeshDescription.Edges().GetElementIDs())
	{
		// Don't look at internal edges (edges within a polygon which break it into triangles)
		if (!MeshDescription.IsEdgeInternal(EdgeID))
		{
			bool bAllNormalsEqual = true;
			TArray<FPolygonID, TInlineAllocator<2>> EdgePolygonIDs = MeshDescription.GetEdgeConnectedPolygons<TInlineAllocator<2>>(EdgeID);

			for (const FVertexID& EdgeVertexID : MeshDescription.GetEdgeVertices(EdgeID))
			{
				if (bAllNormalsEqual)
				{
					// This is the vertex index as it appears in the .obj
					int32 ObjVertexIndex = VertexIndexMapping[EdgeVertexID];

					// For the vertex we are considering, look at the first face adjacent to the edge, and find the vertex data which includes it
					// This is a baseline we will use to compare all the other adjacent faces
					const FVertexData& VertexData = Polygons[EdgePolygonIDs[0]]->GetVertexDataContainingVertexIndex(ObjVertexIndex);
					if (VertexData.NormalIndex == INDEX_NONE)
					{
						continue;
					}

					for (int32 Index = 1; Index < EdgePolygonIDs.Num(); Index++)
					{
						// For all other adjacent faces, find the vertex data which includes the vertex we are considering.
						// If the normal index is not the same as the baseline, we know this must be a hard edge.
						const FVertexData& VertexDataToCompare = Polygons[EdgePolygonIDs[Index]]->GetVertexDataContainingVertexIndex(ObjVertexIndex);

						if (VertexDataToCompare.NormalIndex != INDEX_NONE && VertexData.NormalIndex != VertexDataToCompare.NormalIndex)
						{
							bAllNormalsEqual = false;
							break;
						}
					}
				}
			}

			if (!bAllNormalsEqual)
			{
				Attributes.GetEdgeHardnesses()[EdgeID] = true;
			}

		}
	}

	// Create default UVs if mesh does not have UVs

	if (UVIndexMapping.Num() == 0)
	{
		FBox MeshBoundingBox = MeshDescription.ComputeBoundingBox();
		FUVMapParameters UVParameters(MeshBoundingBox.GetCenter(), FQuat::Identity, MeshBoundingBox.GetSize(), FVector::OneVector, FVector2D::UnitVector);
		TMap<FVertexInstanceID, FVector2D> TexCoords;
		FStaticMeshOperations::GenerateBoxUV(MeshDescription, UVParameters, TexCoords);

		TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
		if (VertexInstanceUVs.GetNumChannels() == 0)
		{
			VertexInstanceUVs.SetNumChannels(1);
		}

		for (const FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
		{
			if (const FVector2D* UVCoord = TexCoords.Find(VertexInstanceID))
			{
				VertexInstanceUVs.Set(VertexInstanceID, 0, FVector2f(*UVCoord));
			}
			else
			{
				ensureMsgf(false, TEXT("Interchange Obj translator: Tried to apply UV data that did not match the MeshDescription."));
			}
		}
	}

	// Fill in the object group name onto all the triangles of the mesh.
	TPolygonAttributesRef<FName> PolygonObjectNames = Attributes.GetPolygonObjectNames();
	const FName ObjectName(GroupName);
	for (const FPolygonID PolygonID: MeshDescription.Polygons().GetElementIDs())
	{
		PolygonObjectNames.Set(PolygonID, ObjectName);
	}
	

	MeshDescription.ResumeVertexInstanceIndexing();
	MeshDescription.ResumePolygonIndexing();
	MeshDescription.ResumePolygonGroupIndexing();
	MeshDescription.ResumeUVIndexing();

	return MeshDescription;
}


namespace UE {
namespace Interchange {

namespace ObjParserUtils
{
	using FFunctionType = TFunction<bool(FObjData&, FStringView)>;
	using FKeywordMap = TMap<FString, FFunctionType>;


	// Determine if the given character can be counted as a token terminator
	static bool IsTerminatorChar(TCHAR CharToCheck, TCHAR PossibleTerminator = '\0')
	{
		return CharToCheck == TEXT(' ') || CharToCheck == TEXT('\t') || CharToCheck == PossibleTerminator;
	}


	// Skip over any leading whitespace, modifying the passed stringview to start at the first non-whitespace character
	static void SkipWhitespace(FStringView& Line)
	{
		int32 Index = 0;
		while (Index < Line.Len() && (Line[Index] == TEXT(' ') || Line[Index] == TEXT('\t')))
		{
			Index++;
		}

		Line.RightChopInline(Index);
	}


	// Return the next token on the line (delimited by the optionally specified terminator character).
	// Modify the passed stringview to start at the beginning of the next token.
	static FStringView GetToken(FStringView& Line, TCHAR Terminator = TEXT(' '))
	{
		SkipWhitespace(Line);

		// Skip comments
		if (!Line.IsEmpty() && Line[0] == TEXT('#'))
		{
			Line = FStringView();
			return FStringView();
		}

		for (int32 Index = 0; Index < Line.Len(); Index++)
		{
			if (IsTerminatorChar(Line[Index], Terminator))
			{
				FStringView Result = Line.Left(Index);
				Line.RightChopInline(Index + 1);
				SkipWhitespace(Line);
				return Result;
			}
		}

		FStringView Result = Line;
		Line = FStringView();
		return Result;
	}


	// Parse a floating-point literal from the start of the given line, returning true if it was successful.
	// The token containing the floating-point literal will be consumed and removed from the line.
	static bool GetFloat(FStringView& Line, float& Result, TCHAR Terminator = TEXT(' '))
	{
		FStringView Token = GetToken(Line, Terminator);
		if (Token.IsEmpty())
		{
			return false;
		}

		// @todo: have to convert this to a string in order to lex it - this is horrible
		LexFromString(Result, *FString(Token));

		return true;
	}


	static bool GetVector2(FStringView& Line, FVector2f& Result)
	{
		if (!GetFloat(Line, Result.X)) { return false; }
		if (!GetFloat(Line, Result.Y)) { return false; }
		return true;
	}


	static bool GetVector3(FStringView& Line, FVector3f& Result)
	{
		if (!GetFloat(Line, Result.X)) { return false; }
		if (!GetFloat(Line, Result.Y)) { return false; }
		if (!GetFloat(Line, Result.Z)) { return false; }
		return true;
	}


	// Parse an integer literal from the start of the given line, returning true if it was successful.
	// The token containing the integer literal will be consumed and removed from the line.
	static bool GetInteger(FStringView& Line, int32& Result, TCHAR Terminator = TEXT(' '))
	{
		FStringView Token = GetToken(Line, Terminator);
		if (Token.IsEmpty())
		{
			return false;
		}

		// @todo: have to convert this to a string in order to lex it - this is horrible
		LexFromString(Result, *FString(Token));

		return true;
	}


	static bool ParseVertexPosition(FObjData& ObjData, FStringView Line)
	{
		FVector3f Position;
		bool bSuccess = GetVector3(Line, Position);

		ObjData.Positions.Add(Position);

		if (!Line.IsEmpty())
		{
			// @todo: emit warning through Interchange results container
			// @todo: get new API working through InterchangeManager (keyed multiple times on translator, pipeline, factory objects)
			// We have a pointer to the Translator inside the FObjData, so we can get the results container from there.
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Unexpected extra arguments on v keyword"));
		}

		return bSuccess;
	}


	static bool ParseTextureCoordinate(FObjData& ObjData, FStringView Line)
	{
		FVector2f UV;
		bool bSuccess = GetVector2(Line, UV);

		ObjData.UVs.Add(UV);

		if (!Line.IsEmpty())
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Unexpected extra arguments on vt keyword"));
		}

		return bSuccess;
	}


	static bool ParseNormalVector(FObjData& ObjData, FStringView Line)
	{
		FVector3f Normal;
		bool bSuccess = GetVector3(Line, Normal);

		ObjData.Normals.Add(Normal);

		if (!Line.IsEmpty())
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Unexpected extra arguments on vn keyword"));
		}

		return bSuccess;
	}


	static bool ParseFace(FObjData& ObjData, FStringView Line)
	{
		FObjData::FFaceData FaceData;

		while (!Line.IsEmpty())
		{
			FStringView FaceToken = GetToken(Line);

			auto GetFaceIndex = [](FStringView& Token, int32 NumElements)
			{
				int32 Index = INDEX_NONE;
				if (GetInteger(Token, Index, TEXT('/')))
				{
					if (Index < 0)
					{
						// Negative indices refer to vertices declared relatively in the file
						Index += NumElements;
					}
					else
					{
						// Indices start at one, so adjust for zero-base
						--Index;
					}

					if (Index >= 0 && Index < NumElements)
					{
						return Index;
					}
				}

				return static_cast<int32>(INDEX_NONE);
			};

			int32 VertexIndex = GetFaceIndex(FaceToken, ObjData.Positions.Num());
			int32 UVIndex = GetFaceIndex(FaceToken, ObjData.UVs.Num());
			int32 NormalIndex = GetFaceIndex(FaceToken, ObjData.Normals.Num());

			FaceData.Vertices.Emplace(VertexIndex, UVIndex, NormalIndex);
		}

		if (FaceData.Vertices.Num() > 2)
		{
			FObjData::FGroupData& GroupData = ObjData.Groups.FindOrAdd(ObjData.CurrentGroup);
			// Start new face group if last one had different material
			if (GroupData.FaceGroups.IsEmpty() || (GroupData.FaceGroups.Last().MaterialName != ObjData.CurrentMaterial))
			{
				GroupData.FaceGroups.Emplace();
				GroupData.FaceGroups.Last().MaterialName = ObjData.CurrentMaterial;
			}

			GroupData.FaceGroups.Last().Faces.Emplace(FaceData);
		}
		else
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Too few vertices on f keyword"));
			return false;
		}

		return true;
	}


	static bool ParseUseMaterial(FObjData& ObjData, FStringView Line)
	{
		FStringView MaterialName = GetToken(Line);

		if (MaterialName.IsEmpty())
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Missing material name on usemtl keyword"));
		}

		if (!Line.IsEmpty())
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Unexpected extra arguments on usemtl keyword"));
		}

		ObjData.CurrentMaterial = FString(MaterialName);

		return true;
	}

	static bool ParseObjectName(FObjData& ObjData, FStringView Line)
	{
		FStringView ObjectName = GetToken(Line);
		if (ObjectName.IsEmpty())
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Missing object name on o keyword"));
		}

		if (!Line.IsEmpty())
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Unexpected extra arguments on o keyword"));
		}

		// Use Object Name as Current Group and if there is no Group underneath 
		// all Data will be in that default group
		ObjData.CurrentGroup = ObjectName;

		//We do not do anything yet with the object name.
		return true;
	}

	static bool ParseGroup(FObjData& ObjData, FStringView Line)
	{
		FStringView GroupName = GetToken(Line);
		if (GroupName.IsEmpty())
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Missing group name on g keyword"));
		}

		if (!Line.IsEmpty())
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Unexpected extra arguments on g keyword"));
		}

		ObjData.CurrentGroup = GroupName;
		return true;
	}

	static bool ParseLine(FObjData& ObjData, FStringView Line, const FKeywordMap& KeywordMap)
	{
		// Dispatch by keyword to appropriate parsing function
		FString Keyword = FString(GetToken(Line));
		if (Keyword.IsEmpty())
		{
			return true;
		}

		if (const FFunctionType* Handler = KeywordMap.Find(Keyword))
		{
			return (*Handler)(ObjData, Line);
		}
		else
		{
			// Unknown keyword
			UE_LOGF(LogInterchangeObjImport, Warning, "Interchange Obj translator: Unknown keyword: %ls", *Keyword);
		}

		return true;
	}


	static bool ParseFile(FObjData& ObjData, const TCHAR* Filename, const FKeywordMap& KeywordMap)
	{
		// Read the file line by line, passing each line to ParseLine

		bool bSuccess = true;
		FFileHelper::LoadFileToStringWithLineVisitor(Filename,
			[&ObjData, &KeywordMap, &bSuccess](FStringView Line)
			{
				bool bResult = ObjParserUtils::ParseLine(ObjData, Line, KeywordMap);
				if (bResult == false)
				{
					bSuccess = false;
				}
			}
		);

		return bSuccess;
	}


	static bool ParseNewMaterial(FObjData& ObjData, FStringView Line)
	{
		FStringView MaterialName = GetToken(Line);

		if (MaterialName.IsEmpty())
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Missing material name on newmtl keyword"));
		}

		if (!Line.IsEmpty())
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Unexpected extra arguments on newmtl keyword"));
		}

		ObjData.Materials.Add(FString(MaterialName));
		ObjData.MaterialBeingDefined = MaterialName;
		return true;
	}


	static inline bool GetMaterialProperty(FStringView& Line, int32& Result) { return GetInteger(Line, Result); }
	static inline bool GetMaterialProperty(FStringView& Line, float& Result) { return GetFloat(Line, Result); }
	static inline bool GetMaterialProperty(FStringView& Line, FVector3f& Result) { return GetVector3(Line, Result); }


	template <typename PropertyType, PropertyType FObjData::FMaterialData::*Property>
	static bool ParseMaterialProperty(FObjData& ObjData, FStringView Line)
	{
		PropertyType Value = PropertyType();
		if (GetMaterialProperty(Line, Value))
		{
			if (FObjData::FMaterialData* MaterialData = ObjData.Materials.Find(ObjData.MaterialBeingDefined))
			{
				MaterialData->*Property = Value;
			}
			else
			{
				INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Missing newmtl keyword"));
				return false;
			}
		}

		if (!Line.IsEmpty())
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Unexpected extra arguments on material property"));
		}

		return true;
	}


	template <FString FObjData::FMaterialData::* Property>
	static bool ParseMaterialPath(FObjData& ObjData, FStringView Line)
	{
		if (Line.IsEmpty())
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Expecting more parameters(including texture filename) on a map keyword"));
		}

		if (FObjData::FMaterialData* MaterialData = ObjData.Materials.Find(ObjData.MaterialBeingDefined))
		{
			MaterialData->*Property = FPaths::GetPath(ObjData.ObjFilename) / FString(Line);
		}
		else
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Missing newmtl keyword"));
			return false;
		}

		return true;
	}

	template <FObjData::FTextureParameter FObjData::FMaterialData::* Property>
	static bool ParseTextureStatement(FObjData& ObjData, FStringView Line)
	{
		if (Line.IsEmpty())
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Expecting more parameters(including texture filename) on a map keyword"));
		}

		if (FObjData::FMaterialData* MaterialData = ObjData.Materials.Find(ObjData.MaterialBeingDefined))
		{
			FObjData::FTextureParameter& Parameter = (MaterialData->*Property);

			FStringView FileName;

			FStringView LastToken;
			while (true)
			{
				FStringView Token = GetToken(Line);
				if (Token.IsEmpty())
				{
					FileName = LastToken;
					break;
				}

				LastToken = Token;

				if (Token == TEXT("-bm"))
				{
					float BumpMultiplier;
					if (GetFloat(Line, BumpMultiplier))
					{
						Parameter.BumpMultiplier = BumpMultiplier;
					}
				}
			}

			if (!FileName.IsEmpty())
			{
				Parameter.Path = FPaths::GetPath(ObjData.ObjFilename) / FString(FileName);

				FString NormalizedTexturePath(Parameter.Path);
				FPaths::NormalizeFilename(NormalizedTexturePath);
				if (!FPaths::FileExists(NormalizedTexturePath))
				{
					Parameter.Path = FString();
					UE_LOGF(LogInterchangeObjImport, Warning, "Interchange Obj translator: Cannot find texture %ls", *NormalizedTexturePath);
				}
			}
		}
		else
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Missing newmtl keyword"));
			return false;
		}

		return true;
	}

	static bool ParseMaterialLib(FObjData& ObjData, FStringView Line)
	{
		static FKeywordMap MtlKeywordMap =
		{
			{ TEXT("newmtl"),  ParseNewMaterial },
			{ TEXT("illum"),   ParseMaterialProperty<int32,     &FObjData::FMaterialData::IlluminationModel> },
			{ TEXT("Kd"),      ParseMaterialProperty<FVector3f, &FObjData::FMaterialData::DiffuseColor> },
			{ TEXT("Ka"),      ParseMaterialProperty<FVector3f, &FObjData::FMaterialData::AmbientColor> },
			{ TEXT("Ks"),      ParseMaterialProperty<FVector3f, &FObjData::FMaterialData::SpecularColor> },
			{ TEXT("Ke"),      ParseMaterialProperty<FVector3f, &FObjData::FMaterialData::EmissiveColor> },
			{ TEXT("Tf"),      ParseMaterialProperty<FVector3f, &FObjData::FMaterialData::TransmissionFilter> },
			{ TEXT("Ni"),      ParseMaterialProperty<float,     &FObjData::FMaterialData::IndexOfRefraction> },
			{ TEXT("Ns"),      ParseMaterialProperty<float,     &FObjData::FMaterialData::SpecularExponent> },
			{ TEXT("d"),       ParseMaterialProperty<float,     &FObjData::FMaterialData::Opacity> },
			{ TEXT("Tr"),      ParseMaterialProperty<float,     &FObjData::FMaterialData::Transparency> },
			{ TEXT("map_Kd"),  ParseTextureStatement<&FObjData::FMaterialData::DiffuseTexture> },
			{ TEXT("map_Ka"),  ParseTextureStatement<&FObjData::FMaterialData::AmbientTexture> },
			{ TEXT("map_Ks"),  ParseTextureStatement<&FObjData::FMaterialData::SpecularTexture> },
			{ TEXT("map_Ke"),  ParseTextureStatement<&FObjData::FMaterialData::EmissiveTexture> },
			{ TEXT("map_d"),   ParseTextureStatement<&FObjData::FMaterialData::OpacityTexture> },
			{ TEXT("map_Tr"),  ParseTextureStatement<&FObjData::FMaterialData::TransparencyTexture> },
			{ TEXT("map_Ns"),  ParseTextureStatement<&FObjData::FMaterialData::SpecularExponentTexture> },
			{ TEXT("map_bump"),ParseTextureStatement<&FObjData::FMaterialData::BumpmapTexture> },
			{ TEXT("bump"),	   ParseTextureStatement<&FObjData::FMaterialData::BumpmapTexture> },

			{ TEXT("Pr"),      ParseMaterialProperty<float,     &FObjData::FMaterialData::Roughness> },
			{ TEXT("map_Pr"),  ParseTextureStatement<&FObjData::FMaterialData::RoughnessTexture> },

			{ TEXT("Pm"),      ParseMaterialProperty<float,     &FObjData::FMaterialData::Metallic> },
			{ TEXT("map_Pm"),  ParseTextureStatement<&FObjData::FMaterialData::MetallicTexture> },

			{ TEXT("Pc"),      ParseMaterialProperty<float,     &FObjData::FMaterialData::ClearCoatThickness> },
			{ TEXT("Pcr"),     ParseMaterialProperty<float,     &FObjData::FMaterialData::ClearCoatRoughness> },
			{ TEXT("aniso"),   ParseMaterialProperty<float,     &FObjData::FMaterialData::Anisotropy> },

			{ TEXT("norm"),    ParseTextureStatement<&FObjData::FMaterialData::NormalmapTexture> },

		};

		// ZBrush doesn't surround filenames with spaces in quotes, so just take the entire remainder of the line as the filename,
		// instead of getting the next token.
		FStringView MtlFilename = Line; // (instead of GetToken(Line);)
		if (MtlFilename.IsEmpty())
		{
			INTERCHANGE_OBJ_TRANSLATOR_LOG_ONCE(Warning, TEXT("Interchange Obj translator: Missing filename on mtllib keyword"));
		}

		// Parse the file referenced by the .obj
		// This will set the current material name
		FString FileToParse = FPaths::GetPath(ObjData.ObjFilename) / FString(MtlFilename);

		bool bSuccess = true;

		// Only .mtl material files are supported
		if (FPaths::GetExtension(FileToParse).Equals(TEXT("mtl"), ESearchCase::IgnoreCase))
		{
			bSuccess &= ParseFile(ObjData, *FileToParse, MtlKeywordMap);
		}
		else
		{
			UE_LOGF(LogInterchangeObjImport, Warning, "Interchange Obj translator: Unsupported material file extension: %ls", *FileToParse);
		}

		// After parsing, clear the current material name
		ObjData.MaterialBeingDefined.Empty();

		return bSuccess;
	}
} // namespace ObjParserUtils

namespace ObjTranslatorUtils
{
	static bool IsColorInitialized(const FVector3f& Color)
	{
		return Color.GetMin() >= 0.f;
	};

	static bool IsScalarInitialized(const float& Scalar)
	{
		return Scalar >= 0.f;
	};

	static FString MakeSceneNodeUid(const FString& Name)
	{
		return TEXT("\\Scene\\") + (Name.IsEmpty() ? FString(TEXT("Null")) : Name);
	}

	static FString MakeActorNodeUid(const FString& Name)
	{
		return TEXT("\\Actor\\") + (Name.IsEmpty() ? FString(TEXT("Null")) : Name);
	}

	/**
	 * Build and return a UID name for a mesh node.
	 * @todo: move this to be a static method on the MeshNode class itself?
	 */
	static FString MakeMeshNodeUid(const FString& Name)
	{
		return TEXT("\\Mesh\\") + (Name.IsEmpty() ? FString(TEXT("Null")) : Name);
	}


	/**
	 * Build and return a UID name for a shader graph node.
	 * @todo: move this to be a static method on the ShaderGraphNode class itself?
	 */
	static FString MakeMaterialInstanceNodeName(const FString& Name)
	{
		return TEXT("MAT_") + (Name.IsEmpty() ? FString(TEXT("Null")) : Name);
	}


	/**
	 * Build and return a UID name for a texture node.
	 * @todo: move this to be a static method on the TextureNode class itself?
	 */
	static FString MakeTextureNodeName(const FString& Name)
	{
		return TEXT("TEX_") + (Name.IsEmpty() ? FString(TEXT("Null")) : Name);
	}


	static const UInterchangeTexture2DNode* CreateTexture2DNode(UInterchangeBaseNodeContainer& BaseNodeContainer, const FString& TexturePath, FString& InOutTextureName) 
	{
		FString TextureName = InOutTextureName;
		for (int32 Index = 0;; ++Index)
		{
			FString NormalizedTexturePath(TexturePath);
			FPaths::NormalizeFilename(NormalizedTexturePath);

			FString TextureNodeUid = UInterchangeTextureNode::MakeNodeUid(TextureName);

			UInterchangeTexture2DNode* TextureNode = const_cast<UInterchangeTexture2DNode*>(Cast<const UInterchangeTexture2DNode>(BaseNodeContainer.GetNode(TextureNodeUid)));
			if (!TextureNode)
			{
				TextureNode = UInterchangeTexture2DNode::Create(&BaseNodeContainer, TextureNodeUid, TextureName);
				TextureNode->SetPayLoadKey(NormalizedTexturePath);

				InOutTextureName = TextureName;
				return TextureNode;
			}

			// Check if existing texture node was created for the same image file 
			if (NormalizedTexturePath == TextureNode->GetPayLoadKey())
			{
				InOutTextureName = TextureName;
				return TextureNode;
			}

			// Generate different texture name(and therefore NodeUid)
			TextureName = InOutTextureName + FString::FromInt(Index);
		}
	}

	static const UInterchangeTexture2DNode* CreateTexture2DNode(UInterchangeBaseNodeContainer& BaseNodeContainer, const FObjData::FTextureParameter& TextureParameter)
	{
		const UInterchangeTexture2DNode* TextureNode = nullptr;
		if (!TextureParameter.IsEmpty())
		{
			FString TextureName = MakeTextureNodeName(FPaths::GetBaseFilename(TextureParameter.Path));
			TextureNode = CreateTexture2DNode(BaseNodeContainer, TextureParameter.Path, TextureName);
		}

		return TextureNode;
	}

	// multiply texture by color, handling cases when any of the arguments is missing
	static bool AddTexturedColoredInput(UInterchangeBaseNodeContainer& BaseNodeContainer, UInterchangeMaterialInstanceNode* MaterialInstanceNode, const FString& InputName, const FVector3f& Color, const FObjData::FTextureParameter& TexturePath)
	{
		if (const UInterchangeTexture2DNode* TextureNode = CreateTexture2DNode(BaseNodeContainer, TexturePath))
		{
			const FString InputNameLabel = InputName + TEXT("Map");
			if (MaterialInstanceNode->AddTextureParameterValue(InputNameLabel, TextureNode->GetUniqueID()))
			{
				MaterialInstanceNode->AddScalarParameterValue(InputNameLabel + TEXT("Weight"), 1.0f);
				return true;
			}
		}
		else if (IsColorInitialized(Color))
		{
			return MaterialInstanceNode->AddVectorParameterValue(InputName, Color);
		}

		return false;
	}

	// multiply texture by scalar, handling cases when any of the arguments is missing
	static bool AddTexturedWeightedInput(UInterchangeBaseNodeContainer& BaseNodeContainer, UInterchangeMaterialInstanceNode* MaterialInstanceNode, const FString& InputName, const float& Weight, const FObjData::FTextureParameter& TexturePath) 
	{
		bool bResult = false;
		if (const UInterchangeTexture2DNode* TextureNode = CreateTexture2DNode(BaseNodeContainer, TexturePath))
		{
			const FString InputNameLabel = InputName + TEXT("Map");
			bResult |= MaterialInstanceNode->AddTextureParameterValue(InputNameLabel, TextureNode->GetUniqueID());
            bResult |= MaterialInstanceNode->AddScalarParameterValue(InputNameLabel + TEXT("Weight"), 1.0f);
		}
		else if (IsScalarInitialized(Weight))
		{
			bResult |= MaterialInstanceNode->AddScalarParameterValue(InputName, Weight);
		}

		return bResult;
	}

	static bool HandlePbrAttributes(UInterchangeBaseNodeContainer& BaseNodeContainer, const FObjData::FMaterialData& MaterialData, UInterchangeMaterialInstanceNode* MaterialInstanceNode)
	{
		bool bPbr = !MaterialData.RoughnessTexture.IsEmpty() || IsScalarInitialized(MaterialData.Roughness)
		         || !MaterialData.MetallicTexture.IsEmpty()  || IsScalarInitialized(MaterialData.Metallic)
				 || IsScalarInitialized(MaterialData.ClearCoatThickness)
			     || IsScalarInitialized(MaterialData.Anisotropy);

		if (!bPbr)
		{
			return false;
		}

		AddTexturedColoredInput(BaseNodeContainer, MaterialInstanceNode, Materials::PBRMR::Parameters::BaseColor.ToString(), MaterialData.DiffuseColor, MaterialData.DiffuseTexture);
		AddTexturedWeightedInput(BaseNodeContainer, MaterialInstanceNode, Materials::PBRMR::Parameters::Metallic.ToString(), MaterialData.Metallic, MaterialData.MetallicTexture);

		const float SpecularValue = (MaterialData.SpecularColor.X + MaterialData.SpecularColor.Y + MaterialData.SpecularColor.Z) / 3;
		AddTexturedWeightedInput(BaseNodeContainer, MaterialInstanceNode, Materials::PBRMR::Parameters::Specular.ToString(), SpecularValue, MaterialData.SpecularTexture);

		// Roughness (Glossiness, Shininess, Specular Power/Exponent in non-PBR terms)
		AddTexturedWeightedInput(BaseNodeContainer, MaterialInstanceNode, Materials::PBRMR::Parameters::Roughness.ToString(), MaterialData.Roughness, MaterialData.RoughnessTexture);

		if (IsScalarInitialized(MaterialData.ClearCoatThickness) && !FMath::IsNearlyZero(MaterialData.ClearCoatThickness))
		{
			MaterialInstanceNode->AddScalarParameterValue(Materials::ClearCoat::Parameters::ClearCoat.ToString(), MaterialData.ClearCoatThickness);

			if (IsScalarInitialized(MaterialData.ClearCoatRoughness))
			{
				MaterialInstanceNode->AddScalarParameterValue(Materials::ClearCoat::Parameters::ClearCoatRoughness.ToString(), MaterialData.ClearCoatRoughness);
			}
		}

		if (IsScalarInitialized(MaterialData.Anisotropy)  && !FMath::IsNearlyZero(MaterialData.Anisotropy))
		{
			MaterialInstanceNode->AddScalarParameterValue(Materials::PBRMR::Parameters::Anisotropy.ToString(), MaterialData.Anisotropy);
		}

		return true;
	}
}  // namespace ObjTranslatorUtils

} // namespace Interchange
} // namespace UE


UInterchangeOBJTranslator::UInterchangeOBJTranslator()
	: ObjDataPtr(MakePimpl<FObjData>())
{
}


UInterchangeOBJTranslator::~UInterchangeOBJTranslator()
{
}

TArray<FString> UInterchangeOBJTranslator::GetSupportedFormats() const
{
	if (GInterchangeEnableOBJImport || GIsAutomationTesting)
	{
		TArray<FString> Formats{ TEXT("obj;OBJ File Format") };
		return Formats;
	}
	else
	{
		return TArray<FString>{};
	}
}

EInterchangeTranslatorAssetType UInterchangeOBJTranslator::GetSupportedAssetTypes() const
{
	//Obj translator support Meshes and Materials
	return EInterchangeTranslatorAssetType::Materials | EInterchangeTranslatorAssetType::Meshes;
}

EInterchangeTranslatorType UInterchangeOBJTranslator::GetTranslatorType() const
{
	return GInterchangeEnableOBJLevelImport ? EInterchangeTranslatorType::Scenes : EInterchangeTranslatorType::Assets;
}

bool UInterchangeOBJTranslator::Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const
{
	// Get filename from SourceData

	FString Filename = GetSourceData()->GetFilename();
	if (!FPaths::FileExists(Filename))
	{
		return false;
	}

	ObjDataPtr->ObjFilename = Filename;

	using namespace UE::Interchange;
	using namespace ObjTranslatorUtils;

	static ObjParserUtils::FKeywordMap ObjKeywordMap =
	{
		{ TEXT("o"),      ObjParserUtils::ParseObjectName },
		{ TEXT("v"),      ObjParserUtils::ParseVertexPosition },
		{ TEXT("vt"),     ObjParserUtils::ParseTextureCoordinate },
		{ TEXT("vn"),     ObjParserUtils::ParseNormalVector },
		{ TEXT("f"),      ObjParserUtils::ParseFace },
		{ TEXT("g"),      ObjParserUtils::ParseGroup },
		{ TEXT("mtllib"), ObjParserUtils::ParseMaterialLib },
		{ TEXT("usemtl"), ObjParserUtils::ParseUseMaterial }
	};

	bool bSuccess = ObjParserUtils::ParseFile(*ObjDataPtr.Get(), *Filename, ObjKeywordMap);

	if (bSuccess)
	{
		if (ObjDataPtr->Groups.IsEmpty() && ObjDataPtr->Materials.IsEmpty())
		{
			UInterchangeResultError_Generic* ErrorResult = AddMessage<UInterchangeResultError_Generic>();
			ErrorResult->SourceAssetName = FPaths::GetBaseFilename(Filename);
			ErrorResult->Text = LOCTEXT("EmptyFileError", "The OBJ file appears to be empty.");

			return false;
		}

		// Add base scene node.
		UInterchangeSceneNode* SceneNode = NewObject< UInterchangeSceneNode >(&BaseNodeContainer);
		const FString SceneName = FPaths::GetBaseFilename(Filename);
		const FString SceneNodeUid = MakeSceneNodeUid(SceneName);
		BaseNodeContainer.SetupNode(SceneNode, SceneNodeUid, SceneName, EInterchangeNodeContainerType::TranslatedScene);

		// Add mesh nodes to the container for each mesh group

		for (const TPair<FString, FObjData::FGroupData>& Group : ObjDataPtr->Groups)
		{
			const FString& GroupName = Group.Key;

			const FString ActorNodeUid = MakeActorNodeUid(GroupName);
			const FString MeshNodeUid = MakeMeshNodeUid(GroupName);

			UInterchangeSceneNode* InterchangeSceneNode = NewObject<UInterchangeSceneNode>(&BaseNodeContainer);
			BaseNodeContainer.SetupNode(InterchangeSceneNode, ActorNodeUid, GroupName, EInterchangeNodeContainerType::TranslatedScene, SceneNodeUid);
			InterchangeSceneNode->SetCustomAssetInstanceUid(MeshNodeUid);

			UInterchangeMeshNode* MeshNode = NewObject<UInterchangeMeshNode>(&BaseNodeContainer);
			BaseNodeContainer.SetupNode(MeshNode, MeshNodeUid, GroupName, EInterchangeNodeContainerType::TranslatedAsset);

			// The payload key is the group name.
			// A MeshDescription is generated for each group of faces.
			MeshNode->SetPayLoadKey(GroupName, EInterchangeMeshPayLoadType::STATIC);

			MeshNode->SetCustomBoundingBox(ObjDataPtr->GetGroupBoundingBox(GroupName));
			MeshNode->SetCustomVertexCount(ObjDataPtr->GetGroupVertexCount(GroupName));
			MeshNode->SetCustomPolygonCount(ObjDataPtr->GetGroupPolygonCount(GroupName));
			MeshNode->SetCustomHasVertexNormal(true);
			MeshNode->SetCustomHasVertexBinormal(false);
			MeshNode->SetCustomHasVertexTangent(false);
			MeshNode->SetCustomHasSmoothGroup(false);
			MeshNode->SetCustomHasVertexColor(false);

			for (const FObjData::FFaceGroupData& FaceGroup : Group.Value.FaceGroups)
			{
				if (!FaceGroup.MaterialName.IsEmpty())
				{					
					const FString MaterialDependencyUid = UInterchangeMaterialInstanceNode::MakeNodeUid(MakeMaterialInstanceNodeName(FaceGroup.MaterialName), TEXT(""));
					MeshNode->SetSlotMaterialDependencyUid(FaceGroup.MaterialName, MaterialDependencyUid);
				}
			}
		}

		// Add material nodes to the container
		for (const TPair<FString, FObjData::FMaterialData>& Material : ObjDataPtr->Materials)
		{
			const FString& MaterialName = Material.Key;
			const FObjData::FMaterialData& MaterialData = Material.Value;
			FString NodeUid = UInterchangeMaterialInstanceNode::MakeNodeUid(MakeMaterialInstanceNodeName(MaterialName), TEXT(""));

			UInterchangeMaterialInstanceNode* MaterialInstanceNode = NewObject<UInterchangeMaterialInstanceNode>(&BaseNodeContainer);
			MaterialInstanceNode->SetCustomParent(TEXT("/InterchangeAssets/Materials/PBRSurfaceMaterial_MR.PBRSurfaceMaterial_MR"));
			BaseNodeContainer.SetupNode(MaterialInstanceNode, NodeUid, MaterialName, EInterchangeNodeContainerType::TranslatedAsset);

			bool bPBR = true;
			bool bOpacity = false;

			bool bUnlit = false;
			bool bSpecular = true;
			bool bReflectionMetal = false;
			bool bReflectionFresnel = false;
			bool bTransmission = false;
			bool bRefraction = false;

			if (!HandlePbrAttributes(BaseNodeContainer, MaterialData, MaterialInstanceNode))
			{
				bPBR = false;
				// It's not a Pbr material - in this case parse the illumination model

				// What is relevant to us about the illumination model in OBJ spec
				// 0 - Constant color
				// 1 - Lambertian diffuse (i.e. no specular)
				// 2 - Phong (i.e. includes specular)
				// 3 - Has 'reflection map'(non-fresnel, metal)
				// 4 - "Simulating Glass" - transparency with strong reflection(non-fresnel, metal)
				// 5 - 3 but Fresnel reflection(non-metal)
				// 6 - Reflection and refraction, Tf transmission is used
				// 7 - Reflection and refraction, Tf transmission is used(just Fresnel)
				// 8 - as 3 
				// 9 - as 4

				switch (MaterialData.IlluminationModel)
				{
					case 0:
					{
						bUnlit = true;
						bSpecular = false;
					}
					break;

					case 1:
					{
						bSpecular = false;
					}
					break;

					case 2:
					{
					}
					break;

					case 3:
					case 4:
					case 8:
					case 9:
					{
						bReflectionMetal = true;
					}
					break;

					case 5:
					{
						bReflectionFresnel = true;
					}
					break;

					case 6:
					{
						bReflectionMetal = true;
						bTransmission = true;
						bRefraction = true;
					}
					break;

					case 7:
					{
						bReflectionFresnel = true;
						bTransmission = true;
						bRefraction = true;
					}
					break;


					default: ;
				}

				if(bUnlit)
				{
					MaterialInstanceNode->SetCustomParent(TEXT("/InterchangeAssets/Materials/UnlitMaterial.UnlitMaterial"));
					AddTexturedColoredInput(BaseNodeContainer, MaterialInstanceNode, Materials::Unlit::Parameters::UnlitColor.ToString(), MaterialData.DiffuseColor, MaterialData.DiffuseTexture);
				}
				else
				{
					MaterialInstanceNode->SetCustomParent(TEXT("/InterchangeAssets/Materials/PhongSurfaceMaterial.PhongSurfaceMaterial"));
					AddTexturedColoredInput(BaseNodeContainer, MaterialInstanceNode, Materials::Phong::Parameters::DiffuseColor.ToString(), (MaterialData.DiffuseColor.GetMin() < 0) ? FVector3f(0.8f, 0.8f, 0.8f) : MaterialData.DiffuseColor, MaterialData.DiffuseTexture);

					if (bSpecular)
					{
						AddTexturedColoredInput(BaseNodeContainer, MaterialInstanceNode, Materials::Phong::Parameters::SpecularColor.ToString(), MaterialData.SpecularColor, MaterialData.SpecularTexture);

						// OBJ Specular Exponent('Ns') is in range 0 to 1000(for sharpest highlight) and Interchange Shininess is up to 100
						const float SpecularExponentToShininessFactor = 0.1f;
						const FString ShininessParameterName = Materials::Phong::Parameters::Shininess.ToString();
						if (const UInterchangeTexture2DNode* SpecularExponentTextureNode= CreateTexture2DNode(BaseNodeContainer, MaterialData.SpecularExponentTexture))
						{
							using namespace UE::Interchange::Materials::Standard::Nodes;

							float SpecularExponentScale = (IsScalarInitialized(MaterialData.SpecularExponent) ? MaterialData.SpecularExponent : 1.0f) * SpecularExponentToShininessFactor;
							MaterialInstanceNode->AddTextureParameterValue(ShininessParameterName + TEXT("Map"), SpecularExponentTextureNode->GetUniqueID());
							MaterialInstanceNode->AddScalarParameterValue(ShininessParameterName + TEXT("MapWeight"), SpecularExponentScale);
						}
						else if (IsScalarInitialized(MaterialData.SpecularExponent))
						{
							MaterialInstanceNode->AddScalarParameterValue(ShininessParameterName, MaterialData.SpecularExponent);
						}
					}

					const FString AmbientColorPropertyName = Materials::Phong::Parameters::AmbientColor.ToString();
					if (!bReflectionMetal)
					{
						const FVector3f AmbientColor = MaterialData.AmbientColor.GetMin() < 0.f ? FVector3f(0.8f, 0.8f, 0.8f) : MaterialData.AmbientColor;
						AddTexturedColoredInput(BaseNodeContainer, MaterialInstanceNode, AmbientColorPropertyName, AmbientColor, MaterialData.AmbientTexture);
					}
				}
			}

			// Common parameters

			// Opacity
			{
				const FString OpacityInputName = Materials::Common::Parameters::Opacity.ToString();

				// Transparency texture replaces Transparency scalar
                if (const UInterchangeTexture2DNode* TransparencyTextureNode = CreateTexture2DNode(BaseNodeContainer, MaterialData.TransparencyTexture))
				{
					bOpacity = true;
					if (MaterialInstanceNode->AddTextureParameterValue(OpacityInputName + TEXT("Map"), TransparencyTextureNode->GetUniqueID()))
					{
						MaterialInstanceNode->AddScalarParameterValue(OpacityInputName + TEXT("MapWeight"), 1.0f);
					}
				}
				// Opacity texture is weighted by scalar opacity("d")
				else if (!AddTexturedWeightedInput(BaseNodeContainer, MaterialInstanceNode, OpacityInputName, MaterialData.Opacity, MaterialData.OpacityTexture))
				{
					// When no textures are present compute opacity value depending on which is defined - opacity or transparency
					// OBJ has two ways to define Opacity/Transparency - "d"(for opacity) and "Tr"(for "inverse opacity" - transparency)
					float OpacityScalar = IsScalarInitialized(MaterialData.Opacity) ? MaterialData.Opacity : (IsScalarInitialized(MaterialData.Transparency) ? 1.f - MaterialData.Transparency : -1.f);
					if (IsScalarInitialized(OpacityScalar))
					{
						bOpacity = true;
						MaterialInstanceNode->AddScalarParameterValue(OpacityInputName, OpacityScalar);
					}
				}

				if (bOpacity)
				{
					if (bPBR)
					{
						MaterialInstanceNode->SetCustomParent(TEXT("/InterchangeAssets/Materials/ThinTranslucentMaterial_MR.ThinTranslucentMaterial_MR"));
					}
					else
					{
						MaterialInstanceNode->SetCustomParent(TEXT("/InterchangeAssets/Materials/PhongSurfaceTranslucentMaterial.PhongSurfaceTranslucentMaterial"));
					}
				}
			}

			if (bUnlit)
			{
				continue;
			}

			// Emissive
			AddTexturedColoredInput(BaseNodeContainer, MaterialInstanceNode, Materials::Common::Parameters::EmissiveColor.ToString(), MaterialData.EmissiveColor, MaterialData.EmissiveTexture);
				
			// Ior
			if (IsScalarInitialized(MaterialData.IndexOfRefraction) || bRefraction)
			{
				const float DefaultIndexOfRefraction = 1.52f;  // Use glass index of refraction by default
				const float IndexOfRefraction = IsScalarInitialized(MaterialData.IndexOfRefraction) ? MaterialData.IndexOfRefraction : DefaultIndexOfRefraction;
				const FString ParameterName = Materials::Common::Parameters::IndexOfRefraction.ToString();
				MaterialInstanceNode->AddScalarParameterValue(ParameterName, IndexOfRefraction);
			}

			if ((IsColorInitialized(MaterialData.TransmissionFilter) && !(MaterialData.TransmissionFilter - FVector3f::One()).IsNearlyZero()) || bTransmission)
			{
				FLinearColor TransmissionColor = IsColorInitialized(MaterialData.TransmissionFilter) ? MaterialData.TransmissionFilter : FLinearColor::White;
				const FString ParameterName = Materials::ThinTranslucent::Parameters::TransmissionColor.ToString();
				MaterialInstanceNode->AddVectorParameterValue(ParameterName, TransmissionColor);
			}

			// Normal/bump
			if (const UInterchangeTexture2DNode* NormalmapTextureNode = CreateTexture2DNode(BaseNodeContainer, MaterialData.NormalmapTexture))
			{
				MaterialInstanceNode->AddTextureParameterValue(Materials::Common::Parameters::Normal.ToString() + TEXT("Map"), NormalmapTextureNode->GetUniqueID());
			}
			else if (!MaterialData.BumpmapTexture.IsEmpty())
			{
				using namespace Materials::Standard::Nodes;

				FString TexturePath = MaterialData.BumpmapTexture.Path;
				FString TextureName = MakeTextureNodeName(FPaths::GetBaseFilename(TexturePath));

				if (const UInterchangeTexture2DNode* BumpmapTextureNode = CreateTexture2DNode(BaseNodeContainer, TexturePath, TextureName))
				{
					MaterialInstanceNode->AddTextureParameterValue(Materials::Common::Parameters::Normal.ToString() + TEXT("Map"), BumpmapTextureNode->GetUniqueID());
					MaterialInstanceNode->AddScalarParameterValue(TEXT("NormalMapWeight"), MaterialData.BumpmapTexture.BumpMultiplier);
					MaterialInstanceNode->AddStaticSwitchParameterValue(TEXT("bHasBump"), true);
				}
			}
		}
	}

	return bSuccess;
}

TOptional<UE::Interchange::FMeshPayloadData> UInterchangeOBJTranslator::GetMeshPayloadData(const FInterchangeMeshPayLoadKey& PayLoadKey, const UE::Interchange::FAttributeStorage& PayloadAttributes) const
{
	using namespace UE::Interchange;

	FTransform MeshGlobalTransform;
	PayloadAttributes.GetAttribute(UE::Interchange::FAttributeKey{ MeshPayload::Attributes::MeshGlobalTransform }, MeshGlobalTransform);

	FMeshPayloadData Payload;
	Payload.MeshDescription = ObjDataPtr->MakeMeshDescriptionForGroup(PayLoadKey.UniqueId, MeshGlobalTransform);

	if (!FStaticMeshOperations::ValidateAndFixData(Payload.MeshDescription, PayLoadKey.UniqueId))
	{
		UInterchangeResultError_Generic* ErrorResult = AddMessage<UInterchangeResultError_Generic>();
		ErrorResult->SourceAssetName = SourceData ? SourceData->GetFilename() : FString();
		ErrorResult->Text = LOCTEXT("GetMeshPayloadData_ValidateMeshDescriptionFail", "Invalid mesh data (NAN) was found and fix to zero. Mesh render can be bad.");
	}
	return Payload;
}

TOptional<UE::Interchange::FImportImage> UInterchangeOBJTranslator::GetTexturePayloadData(const FString& PayLoadKey, TOptional<FString>& AlternateTexturePath) const
{
	UE::Interchange::Private::FScopedTranslator ScopedTranslator(PayLoadKey, Results, AnalyticsHandler);
	const IInterchangeTexturePayloadInterface* TextureTranslator = ScopedTranslator.GetPayLoadInterface<IInterchangeTexturePayloadInterface>();
	if (!ensure(TextureTranslator))
	{
		return TOptional<UE::Interchange::FImportImage>();
	}
	AlternateTexturePath = PayLoadKey;
	return TextureTranslator->GetTexturePayloadData(PayLoadKey, AlternateTexturePath);
}

#undef LOCTEXT_NAMESPACE