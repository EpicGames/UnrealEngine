// Copyright Epic Games, Inc. All Rights Reserved. 
#include "Mesh/InterchangePhysicsAssetFactory.h"

#include "Engine/SkeletalMesh.h"
#include "InterchangeImportCommon.h"
#include "InterchangeImportLog.h"
#include "InterchangePhysicsAssetFactoryNode.h"
#include "InterchangeSkeletalMeshFactoryNode.h"
#include "InterchangeSourceData.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "PhysicsEngine/PhysicsAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangePhysicsAssetFactory)

UClass* UInterchangePhysicsAssetFactory::GetFactoryClass() const
{
	return UPhysicsAsset::StaticClass();
}

UInterchangeFactoryBase::FImportAssetResult UInterchangePhysicsAssetFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangePhysicsAssetFactory::BeginImportAsset_GameThread);
	FImportAssetResult ImportAssetResult;
	UObject* PhysicsAsset = nullptr;

#if WITH_EDITORONLY_DATA
	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	UInterchangePhysicsAssetFactoryNode* PhysicsAssetNode = Cast<UInterchangePhysicsAssetFactoryNode>(Arguments.AssetNode);
	if (PhysicsAssetNode == nullptr)
	{
		return ImportAssetResult;
	}

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (PhysicsAssetNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}
	}

	// create a new PhysicsAsset or overwrite existing asset, if possible
	if (!ExistingAsset)
	{
		PhysicsAsset = NewObject<UObject>(Arguments.Parent, UPhysicsAsset::StaticClass(), *Arguments.AssetName, RF_Public | RF_Standalone);
	}
	else if (ExistingAsset->GetClass()->IsChildOf(UPhysicsAsset::StaticClass()))
	{
		//This is a reimport, we are just re-updating the source data
		PhysicsAsset = ExistingAsset;
	}

	// Premptively set the relevant SkeletalMesh as the PreviewMesh on our creted PhysicsAsset.
	//
	// This is needed because otherwise we'd only have the PreviewSkeletalMesh property correctly filled
	// by the time UInterchangeGenericMeshPipeline::PostImportPhysicsAssetImport() runs, which happens very
	// late in the import process, and even later than the OnAssetDoneNative callbacks.
	if (UPhysicsAsset* TypedPhysicsAsset = Cast<UPhysicsAsset>(PhysicsAsset); TypedPhysicsAsset && Arguments.NodeContainer)
	{
		for (const FString& DependencyUid : PhysicsAssetNode->GetFactoryDependencies())
		{
			const UInterchangeSkeletalMeshFactoryNode* SkeletalMeshFactoryNode = Cast<UInterchangeSkeletalMeshFactoryNode>(
				Arguments.NodeContainer->GetFactoryNode(DependencyUid)
			);
			if (!SkeletalMeshFactoryNode)
			{
				continue;
			}

			FSoftObjectPath SkeletalMeshReference;
			if (!SkeletalMeshFactoryNode->GetCustomReferenceObject(SkeletalMeshReference))
			{
				continue;
			}

			if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(SkeletalMeshReference.TryLoad()))
			{
				const bool bMarkAsDirty = false;
				TypedPhysicsAsset->SetPreviewMesh(SkeletalMesh, bMarkAsDirty);
				break;
			}
		}
	}

	if (!PhysicsAsset)
	{
		UE_LOGF(LogInterchangeImport, Warning, "Could not create PhysicsAsset asset %ls", *Arguments.AssetName);
		return ImportAssetResult;
	}
	PhysicsAssetNode->SetCustomReferenceObject(FSoftObjectPath(PhysicsAsset));

	PhysicsAsset->PreEditChange(nullptr);
#endif //WITH_EDITORONLY_DATA

	ImportAssetResult.ImportedObject = PhysicsAsset;
	return ImportAssetResult;
}

UInterchangeFactoryBase::FImportAssetResult UInterchangePhysicsAssetFactory::ImportAsset_Async(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangePhysicsAssetFactory::ImportAsset_Async);
	FImportAssetResult ImportAssetResult;
#if WITH_EDITORONLY_DATA
	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	UInterchangePhysicsAssetFactoryNode* PhysicsAssetNode = Cast<UInterchangePhysicsAssetFactoryNode>(Arguments.AssetNode);
	if (PhysicsAssetNode == nullptr)
	{
		return ImportAssetResult;
	}

	UObject* PhysicsAssetObject = UE::Interchange::FFactoryCommon::AsyncFindObject(PhysicsAssetNode, GetFactoryClass(), Arguments.Parent, Arguments.AssetName);

	if (!PhysicsAssetObject)
	{
		UE_LOGF(LogInterchangeImport, Error, "Could not import the PhysicsAsset asset %ls because the asset does not exist.", *Arguments.AssetName);
		return ImportAssetResult;
	}

	UPhysicsAsset* PhysicsAsset = Cast<UPhysicsAsset>(PhysicsAssetObject);
	if (!ensure(PhysicsAsset))
	{
		UE_LOGF(LogInterchangeImport, Error, "Could not cast to PhysicsAsset asset %ls.", *Arguments.AssetName);
		return ImportAssetResult;
	}

	//Currently PhysicsAsset re-import will not touch the PhysicsAsset at all
	//TODO design a re-import process for the PhysicsAsset
	if(!Arguments.ReimportObject)
	{
		PhysicsAssetNode->SetCustomReferenceObject(FSoftObjectPath(PhysicsAsset));
	}
		
	//Getting the file Hash will cache it into the source data
	Arguments.SourceData->GetFileContentHash();

	ImportAssetResult.ImportedObject = PhysicsAssetObject;
#endif
	return ImportAssetResult;
}

