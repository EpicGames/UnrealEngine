// Copyright Epic Games, Inc. All Rights Reserved. 
#include "Mesh/InterchangeSkeletalMeshFactory.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/Async.h"
#include "Async/ParallelFor.h"
#include "BoneWeights.h"
#include "ClothingAsset.h"
#include "Components.h"
#include "CoreGlobals.h"
#include "Engine/NaniteAssemblyData.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Engine/SkinnedAssetAsyncCompileUtils.h"
#include "Engine/SkinnedAssetCommon.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "GPUSkinPublicDefs.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeCommonPipelineDataFactoryNode.h"
#include "InterchangeImportCommon.h"
#include "InterchangeImportLog.h"
#include "InterchangeManager.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeMeshDefinitions.h"
#include "InterchangeMeshNode.h"
#include "InterchangeMeshUtilities.h"
#include "InterchangeSceneComponentNodes.h"
#include "InterchangeJointNode.h"
#include "InterchangeSkeletalMeshLodDataNode.h"
#include "InterchangeSkeletalMeshFactoryNode.h"
#include "InterchangeSkeletonFactoryNode.h"
#include "InterchangeSkeletonHelper.h"
#include "InterchangeSourceData.h"
#include "InterchangeTranslatorBase.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Mesh/InterchangeMeshHelper.h"
#include "Mesh/InterchangeMeshPayload.h"
#include "Mesh/InterchangeMeshPayloadInterface.h"
#include "Misc/App.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeSourceNode.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "SkeletalMeshAttributes.h"
#include "SkeletalMeshOperations.h"
#include "SkinnedAssetCompiler.h"

#if WITH_EDITOR
#include "LODUtilities.h"
#include "SkinWeightsUtilities.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSkeletalMeshFactory)

#if WITH_EDITORONLY_DATA

#include "EditorFramework/AssetImportData.h"

#endif //WITH_EDITORONLY_DATA

FScopedSkeletalMeshReimportUtility::FScopedSkeletalMeshReimportUtility(USkeletalMesh* InSkeletalMesh)
	: FSkinnedMeshComponentRecreateRenderStateContext(InSkeletalMesh)
{
	InSkeletalMesh->ReleaseResources();

#if WITH_EDITOR
	// Scope PostEditChange before locking the skeletal mesh's properties
	ScopedPostEditChange = MakeUnique<FScopedSkeletalMeshPostEditChange>(InSkeletalMesh);
	// Lock the skeletal mesh's properties to prevent any async access
	LockPropertiesEvent = InSkeletalMesh->LockPropertiesUntil();
#endif
}
FScopedSkeletalMeshReimportUtility::~FScopedSkeletalMeshReimportUtility()
{
#if WITH_EDITOR
	// Unlock the skeletal mesh's properties first
	if (ensure(LockPropertiesEvent))
	{
		LockPropertiesEvent->Trigger();
	}
	LockPropertiesEvent = nullptr;
	// Now apply PostEditChange to update skeletal mesh's resources
	ScopedPostEditChange.Reset();
#endif
}

#if WITH_EDITOR
namespace UE
{
	namespace Interchange
	{
		FInterchangeMeshPayLoadKey FMeshNodeContext::GetTranslatorAndTransformPayloadKey() const
		{
			FInterchangeMeshPayLoadKey GlobalPayloadKey = TranslatorPayloadKey;
			GlobalPayloadKey.UniqueId = GetUniqueId();
			return GlobalPayloadKey;
		}
		FInterchangeMeshPayLoadKey FMeshNodeContext::GetMorphTargetAndTransformPayloadKey(const FInterchangeMeshPayLoadKey& MorphTargetKey) const
		{
			FInterchangeMeshPayLoadKey GlobalPayloadKey = MorphTargetKey;
			GlobalPayloadKey.UniqueId += FInterchangeMeshPayLoadKey::GetTransformString(SceneGlobalTransform);
			return GlobalPayloadKey;
		}

		//Return the translator key merge with the transform
		FString FMeshNodeContext::GetUniqueId() const
		{
			FString UniqueId = TranslatorPayloadKey.UniqueId;
			UniqueId += FInterchangeMeshPayLoadKey::GetTransformString(SceneGlobalTransform);
			return UniqueId;
		}

		namespace Private
		{
			void FindAllSockets(const UInterchangeBaseNodeContainer* NodeContainer, const UInterchangeBaseNode* BaseNode, TMap<FString, TSet<const UInterchangeSceneNode*>>& OutSocketsPerBoneMap)
			{
				const TArray<FString> ChildrenIds = NodeContainer->GetNodeChildrenUids(BaseNode->GetUniqueID());
				for (int32 ChildIndex = 0; ChildIndex < ChildrenIds.Num(); ++ChildIndex)
				{
					const FString& ChildUid = ChildrenIds[ChildIndex];
					if (const UInterchangeSceneNode* ChildNode = Cast<UInterchangeSceneNode>(NodeContainer->GetNode(ChildUid)))
					{
						if (FSkeletonHelper::IsValidSocket(NodeContainer, ChildNode))
						{
							//We found a socket add it to the map
							TSet<const UInterchangeSceneNode*>& Sockets = OutSocketsPerBoneMap.FindOrAdd(BaseNode->GetDisplayLabel());
							Sockets.Add(ChildNode);
							//Socket cannot have children
							continue;
						}
						FindAllSockets(NodeContainer, ChildNode, OutSocketsPerBoneMap);
					}
				}
			}
			
			void FillMorphTargetMeshDescriptionsPerMorphTargetName(const FMeshNodeContext& MeshNodeContext
																 , TMap<FString, TOptional<UE::Interchange::FMeshPayloadData>>& MorphTargetMeshDescriptionsPerMorphTargetName
																 , UInterchangeSkeletalMeshFactory::FLodPayloads& LodPayloads
																 , const int32 VertexOffset
																 , const UInterchangeBaseNodeContainer* NodeContainer
																 , FString AssetName)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FillMorphTargetMeshDescriptionsPerMorphTargetName)
				TArray<FString> MorphTargetUids;
				MeshNodeContext.MeshNode->GetMorphTargetDependencies(MorphTargetUids);
				TMap<FString, TOptional<UE::Interchange::FMeshPayloadData>> TempMorphTargetMeshDescriptionsPerMorphTargetName;
				TempMorphTargetMeshDescriptionsPerMorphTargetName.Reserve(MorphTargetUids.Num());
				for (const FString& MorphTargetUid : MorphTargetUids)
				{
					if (const UInterchangeMeshNode* MorphTargetMeshNode = Cast<UInterchangeMeshNode>(NodeContainer->GetNode(MorphTargetUid)))
					{
						const TOptional<FInterchangeMeshPayLoadKey>& OptionalPayLoadKey = MorphTargetMeshNode->GetPayLoadKey();
						if (!OptionalPayLoadKey.IsSet())
						{
							UE_LOGF(LogInterchangeImport, Warning, "Empty LOD morph target mesh reference payload when importing SkeletalMesh asset %ls.", *AssetName);
							continue;
						}
						const FInterchangeMeshPayLoadKey& PayLoadKey = OptionalPayLoadKey.GetValue();
						const FInterchangeMeshPayLoadKey& GlobalMorphPayLoadKey = MeshNodeContext.GetMorphTargetAndTransformPayloadKey(PayLoadKey);
						//Add the map entry key, the translator will be call after to bulk get all the needed payload
						TempMorphTargetMeshDescriptionsPerMorphTargetName.Add(PayLoadKey.UniqueId, LodPayloads.MorphPayloadPerKey.FindChecked(GlobalMorphPayLoadKey));
					}
				}

				for (const FString& MorphTargetUid : MorphTargetUids)
				{
					if (const UInterchangeMeshNode* MorphTargetMeshNode = Cast<UInterchangeMeshNode>(NodeContainer->GetNode(MorphTargetUid)))
					{
						
						const TOptional<FInterchangeMeshPayLoadKey>& OptionalPayLoadKey = MorphTargetMeshNode->GetPayLoadKey();
						if (!OptionalPayLoadKey.IsSet())
						{
							continue;
						}
						const FInterchangeMeshPayLoadKey& PayLoadKey = OptionalPayLoadKey.GetValue();

						const FString& MorphTargetPayloadKeyString = PayLoadKey.UniqueId;
						if (!ensure(TempMorphTargetMeshDescriptionsPerMorphTargetName.Contains(MorphTargetPayloadKeyString)))
						{
							continue;
						}

						TOptional<UE::Interchange::FMeshPayloadData>& MorphTargetMeshPayload = TempMorphTargetMeshDescriptionsPerMorphTargetName.FindChecked(MorphTargetPayloadKeyString);
						if (!MorphTargetMeshPayload.IsSet())
						{
							UE_LOGF(LogInterchangeImport, Warning, "Invalid skeletal mesh morph target payload key [%ls] for SkeletalMesh asset %ls.", *MorphTargetPayloadKeyString, *AssetName);
							continue;
						}
						MorphTargetMeshPayload->VertexOffset = VertexOffset;

						if (!MorphTargetMeshNode->GetMorphTargetName(MorphTargetMeshPayload->MorphTargetName))
						{
							MorphTargetMeshPayload->MorphTargetName = MorphTargetPayloadKeyString;
						}
						//Add the morph target to the morph target map
						MorphTargetMeshDescriptionsPerMorphTargetName.Add(MorphTargetPayloadKeyString, MorphTargetMeshPayload); //Note: this is adding a copy of MorphTargetMeshPayload to the TMap.
					}
				}
			}

			const UInterchangeJointNode* RecursiveFindJointByName(const UInterchangeBaseNodeContainer* NodeContainer, const FString& JointNodeId, const FString& JointName)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RecursiveFindJointByName)
				if (const UInterchangeJointNode* JointNode = Cast<UInterchangeJointNode>(NodeContainer->GetNode(JointNodeId)))
				{
					if (JointNode->GetDisplayLabel().Equals(JointName))
					{
						return JointNode;
					}
				}
				TArray<FString> NodeChildrenUids = NodeContainer->GetNodeChildrenUids(JointNodeId);
				for (int32 ChildIndex = 0; ChildIndex < NodeChildrenUids.Num(); ++ChildIndex)
				{
					if (const UInterchangeJointNode* JointNode = RecursiveFindJointByName(NodeContainer, NodeChildrenUids[ChildIndex], JointName))
					{
						return JointNode;
					}
				}

				return nullptr;
			}

			//We assume normalize weight method in this bind pose conversion
			void SkinVertexPositionToTimeZero(UE::Interchange::FMeshPayloadData& LodMeshPayload
				, const UInterchangeBaseNodeContainer* NodeContainer
				, const FString& RootJointNodeId
				, const UInterchangeMeshNode* MeshNode
				, const FTransform GeometricTransform
				, const FTransform& SceneNodeTransform)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(SkinVertexPositionToTimeZero)
				FMeshDescription& MeshDescription = LodMeshPayload.MeshDescription;
				const int32 VertexCount = MeshDescription.Vertices().Num();
				const TArray<FString>& JointNames = LodMeshPayload.JointNames;
				// Create a copy of the vertex array to receive vertex deformations.
				TArray<FVector3f> DestinationVertexPositions;
				DestinationVertexPositions.AddZeroed(VertexCount);

				FSkeletalMeshAttributes Attributes(MeshDescription);
				TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
				FSkinWeightsVertexAttributesRef VertexSkinWeights = Attributes.GetVertexSkinWeights();
				TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
				TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
				TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
				TVertexInstanceAttributesConstRef<FVertexID> VertexInstanceVertexIndices = Attributes.GetVertexInstanceVertexIndices();
				

				for (FVertexID VertexID : MeshDescription.Vertices().GetElementIDs())
				{
					//We can use GetValue because the Meshdescription was compacted before the copy
					DestinationVertexPositions[VertexID.GetValue()] = VertexPositions[VertexID];
				}

				// Deform the vertex array with the links contained in the mesh.
				TArray<FMatrix> SkinDeformations;
				SkinDeformations.AddZeroed(VertexCount);

				TArray<double> SkinWeights;
				SkinWeights.AddZeroed(VertexCount);

				FMatrix SceneNodeMatrixInverse = SceneNodeTransform.ToInverseMatrixWithScale();
				FMatrix SceneNodeMatrixInverseWithoutGlobalOffset = SceneNodeMatrixInverse;

				if (UInterchangeCommonPipelineDataFactoryNode* CommonPipelineDataFactoryNode = UInterchangeCommonPipelineDataFactoryNode::GetUniqueInstance(NodeContainer))
				{
					FTransform GlobalOffsetTransform = FTransform::Identity;
					if (CommonPipelineDataFactoryNode->GetCustomGlobalOffsetTransform(GlobalOffsetTransform))
					{
						bool bBakeMeshes = false;
						if (CommonPipelineDataFactoryNode->GetBakeMeshes(bBakeMeshes) && bBakeMeshes)
						{
							SceneNodeMatrixInverseWithoutGlobalOffset = (SceneNodeTransform * GlobalOffsetTransform.Inverse()).ToInverseMatrixWithScale();
						}
					}
				}

				const int32 JointCount = JointNames.Num();
				for (int32 JointIndex = 0; JointIndex < JointCount; ++JointIndex)
				{
					const FString& JointName = JointNames[JointIndex];

					const UInterchangeJointNode* JointNode = RecursiveFindJointByName(NodeContainer, RootJointNodeId, JointName);
					if (!JointNode)
					{
						continue;
					}

					FMatrix JointBindPoseMatrix = FMatrix::Identity;
					FMatrix JointT0Matrix = FMatrix::Identity;
					FMatrix JointReferenceMatrix = FMatrix::Identity;

					//Scope any transform variable to be sure they are not use to create the vertex transformation matrix
					{
						FString AttributeKey = TEXT("JointBindPosePerMesh_") + MeshNode->GetUniqueID();
						if (!JointNode->GetAttribute<FMatrix>(AttributeKey, JointBindPoseMatrix))
						{
							FTransform JointBindPose;
							if (!ensure(JointNode->GetBindPoseGlobalTransform(NodeContainer, FTransform::Identity, JointBindPose)))
							{
								//BindPose will fall back on LocalTransform in case its not present.
								//If neither is present: No Value to convert from, skip this joint.
								continue;
							}
							JointBindPoseMatrix = JointBindPose.ToMatrixWithScale();
						}
					
						if (!JointNode->GetGlobalMatrixForT0Rebinding(JointT0Matrix))
						{
							FTransform JointT0;
							if (!JointNode->GetTimeZeroGlobalTransform(NodeContainer, FTransform::Identity, JointT0))
							{
								//If there is no time zero global transform we cannot set the bind pose to time zero.
								//We must skip this joint.
								continue;
							}
							JointT0Matrix = JointT0.ToMatrixWithScale();
						}

						if (!JointNode->GetGlobalBindPoseReferenceFromMeshUID(MeshNode->GetUniqueID(), JointReferenceMatrix))
						{
							//Skip this joint
							continue;
						}

						//We must add the geometric node transform
						JointReferenceMatrix = GeometricTransform.ToMatrixWithScale() * JointReferenceMatrix;
					}

					FMatrix VertexTransformMatrix;
					{
						FMatrix ReferenceMatrixRelativeToBindPose = JointReferenceMatrix * JointBindPoseMatrix.Inverse();
						FMatrix JointTimeZeroRelativeToMeshNodeMatrix = JointT0Matrix * SceneNodeMatrixInverseWithoutGlobalOffset;
						//Compute the transform to apply to vertex to change the bind pose from the existing one to use time zero pose instead
						VertexTransformMatrix = SceneNodeMatrixInverse * (ReferenceMatrixRelativeToBindPose * JointTimeZeroRelativeToMeshNodeMatrix);
					}

					//Iterate all bone vertices
					for (FVertexID VertexID : MeshDescription.Vertices().GetElementIDs())
					{
						const int32 VertexIndex = VertexID.GetValue();
						const FVertexBoneWeights BoneWeights = VertexSkinWeights.Get(VertexID);
						const int32 InfluenceCount = BoneWeights.Num();
						float Weight = 0.0f;
						for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
						{
							FBoneIndexType BoneIndex = BoneWeights[InfluenceIndex].GetBoneIndex();
							if (JointIndex == BoneIndex)
							{
								Weight = BoneWeights[InfluenceIndex].GetWeight();
								break;
							}
						}
						if (FMath::IsNearlyZero(Weight))
						{
							continue;
						}

						//The weight multiply the vertex transform matrix so we can have multiple joint affecting this vertex.
						const FMatrix Influence = VertexTransformMatrix * Weight;
						//Add the weighted result
						SkinDeformations[VertexIndex] += Influence;
						//Add the total weight so we can normalize the result in case the accumulated weight is different then 1
						SkinWeights[VertexIndex] += Weight;
					}
				}

				if (VertexInstanceNormals.IsValid())
				{
					for (FVertexID VertexID : MeshDescription.Vertices().GetElementIDs())
					{
						const int32 VertexIndex = VertexID.GetValue();
						const FVector lSrcVertex = FVector(DestinationVertexPositions[VertexIndex]);
						FVector3f& lDstVertex = DestinationVertexPositions[VertexIndex];
						double Weight = SkinWeights[VertexIndex];

						// Deform the vertex if there was at least a link with an influence on the vertex,
						if (!FMath::IsNearlyZero(Weight))
						{
							//Apply skinning of all joints
							lDstVertex = FVector4f(SkinDeformations[VertexIndex].TransformPosition(lSrcVertex));
							//Normalized, in case the weight is different then 1
							lDstVertex /= Weight;
							//Set the new vertex position in the mesh description
							VertexPositions[VertexID] = FVector3f(FVector4f(SceneNodeTransform.TransformFVector4(FVector(lDstVertex))));

							//Vertex Instance Normals:
							TArrayView<const FVertexInstanceID> VertexInstanceIDs = MeshDescription.GetVertexVertexInstanceIDs(VertexID);
							for (const FVertexInstanceID VertexInstanceID : VertexInstanceIDs)
							{
								FVector3f& Normal = VertexInstanceNormals[VertexInstanceID];
								FVector3f OriginalNormal = Normal;

								if (Normal.IsZero())
								{
									continue;
								}

								FVector NormalTransformed = SkinDeformations[VertexIndex].TransformVector(FVector(Normal));
								NormalTransformed /= Weight;
								Normal = FVector4f(SceneNodeTransform.TransformFVector4(NormalTransformed)).GetSafeNormal();

								if (!VertexInstanceTangents.IsValid())
								{
									continue;
								}

								FVector3f& Tangent = VertexInstanceTangents[VertexInstanceID];
								
								if (Tangent.IsZero())
								{
									continue;
								}

								//Calculate transformed Tangent:
								FVector3f OriginalTangent = Tangent;
								FVector TangentTransformed = SkinDeformations[VertexIndex].TransformVector(FVector(Tangent));
								TangentTransformed /= Weight;
								Tangent = FVector4f(SceneNodeTransform.TransformFVector4(TangentTransformed)).GetSafeNormal();

								if (!VertexInstanceBinormalSigns.IsValid())
								{
									continue;
								}

								float& BinormalSign = VertexInstanceBinormalSigns[VertexInstanceID];

								//Calculate original binormal vector to correct handidness after transform applications:
								FVector3f OriginalBinormal = FVector3f::CrossProduct(OriginalNormal, OriginalTangent) * BinormalSign;

								//Calclate transformed binormal vector:
								FVector BinormalTransformed = SkinDeformations[VertexIndex].TransformVector(FVector(OriginalBinormal));
								BinormalTransformed /= Weight;
								BinormalTransformed = SceneNodeTransform.TransformFVector4(BinormalTransformed).GetSafeNormal();

								//Get Candidate Binormal from transformed Tangent and Normal:
								FVector3f CandidateBinormal = FVector3f::CrossProduct(Normal, Tangent).GetSafeNormal();

								//Get the handidness corrected:
								// As in: Check if BinormalTransformed and the CandidateBinormal = (NormalTransformed x TangentTransformed) are aligned, if yes +1, if no -1.
								BinormalSign = (FVector3f::DotProduct(CandidateBinormal, FVector3f(BinormalTransformed)) >= 0.0f) ? +1.0f : -1.0f;
							}
						}
					}
				}
				else
				{
					for (FVertexID VertexID : MeshDescription.Vertices().GetElementIDs())
					{
						const int32 VertexIndex = VertexID.GetValue();
						const FVector lSrcVertex = FVector(DestinationVertexPositions[VertexIndex]);
						FVector3f& lDstVertex = DestinationVertexPositions[VertexIndex];
						double Weight = SkinWeights[VertexIndex];

						// Deform the vertex if there was at least a link with an influence on the vertex,
						if (!FMath::IsNearlyZero(Weight))
						{
							//Apply skinning of all joints
							lDstVertex = FVector4f(SkinDeformations[VertexIndex].TransformPosition(lSrcVertex));
							//Normalized, in case the weight is different then 1
							lDstVertex /= Weight;
							//Set the new vertex position in the mesh description
							VertexPositions[VertexID] = FVector3f(FVector4f(SceneNodeTransform.TransformFVector4(FVector(lDstVertex))));
						}
					}
				}
			}

			void RetrieveAllSkeletalMeshPayloads(const UInterchangeSkeletalMeshFactoryNode* SkeletalMeshFactoryNode
													, UInterchangeSkeletalMeshFactory::FImportAssetObjectLODData& ImportAssetObjectLODData
													, UInterchangeSkeletalMeshFactory::FLodPayloads& LodPayloads
													, const UInterchangeSkeletalMeshFactory::FImportAssetObjectParams& Arguments
													, const UInterchangeBaseNodeContainer* NodeContainer
													, const FString& RootJointNodeId
													, USkeletalMesh* SkeletalMesh
													, const int32 CurrentLodIndex
													, FMeshDescription& DestinationMeshDescription)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RetrieveAllSkeletalMeshPayloads)
				TArray<FMeshNodeContext>& MeshReferences = ImportAssetObjectLODData.MeshNodeContexts;
				const TArray<SkeletalMeshImportData::FBone>& RefBonesBinary = ImportAssetObjectLODData.RefBonesBinary;
				const bool bSkinControlPointToTimeZero = ImportAssetObjectLODData.bUseTimeZeroAsBindPose && ImportAssetObjectLODData.bDiffPose;

				FSkeletalMeshAttributes DestinationMeshAttributes(DestinationMeshDescription);
				DestinationMeshAttributes.Register();

				DestinationMeshAttributes.RegisterSourceGeometryPartsAttributes();
				FSkeletalMeshAttributes::FSourceGeometryPartNameRef NameAttribute = DestinationMeshAttributes.GetSourceGeometryPartNames();
				FSkeletalMeshAttributes::FSourceGeometryPartVertexOffsetAndCountRef VertexAndCountAttribute = DestinationMeshAttributes.GetSourceGeometryPartVertexOffsetAndCounts();
				DestinationMeshAttributes.SourceGeometryParts().Reserve(MeshReferences.Num());

				FStaticMeshOperations::FAppendSettings AppendSettings;
				for (int32 ChannelIdx = 0; ChannelIdx < FStaticMeshOperations::FAppendSettings::MAX_NUM_UV_CHANNELS; ++ChannelIdx)
				{
					AppendSettings.bMergeUVChannels[ChannelIdx] = true;
				}
				AppendSettings.bMergeObjectName = true;
				
				bool bKeepSectionsSeparate = false;
				SkeletalMeshFactoryNode->GetCustomKeepSectionsSeparate(bKeepSectionsSeparate);

				bool bImportVertexAttributes = false;
				SkeletalMeshFactoryNode->GetCustomImportVertexAttributes(bImportVertexAttributes);

				bool bImportMorphTarget = true;
				SkeletalMeshFactoryNode->GetCustomImportMorphTarget(bImportMorphTarget);

				TMap<const FMeshNodeContext*, TOptional<UE::Interchange::FMeshPayloadData>> LodMeshPayloadPerTranslatorPayloadKey;
				LodMeshPayloadPerTranslatorPayloadKey.Reserve(MeshReferences.Num());

				TMap<FString, TOptional<UE::Interchange::FMeshPayloadData>> MorphTargetMeshDescriptionsPerMorphTargetName;
				int32 MorphTargetCount = 0;

				struct FInternalInstanceData
				{
					bool ScaleGreaterThenOne = false;
					int32 Count = 0;
					bool ShouldFetchWithTransform() const
					{
						return Count == 1 || ScaleGreaterThenOne;
					}
				};
				TMap<FString, FInternalInstanceData> MeshInstancesDatas;
				for (const FMeshNodeContext& MeshNodeContext : MeshReferences)
				{
					FInternalInstanceData& InstanceData = MeshInstancesDatas.FindOrAdd(MeshNodeContext.TranslatorPayloadKey.UniqueId);
					InstanceData.Count++;
					InstanceData.ScaleGreaterThenOne |= MeshNodeContext.SceneGlobalTransform.GetScale3D().GetAbs().GetMax() > 1.0;
				}

				for (const FMeshNodeContext& MeshNodeContext : MeshReferences)
				{
					//Add the payload entry key, the payload data will be fill later in bulk by the translator
					LodMeshPayloadPerTranslatorPayloadKey.Add(&MeshNodeContext, LodPayloads.MeshPayloadPerKey.FindChecked(MeshNodeContext.GetTranslatorAndTransformPayloadKey()));
					//Count the morph target dependencies so we can reserve the right amount
					MorphTargetCount += (bImportMorphTarget && MeshNodeContext.MeshNode) ? MeshNodeContext.MeshNode->GetMorphTargetDependeciesCount() : 0;
				}
				MorphTargetMeshDescriptionsPerMorphTargetName.Reserve(MorphTargetCount);

				//Fill the lod mesh description using all combined mesh part
				for(TPair<const FMeshNodeContext*, TOptional<UE::Interchange::FMeshPayloadData>>& MeshNodeContextAndFuture: LodMeshPayloadPerTranslatorPayloadKey)
				{
					if (!MeshNodeContextAndFuture.Key)
					{
						continue;
					}
					const FMeshNodeContext& MeshNodeContext = *MeshNodeContextAndFuture.Key;
					TRACE_CPUPROFILER_EVENT_SCOPE(RetrieveAllSkeletalMeshPayloadsAndFillImportData::GetPayload)
					TOptional<UE::Interchange::FMeshPayloadData> LodMeshPayload = MeshNodeContextAndFuture.Value;
					if (!LodMeshPayload.IsSet())
					{
						UE_LOGF(LogInterchangeImport, Warning, "Invalid skeletal mesh payload key [%ls] for SkeletalMesh asset %ls.", *MeshNodeContext.TranslatorPayloadKey.UniqueId, *Arguments.AssetName);
						continue;
					}
					
					const int32 VertexOffset = DestinationMeshDescription.Vertices().Num();

					const bool bIsRiggedMesh = LodMeshPayload->JointNames.Num() > 0;
					if (MeshNodeContext.MeshNode && bIsRiggedMesh)
					{
						FSourceGeometryPartID PartID = DestinationMeshAttributes.CreateSourceGeometryPart();

						NameAttribute.Set(PartID, FName(MeshNodeContext.SceneNodeName));
						VertexAndCountAttribute.Set(PartID, { DestinationMeshDescription.Vertices().Num(), LodMeshPayload->MeshDescription.Vertices().Num()});
					}

					FString AssetName = SkeletalMeshFactoryNode->GetAssetName();

					FSkeletalMeshOperations::FSkeletalMeshAppendSettings SkeletalMeshAppendSettings;
					SkeletalMeshAppendSettings.bAppendVertexAttributes = bImportVertexAttributes;
					SkeletalMeshAppendSettings.SourceVertexIDOffset = VertexOffset;
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(RetrieveAllSkeletalMeshPayloadsAndFillImportData::CompactPayload)
						FElementIDRemappings ElementIDRemappings;
						LodMeshPayload->MeshDescription.Compact(ElementIDRemappings);
					}

					if (bSkinControlPointToTimeZero && bIsRiggedMesh)
					{
						//We need to rebind the mesh at time 0. Skeleton joint have the time zero transform, so we need to apply the skinning to the mesh
						//With the skeleton transform at time zero
						SkinVertexPositionToTimeZero(LodMeshPayload.GetValue(), NodeContainer, RootJointNodeId, MeshNodeContext.MeshNode, MeshNodeContext.GeometricTransform, MeshNodeContext.SceneGlobalTransform);
					}

					const int32 RefBoneCount = RefBonesBinary.Num();
					
					//Remap the influence vertex index to point on the correct index
					if (LodMeshPayload->JointNames.Num() > 0)
					{
						const int32 LocalJointCount = LodMeshPayload->JointNames.Num();
						
						SkeletalMeshAppendSettings.SourceRemapBoneIndex.AddZeroed(LocalJointCount);
						for (int32 LocalJointIndex = 0; LocalJointIndex < LocalJointCount; ++LocalJointIndex)
						{
							SkeletalMeshAppendSettings.SourceRemapBoneIndex[LocalJointIndex] = LocalJointIndex;
							const FString& LocalJointName = LodMeshPayload->JointNames[LocalJointIndex];
							for (int32 RefBoneIndex = 0; RefBoneIndex < RefBoneCount; ++RefBoneIndex)
							{
								const SkeletalMeshImportData::FBone& Bone = RefBonesBinary[RefBoneIndex];
								if (Bone.Name.Equals(LocalJointName))
								{
									SkeletalMeshAppendSettings.SourceRemapBoneIndex[LocalJointIndex] = RefBoneIndex;
									break;
								}
							}
						}
					}
					else if(!bIsRiggedMesh)
					{
						// We have a rigid mesh instance (a scene node point on the mesh, the scene node will be the bone on which the rigid mesh is skin).
						// We must add skinning to the mesh description on bone 0 and remap it to the correct RefBonesBinary in the append settings
						const FString ToSkinBoneName = MeshNodeContext.SceneNodeName;
						for (int32 RefBoneIndex = 0; RefBoneIndex < RefBoneCount; ++RefBoneIndex)
						{
							const SkeletalMeshImportData::FBone& Bone = RefBonesBinary[RefBoneIndex];
							if (Bone.Name.Equals(ToSkinBoneName))
							{
								SkeletalMeshAppendSettings.SourceRemapBoneIndex.AddZeroed_GetRef() = RefBoneIndex;
								break;
							}
						}
						//Add the skinning in the mesh description
						{
							FSkeletalMeshAttributes PayloadSkeletalMeshAttributes(LodMeshPayload->MeshDescription);
							constexpr bool bKeepExistingAttribute = true;
							PayloadSkeletalMeshAttributes.Register(bKeepExistingAttribute);
							using namespace UE::AnimationCore;
							TArray<FBoneWeight> BoneWeights;
							FBoneWeight& BoneWeight = BoneWeights.AddDefaulted_GetRef();
							BoneWeight.SetBoneIndex(0);
							BoneWeight.SetWeight(1.0f);
							FSkinWeightsVertexAttributesRef PayloadVertexSkinWeights = PayloadSkeletalMeshAttributes.GetVertexSkinWeights();
							for (const FVertexID& PayloadVertexID : LodMeshPayload->MeshDescription.Vertices().GetElementIDs())
							{
								PayloadVertexSkinWeights.Set(PayloadVertexID, BoneWeights);
							}
						}
					}
					
					//The Mesh node parent bake transform can be pass to the payload request or not, it depend on the count of instance and the scale of the transform.
					const FInternalInstanceData& InstanceData = MeshInstancesDatas.FindChecked(MeshNodeContext.TranslatorPayloadKey.UniqueId);
					AppendSettings.MeshTransform = InstanceData.ShouldFetchWithTransform() ? FTransform::Identity : MeshNodeContext.SceneGlobalTransform;
					if (bKeepSectionsSeparate)
					{
						AppendSettings.PolygonGroupsDelegate = FAppendPolygonGroupsDelegate::CreateLambda([](const FMeshDescription& SourceMesh, FMeshDescription& TargetMesh, PolygonGroupMap& RemapPolygonGroup)
							{
								UE::Interchange::Private::MeshHelper::RemapPolygonGroups(SourceMesh, TargetMesh, RemapPolygonGroup);
							});
					}

					FStaticMeshOperations::AppendMeshDescription(LodMeshPayload->MeshDescription, DestinationMeshDescription, AppendSettings);
					if (MeshNodeContext.MeshNode->IsSkinnedMesh() || !bIsRiggedMesh)
					{
						FSkeletalMeshOperations::AppendSkinWeight(LodMeshPayload->MeshDescription, DestinationMeshDescription, SkeletalMeshAppendSettings);
					}
					if (bImportMorphTarget)
					{
						FillMorphTargetMeshDescriptionsPerMorphTargetName(MeshNodeContext
							, MorphTargetMeshDescriptionsPerMorphTargetName
							, LodPayloads
							, VertexOffset
							, Arguments.NodeContainer
							, Arguments.AssetName);
					}
				}

				//The color data is linearized twice by the translator, we need to convert to sRGB to have proper linear
				//TODO: change the translator to put linear instead of linear of linear
				//      remove the ToFColor in StaticMeshBuilder
				//      version the meshdescription properly to force the ToFColor when loading old static meshdescription
				{
					TVertexInstanceAttributesRef<FVector4f> VertexColor = DestinationMeshAttributes.GetVertexInstanceColors();
					for (FVertexInstanceID VertexInstanceID : DestinationMeshDescription.VertexInstances().GetElementIDs())
					{
						const FColor LinearUint8Color = FLinearColor(VertexColor[VertexInstanceID]).ToFColor(true);
						constexpr float ColorScale = (1.0f / 255.0f);
						VertexColor[VertexInstanceID] = FVector4f(static_cast<float>(LinearUint8Color.R) * ColorScale
							, static_cast<float>(LinearUint8Color.G) * ColorScale
							, static_cast<float>(LinearUint8Color.B) * ColorScale
							, static_cast<float>(LinearUint8Color.A) * ColorScale);
					}
				}

				bool bMergeMorphTargetWithSameName = false;
				SkeletalMeshFactoryNode->GetCustomMergeMorphTargetShapeWithSameName(bMergeMorphTargetWithSameName);
				
				//Copy all the lod morph targets data to the DestinationMeshDescription.
				UE::Interchange::Private::MeshHelper::CopyMorphTargetsMeshDescriptionToSkeletalMeshDescription(ImportAssetObjectLODData.SkeletonMorphCurveMetadataNames, MorphTargetMeshDescriptionsPerMorphTargetName, DestinationMeshDescription, bMergeMorphTargetWithSameName);

				TArray<uint32> FaceSmoothingMasks;
				FMeshDescription FixedMeshDescription;
				TArray<uint32> FixedFaceSmoothingMasks;

				FSkeletalMeshOperations::ConvertHardEdgesToSmoothMasks(DestinationMeshDescription, FaceSmoothingMasks); //Create Smooth Masks from original mesh description
				FSkeletalMeshOperations::FixVertexInstanceStructure(DestinationMeshDescription, FixedMeshDescription, FaceSmoothingMasks, FixedFaceSmoothingMasks);
				FString SkeletalMeshPath = SkeletalMesh ? SkeletalMesh->GetOuter()->GetPathName() : TEXT("");
				bool bComputeWeightedNormals = SkeletalMesh ? SkeletalMesh->GetLODInfo(CurrentLodIndex) && SkeletalMesh->GetLODInfo(CurrentLodIndex)->BuildSettings.bComputeWeightedNormals : false;
				FSkeletalMeshOperations::ValidateFixComputeMeshDescriptionData(FixedMeshDescription, FixedFaceSmoothingMasks, CurrentLodIndex, bComputeWeightedNormals, SkeletalMeshPath); //Validates and Fixes MeshDescription including re-instating Smooth masks.

				DestinationMeshDescription = MoveTemp(FixedMeshDescription);
			}

			struct FContentInfo
			{
				bool bApplyGeometry = false;
				bool bApplySkinning = false;
				bool bApplyPartialContent = false;
				bool bApplyGeometryOnly = false;
				bool bApplySkinningOnly = false;
			};

			FContentInfo GetContentInfo(const UInterchangeSkeletalMeshFactoryNode* SkeletalMeshFactoryNode, const bool bIsReImport)
			{
				FContentInfo ContentInfo;
				EInterchangeSkeletalMeshContentType ImportContent = EInterchangeSkeletalMeshContentType::All;
				SkeletalMeshFactoryNode->GetCustomImportContentType(ImportContent);
				ContentInfo.bApplyGeometry = !bIsReImport || (ImportContent == EInterchangeSkeletalMeshContentType::All || ImportContent == EInterchangeSkeletalMeshContentType::Geometry);
				ContentInfo.bApplySkinning = !bIsReImport || (ImportContent == EInterchangeSkeletalMeshContentType::All || ImportContent == EInterchangeSkeletalMeshContentType::SkinningWeights);
				ContentInfo.bApplyPartialContent = bIsReImport && ImportContent != EInterchangeSkeletalMeshContentType::All;
				ContentInfo.bApplyGeometryOnly = ContentInfo.bApplyPartialContent && ContentInfo.bApplyGeometry;
				ContentInfo.bApplySkinningOnly = ContentInfo.bApplyPartialContent && ContentInfo.bApplySkinning;
				return ContentInfo;
			}

			bool BuildMeshReferences(const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments
				, const FString& RootJointNodeId
				, UInterchangeSkeletalMeshFactory::FImportAssetObjectLODData& ImportAssetObjectLODDataRef
				, const UInterchangeSkeletalMeshLodDataNode* LodDataNode
				, bool bImportSockets)
			{
				if (!LodDataNode)
				{
					return false;
				}

				const UInterchangeBaseNode* BaseNode = Arguments.NodeContainer->GetNode(RootJointNodeId);
				if (!BaseNode)
				{
					return false;
				}

				ImportAssetObjectLODDataRef.bDiffPose = false;
				TArray <Private::FJointInfo> JointInfos;
				TArray<FString> BoneNotBindNames;
				TSet<FName> UsedBoneNames;
				Private::FSkeletonHelper::RecursiveAddBones(Arguments.NodeContainer
					, RootJointNodeId
					, JointInfos
					, INDEX_NONE
					, ImportAssetObjectLODDataRef.RefBonesBinary
					, UsedBoneNames
					, ImportAssetObjectLODDataRef.bUseTimeZeroAsBindPose
					, ImportAssetObjectLODDataRef.bDiffPose
					, BoneNotBindNames
					, bImportSockets);

				//Scope to query the mesh node
				{
					TArray<FInterchangeLODMeshData> LODMeshDataArray;
					LodDataNode->GetLODMeshDataArray(LODMeshDataArray);
					ImportAssetObjectLODDataRef.MeshNodeContexts.Reserve(LODMeshDataArray.Num());

					for (const FInterchangeLODMeshData& LODMeshData : LODMeshDataArray)
					{
						FMeshNodeContext MeshReference;
						const UInterchangeBaseNode* MeshBaseNode = Arguments.NodeContainer->GetNode(LODMeshData.MeshUid);
						MeshReference.MeshNode = Cast<UInterchangeMeshNode>(MeshBaseNode);
						MeshReference.SceneGlobalTransform = LODMeshData.Transform;
						MeshReference.GeometricTransform = LODMeshData.GeometricTransform;
						MeshReference.SceneNodeName = LODMeshData.SceneNodeName;

						if (!ensure(MeshReference.MeshNode != nullptr))
						{
							continue;
						}

						TOptional<FInterchangeMeshPayLoadKey> OptionalPayLoadKey = MeshReference.MeshNode->GetPayLoadKey();
						if (OptionalPayLoadKey.IsSet())
						{
							MeshReference.TranslatorPayloadKey = OptionalPayLoadKey.GetValue();
						}
						else
						{
							continue;
						}
						ImportAssetObjectLODDataRef.MeshNodeContexts.Add(MeshReference);
					}
				}
				return true;
			}

			TMap<FString, int32> GetSkeletonBoneIndexByBoneNameTable(const USkeleton& Skeleton)
			{
				TMap<FString, int32> BoneIndexByBoneNameTable;

				const FReferenceSkeleton& ReferenceSkeleton = Skeleton.GetReferenceSkeleton();
				const int32 NumBones = ReferenceSkeleton.GetNum();
				for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
				{
					// Get bone name by index
					const FName BoneName = ReferenceSkeleton.GetBoneName(BoneIndex);
					BoneIndexByBoneNameTable.Add(BoneName.ToString(), BoneIndex);
				}

				return BoneIndexByBoneNameTable;
			}

			bool RemapNaniteAssemblyBoneIndices(
				UE::Interchange::FNaniteAssemblyDescription& InOutDescription
				, const TArray<FString>& JointNames
				, const TMap<FString, int32>& NewBoneIndexByBoneNameMap
			    , const FString& AssetName)
			{
				// Remap the bone influence bone indices to our actual skeleton.  Note that Nanite assembly 
				// bone indices are uint32, while the indices held in the table are int32.

				int32 bNumRemapped = 0;
				for (FNaniteAssemblyDescription::FBoneInfluence& BoneInfluence : InOutDescription.BoneInfluences)
				{
					const uint32 OldBoneIndex = BoneInfluence.BoneIndex;
					if (IntFitsIn<uint32, int32>(OldBoneIndex) && JointNames.IsValidIndex(static_cast<int32>(OldBoneIndex)))
					{
						const FString& OriginalName = JointNames[OldBoneIndex];
						if (const int32* RemappedIndexPtr = NewBoneIndexByBoneNameMap.Find(OriginalName))
						{
							BoneInfluence.BoneIndex = static_cast<uint32>(*RemappedIndexPtr);
							bNumRemapped++;
						}
					}
				}

				if (bNumRemapped != InOutDescription.BoneInfluences.Num())
				{
					UE_LOGF(LogInterchangeImport, Warning
						, "Failed to remap one or more Nanite Assembly Bone Influence's while processing skeletal mesh '%ls'"
						, *AssetName
					);
					return false;
				}

				return true;
			}

		} //Namespace Private
	} //namespace Interchange
} //namespace UE

#endif //#if WITH_EDITOR

UClass* UInterchangeSkeletalMeshFactory::GetFactoryClass() const
{
	return USkeletalMesh::StaticClass();
}

void UInterchangeSkeletalMeshFactory::CreatePayloadTasks(const FImportAssetObjectParams& Arguments, bool bAsync, TArray<TSharedPtr<UE::Interchange::FInterchangeTaskBase>>& PayloadTasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSkeletalMeshFactory::CreateAsset)

#if WITH_EDITOR && WITH_EDITORONLY_DATA
	using namespace UE::Interchange;

	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return;
	}

	UInterchangeSkeletalMeshFactoryNode* SkeletalMeshFactoryNode = Cast<UInterchangeSkeletalMeshFactoryNode>(Arguments.AssetNode);
	if (SkeletalMeshFactoryNode == nullptr)
	{
		return;
	}

	const IInterchangeMeshPayloadInterface* MeshTranslatorPayloadInterface = Cast<IInterchangeMeshPayloadInterface>(Arguments.Translator);
	if (!MeshTranslatorPayloadInterface)
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->Text = FText::Format(NSLOCTEXT("InterchangeSkeletalMeshFactory", "CreatePayloadTasks_TranslatorInterfaceMissing", "Cannot import skeletalMesh {0}, the translator {1} does not implement the IInterchangeSkeletalMeshPayloadInterface.")
			, FText::FromString(Arguments.AssetName)
			, FText::FromString(Arguments.Translator ? Arguments.Translator->GetName() : TEXT("UnknownTranslator")));
		return;
	}

	FTransform GlobalOffsetTransform = FTransform::Identity;
	bool bBakeMeshes = false;
	if (UInterchangeCommonPipelineDataFactoryNode* CommonPipelineDataFactoryNode = UInterchangeCommonPipelineDataFactoryNode::GetUniqueInstance(Arguments.NodeContainer))
	{
		CommonPipelineDataFactoryNode->GetCustomGlobalOffsetTransform(GlobalOffsetTransform);
		CommonPipelineDataFactoryNode->GetBakeMeshes(bBakeMeshes);
	}

	bool bImportMorphTarget = true;
	SkeletalMeshFactoryNode->GetCustomImportMorphTarget(bImportMorphTarget);

	bool bImportSockets = false;
	SkeletalMeshFactoryNode->GetCustomImportSockets(bImportSockets);

	int32 LodCount = SkeletalMeshFactoryNode->GetLodDataCount();
	TArray<FString> LodDataUniqueIds;
	SkeletalMeshFactoryNode->GetLodDataUniqueIds(LodDataUniqueIds);
	ensure(LodDataUniqueIds.Num() == LodCount);
	PayloadsPerLodIndex.Reserve(LodCount);
	int32 CurrentLodIndex = 0;
	for (int32 LodIndex = 0; LodIndex < LodCount; ++LodIndex)
	{
		FString LodUniqueId = LodDataUniqueIds[LodIndex];
		const UInterchangeSkeletalMeshLodDataNode* LodDataNode = Cast<UInterchangeSkeletalMeshLodDataNode>(Arguments.NodeContainer->GetNode(LodUniqueId));
		if (!LodDataNode)
		{
			continue;
		}

		FString SkeletonNodeUid;
		if (!LodDataNode->GetCustomSkeletonUid(SkeletonNodeUid))
		{
			continue;
		}
		const UInterchangeSkeletonFactoryNode* SkeletonNode = Cast<UInterchangeSkeletonFactoryNode>(Arguments.NodeContainer->GetNode(SkeletonNodeUid));
		if (!SkeletonNode)
		{
			continue;
		}

		FString RootJointNodeId;
		if (!SkeletonNode->GetCustomRootJointUid(RootJointNodeId))
		{
			continue;
		}

		FTransform RootJointGlobalTransform;
		const UInterchangeBaseNode* RootJointNode = Arguments.NodeContainer->GetNode(RootJointNodeId);
		if (const UInterchangeSceneNode* RootJointSceneNode = Cast<const UInterchangeSceneNode>(RootJointNode))
		{
			RootJointSceneNode->GetCustomGlobalTransform(Arguments.NodeContainer, GlobalOffsetTransform, RootJointGlobalTransform);
		}

		FLodPayloads& LodPayloads = PayloadsPerLodIndex.FindOrAdd(LodIndex);

		FImportAssetObjectLODData& ImportAssetObjectLODData = ImportAssetObjectData.LodDatas.AddDefaulted_GetRef();
		ImportAssetObjectLODData.LodIndex = CurrentLodIndex;

		SkeletonNode->GetCustomUseTimeZeroForBindPose(ImportAssetObjectLODData.bUseTimeZeroAsBindPose);

		Private::BuildMeshReferences(
			Arguments
			, RootJointNodeId
			, ImportAssetObjectLODData
			, LodDataNode
			, bImportSockets
		);

		struct FInternalInstanceData
		{
			bool ScaleGreaterThenOne = false;
			int32 Count = 0;
			bool ShouldFetchWithTransform() const
			{
				return Count == 1 || ScaleGreaterThenOne;
			}
		};
		TMap<FString, FInternalInstanceData> MeshInstancesDatas;
		for (const FMeshNodeContext& MeshNodeContext : ImportAssetObjectLODData.MeshNodeContexts)
		{
			FInternalInstanceData& InstanceData = MeshInstancesDatas.FindOrAdd(MeshNodeContext.TranslatorPayloadKey.UniqueId);
			InstanceData.Count++;
			InstanceData.ScaleGreaterThenOne |= MeshNodeContext.SceneGlobalTransform.GetScale3D().GetAbs().GetMax() > 1.0;
		}
		
		//Reserve the correct amount since we point in the array for the lambda so array must not be resized at any moments after we create the tasks
		LodPayloads.MeshPayloadPerKey.Reserve(ImportAssetObjectLODData.MeshNodeContexts.Num());
		int32 MorphTargetCount = 0;
		for (const FMeshNodeContext& MeshNodeContext : ImportAssetObjectLODData.MeshNodeContexts)
		{
			//Count the morph target dependencies so we can reserve the right amount
			if (bImportMorphTarget)
			{
				MorphTargetCount += MeshNodeContext.MeshNode->GetMorphTargetDependeciesCount();
			}
		}
		LodPayloads.MorphPayloadPerKey.Reserve(MorphTargetCount);

		UE::Interchange::FAttributeStorage PayloadAttributes;
		UInterchangeMeshFactoryNode::CopyPayloadKeyStorageAttributes(SkeletalMeshFactoryNode, PayloadAttributes);

		PayloadAttributes.RegisterAttribute(UE::Interchange::FAttributeKey{ UE::Interchange::MeshPayload::Attributes::BakeMeshes }, bBakeMeshes);
		PayloadAttributes.RegisterAttribute(UE::Interchange::FAttributeKey{ UE::Interchange::MeshPayload::Attributes::RootJointGlobalTransform }, RootJointGlobalTransform);
		PayloadAttributes.RegisterAttribute(UE::Interchange::FAttributeKey{ UE::Interchange::MeshPayload::Attributes::GlobalOffsetTransform }, GlobalOffsetTransform);

		for (const FMeshNodeContext& MeshNodeContext : ImportAssetObjectLODData.MeshNodeContexts)
		{
			const FInternalInstanceData& InstanceData = MeshInstancesDatas.FindChecked(MeshNodeContext.TranslatorPayloadKey.UniqueId);
			FTransform ApplyTransformWhenFetchPayload = InstanceData.ShouldFetchWithTransform() ? MeshNodeContext.SceneGlobalTransform : FTransform::Identity;
			PayloadAttributes.RegisterAttribute(UE::Interchange::FAttributeKey{ UE::Interchange::MeshPayload::Attributes::MeshGlobalTransform }, ApplyTransformWhenFetchPayload);
			//Create the payload task
			TOptional<FMeshPayloadData>& MeshPayload = LodPayloads.MeshPayloadPerKey.FindOrAdd(MeshNodeContext.GetTranslatorAndTransformPayloadKey());
			TSharedPtr<FInterchangeTaskLambda, ESPMode::ThreadSafe> TaskGetMeshPayload = MakeShared<FInterchangeTaskLambda, ESPMode::ThreadSafe>(bAsync ? EInterchangeTaskThread::AsyncThread : EInterchangeTaskThread::GameThread
				, [&MeshPayload, MeshTranslatorPayloadInterface, PayLoadKey = MeshNodeContext.TranslatorPayloadKey, PayloadAttributes]() mutable
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSkeletalMeshFactory::GetMeshPayloadDataTask);										
					MeshPayload = MeshTranslatorPayloadInterface->GetMeshPayloadData(PayLoadKey, PayloadAttributes);
				});
			PayloadTasks.Add(TaskGetMeshPayload);

			//Count the morph target dependencies so we can reserve the right amount
			if (bImportMorphTarget)
			{
				TArray<FString> MorphTargetUids;
				MeshNodeContext.MeshNode->GetMorphTargetDependencies(MorphTargetUids);
				for (const FString& MorphTargetUid : MorphTargetUids)
				{
					if (const UInterchangeMeshNode* MorphTargetMeshNode = Cast<UInterchangeMeshNode>(Arguments.NodeContainer->GetNode(MorphTargetUid)))
					{
						TOptional<FInterchangeMeshPayLoadKey> OptionalPayLoadKey = MorphTargetMeshNode->GetPayLoadKey();
						if (!OptionalPayLoadKey.IsSet())
						{
							UE_LOGF(LogInterchangeImport, Warning, "Empty LOD morph target mesh reference payload when importing SkeletalMesh asset %ls.", *Arguments.AssetName);
							continue;
						}
						FInterchangeMeshPayLoadKey& MorphPayLoadKey = OptionalPayLoadKey.GetValue();
						FInterchangeMeshPayLoadKey GlobalMorphPayLoadKey = MeshNodeContext.GetMorphTargetAndTransformPayloadKey(MorphPayLoadKey);
						TOptional<FMeshPayloadData>& MorphPayload = LodPayloads.MorphPayloadPerKey.FindOrAdd(GlobalMorphPayLoadKey);						
						TSharedPtr<FInterchangeTaskLambda, ESPMode::ThreadSafe> TaskGetMorphPayload = MakeShared<FInterchangeTaskLambda, ESPMode::ThreadSafe>(bAsync ? EInterchangeTaskThread::AsyncThread : EInterchangeTaskThread::GameThread
							, [&MorphPayload, MeshTranslatorPayloadInterface, MorphPayLoadKey, PayloadAttributes]()
							{
								TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSkeletalMeshFactory::GetMeshMorphTargetPayloadDataTask);
								MorphPayload = MeshTranslatorPayloadInterface->GetMeshPayloadData(MorphPayLoadKey, PayloadAttributes);
							});
						PayloadTasks.Add(TaskGetMorphPayload);
					}
				}
			}
		}
		CurrentLodIndex++;
	}
#endif
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeSkeletalMeshFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSkeletalMeshFactory::BeginImportAssetObject_GameThread)

	FImportAssetResult ImportAssetResult;

#if WITH_EDITOR && WITH_EDITORONLY_DATA
	USkeletalMesh* SkeletalMesh = nullptr;
	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	UInterchangeSkeletalMeshFactoryNode* SkeletalMeshFactoryNode = Cast<UInterchangeSkeletalMeshFactoryNode>(Arguments.AssetNode);
	if (SkeletalMeshFactoryNode == nullptr)
	{
		return ImportAssetResult;
	}

	bool bImportSockets = false;
	SkeletalMeshFactoryNode->GetCustomImportSockets(bImportSockets);

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (SkeletalMeshFactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}
	}

	// create a new material or overwrite existing asset, if possible
	if (!ExistingAsset)
	{
		SkeletalMesh = NewObject<USkeletalMesh>(Arguments.Parent, *Arguments.AssetName, RF_Public | RF_Standalone);
	}
	else
	{
		//This is a reimport or an override, simply cast it to USkeletalMesh, the class has been verified by the caller (UE::Interchange::FTaskImportObject_GameThread::DoTask)
		SkeletalMesh = Cast<USkeletalMesh>(ExistingAsset);
	}
	
	//This should not happen
	if (!ensure(SkeletalMesh))
	{
		if (!Arguments.ReimportObject)
		{
			UE_LOGF(LogInterchangeImport, Warning, "Could not create or find a SkeletalMesh asset named %ls.", *Arguments.AssetName);
		}
		return ImportAssetResult;
	}

	SkeletalMesh->PreEditChange(nullptr);

	//Lock the skeletalmesh properties if the skeletal mesh already exist (re-import)
	if (ExistingAsset)
	{
		ScopedReimportUtility = MakeUnique<FScopedSkeletalMeshReimportUtility>(SkeletalMesh);
	}

	ImportAssetResult.ImportedObject = SkeletalMesh;

	//Make sure we can modify the skeletalmesh properties
	FSkinnedAssetAsyncBuildScope AsyncBuildScope(SkeletalMesh);

	//This is consider has a re-import if we have a reimport object or if the object exist and have some valid LOD
	const bool bIsReImport = (Arguments.ReimportObject != nullptr) || (SkeletalMesh->GetLODNum() > 0);

	//Dirty the DDC Key for any imported Skeletal Mesh
	SkeletalMesh->InvalidateDeriveDataCacheGUID();
	USkeleton* SkeletonReference = nullptr;

	if (bIsReImport)
	{
		//Save all existing source data that are imported only by the editor UI

		int32 ExistingLodCount = SkeletalMesh->GetLODNum();

		//Skin weight profiles, The skin weight alternate data will be extracted when iterating the lods
		ImportAssetObjectData.ExistingSkinWeightProfileInfos = SkeletalMesh->GetSkinWeightProfiles();

		//Unbind clothing and save the data to rebind it later in the post import task
		SkeletalMesh->GetSkinWeightProfiles().Reset();
		for (int32 LodIndex = 0; LodIndex < ExistingLodCount; ++LodIndex)
		{
			FSkeletalMeshLODModel& BuildLODModel = SkeletalMesh->GetImportedModel()->LODModels[LodIndex];
			BuildLODModel.SkinWeightProfiles.Reset();

			//Store the LOD alternate skinning profile data
			if (SkeletalMesh->HasMeshDescription(LodIndex))
			{
				ImportAssetObjectData.ExistingAlternateMeshDescriptionPerLOD.Add(*SkeletalMesh->GetMeshDescription(LodIndex));
			}

			//Cloth
			TArray<ClothingAssetUtils::FClothingAssetMeshBinding> ExistingClothingBindingsLod;
			FLODUtilities::UnbindClothingAndBackup(SkeletalMesh, ExistingClothingBindingsLod, LodIndex);
			ImportAssetObjectData.ExistingClothingBindings.Append(ExistingClothingBindingsLod);
		}
	}

	int32 LodCount = SkeletalMeshFactoryNode->GetLodDataCount();
	TArray<FString> LodDataUniqueIds;
	SkeletalMeshFactoryNode->GetLodDataUniqueIds(LodDataUniqueIds);
	ensure(LodDataUniqueIds.Num() == LodCount);
	int32 CurrentLodIndex = 0;

	UE::Interchange::Private::FContentInfo ContentInfo = UE::Interchange::Private::GetContentInfo(SkeletalMeshFactoryNode, bIsReImport);
	ImportAssetObjectData.bIsReImport = bIsReImport;
	ImportAssetObjectData.bApplyGeometryOnly = ContentInfo.bApplyGeometryOnly;

	for (int32 LodIndex = 0; LodIndex < LodCount; ++LodIndex)
	{
		FText WarningMessage_InvalidSkeleton = FText::Format(NSLOCTEXT("InterchangeSkeletalMeshFactory", "BeginImportAsset_GameThread_InvalidSkeletonLOD", "Invalid Skeleton LOD {0} when importing SkeletalMesh asset {1}.")
			, FText::AsNumber(LodIndex)
			, FText::FromString(Arguments.AssetName));
		FText WarningMessage_InvalidRootJoint = FText::Format(NSLOCTEXT("InterchangeSkeletalMeshFactory", "BeginImportAsset_GameThread_InvalidSkeletonRootJoint", "Invalid Skeleton LOD {0}'s Root Joint when importing SkeletalMesh asset {1}.")
			, FText::AsNumber(LodIndex)
			, FText::FromString(Arguments.AssetName));

		FString LodUniqueId = LodDataUniqueIds[LodIndex];
		const UInterchangeSkeletalMeshLodDataNode* LodDataNode = Cast<UInterchangeSkeletalMeshLodDataNode>(Arguments.NodeContainer->GetNode(LodUniqueId));
		if (!LodDataNode)
		{
			UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
			Message->Text = FText::Format(NSLOCTEXT("InterchangeSkeletalMeshFactory", "BeginImportAsset_GameThread_InvalidLOD", "Invalid LOD {0} when importing SkeletalMesh asset {1}.")
				, FText::AsNumber(LodIndex)
				, FText::FromString(Arguments.AssetName));
			continue;
		}

		FString SkeletonNodeUid;
		if (!LodDataNode->GetCustomSkeletonUid(SkeletonNodeUid))
		{
			UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
			Message->Text = WarningMessage_InvalidSkeleton;
			continue;
		}
		const UInterchangeSkeletonFactoryNode* SkeletonNode = Cast<UInterchangeSkeletonFactoryNode>(Arguments.NodeContainer->GetNode(SkeletonNodeUid));
		if (!SkeletonNode)
		{
			UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
			Message->Text = WarningMessage_InvalidSkeleton;
			continue;
		}
		FSoftObjectPath SkeletonNodeReferenceObject;
		SkeletonNode->GetCustomReferenceObject(SkeletonNodeReferenceObject);

		FSoftObjectPath SpecifiedSkeleton;
		SkeletalMeshFactoryNode->GetCustomSkeletonSoftObjectPath(SpecifiedSkeleton);
		bool bSpecifiedSkeleton = SpecifiedSkeleton.IsValid();
		if (SkeletonReference == nullptr)
		{
			UObject* SkeletonObject = nullptr;

			if (SpecifiedSkeleton.IsValid())
			{
				SkeletonObject = SpecifiedSkeleton.TryLoad();
			}
			else if (SkeletonNodeReferenceObject.IsValid())
			{
				SkeletonObject = SkeletonNodeReferenceObject.TryLoad();
			}

			if (SkeletonObject)
			{
				SkeletonReference = Cast<USkeleton>(SkeletonObject);

			}

			if (!Arguments.ReimportObject)
			{
				//In case its a SkeletalMesh AssetReimport without the Skeleton, then we won't have SkeletonReference which is expected
				if (!ensure(SkeletonReference))
				{
					UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
					Message->Text = WarningMessage_InvalidSkeleton;
					break;
				}
			}
			
			ImportAssetObjectData.SkeletonReference = SkeletonReference;
		}

		FString RootJointNodeId;
		if (!SkeletonNode->GetCustomRootJointUid(RootJointNodeId))
		{
			UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
			Message->Text = WarningMessage_InvalidRootJoint;
			continue;
		}

		const UInterchangeBaseNode* RootJointNode = Arguments.NodeContainer->GetNode(RootJointNodeId);
		if (!RootJointNode)
		{
			UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
			Message->Text = WarningMessage_InvalidRootJoint;
			continue;
		}
		if (!ImportAssetObjectData.LodDatas.IsValidIndex(CurrentLodIndex))
		{
			UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
			Message->Text = FText::Format(NSLOCTEXT("InterchangeSkeletalMeshFactory", "BeginImportAsset_GameThread_BadLodIndexPrecomputed", "Invalid precompute LOD {0} data when importing SkeletalMesh asset {1}.")
				, FText::AsNumber(LodIndex)
				, FText::FromString(Arguments.AssetName));
			continue;
		}
		FImportAssetObjectLODData& ImportAssetObjectLODData = ImportAssetObjectData.LodDatas[CurrentLodIndex];
		
		//Do not alter the skeletal mesh reference skeleton when importing geometry only
		FReferenceSkeleton RefSkeleton;
		UE::Interchange::Private::FSkeletonHelper::ProcessImportMeshSkeleton( Results
			, SkeletonReference
			, ContentInfo.bApplyGeometryOnly ? RefSkeleton : SkeletalMesh->GetRefSkeleton()
			, Arguments.NodeContainer
			, RootJointNodeId
			, ImportAssetObjectLODData.RefBonesBinary
			, ImportAssetObjectLODData.bUseTimeZeroAsBindPose
			, ImportAssetObjectLODData.bDiffPose
			, bImportSockets);

		if (bSpecifiedSkeleton && !SkeletonReference->IsCompatibleMesh(SkeletalMesh))
		{
			UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
			Message->Text = FText::Format(NSLOCTEXT("InterchangeSkeletalMeshFactory", "BeginImportAsset_GameThread_IncompatibleSkeleton", "The skeleton {0} is incompatible with the imported LOD {1} skeletalmesh asset {2}.")
				, FText::FromString(SkeletonReference->GetName())
				, FText::AsNumber(LodIndex)
				, FText::FromString(Arguments.AssetName));
		}

		CurrentLodIndex++;
	}
#endif

	return ImportAssetResult;
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeSkeletalMeshFactory::ImportAsset_Async(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSkeletalMeshFactory::CreateAsset)

	FImportAssetResult ImportAssetResult;

#if WITH_EDITOR && WITH_EDITORONLY_DATA
	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	UInterchangeSkeletalMeshFactoryNode* SkeletalMeshFactoryNode = Cast<UInterchangeSkeletalMeshFactoryNode>(Arguments.AssetNode);
	if (SkeletalMeshFactoryNode == nullptr)
	{
		return ImportAssetResult;
	}

	const IInterchangeMeshPayloadInterface* MeshTranslatorPayloadInterface = Cast<IInterchangeMeshPayloadInterface>(Arguments.Translator);
	if (!MeshTranslatorPayloadInterface)
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->Text = FText::Format(NSLOCTEXT("InterchangeSkeletalMeshFactory", "ImportAsset_Async_TranslatorInterfaceMissing", "Cannot import skeletalMesh {0}, the translator {1} does not implement the IInterchangeSkeletalMeshPayloadInterface.")
			, FText::FromString(Arguments.AssetName)
			, FText::FromString(Arguments.Translator ? Arguments.Translator->GetName() : TEXT("UnknownTranslator")));
		return ImportAssetResult;
	}

	FText ErrorMessage_SkeletalMeshDontExist = FText::Format(NSLOCTEXT("InterchangeSkeletalMeshFactory", "ImportAsset_Async_AssetDontExist", "Could not import the SkeletalMesh asset {0}, because the asset do not exist.")
		, FText::FromString(Arguments.AssetName));

	UObject* SkeletalMeshObject = UE::Interchange::FFactoryCommon::AsyncFindObject(SkeletalMeshFactoryNode, GetFactoryClass(), Arguments.Parent, Arguments.AssetName);

	if (!SkeletalMeshObject)
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->Text = ErrorMessage_SkeletalMeshDontExist;
		return ImportAssetResult;
	}

	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(SkeletalMeshObject);
	if (!ensure(SkeletalMesh))
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->Text = ErrorMessage_SkeletalMeshDontExist;
		return ImportAssetResult;
	}

	//Make sure we can modify the skeletalmesh properties
	FSkinnedAssetAsyncBuildScope AsyncBuildScope(SkeletalMesh);

	TMap<FString, TArray<FLODUtilities::FMorphTargetLodBackupData>> BackupImportedMorphTargetData;
	FLODUtilities::BackupCustomImportedMorphTargetData(SkeletalMesh, BackupImportedMorphTargetData);

	USkeleton* SkeletonReference = ImportAssetObjectData.SkeletonReference;
		
	FSkeletalMeshModel* ImportedResource = SkeletalMesh->GetImportedModel();
	if (!ImportAssetObjectData.bIsReImport)
	{
		if (!ensure(ImportedResource->LODModels.Num() == 0))
		{
			ImportedResource->LODModels.Empty();
		}
	}
	else
	{
		//When we re-import, we force the current skeletalmesh skeleton, to be specified and to be the reference
		FSoftObjectPath SpecifiedSkeleton = SkeletalMesh->GetSkeleton();
		SkeletalMeshFactoryNode->SetCustomSkeletonSoftObjectPath(SpecifiedSkeleton);
	}
			
	int32 LodCount = SkeletalMeshFactoryNode->GetLodDataCount();
	TArray<FString> LodDataUniqueIds;
	SkeletalMeshFactoryNode->GetLodDataUniqueIds(LodDataUniqueIds);
	ensure(LodDataUniqueIds.Num() == LodCount);
	int32 CurrentLodIndex = 0;

	UE::Interchange::Private::FContentInfo ContentInfo = UE::Interchange::Private::GetContentInfo(SkeletalMeshFactoryNode, ImportAssetObjectData.bIsReImport);
	
	if (ContentInfo.bApplySkinningOnly)
	{
		//Ignore vertex color when we import only the skinning
		constexpr bool bForceIgnoreVertexColor = true;
		SkeletalMeshFactoryNode->SetCustomVertexColorIgnore(bForceIgnoreVertexColor);
		constexpr bool bFalseSetting = false;
		SkeletalMeshFactoryNode->SetCustomVertexColorReplace(bFalseSetting);
	}


	//Call the mesh helper to create the missing material and to use the unmatched existing slot with the unmatch import slot
	{
		using namespace UE::Interchange::Private::MeshHelper;
		TMap<FString, FString> SlotMaterialDependencies;
		SkeletalMeshFactoryNode->GetSlotMaterialDependencies(SlotMaterialDependencies);
		SkeletalMeshFactorySetupAssetMaterialArray(SkeletalMesh->GetMaterials(), SlotMaterialDependencies, Arguments.NodeContainer, ImportAssetObjectData.bIsReImport);
	}

	for (int32 LodIndex = 0; LodIndex < LodCount; ++LodIndex)
	{
		FText WarningMessage_InvalidSkeleton = FText::Format(NSLOCTEXT("InterchangeSkeletalMeshFactory", "ImportAsset_Async_InvalidSkeletonLOD", "Invalid Skeleton LOD {0} when importing SkeletalMesh asset {1}")
			, FText::AsNumber(LodIndex)
			, FText::FromString(Arguments.AssetName));
		FText WarningMessage_InvalidRootJoint = FText::Format(NSLOCTEXT("InterchangeSkeletalMeshFactory", "ImportAsset_Async_InvalidSkeletonRootJoint", "Invalid Skeleton LOD {0} Root Joint when importing SkeletalMesh asset {1}")
			, FText::AsNumber(LodIndex)
			, FText::FromString(Arguments.AssetName));


		TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSkeletalMeshFactory::CreateAsset_LOD)

		FString LodUniqueId = LodDataUniqueIds[LodIndex];
		const UInterchangeSkeletalMeshLodDataNode* LodDataNode = Cast<UInterchangeSkeletalMeshLodDataNode>(Arguments.NodeContainer->GetNode(LodUniqueId));
		if (!LodDataNode)
		{
			UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
			Message->Text = FText::Format(NSLOCTEXT("InterchangeSkeletalMeshFactory", "ImportAsset_Async_InvalidLOD", "Invalid LOD {0} when importing SkeletalMesh asset {1}.")
				, FText::AsNumber(LodIndex)
				, FText::FromString(Arguments.AssetName));
			continue;
		}

		FString SkeletonNodeUid;
		if (!LodDataNode->GetCustomSkeletonUid(SkeletonNodeUid))
		{
			UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
			Message->Text = WarningMessage_InvalidSkeleton;
			continue;
		}
		const UInterchangeSkeletonFactoryNode* SkeletonNode = Cast<UInterchangeSkeletonFactoryNode>(Arguments.NodeContainer->GetNode(SkeletonNodeUid));
		if (!SkeletonNode)
		{
			UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
			Message->Text = WarningMessage_InvalidSkeleton;
			continue;
		}

		FSoftObjectPath SpecifiedSkeleton;
		SkeletalMeshFactoryNode->GetCustomSkeletonSoftObjectPath(SpecifiedSkeleton);
		bool bSpecifiedSkeleton = SpecifiedSkeleton.IsValid();
		if (SkeletonReference == nullptr)
		{
			break;
		}

		FString RootJointNodeId;
		if (!SkeletonNode->GetCustomRootJointUid(RootJointNodeId))
		{
			UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
			Message->Text = WarningMessage_InvalidRootJoint;
			continue;
		}
		
		const UInterchangeBaseNode* RootJointNode = Arguments.NodeContainer->GetNode(RootJointNodeId);
		if (!RootJointNode)
		{
			UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
			Message->Text = WarningMessage_InvalidRootJoint;
			continue;
		}

		//We should have a valid lod payload data
		FLodPayloads& LodPayloads = PayloadsPerLodIndex.FindChecked(LodIndex);

		FImportAssetObjectLODData& ImportAssetObjectLODData = ImportAssetObjectData.LodDatas[CurrentLodIndex];
		ensure(ImportAssetObjectLODData.LodIndex == CurrentLodIndex);

		//Add the lod mesh data to the skeletalmesh
		FMeshDescription MeshDescription;
		//Get all meshes and morph targets payload and fill the MeshDescription structure
		UE::Interchange::Private::RetrieveAllSkeletalMeshPayloads(SkeletalMeshFactoryNode
																	, ImportAssetObjectLODData
																	, LodPayloads
																	, Arguments
																	, Arguments.NodeContainer
																	, RootJointNodeId
																	, SkeletalMesh
																	, CurrentLodIndex
																	, MeshDescription);

		//////////////////////////////////////////////////////////////////////////
		//Manage vertex color, we want to use the translated source data
		//Replace -> do nothing
		//Ignore -> remove vertex color from import data (when we re-import, ignore have to put back the current mesh vertex color)
		//Override -> replace the vertex color by the override color
		{
			FSkeletalMeshAttributes MeshAttributes(MeshDescription);
			bool bHasVertexColor = MeshAttributes.GetVertexInstanceColors().IsValid() && (MeshAttributes.GetVertexInstanceColors().GetNumElements() > 0);

			bool bReplaceVertexColor = false;
			SkeletalMeshFactoryNode->GetCustomVertexColorReplace(bReplaceVertexColor);
			if (!bReplaceVertexColor)
			{
				bool bIgnoreVertexColor = false;
				SkeletalMeshFactoryNode->GetCustomVertexColorIgnore(bIgnoreVertexColor);
				if (bIgnoreVertexColor)
				{
					if (ImportAssetObjectData.bIsReImport)
					{
						//Get the vertex color we have in the current asset, 
						UE::Interchange::Private::MeshHelper::RemapSkeletalMeshVertexColorToMeshDescription(SkeletalMesh, LodIndex, MeshDescription);
						bHasVertexColor = true;
					}
					else
					{
						//Flush the vertex color
						TVertexInstanceAttributesRef<FVector4f> VertexColors = MeshAttributes.GetVertexInstanceColors();

						for (FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
						{
							VertexColors[VertexInstanceID] = FVector4f(1, 1, 1, 1);
						}
					}
				}
				else
				{
					FColor OverrideVertexColor;
					if (SkeletalMeshFactoryNode->GetCustomVertexColorOverride(OverrideVertexColor))
					{
						TVertexInstanceAttributesRef<FVector4f> VertexColors = MeshAttributes.GetVertexInstanceColors();

						for (FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
						{
							VertexColors[VertexInstanceID] = OverrideVertexColor.ReinterpretAsLinear();
						}
					}

					bHasVertexColor = true;
				}
			}

			if (ContentInfo.bApplyGeometry)
			{
				// Store whether or not this mesh has vertex colors
				SkeletalMesh->SetHasVertexColors(bHasVertexColor);
				SkeletalMesh->SetVertexColorGuid(SkeletalMesh->GetHasVertexColors() ? FGuid::NewGuid() : FGuid());
			}
		}

		if (ImportAssetObjectData.bIsReImport)
		{
			while (ImportedResource->LODModels.Num() <= CurrentLodIndex)
			{
				ImportedResource->LODModels.Add(new FSkeletalMeshLODModel());
			}

			// The LOD slot may have survived from a previous import with stale
			// RequiredBones / ActiveBoneIndices / MeshToImportVertexMap / IndexBuffer
			// /etc. The downstream build only overwrites the fields it produces, so
			// wipe the slot back to a default-constructed state before refilling it.
			ImportedResource->LODModels[CurrentLodIndex].Empty();
		}
		else
		{
			ensure(ImportedResource->LODModels.Add(new FSkeletalMeshLODModel()) == CurrentLodIndex);
		}

		auto AddLodInfo = [&SkeletalMesh]()
		{
			FSkeletalMeshLODInfo& NewLODInfo = SkeletalMesh->AddLODInfo();
			NewLODInfo.ReductionSettings.NumOfTrianglesPercentage = 1.0f;
			NewLODInfo.ReductionSettings.NumOfVertPercentage = 1.0f;
			NewLODInfo.ReductionSettings.MaxDeviationPercentage = 0.0f;
			NewLODInfo.LODHysteresis = 0.02f;
			NewLODInfo.bImportWithBaseMesh = true;
		};

		if (ImportAssetObjectData.bIsReImport)
		{
			while (SkeletalMesh->GetLODNum() <= CurrentLodIndex)
			{
				AddLodInfo();
			}
		}
		else
		{
			AddLodInfo();
		}
		
		FSkeletalMeshLODModel& LODModel = ImportedResource->LODModels[CurrentLodIndex];

		bool bOutInfluenceCountLimitHit = false;
		FSkeletalMeshOperations::ValidateAndFixInfluences(MeshDescription, bOutInfluenceCountLimitHit);

		if (bOutInfluenceCountLimitHit)
		{
			UE_LOGF(LogInterchangeImport, Warning, "Asset [%ls] contains too many influences.", *Arguments.AssetName);
		}

		if (ContentInfo.bApplyGeometryOnly)
		{
			FMeshDescription* OriginalMeshDescription = SkeletalMesh->GetMeshDescription(CurrentLodIndex);

			if (OriginalMeshDescription)
			{
				FSkeletalMeshOperations::ApplyRigToGeo(*OriginalMeshDescription, MeshDescription);
			}
		}
		else if (ContentInfo.bApplySkinningOnly)
		{
			FMeshDescription* OriginalMeshDescription = SkeletalMesh->GetMeshDescription(CurrentLodIndex);

			if (OriginalMeshDescription)
			{
				FSkeletalMeshOperations::ApplyRigToGeo(MeshDescription, *OriginalMeshDescription);
			}

			MeshDescription = *OriginalMeshDescription;
		}

		//Store the existing material import data before updating it so we can remap properly the material and section data
		TArray<FName> ExistingOriginalPerSectionMaterialImportName;
		if (ImportAssetObjectData.bIsReImport)
		{
			if (CurrentLodIndex != 0)
			{
				if (SkeletalMesh->HasMeshDescription(CurrentLodIndex))
				{
					FMeshDescription* ExistingMeshDescription = SkeletalMesh->GetMeshDescription(CurrentLodIndex);
					if (ExistingMeshDescription)
					{
						FSkeletalMeshAttributes MeshAttributes(*ExistingMeshDescription);
						TPolygonGroupAttributesConstRef<FName> SourceImportedMaterialSlotNames = MeshAttributes.GetPolygonGroupMaterialSlotNames();
						for (int32 MaterialSlotIndex = 0; MaterialSlotIndex < SourceImportedMaterialSlotNames.GetNumElements(); ++MaterialSlotIndex)
						{
							ExistingOriginalPerSectionMaterialImportName.Add(SourceImportedMaterialSlotNames[MaterialSlotIndex]);
						}
					}
				}
			}
			else
			{
				TArray<FSkeletalMaterial>& Materials = SkeletalMesh->GetMaterials();
				//LOD 0 import data is reorder to the material array before when building the LOD 0
				for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
				{
					ExistingOriginalPerSectionMaterialImportName.Add(Materials[MaterialIndex].ImportedMaterialSlotName);
				}
			}
		}

		//Store the original fbx import data the SkeletalMeshImportData should not be modified after this
		{
			//Restore the morph target into the imported mesh description
			FLODUtilities::RestoreCustomImportedMorphTargetData(SkeletalMesh, CurrentLodIndex, MeshDescription, BackupImportedMorphTargetData);
			SkeletalMesh->CreateMeshDescription(CurrentLodIndex, MoveTemp(MeshDescription));
			SkeletalMesh->CommitMeshDescription(CurrentLodIndex);
		}

		//Update the bounding box if we are importing the LOD 0
		if(CurrentLodIndex == 0)
		{
			FMeshDescription* StoredMeshDescription = SkeletalMesh->GetMeshDescription(CurrentLodIndex);
			if (StoredMeshDescription)
			{
				const FBox BoundingBox = StoredMeshDescription->ComputeBoundingBox();
				const FVector BoundingBoxSize = BoundingBox.GetSize();
				if (StoredMeshDescription->Vertices().Num() > 2 && BoundingBoxSize.X < UE_THRESH_POINTS_ARE_SAME && BoundingBoxSize.Y < UE_THRESH_POINTS_ARE_SAME && BoundingBoxSize.Z < UE_THRESH_POINTS_ARE_SAME)
				{
					UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
					Message->Text = FText::Format(NSLOCTEXT("InterchangeSkeletalMeshFactory", "ImportAsset_Async_ErrorMeshTooSmall", "The mesh {0} bounding box is smaller than the supported threshold[{1}]. All Vertices will be merge into one vertex.")
						, FText::FromString(Arguments.AssetName)
						, FText::AsNumber(UE_THRESH_POINTS_ARE_SAME));
				}
				FBoxSphereBounds BoxSphereBound((FBox)BoundingBox);
				SkeletalMesh->SetImportedBounds(FBoxSphereBounds((FBox)BoundingBox));
			}
		}
		//Copy the data into the game thread structure so we can finish the import in the game thread callback
		ImportAssetObjectLODData.ExistingOriginalPerSectionMaterialImportName = ExistingOriginalPerSectionMaterialImportName;

		//Acquire and Store ImportedMaterials (Aka MaterialSlotNames)
		{
			FSkeletalMeshConstAttributes MeshAttributes(MeshDescription);
			TPolygonGroupAttributesConstRef<FName> PolygonGroupMaterialSlotNames = MeshAttributes.GetPolygonGroupMaterialSlotNames();
			ImportAssetObjectLODData.ImportedMaterials.Reserve(MeshDescription.PolygonGroups().Num());
			for (FPolygonGroupID PolygonGroupID : MeshDescription.PolygonGroups().GetElementIDs())
			{
				SkeletalMeshImportData::FMaterial Material;
				Material.MaterialImportName = PolygonGroupMaterialSlotNames[PolygonGroupID].ToString();
				//The material interface will be added later by the factory
				ImportAssetObjectLODData.ImportedMaterials.Add(Material);
			}
		}

		// Make sure to retain Nanite assembly descriptions (needed in SetupObject_GameThread).
		if (FLodPayloads* Payloads = PayloadsPerLodIndex.Find(LodIndex))
		{
			for (TTuple<FInterchangeMeshPayLoadKey, TOptional<UE::Interchange::FMeshPayloadData>>& PayloadPair : Payloads->MeshPayloadPerKey)
			{
				TOptional<UE::Interchange::FMeshPayloadData>& PayloadData = PayloadPair.Value;
				if (PayloadData.IsSet() && PayloadData->NaniteAssemblyDescription.IsSet())
				{
					UE::Interchange::FNaniteAssemblyDescription& Description = PayloadData->NaniteAssemblyDescription.GetValue();
					NaniteAssemblyData.Emplace(MoveTemp(Description), MoveTemp(PayloadData->JointNames));
				}
			}
		}

		PayloadsPerLodIndex.Remove(LodIndex);

		CurrentLodIndex++;
	}

	//Getting the file Hash will cache it into the source data
	Arguments.SourceData->GetFileContentHash();

	ImportAssetResult.ImportedObject = SkeletalMeshObject;
#endif

	return ImportAssetResult;
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeSkeletalMeshFactory::EndImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSkeletalMeshFactory::EndImportAssetObject_GameThread)
	
	check(IsInGameThread());
	FImportAssetResult ImportAssetResult;

#if WITH_EDITOR && WITH_EDITORONLY_DATA
	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	UInterchangeSkeletalMeshFactoryNode* SkeletalMeshFactoryNode = Cast<UInterchangeSkeletalMeshFactoryNode>(Arguments.AssetNode);
	if (SkeletalMeshFactoryNode == nullptr)
	{
		return ImportAssetResult;
	}

	const UClass* SkeletalMeshClass = SkeletalMeshFactoryNode->GetObjectClass();
	check(SkeletalMeshClass && SkeletalMeshClass->IsChildOf(GetFactoryClass()));

	//Get the skeletal mesh asset from the factory node or a find if the node was not set properly
	USkeletalMesh* SkeletalMesh = nullptr;
	FSoftObjectPath ReferenceObject;
	if (SkeletalMeshFactoryNode->GetCustomReferenceObject(ReferenceObject))
	{
		SkeletalMesh = Cast<USkeletalMesh>(ReferenceObject.TryLoad());
	}
	if (!SkeletalMesh)
	{
		SkeletalMesh = Cast<USkeletalMesh>(StaticFindObject(nullptr, Arguments.Parent, *Arguments.AssetName));
	}
	
	if (!ensure(SkeletalMesh))
	{
		if (Arguments.ReimportObject == nullptr)
		{
			UE_LOGF(LogInterchangeImport, Error, "Could not create SkeletalMesh asset %ls.", *Arguments.AssetName);
		}
		else
		{
			UE_LOGF(LogInterchangeImport, Error, "Could not find reimported SkeletalMesh asset %ls.", *Arguments.AssetName);
		}
		return ImportAssetResult;
	}

	//Make sure we can modify the skeletalmesh properties
	FSkinnedAssetAsyncBuildScope AsyncBuildScope(SkeletalMesh);

	//Finish the import in the game thread so we can pop dialog if needed
	if (ImportAssetObjectData.IsValid())
	{
		for (FImportAssetObjectLODData& ImportAssetObjectLODData : ImportAssetObjectData.LodDatas)
		{
			if (FSkeletalMeshLODInfo* LodInfo = SkeletalMesh->GetLODInfo(ImportAssetObjectLODData.LodIndex))
			{
				if (SkeletalMesh->GetMaterials().IsEmpty() && !ImportAssetObjectLODData.ImportedMaterials.IsEmpty() )
				{
					for (SkeletalMeshImportData::FMaterial ImportMaterial : ImportAssetObjectLODData.ImportedMaterials)
					{
						UMaterialInterface* MaterialInterface = ImportMaterial.Material.Get() ? ImportMaterial.Material.Get() : UMaterial::GetDefaultMaterial(MD_Surface);
						SkeletalMesh->GetMaterials().Add(FSkeletalMaterial(MaterialInterface, FName(ImportMaterial.MaterialImportName), FName(ImportMaterial.MaterialImportName)));
					}
				}

				if (SkeletalMesh->GetMaterials().Num() > 0)
				{
					FLODUtilities::FSkeletalMeshMatchImportedMaterialsParameters Parameters;
					Parameters.bIsReImport = ImportAssetObjectData.bIsReImport;
					Parameters.LodIndex = ImportAssetObjectLODData.LodIndex;
					Parameters.SkeletalMesh = SkeletalMesh;
					Parameters.ImportedMaterials = &ImportAssetObjectLODData.ImportedMaterials;
					Parameters.ExistingOriginalPerSectionMaterialImportName = &ImportAssetObjectLODData.ExistingOriginalPerSectionMaterialImportName;
					FLODUtilities::MatchImportedMaterials(Parameters);
					//Flush the old LOD sections after we rematch the materials
					if (Parameters.bIsReImport)
					{
						if (FSkeletalMeshModel* ImportedModel = SkeletalMesh->GetImportedModel())
						{
							if (ImportedModel->LODModels.IsValidIndex(ImportAssetObjectLODData.LodIndex))
							{
								SkeletalMesh->GetImportedModel()->LODModels[ImportAssetObjectLODData.LodIndex].Sections.Empty();
							}
						}
					}
				}
			}
		}

		//Now that materials are matched we can re-order them and remove the unused.
		if (ImportAssetObjectData.bIsReImport)
		{
			FLODUtilities::ReorderMaterialSlotToBaseLod(SkeletalMesh);
			FLODUtilities::RemoveUnusedMaterialSlot(SkeletalMesh);
		}
	}

	if (USkeleton* SkeletonReference = ImportAssetObjectData.SkeletonReference)
	{
		if (SkeletalMesh->GetSkeleton() != SkeletonReference)
		{
			SkeletalMesh->SetSkeleton(SkeletonReference);
		}

		constexpr bool bShowProgress = false;
		if ((!ImportAssetObjectData.bApplyGeometryOnly || !ImportAssetObjectData.bIsReImport) && !SkeletonReference->MergeAllBonesToBoneTree(SkeletalMesh, bShowProgress))
		{
			TUniqueFunction<bool()> RecreateSkeleton = [this, WeakSkeletalMesh = TWeakObjectPtr<USkeletalMesh>(SkeletalMesh), WeakSkeleton = TWeakObjectPtr<USkeleton>(SkeletonReference)]()
			{

				check(IsInGameThread());

				USkeleton* SkeletonPtr = WeakSkeleton.Get();
				USkeletalMesh* SkeletalMeshPtr = WeakSkeletalMesh.Get();

				if (!SkeletonPtr || !SkeletalMeshPtr)
				{
					return false;
				}

				if (GIsRunningUnattendedScript)
				{
					UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
					Message->Text = NSLOCTEXT("InterchangeSkeletalMeshFactory", "ImportWithScriptIncompatibleSkeleton", "Interchange Import UInterchangeSkeletalMeshFactory::EndImportAssetObject_GameThread, cannot merge bone tree with the existing skeleton.");
					return false;
				}

				EAppReturnType::Type MergeBonesChoice = FMessageDialog::Open(EAppMsgType::YesNo
					, EAppReturnType::No
					, NSLOCTEXT("InterchangeSkeletalMeshFactory", "SkeletonFailed_BoneMerge", "Failed to merge bones.\n\n This can happen if significant hierarchical changes have been made,\nsuch as inserting a bone between nodes.\n\nWould you like to regenerate the Skeleton from this mesh? This may invalidate or require recompression of animation data.\n"));
				if (MergeBonesChoice == EAppReturnType::Yes)
				{
					//Allow this thread scope to read and write skeletalmesh locked properties
					FSkinnedAssetAsyncBuildScope AsyncBuildScope(SkeletalMeshPtr);

					if (SkeletonPtr->RecreateBoneTree(SkeletalMeshPtr))
					{
						TArray<const USkeletalMesh*> OtherSkeletalMeshUsingSkeleton;
						FString SkeletalMeshList;
						FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
						TArray<FAssetData> SkeletalMeshAssetData;

						FARFilter ARFilter;
						ARFilter.ClassPaths.Add(USkeletalMesh::StaticClass()->GetClassPathName());
						ARFilter.TagsAndValues.Add(TEXT("Skeleton"), FAssetData(SkeletonPtr).GetExportTextName());

						IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
						if (AssetRegistry.GetAssets(ARFilter, SkeletalMeshAssetData))
						{
							// look through all skeletalmeshes that uses this skeleton
							for (int32 AssetId = 0; AssetId < SkeletalMeshAssetData.Num(); ++AssetId)
							{
								FAssetData& CurAssetData = SkeletalMeshAssetData[AssetId];
								const USkeletalMesh* ExtraSkeletalMesh = Cast<USkeletalMesh>(CurAssetData.GetAsset());
								if (SkeletalMeshPtr != ExtraSkeletalMesh && IsValid(ExtraSkeletalMesh))
								{
									OtherSkeletalMeshUsingSkeleton.Add(ExtraSkeletalMesh);
									SkeletalMeshList += TEXT("\n") + ExtraSkeletalMesh->GetPathName();
								}
							}
						}
						if (OtherSkeletalMeshUsingSkeleton.Num() > 0)
						{
							FText MessageText = FText::Format(
								NSLOCTEXT("InterchangeSkeletalMeshFactory", "Skeleton_ReAddAllMeshes", "Would you like to merge all SkeletalMeshes using this skeleton to ensure all bones are merged? This will require to load those SkeletalMeshes.{0}")
								, FText::FromString(SkeletalMeshList));
							if (FMessageDialog::Open(EAppMsgType::YesNo, MessageText) == EAppReturnType::Yes)
							{
								// look through all skeletalmeshes that uses this skeleton
								for (const USkeletalMesh* ExtraSkeletalMesh : OtherSkeletalMeshUsingSkeleton)
								{
									// merge still can fail
									if (!SkeletonPtr->MergeAllBonesToBoneTree(ExtraSkeletalMesh, bShowProgress))
									{
										FMessageDialog::Open(EAppMsgType::Ok,
											FText::Format(NSLOCTEXT("InterchangeSkeletalMeshFactory", "SkeletonRegenError_RemergingBones", "Failed to merge SkeletalMesh '{0}'."), FText::FromString(ExtraSkeletalMesh->GetName())));
									}
								}
							}
						}
					}
				}
				return true;
			};

			if (IsInGameThread())
			{
				RecreateSkeleton();
			}
			else
			{
				//Wait until the skeleton is recreate on the game thread
				Async(EAsyncExecution::TaskGraphMainThread, MoveTemp(RecreateSkeleton)).Wait();
			}
		}

		//Cleanup the reference skeleton sockets that doesn't fit anymmore a bone name.
		TArray<TObjectPtr<USkeletalMeshSocket>>& Sockets = SkeletalMesh->GetMeshOnlySocketList();

		for (int32 SocketIndex = Sockets.Num()-1; SocketIndex >= 0; SocketIndex--)
		{
			// Find bone index the socket is attached to.
			USkeletalMeshSocket* Socket = Sockets[SocketIndex];
			int32 SocketBoneIndex = SkeletalMesh->GetRefSkeleton().FindBoneIndex(Socket->BoneName);
			// If this LOD does not contain the socket bone, abort import.
			if (SocketBoneIndex == INDEX_NONE)
			{
				Sockets.RemoveAt(SocketIndex, EAllowShrinking::No);
			}
		}

		//Add imported sockets
		bool bImportSockets = false;
		SkeletalMeshFactoryNode->GetCustomImportSockets(bImportSockets);
		UE::Interchange::Private::FContentInfo ContentInfo = UE::Interchange::Private::GetContentInfo(SkeletalMeshFactoryNode, ImportAssetObjectData.bIsReImport);
		if (bImportSockets && !ContentInfo.bApplySkinningOnly)
		{
			FString RootSkeletonNodeUid;
			//Find the Root skeleton uid
			{
				TArray<FString> LodDataUniqueIds;
				SkeletalMeshFactoryNode->GetLodDataUniqueIds(LodDataUniqueIds);
				//We import socket only for the base lod
				const int32 LodIndex = 0;
				if(LodDataUniqueIds.IsValidIndex(LodIndex))
				{
					FString LodUniqueId = LodDataUniqueIds[LodIndex];
					const UInterchangeSkeletalMeshLodDataNode* LodDataNode = Cast<UInterchangeSkeletalMeshLodDataNode>(Arguments.NodeContainer->GetNode(LodUniqueId));
					if (LodDataNode)
					{
						FString SkeletonNodeUid;
						if (LodDataNode->GetCustomSkeletonUid(SkeletonNodeUid))
						{
							const UInterchangeSkeletonFactoryNode* SkeletonFactoryNode = Cast<UInterchangeSkeletonFactoryNode>(Arguments.NodeContainer->GetNode(SkeletonNodeUid));
							if (SkeletonFactoryNode)
							{
								SkeletonFactoryNode->GetCustomRootJointUid(RootSkeletonNodeUid);
							}
						}
					}
				}
			}
			//If we find the root skeleton uid we can create the sockets
			if(!RootSkeletonNodeUid.IsEmpty())
			{
				const FReferenceSkeleton& RefSkeleton = SkeletonReference->GetReferenceSkeleton();
				TMap<FString, TSet<const UInterchangeSceneNode*>> SocketsPerBoneMap;

				if (const UInterchangeBaseNode* RootSkeletonNode = Arguments.NodeContainer->GetNode(RootSkeletonNodeUid))
				{
					UE::Interchange::Private::FindAllSockets(Arguments.NodeContainer, RootSkeletonNode, SocketsPerBoneMap);
				}

				for (const TPair<FString, TSet<const UInterchangeSceneNode*>>& BoneAndSockets : SocketsPerBoneMap)
				{
					//Find the bone in the skeleton
					const FString& BoneName = BoneAndSockets.Key;
					int32 BoneIndex = RefSkeleton.FindBoneIndex(*BoneName);
					if (RefSkeleton.IsValidIndex(BoneIndex))
					{
						//Add or update sockets
						for (const UInterchangeSceneNode* SocketSceneNode : BoneAndSockets.Value)
						{
							const FString SocketNameString = SocketSceneNode->GetDisplayLabel();

							//Remove the SOCKET_ prefix from the socket node name
							const int32 SocketPrefixLength = UInterchangeMeshFactoryNode::GetMeshSocketPrefix().Len();
							FName SocketName = (SocketNameString.Len() > SocketPrefixLength) ? *FString(SocketNameString.RightChop(SocketPrefixLength)) : *SocketNameString;
							USkeletalMeshSocket* Socket = SkeletalMesh->FindSocket(SocketName);
							if (!Socket)
							{
								//Create the socket
								Socket = NewObject<USkeletalMeshSocket>(SkeletalMesh);
								Socket->SocketName = SocketName;
								Sockets.Add(Socket);
							}

							if (ensure(Socket))
							{
								//Update the socket bone here in case the socket exist but the bone change
								Socket->BoneName = *BoneName;

								FTransform LocalSocketTransform;
								SocketSceneNode->GetCustomLocalTransform(LocalSocketTransform);
								Socket->RelativeLocation = LocalSocketTransform.GetLocation();
								Socket->RelativeRotation = LocalSocketTransform.GetRotation().Rotator();
								Socket->RelativeScale = LocalSocketTransform.GetScale3D();
							}
						}
					}
				}
			}
		}

		Sockets.Shrink();
	}
	else
	{
		UE_LOGF(LogInterchangeImport, Error, "Interchange Import UInterchangeSkeletalMeshFactory::EndImportAssetObject_GameThread, USkeleton* SkeletonReference is nullptr.");
	}

	if (ImportAssetObjectData.bIsReImport)
	{
		//We must reset the matrixs since CalculateInvRefMatrices only do the calculation if matrix count differ from the bone count.
		SkeletalMesh->GetRefBasesInvMatrix().Reset();
	}

	SkeletalMesh->CalculateInvRefMatrices();

	if (!ImportAssetObjectData.bIsReImport)
	{
		/** Apply all SkeletalMeshFactoryNode custom attributes to the skeletal mesh asset */
		SkeletalMeshFactoryNode->ApplyAllCustomAttributeToObject(SkeletalMesh);

		bool bCreatePhysicsAsset = false;
		SkeletalMeshFactoryNode->GetCustomCreatePhysicsAsset(bCreatePhysicsAsset);

		if (!bCreatePhysicsAsset)
		{
			FSoftObjectPath SpecifiedPhysicAsset;
			SkeletalMeshFactoryNode->GetCustomPhysicAssetSoftObjectPath(SpecifiedPhysicAsset);
			if (SpecifiedPhysicAsset.IsValid())
			{
				UPhysicsAsset* PhysicsAsset = Cast<UPhysicsAsset>(SpecifiedPhysicAsset.TryLoad());
				SkeletalMesh->SetPhysicsAsset(PhysicsAsset);
			}
		}
	}
	else
	{
		//Apply the re import strategy 
		UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(SkeletalMesh->GetAssetImportData());
		UInterchangeFactoryBaseNode* PreviousNode = nullptr;
		if (InterchangeAssetImportData)
		{
			PreviousNode = InterchangeAssetImportData->GetStoredFactoryNode(InterchangeAssetImportData->NodeUniqueID);
		}
		UInterchangeFactoryBaseNode* CurrentNode = UInterchangeFactoryBaseNode::DuplicateWithObject(SkeletalMeshFactoryNode, SkeletalMesh);
		UE::Interchange::FFactoryCommon::ApplyReimportStrategyToAsset(SkeletalMesh, PreviousNode, CurrentNode, SkeletalMeshFactoryNode);
	}

	//For UAnimSequences we also have to check the existance of USkeletalMeshes not just USkeletons.
	// (USkeletalMesh creation can fail while USkeletons can succeed still)
	SkeletalMeshFactoryNode->SetCustomReferenceObject(FSoftObjectPath(SkeletalMesh));

	ImportAssetResult.ImportedObject = SkeletalMesh;

#endif

	if (ScopedReimportUtility.IsValid())
	{
		ScopedReimportUtility.Reset();
	}

	return ImportAssetResult;
}

void UInterchangeSkeletalMeshFactory::Cancel()
{
	if (ScopedReimportUtility.IsValid())
	{
		ScopedReimportUtility.Reset();
	}
}

/* This function is call in the completion task on the main thread, use it to call main thread post creation step for your assets*/
void UInterchangeSkeletalMeshFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSkeletalMeshFactory::SetupObject_GameThread)
	check(IsInGameThread());
	Super::SetupObject_GameThread(Arguments);

	const UInterchangeSkeletalMeshFactoryNode* SkeletalMeshFactoryNode = Cast<const UInterchangeSkeletalMeshFactoryNode>(Arguments.NodeContainer->GetFactoryNode(Arguments.NodeUniqueID));

	USkeletalMesh* SkeletalMesh = CastChecked<USkeletalMesh>(Arguments.ImportedObject);
	if(ensure(SkeletalMesh))
	{
		//TODO make sure this work at runtime
#if WITH_EDITORONLY_DATA
		if (ensure(Arguments.SourceData))
		{
			//We must call the Update of the asset source file in the main thread because UAssetImportData::Update execute some delegate we do not control

			UAssetImportData* ImportDataPtr = SkeletalMesh->GetAssetImportData();
			UE::Interchange::FFactoryCommon::FUpdateImportAssetDataParameters UpdateImportAssetDataParameters(SkeletalMesh
				, ImportDataPtr
				, Arguments.SourceData
				, Arguments.NodeUniqueID
				, Arguments.NodeContainer
				, Arguments.OriginalPipelines
				, Arguments.Translator);

			ImportDataPtr = UE::Interchange::FFactoryCommon::UpdateImportAssetData(UpdateImportAssetDataParameters, [&Arguments, SkeletalMeshFactoryNode, SkeletalMesh](UInterchangeAssetImportData* AssetImportData)
				{
					auto GetSourceIndexFromContentType = [](const EInterchangeSkeletalMeshContentType& ImportContentType)->int32
						{
							return (ImportContentType == EInterchangeSkeletalMeshContentType::Geometry)
								? 1
								: (ImportContentType == EInterchangeSkeletalMeshContentType::SkinningWeights)
								? 2
								: 0;
						};

					auto GetSourceLabelFromSourceIndex = [](const int32 SourceIndex)->FString
						{
							return (SourceIndex == 1)
								? NSSkeletalMeshSourceFileLabels::GeometryText().ToString()
								: (SourceIndex == 2)
								? NSSkeletalMeshSourceFileLabels::SkinningText().ToString()
								: NSSkeletalMeshSourceFileLabels::GeoAndSkinningText().ToString();
						};

					if (SkeletalMeshFactoryNode)
					{
						EInterchangeSkeletalMeshContentType ImportContentType = EInterchangeSkeletalMeshContentType::All;
						SkeletalMeshFactoryNode->GetCustomImportContentType(ImportContentType);
						FMD5Hash SourceFileHash = Arguments.SourceData->GetFileContentHash().Get(FMD5Hash());
						const FString& NewSourceFilename = Arguments.SourceData->GetFilename();
						const int32 NewSourceIndex = GetSourceIndexFromContentType(ImportContentType);
						//NewSourceIndex should be 0, 1 or 2 (All, Geo, Skinning)
						check(NewSourceIndex >= 0 && NewSourceIndex < 3);
						const TArray<FString> OldFilenames = AssetImportData->ScriptExtractFilenames();
						const FString DefaultFilename = OldFilenames.Num() > 0 ? OldFilenames[GetSourceIndexFromContentType(EInterchangeSkeletalMeshContentType::All)] : NewSourceFilename;
						for (int32 SourceIndex = 0; SourceIndex < 3; ++SourceIndex)
						{
							FString SourceLabel = GetSourceLabelFromSourceIndex(SourceIndex);
							if (SourceIndex == NewSourceIndex)
							{
								if (SourceIndex == GetSourceIndexFromContentType(EInterchangeSkeletalMeshContentType::All))
								{
									AssetImportData->Update(NewSourceFilename, SourceFileHash);
									break;
								}
								else
								{
									AssetImportData->ScriptedAddFilename(NewSourceFilename, SourceIndex, SourceLabel);
								}
							}
							else
							{
								//Extract filename create a default path if the FSourceFile::RelativeFilename is empty
								//We want to fill the entry with the base source file (SourceIndex 0, All) in this case.
								const bool bValidOldFilename = AssetImportData->SourceData.SourceFiles.IsValidIndex(SourceIndex)
									&& !AssetImportData->SourceData.SourceFiles[SourceIndex].RelativeFilename.IsEmpty()
									&& OldFilenames.IsValidIndex(SourceIndex);
								const FString& OldFilename = bValidOldFilename ? OldFilenames[SourceIndex] : DefaultFilename;
								AssetImportData->ScriptedAddFilename(OldFilename, SourceIndex, SourceLabel);
							}
						}
					}
				});


			SkeletalMesh->SetAssetImportData(ImportDataPtr);

			//Ensure we have curve meta data for every morph target
			if (SkeletalMeshFactoryNode)
			{
				USkeleton* SkeletonReference = ImportAssetObjectData.SkeletonReference;
				bool bAddMorphTargetToSkeletonCurveMetadata = false;
				if (SkeletonReference)
				{
					bAddMorphTargetToSkeletonCurveMetadata = true;
					SkeletalMeshFactoryNode->GetCustomAddCurveMetadataToSkeleton(bAddMorphTargetToSkeletonCurveMetadata);
				}
				constexpr bool bInTransactFalse = false;
				for (int32 LodIndex = 0; LodIndex < ImportAssetObjectData.LodDatas.Num(); ++LodIndex)
				{
					// Ensure that we have curve metadata for morph target (either skeleton or mesh)
					for (const FString& MorphTargetName : ImportAssetObjectData.LodDatas[LodIndex].SkeletonMorphCurveMetadataNames)
					{
						FName CurveName = *MorphTargetName;
						if (bAddMorphTargetToSkeletonCurveMetadata)
						{
							//bAddMorphTargetToSkeletonCurveMetadata is true only if the skeleton is not null and the node setting is undefined or true
							if (ensure(SkeletonReference))
							{
								SkeletonReference->AddCurveMetaData(CurveName, bInTransactFalse);
								// Ensure we have a morph flag set
								FCurveMetaData* CurveMetaData = SkeletonReference->GetCurveMetaData(CurveName);
								check(CurveMetaData);
								CurveMetaData->Type.bMorphtarget = true;
							}
						}
						else
						{
							UAnimCurveMetaData* AnimCurveMetaData = SkeletalMesh->GetAssetUserData<UAnimCurveMetaData>();
							if (AnimCurveMetaData == nullptr)
							{
								AnimCurveMetaData = NewObject<UAnimCurveMetaData>(SkeletalMesh, NAME_None, RF_Transactional);
								SkeletalMesh->AddAssetUserData(AnimCurveMetaData);
							}
							AnimCurveMetaData->AddCurveMetaData(CurveName, bInTransactFalse);
							// Ensure we have a morph flag set
							FCurveMetaData* CurveMetaData = AnimCurveMetaData->GetCurveMetaData(CurveName);
							check(CurveMetaData);
							CurveMetaData->Type.bMorphtarget = true;
						}
					}
				}
			}

#if WITH_EDITOR
			// Build Nanite Assembly data, if available
			if (NaniteAssemblyData.Num() > 0)
			{
				if (USkeleton* SkeletonReference = ImportAssetObjectData.SkeletonReference)
				{
					using namespace UE::Interchange::Private::MeshHelper;

					const TMap<FString, int32> NewBoneIndexByBoneNameTable = UE::Interchange::Private::GetSkeletonBoneIndexByBoneNameTable(*SkeletonReference);
					FInterchangeNaniteAssemblyBuilder Builder = FInterchangeNaniteAssemblyBuilder::Create(SkeletalMesh, SkeletalMeshFactoryNode, Arguments.NodeContainer);
			
					for (FNaniteAssemblyData& Data : NaniteAssemblyData)
					{
						if (UE::Interchange::Private::RemapNaniteAssemblyBoneIndices(Data.Description, Data.JointNames, NewBoneIndexByBoneNameTable, Arguments.NodeUniqueID))
						{
							Builder.AddDescription(Data.Description);
						}
					}
				}

				NaniteAssemblyData.Empty();
			}

			//Re-apply the alternate skinning data
			if (ImportAssetObjectData.ExistingSkinWeightProfileInfos.Num() > 0)
			{
				//Reimport alternate skin of the reimport lod only by looking at the factory node lod data count.
				int32 LodCount = SkeletalMesh->GetLODNum();
				if (SkeletalMeshFactoryNode)
				{
					int32 ImportedLodCount = SkeletalMeshFactoryNode->GetLodDataCount();
					if (ImportedLodCount > 0 && ImportedLodCount <= SkeletalMesh->GetLODNum())
					{
						LodCount = ImportedLodCount;
					}
				}

				TSharedPtr<FInterchangeSkeletalMeshAlternateSkinWeightPostImportTask> SkeletalMeshPostImportTask = nullptr;
				TArray<FSkinWeightProfileInfo>& SkinProfiles = SkeletalMesh->GetSkinWeightProfiles();
				SkinProfiles = ImportAssetObjectData.ExistingSkinWeightProfileInfos;
				//Since SkeletalMesh->SaveLODImportedData can remove invalid profile we can't use an iterator or a for loop
				//We store all the name in a TArray and we will search for them
				TArray<FName> ProfileList;
				for (const FSkinWeightProfileInfo& ProfileInfo : SkinProfiles)
				{
					ProfileList.AddUnique(ProfileInfo.Name);
				}
				FSkinWeightProfileInfo EmptyProfile = FSkinWeightProfileInfo();
				EmptyProfile.Name = NAME_None;
				//Return a profile reference, if it doesn't found any profile, it will return the empty profile
				auto GetProfileFromName = [&SkinProfiles, &EmptyProfile](const FName ProfileNameToSearch)
					{
						for (const FSkinWeightProfileInfo& ProfileInfo : SkinProfiles)
						{
							if (ProfileInfo.Name == ProfileNameToSearch)
							{
								return ProfileInfo;
							}
						}
						return EmptyProfile;
					};

				TMap<int32, FMeshDescription> MeshDescriptionDestPerLod;
				TBitArray SaveLodIndexes;
				SaveLodIndexes.Init(false, LodCount);

				for (const FName& ProfileName : ProfileList)
				{
					const FSkinWeightProfileInfo& ProfileInfo = GetProfileFromName(ProfileName);
					if (ProfileInfo.Name == NAME_None)
					{
						continue;
					}
					for (int32 LodIndex = 0; LodIndex < LodCount; ++LodIndex)
					{
						if (!SkeletalMesh->IsValidLODIndex(LodIndex)
							|| !ImportAssetObjectData.ExistingAlternateMeshDescriptionPerLOD.IsValidIndex(LodIndex)
							|| !SkeletalMesh->HasMeshDescription(LodIndex))
						{
							continue;
						}
						const FSkeletalMeshLODInfo* LodInfo = SkeletalMesh->GetLODInfo(LodIndex);
						if (!LodInfo)
						{
							continue;
						}

						const FMeshDescription& ExistingMeshDescriptionSrc = ImportAssetObjectData.ExistingAlternateMeshDescriptionPerLOD[LodIndex];

						if (!MeshDescriptionDestPerLod.Contains(LodIndex))
						{
							MeshDescriptionDestPerLod.Add(LodIndex, *SkeletalMesh->GetMeshDescription(LodIndex));
						}
						FMeshDescription& MeshDescriptionDest = MeshDescriptionDestPerLod.FindChecked(LodIndex);

						int32 PointNumberDest = MeshDescriptionDest.Vertices().Num();
						int32 VertexNumberDest = PointNumberDest;

						const bool bIsGeneratedByEngine = !ProfileInfo.PerLODSourceFiles.Contains(LodIndex);

						if (ExistingMeshDescriptionSrc.Vertices().Num() == PointNumberDest)
						{
							FSkeletalMeshAttributes AttributesDest(MeshDescriptionDest);
							if (AttributesDest.GetSkinWeightProfileNames().Contains(ProfileName))
							{
								AttributesDest.UnregisterSkinWeightAttribute(ProfileName);
							}

							int32 DestinationNumOfBones = AttributesDest.GetNumBones();
							bool bSkippedBoneWeight = false;

							FSkeletalMeshConstAttributes AttributesExistingSrc(ExistingMeshDescriptionSrc);
							if (AttributesExistingSrc.GetSkinWeightProfileNames().Contains(ProfileName))
							{
								AttributesDest.RegisterSkinWeightAttribute(ProfileName);
								//Fill the SkinWeightsAttribute
								FSkinWeightsVertexAttributesRef SkinWeightDest = AttributesDest.GetVertexSkinWeights(ProfileName);
								FSkinWeightsVertexAttributesConstRef SkinWeightsExistingSrc = AttributesExistingSrc.GetVertexSkinWeights(ProfileName);

								for (FVertexID VertexID : ExistingMeshDescriptionSrc.Vertices().GetElementIDs())
								{
									FVertexBoneWeightsConst VertexBoneWeightsExistingSrc = SkinWeightsExistingSrc.Get(VertexID);

									TArray<UE::AnimationCore::FBoneWeight> BoneWeights;
									for (const UE::AnimationCore::FBoneWeight BoneWeight : VertexBoneWeightsExistingSrc)
									{
										if (BoneWeight.GetBoneIndex() < DestinationNumOfBones)
										{
											BoneWeights.Add(BoneWeight);
										}
										else
										{
											bSkippedBoneWeight = true;
										}
									}
									SkinWeightDest.Set(VertexID, BoneWeights);
								}
							}

							if (bSkippedBoneWeight)
							{
								UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
								Message->Text = FText::Format(NSLOCTEXT("InterchangeSkeletalMeshFactory", "SetupObject_GameThread_AlternateProfileReinstateIssue", "At least 1 Bone Weight was skipped in [{0}] Alternate Skin Weight Profile on LOD [{1}] upon re-instate due to Bone Number missmatch.")
									, FText::FromName(ProfileName)
									, FText::AsNumber(LodIndex));
								Message->AssetFriendlyName = SkeletalMesh->GetName();
							}

							SaveLodIndexes[LodIndex] = true;
						}
						else if (bIsGeneratedByEngine)
						{
							UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
							Message->Text = FText::Format(NSLOCTEXT("InterchangeSkeletalMeshFactory", "SetupObject_GameThread_AlternateProfileLodCannotImport", "While re-importing skeletal mesh, the alternate skin weight profile '{0}' cannot be re-import for LOD {1}, because mesh topology differ.")
								, FText::FromString(ProfileName.ToString())
								, FText::AsNumber(LodIndex));
							Message->AssetFriendlyName = SkeletalMesh->GetName();
							Message->AssetType = USkeletalMesh::StaticClass();
							Message->SourceAssetName = Arguments.SourceData->GetFilename();
						}

						//We must enqueue a post import task that will re-import all skin weight profile that has some source file. Generated alternate skin cannot be re-imported
						if (!bIsGeneratedByEngine)
						{
							if (!SkeletalMeshPostImportTask.IsValid())
							{
								SkeletalMeshPostImportTask = MakeShared<FInterchangeSkeletalMeshAlternateSkinWeightPostImportTask>(SkeletalMesh);
								SkeletalMeshPostImportTask->ReimportAlternateSkinWeightDelegate.BindLambda([](USkeletalMesh* SkeletalMesh, int32 LodIndex)
									{
										return FSkinWeightsUtilities::ReimportAlternateSkinWeight(SkeletalMesh, LodIndex);
									});
							}
							SkeletalMeshPostImportTask->AddLodToReimportAlternate(LodIndex);
						}
					}
				}

				//Save every modified lod source data
				for (int32 LodIndex = 0; LodIndex < LodCount; ++LodIndex)
				{
					if (SaveLodIndexes[LodIndex] && MeshDescriptionDestPerLod.Contains(LodIndex))
					{
						SkeletalMesh->CreateMeshDescription(LodIndex, MoveTemp(MeshDescriptionDestPerLod[LodIndex]));
						SkeletalMesh->CommitMeshDescription(LodIndex);
					}
				}

				//Enqueue the task if needed
				if (SkeletalMeshPostImportTask.IsValid())
				{
					UInterchangeManager::GetInterchangeManager().EnqueuePostImportTask(SkeletalMeshPostImportTask);
				}
			}
#endif //WITH_EDITOR
		}
#endif //WITH_EDITORONLY_DATA

		if (const UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::GetUniqueInstance(Arguments.NodeContainer))
		{
			uint8 SkeletalMeshFronAxis;
			if (SourceNode->GetCustomSkeletalMeshFrontAxis(SkeletalMeshFronAxis))
			{
				SkeletalMesh->SetForwardAxis((EAxis::Type)SkeletalMeshFronAxis);
			}
		}
	}
}

void UInterchangeSkeletalMeshFactory::BuildObject_GameThread(const FSetupObjectParams& Arguments, bool& OutPostEditchangeCalled)
{
	check(IsInGameThread());
	OutPostEditchangeCalled = false;
#if WITH_EDITOR
	if (Arguments.ImportedObject)
	{
		if (USkeletalMesh* SkeletalMesh = CastChecked<USkeletalMesh>(Arguments.ImportedObject))
		{
			//Start an async build of the staticmesh
			SkeletalMesh->Build();
		}
	}
#endif
}

void UInterchangeSkeletalMeshFactory::FinalizeObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSkeletalMeshFactory::FinalizeObject_GameThread)
	Super::FinalizeObject_GameThread(Arguments);

#if WITH_EDITOR
	//This code works only on the game thread and is not asynchronous
	check(IsInGameThread());

	if (!ensure(Arguments.ImportedObject && Arguments.SourceData))
	{
		return;
	}

	USkeletalMesh* SkeletalMesh = CastChecked<USkeletalMesh>(Arguments.ImportedObject);

	//Rebinding cloth will rebuild the skeletalmesh
	//TODO: To avoid a second build we need to restore the clothing data before in SetupObject_GameThread.
	//to do this we need to put the cloth binding data in the import data use by the build (i.e. Meshdescription)
	if (ImportAssetObjectData.ExistingClothingBindings.Num() > 0)
	{
		//Make sure we rebuild the skeletal mesh after re-importing all skin weight
		FScopedSkeletalMeshPostEditChange ScopePostEditChange(SkeletalMesh);
		//Wait until the asset is finish building then lock the skeletal mesh properties to prevent the UI to update during the alternate skinning reimport
		FEvent* LockEvent = SkeletalMesh->LockPropertiesUntil();
		FSkinnedAssetAsyncBuildScope AsyncBuildScope(SkeletalMesh);
		for (ClothingAssetUtils::FClothingAssetMeshBinding& ExistingClothMeshBinding : ImportAssetObjectData.ExistingClothingBindings)
		{
			if (ExistingClothMeshBinding.Asset)
			{
				ExistingClothMeshBinding.Asset->RefreshBoneMapping(SkeletalMesh);
			}
		}
		FSkeletalMeshModel* ImportedResource = SkeletalMesh->GetImportedModel();
		for (int32 LodIndex = 0; LodIndex < ImportedResource->LODModels.Num(); ++LodIndex)
		{
			// Re-apply our clothing assets
			FLODUtilities::RestoreClothingFromBackup(SkeletalMesh, ImportAssetObjectData.ExistingClothingBindings, LodIndex);
		}
		//Release the skeletal mesh async properties
		LockEvent->Trigger();
	}
#if WITH_EDITORONLY_DATA
	//Clear the skeletal mesh descriptions to free memory.
	//Note that we cannot do this earlier due to the MeshDescriptions being used when building the skeletal mesh.
	SkeletalMesh->ClearAllMeshDescriptions();
#endif
#endif //WITH_EDITOR
}

bool UInterchangeSkeletalMeshFactory::GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSkeletalMeshFactory::GetSourceFilenames)
#if WITH_EDITORONLY_DATA
	if (const USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Object))
	{
		return UE::Interchange::FFactoryCommon::GetSourceFilenames(SkeletalMesh->GetAssetImportData(), OutSourceFilenames);
	}
#endif

	return false;
}

bool UInterchangeSkeletalMeshFactory::SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSkeletalMeshFactory::SetSourceFilename)
#if WITH_EDITORONLY_DATA
	if (const USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Object))
	{
		const FString SourceLabel = USkeletalMesh::GetSourceFileLabelFromIndex(SourceIndex).ToString();
		return UE::Interchange::FFactoryCommon::SetSourceFilename(SkeletalMesh->GetAssetImportData(), SourceFilename, SourceIndex, SourceLabel);
	}
#endif

	return false;
}

void UInterchangeSkeletalMeshFactory::BackupSourceData(const UObject* Object) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSkeletalMeshFactory::BackupSourceData)
#if WITH_EDITORONLY_DATA
	if (const USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Object))
	{
		UE::Interchange::FFactoryCommon::BackupSourceData(SkeletalMesh->GetAssetImportData());
	}
#endif
}

void UInterchangeSkeletalMeshFactory::ReinstateSourceData(const UObject* Object) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSkeletalMeshFactory::ReinstateSourceData)
#if WITH_EDITORONLY_DATA
		if (const USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Object))
		{
			UE::Interchange::FFactoryCommon::ReinstateSourceData(SkeletalMesh->GetAssetImportData());
		}
#endif
}
void UInterchangeSkeletalMeshFactory::ClearBackupSourceData(const UObject* Object) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSkeletalMeshFactory::ClearBackupSourceData)
#if WITH_EDITORONLY_DATA
		if (const USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Object))
		{
			UE::Interchange::FFactoryCommon::ClearBackupSourceData(SkeletalMesh->GetAssetImportData());
		}
#endif
}

bool UInterchangeSkeletalMeshFactory::SetReimportSourceIndex(const UObject* Object, int32 SourceIndex) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSkeletalMeshFactory::SetReimportSourceIndex)
#if WITH_EDITORONLY_DATA
	if (const USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Object))
	{
		return UE::Interchange::FFactoryCommon::SetReimportSourceIndex(SkeletalMesh, SkeletalMesh->GetAssetImportData(), SourceIndex);
	}
#endif

	return false;
}

bool UInterchangeSkeletalMeshFactory::FImportAssetObjectData::IsValid() const
{
	if (LodDatas.Num() < 1)
	{
		return false;
	}
	for (const FImportAssetObjectLODData& ImportAssetObjectLODData : LodDatas)
	{
		if (ImportAssetObjectLODData.LodIndex == INDEX_NONE)
		{
			return false;
		}
#if WITH_EDITOR
		if (!bIsReImport && ImportAssetObjectLODData.ImportedMaterials.Num() < 1)
		{
			return false;
		}
#endif
	}

	return true;
}

