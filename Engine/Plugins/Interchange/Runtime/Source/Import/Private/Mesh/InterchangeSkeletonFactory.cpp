// Copyright Epic Games, Inc. All Rights Reserved. 
#include "Mesh/InterchangeSkeletonFactory.h"

#include "Animation/Skeleton.h"
#include "InterchangeImportCommon.h"
#include "InterchangeImportLog.h"
#include "InterchangeSkeletonFactoryNode.h"
#include "InterchangeSourceData.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Rendering/SkeletalMeshLODImporterData.h"

#if WITH_EDITORONLY_DATA
#include "Engine/SkeletalMesh.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSkeletonFactory)

UClass* UInterchangeSkeletonFactory::GetFactoryClass() const
{
	return USkeleton::StaticClass();
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeSkeletonFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSkeletonFactory::BeginImportAsset_GameThread);

	FImportAssetResult ImportAssetResult;

	USkeleton* Skeleton = nullptr;

#if WITH_EDITORONLY_DATA
	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	UInterchangeSkeletonFactoryNode* SkeletonNode = Cast<UInterchangeSkeletonFactoryNode>(Arguments.AssetNode);
	if (SkeletonNode == nullptr)
	{
		return ImportAssetResult;
	}

	bool bIsReImport = Arguments.ReimportObject != nullptr;

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (SkeletonNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}
	}

	// create a new skeleton or overwrite existing asset, if possible
	if (!ExistingAsset)
	{
		Skeleton = NewObject<USkeleton>(Arguments.Parent, USkeleton::StaticClass(), *Arguments.AssetName, RF_Public | RF_Standalone);
	}
	else
	{
		//This is a reimport, we are just re-updating the source data
		Skeleton = Cast<USkeleton>(ExistingAsset);
	}

	if (!Skeleton)
	{
		UE_LOGF(LogInterchangeImport, Warning, "Could not create Skeleton asset %ls", *Arguments.AssetName);
		return ImportAssetResult;
	}
	SkeletonNode->SetCustomReferenceObject(FSoftObjectPath(Skeleton));

	Skeleton->PreEditChange(nullptr);

	FString RootJointUid;
	if (!SkeletonNode->GetCustomRootJointUid(RootJointUid))
	{
		UE_LOGF(LogInterchangeImport, Warning, "Could not create Skeleton asset %ls, because there is no valid root joint node", *Arguments.AssetName);
		return ImportAssetResult;
	}
	if (bIsReImport)
	{
		if (const UInterchangeBaseNode* RootJointNode = Arguments.NodeContainer->GetNode(RootJointUid))
		{
			FName RootJointName = FName(*RootJointNode->GetDisplayLabel());
			if (RootJointName != Skeleton->GetReferenceSkeleton().GetBoneName(0))
			{
				return ImportAssetResult;
			}
		}
	}
	//The joint will be added by the skeletalmesh factory since we need a valid skeletalmesh to add joint to a skeleton
	SkeletonNode->SetCustomReferenceObject(FSoftObjectPath(Skeleton));

	//Getting the file Hash will cache it into the source data
	Arguments.SourceData->GetFileContentHash();

	//The interchange completion task (call in the GameThread after the factories pass), will call PostEditChange which will trig another asynchronous system that will build all skeleton in parallel

#endif //WITH_EDITORONLY_DATA
	ImportAssetResult.ImportedObject = Skeleton;

	return ImportAssetResult;
}

UObject* UInterchangeSkeletonFactory::GetObjectToReimport(UObject* ReimportObject, const UInterchangeFactoryBaseNode& FactoryNode, const FString& PackageName, const FString& AssetName, const FString& SubPathString)
{
#if WITH_EDITORONLY_DATA
	if (ReimportObject &&
		ReimportObject->GetClass()->IsChildOf(USkeletalMesh::StaticClass()) &&
		FactoryNode.GetObjectClass()->IsChildOf(USkeleton::StaticClass()))
	{
		//The Skeleton property of the UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties will be populated for re-import,
		//which in turn will disable the UInterchangeSkeletonFactoryNodes.
		//Meaning, if at re-import the Skeleton already exists, the SkeletonFactoryNode will be disabled.
		//Hence, if we get to this branch execution that means that we are Reimporting a SkeletalMesh without a Skeleton present.
		return nullptr;
	}
#endif

	return ReimportObject;
}