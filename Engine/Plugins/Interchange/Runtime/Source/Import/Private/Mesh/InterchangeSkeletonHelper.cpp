// Copyright Epic Games, Inc. All Rights Reserved. 
#include "InterchangeSkeletonHelper.h"

#if WITH_EDITOR
#include "Animation/Skeleton.h"
#include "InterchangeCommonPipelineDataFactoryNode.h"
#include "InterchangeImportLog.h"
#include "InterchangeMeshFactoryNode.h"
#include "InterchangeResultsContainer.h"
#include "InterchangeJointNode.h"
#include "ReferenceSkeleton.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"

namespace UE::Interchange::Private
{
	bool FSkeletonHelper::IsValidSocket(const UInterchangeBaseNodeContainer* NodeContainer, const UInterchangeSceneNode* JointNode)
	{
		
		if (JointNode && JointNode->GetDisplayLabel().StartsWith(UInterchangeMeshFactoryNode::GetMeshSocketPrefix(), ESearchCase::CaseSensitive))
		{
			const FString& JointNodeId = JointNode->GetUniqueID();
			return (NodeContainer->GetNodeChildrenCount(JointNodeId) == 0);
		}
		return false;
	}

	bool FSkeletonHelper::ProcessImportMeshSkeleton(TObjectPtr<UInterchangeResultsContainer> Results
		, const USkeleton* SkeletonAsset
		, FReferenceSkeleton& RefSkeleton
		, const UInterchangeBaseNodeContainer* NodeContainer
		, const FString& RootJointNodeId
		, TArray<SkeletalMeshImportData::FBone>& RefBonesBinary
		, const bool bUseTimeZeroAsBindPose
		, bool& bOutDiffPose
		, bool bImportSockets)
	{
		RefBonesBinary.Empty();
		// Setup skeletal hierarchy + names structure.
		RefSkeleton.Empty();

		FReferenceSkeletonModifier RefSkelModifier(RefSkeleton, SkeletonAsset);
		TArray <FJointInfo> JointInfos;
		TArray<FString> BoneNotBindNames;
		TSet<FName> UsedBoneNames;
		RecursiveAddBones(NodeContainer, RootJointNodeId, JointInfos, INDEX_NONE, RefBonesBinary, UsedBoneNames, bUseTimeZeroAsBindPose, bOutDiffPose, BoneNotBindNames, bImportSockets);
		if (bOutDiffPose)
		{
			//bOutDiffPose can only be true if the user ask to bind with time zero transform.
			ensure(bUseTimeZeroAsBindPose);
		}
		//Do not output this warning in automation testing
		if (!GIsAutomationTesting && BoneNotBindNames.Num() > 0 && !bUseTimeZeroAsBindPose)
		{
			FString BonesWithoutBindPoses;
			for (const FString& BoneName : BoneNotBindNames)
			{
				BonesWithoutBindPoses += BoneName;
				BonesWithoutBindPoses += TEXT("  \n");
			}

			FText MissingBindPoseMessage = FText::Format(NSLOCTEXT("FSkeletonHelper", "ProcessImportMeshSkeleton__BonesAreMissingFromBindPose", "The following bones are missing from the bind pose:\n{0}\nThis can happen for bones that are not vert weighted. If they are not in the correct orientation after importing,\nplease set the \"Use T0 as ref pose\" option or add them to the bind pose and reimport the skeletal mesh.")
				, FText::FromString(BonesWithoutBindPoses));
			UInterchangeResultWarning_Generic* Message = Results->Add<UInterchangeResultWarning_Generic>();
			Message->Text = MissingBindPoseMessage;
		}
		// Digest bones to the serializable format.
		for (int32 b = 0; b < JointInfos.Num(); b++)
		{
			const FJointInfo& BinaryBone = JointInfos[b];

			const FString BoneName = BinaryBone.Name;
			const FMeshBoneInfo BoneInfo(FName(*BoneName, FNAME_Add), BinaryBone.Name, BinaryBone.ParentIndex);
			const FTransform BoneTransform(BinaryBone.LocalTransform);
			if (RefSkeleton.FindRawBoneIndex(BoneInfo.Name) != INDEX_NONE)
			{
				UInterchangeResultError_Generic* Message = Results->Add<UInterchangeResultError_Generic>();
				Message->Text = FText::Format(NSLOCTEXT("FSkeletonHelper", "ProcessImportMeshSkeleton_InvalidSkeletonUniqueNames", "Invalid Skeleton because of non - unique bone names [{0}].")
					, FText::FromName(BoneInfo.Name));
				return false;
			}
			RefSkelModifier.Add(BoneInfo, BoneTransform);
		}
		return true;
	}

	// This function should match USkeleton::IsCompatibleMesh semantic for consistency with the "Failed to merge bones" dialog that shows up when skeletons are incompatible.
	bool FSkeletonHelper::IsCompatibleSkeleton(const USkeleton* Skeleton
		, const FString RootJoinUid
		, const UInterchangeBaseNodeContainer* BaseNodeContainer
		, bool bConvertStaticToSkeletalActive
		, bool bCheckForIdenticalSkeleton
		, bool bImportSockets)
	{
		if (!Skeleton)
		{
			return false;
		}

		// at least % of bone should match 
		int32 NumOfBoneMatches = 0;
		//Make sure the specified Skeleton fit this skeletal mesh
		const FReferenceSkeleton& SkeletonRef = Skeleton->GetReferenceSkeleton();
		const int32 SkeletonBoneCount = SkeletonRef.GetRawBoneNum();

		TArray<FMeshBoneInfo> SkeletalLodRawInfos;
		SkeletalLodRawInfos.Reserve(SkeletonBoneCount);
		RecursiveBuildSkeletalSkeleton(RootJoinUid, INDEX_NONE, BaseNodeContainer, SkeletalLodRawInfos, bConvertStaticToSkeletalActive, bImportSockets);
		const int32 SkeletalLodBoneCount = SkeletalLodRawInfos.Num();

		// first ensure the parent exists for each bone
		for (int32 MeshBoneIndex = 0; MeshBoneIndex < SkeletalLodBoneCount; MeshBoneIndex++)
		{
			FName MeshBoneName = SkeletalLodRawInfos[MeshBoneIndex].Name;
			// See if Mesh bone exists in Skeleton.
			int32 SkeletonBoneIndex = SkeletonRef.FindBoneIndex(MeshBoneName);

			// if found, increase num of bone matches count
			if (SkeletonBoneIndex != INDEX_NONE)
			{
				++NumOfBoneMatches;

				// follow the parent chain to verify the chain is same
				if (!DoesParentChainMatch(SkeletonBoneIndex, SkeletonRef, SkeletalLodRawInfos))
				{
					//Not compatible
					return false;
				}
			}
			else
			{
				if (bCheckForIdenticalSkeleton)
				{
					return false;
				}
				int32 CurrentBoneId = MeshBoneIndex;
				// if not look for parents that matches
				while (SkeletonBoneIndex == INDEX_NONE && CurrentBoneId != INDEX_NONE)
				{
					// find Parent one see exists
					const int32 ParentMeshBoneIndex = SkeletalLodGetParentIndex(SkeletalLodRawInfos, CurrentBoneId);
					if (ParentMeshBoneIndex != INDEX_NONE)
					{
						// @TODO: make sure RefSkeleton's root ParentIndex < 0 if not, I'll need to fix this by checking TreeBoneIdx
						FName ParentBoneName = SkeletalLodGetBoneName(SkeletalLodRawInfos, ParentMeshBoneIndex);
						SkeletonBoneIndex = SkeletonRef.FindBoneIndex(ParentBoneName);
					}

					// root is reached
					if (ParentMeshBoneIndex == 0)
					{
						break;
					}
					else
					{
						CurrentBoneId = ParentMeshBoneIndex;
					}
				}

				// still no match, return false, no parent to look for
				if (SkeletonBoneIndex == INDEX_NONE)
				{
					return false;
				}

				// second follow the parent chain to verify the chain is same
				if (!DoesParentChainMatch(SkeletonBoneIndex, SkeletonRef, SkeletalLodRawInfos))
				{
					return false;
				}
			}
		}

		// originally we made sure at least matches more than 50% 
		// but then follower components can't play since they're only partial
		// if the hierarchy matches, and if it's more then 1 bone, we allow
		return (NumOfBoneMatches > 0);
	}

	void FSkeletonHelper::RecursiveAddSkeletonMetaDataValues(UInterchangeBaseNodeContainer* NodeContainer, UInterchangeBaseNode* DestinationNode, const FString& JointUid)
	{
		const UInterchangeJointNode* JointNode = Cast<UInterchangeJointNode>(NodeContainer->GetNode(JointUid));
		if (!JointNode)
		{
			return;
		}
		constexpr bool bAddSourceNodeName = true;
		UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(JointNode, DestinationNode, bAddSourceNodeName);

		// Iterate children
		TArray<FString>* CachedChildren = NodeContainer->GetCachedNodeChildrenUids(JointUid);

		//Note: If CacheChildren is null, that just means that the JointUid does not have children.
		if (!CachedChildren)
		{
			return;
		}

		const TArray<FString>& ChildrenIds = *CachedChildren;
		for (int32 ChildIndex = 0; ChildIndex < ChildrenIds.Num(); ++ChildIndex)
		{
			RecursiveAddSkeletonMetaDataValues(NodeContainer, DestinationNode, ChildrenIds[ChildIndex]);
		}
	}

	void FSkeletonHelper::RecursiveBoneHasBindPose(const UInterchangeBaseNodeContainer* NodeContainer, const FString& JointNodeId, bool& bHasBoneWithoutBindPose)
	{
		if (bHasBoneWithoutBindPose)
		{
			return;
		}

		//In case of rigid meshes, a regular SceneNode can still be considered a Joint.
		const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(NodeContainer->GetNode(JointNodeId));
		if (!SceneNode)
		{
			UE_LOGF(LogInterchangeImport, Warning, "Invalid Skeleton Joint");
			return;
		}
		
		// Only UInterchangeJointNode can have bind pose
		if (const UInterchangeJointNode* JointNode = Cast<UInterchangeJointNode>(SceneNode))
		{
			bool bHasInvalidBindPose;

			// Only consider joint nodes that has been set explicitly as not having a bind pose
			if (JointNode->GetHasInvalidBindPose(bHasInvalidBindPose) && bHasInvalidBindPose)
			{
				bHasBoneWithoutBindPose = true;
			}
		}

		if (!bHasBoneWithoutBindPose)
		{
			const TArray<FString> ChildrenIds = NodeContainer->GetNodeChildrenUids(JointNodeId);
			for (int32 ChildIndex = 0; ChildIndex < ChildrenIds.Num(); ++ChildIndex)
			{
				RecursiveBoneHasBindPose(NodeContainer, ChildrenIds[ChildIndex], bHasBoneWithoutBindPose);
			}
		}
	}

	void FSkeletonHelper::RecursiveAddBones(const UInterchangeBaseNodeContainer* NodeContainer
		, const FString& JointNodeId
		, TArray <FJointInfo>& JointInfos
		, int32 ParentIndex
		, TArray<SkeletalMeshImportData::FBone>& RefBonesBinary
		, TSet<FName>& UsedBoneNames
		, const bool bUseTimeZeroAsBindPose
		, bool& bOutDiffPose
		, TArray<FString>& OutBoneNotBindNames
		, bool bImportSockets)
	{
		// Produces a unique bone name for a particular node, and stores it on UsedBoneNames
		auto GetBoneNameForNode = [&UsedBoneNames](const UInterchangeBaseNode* Node)
		{
			FString Name = Node->GetDisplayLabel();

			if (UsedBoneNames.Contains(*Name))
			{
				const FString UniqueId = Node->GetUniqueID();
				Name += TEXT("_") + FMD5::HashAnsiString(*UniqueId);
			}

			ensure(!UsedBoneNames.Contains(*Name));
			UsedBoneNames.Add(*Name);
			return Name;
		};

		const UInterchangeBaseNode* BaseNode = NodeContainer->GetNode(JointNodeId);
		const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(BaseNode);
		if (!SceneNode)
		{
			if (JointInfos.Num() == 0 && BaseNode)
			{
				FJointInfo& Info = JointInfos.AddZeroed_GetRef();
				Info.Name = GetBoneNameForNode(BaseNode);
				Info.ParentIndex = INDEX_NONE;
				Info.LocalTransform = FTransform::Identity;

				SkeletalMeshImportData::FBone& Bone = RefBonesBinary.AddZeroed_GetRef();
				Bone.Name = Info.Name;
				Bone.ParentIndex = Info.ParentIndex;
				Bone.BonePos.Transform = FTransform3f::Identity;
				Bone.BonePos.Length = 0.0f;
				Bone.BonePos.XSize = 1.0f;
				Bone.BonePos.YSize = 1.0f;
				Bone.BonePos.ZSize = 1.0f;
			}
			else
			{
				UE_LOGF(LogInterchangeImport, Warning, "Invalid Skeleton Joint");
			}
			
			return;
		}

		int32 JointInfoIndex = JointInfos.Num();
		FJointInfo& Info = JointInfos.AddZeroed_GetRef();
		Info.Name = GetBoneNameForNode(SceneNode);

		FTransform LocalTransform;
		FTransform TimeZeroLocalTransform;
		bool bHasTimeZeroTransform = false;
		FTransform BindPoseLocalTransform;
		bool bHasBindPoseTransform = false;

		SceneNode->GetCustomLocalTransform(LocalTransform);

		const UInterchangeJointNode* JointNode = Cast<UInterchangeJointNode>(SceneNode);
		if (JointNode)
		{
			bHasTimeZeroTransform = JointNode->GetTimeZeroLocalTransform(TimeZeroLocalTransform);
			bHasBindPoseTransform = JointNode->GetBindPoseLocalTransform(BindPoseLocalTransform);
		}

		if (ParentIndex == INDEX_NONE)
		{
			FTransform GlobalOffsetTransform = FTransform::Identity;
			bool bBakeMeshes = false;
			if (UInterchangeCommonPipelineDataFactoryNode* CommonPipelineDataFactoryNode = UInterchangeCommonPipelineDataFactoryNode::GetUniqueInstance(NodeContainer))
			{
				CommonPipelineDataFactoryNode->GetCustomGlobalOffsetTransform(GlobalOffsetTransform);
				CommonPipelineDataFactoryNode->GetBakeMeshes(bBakeMeshes);
			}

			if (bBakeMeshes)
			{
				LocalTransform = FTransform::Identity;
				ensure(SceneNode->GetCustomGlobalTransform(NodeContainer, GlobalOffsetTransform, LocalTransform));

				if (JointNode)
				{
					bHasTimeZeroTransform = JointNode->GetTimeZeroGlobalTransform(NodeContainer, GlobalOffsetTransform, TimeZeroLocalTransform);
					bHasBindPoseTransform = JointNode->GetBindPoseGlobalTransform(NodeContainer, GlobalOffsetTransform, BindPoseLocalTransform);
				}
			}
		}

		Info.LocalTransform = bHasBindPoseTransform ? BindPoseLocalTransform : LocalTransform;
		//If user want to bind the mesh at time zero try to get the time zero transform
		if (bUseTimeZeroAsBindPose && bHasTimeZeroTransform)
		{
			if (bHasBindPoseTransform)
			{
				if (!TimeZeroLocalTransform.Equals(Info.LocalTransform))
				{
					bOutDiffPose = true;
				}
			}
			Info.LocalTransform = TimeZeroLocalTransform;
		}
		else if (JointNode && !GIsAutomationTesting && !bHasBindPoseTransform && !bUseTimeZeroAsBindPose)
		{
			//StaticMeshes converted to SkeletalMeshes are not expected to have BindPoses
			OutBoneNotBindNames.Add(Info.Name);
		}

		Info.ParentIndex = ParentIndex;

		SkeletalMeshImportData::FBone& Bone = RefBonesBinary.AddZeroed_GetRef();
		Bone.Name = Info.Name;
		Bone.BonePos.Transform = FTransform3f(Info.LocalTransform);
		Bone.ParentIndex = ParentIndex;
		//Fill the scrap we do not need
		Bone.BonePos.Length = 0.0f;
		Bone.BonePos.XSize = 1.0f;
		Bone.BonePos.YSize = 1.0f;
		Bone.BonePos.ZSize = 1.0f;

		const TArray<FString> ChildrenIds = NodeContainer->GetNodeChildrenUids(JointNodeId);
		Bone.NumChildren = ChildrenIds.Num();
		for (int32 ChildIndex = 0; ChildIndex < ChildrenIds.Num(); ++ChildIndex)
		{
			const FString& ChildUid = ChildrenIds[ChildIndex];
			if(const UInterchangeSceneNode* ChildJointNode = Cast<UInterchangeSceneNode>(NodeContainer->GetNode(ChildUid)))
			{
				if (bImportSockets && IsValidSocket(NodeContainer, ChildJointNode))
				{
					//Sockets will be add by the skeletal mesh factory
					//We don't want a socket to be a skeleton joint
					continue;
				}
				RecursiveAddBones(NodeContainer, ChildrenIds[ChildIndex], JointInfos, JointInfoIndex, RefBonesBinary, UsedBoneNames, bUseTimeZeroAsBindPose, bOutDiffPose, OutBoneNotBindNames, bImportSockets);
			}
		}
	}

	FName FSkeletonHelper::SkeletalLodGetBoneName(const TArray<FMeshBoneInfo>& SkeletalLodRawInfos, int32 BoneIndex)
	{
		if (SkeletalLodRawInfos.IsValidIndex(BoneIndex))
		{
			return SkeletalLodRawInfos[BoneIndex].Name;
		}
		return NAME_None;
	}

	int32 FSkeletonHelper::SkeletalLodFindBoneIndex(const TArray<FMeshBoneInfo>& SkeletalLodRawInfos, FName BoneName)
	{
		const int32 BoneCount = SkeletalLodRawInfos.Num();
		for (int32 BoneIndex = 0; BoneIndex < BoneCount; ++BoneIndex)
		{
			if (SkeletalLodRawInfos[BoneIndex].Name == BoneName)
			{
				return BoneIndex;
			}
		}
		return INDEX_NONE;
	}

	int32 FSkeletonHelper::SkeletalLodGetParentIndex(const TArray<FMeshBoneInfo>& SkeletalLodRawInfos, int32 BoneIndex)
	{
		if (SkeletalLodRawInfos.IsValidIndex(BoneIndex))
		{
			return SkeletalLodRawInfos[BoneIndex].ParentIndex;
		}
		return INDEX_NONE;
	}

	bool FSkeletonHelper::DoesParentChainMatch(int32 StartBoneIndex, const FReferenceSkeleton& SkeletonRef, const TArray<FMeshBoneInfo>& SkeletalLodRawInfos)
	{
		// if start is root bone
		if (StartBoneIndex == 0)
		{
			// verify name of root bone matches
			return (SkeletonRef.GetBoneName(0) == SkeletalLodGetBoneName(SkeletalLodRawInfos, 0));
		}

		int32 SkeletonBoneIndex = StartBoneIndex;
		// If skeleton bone is not found in mesh, fail.
		int32 MeshBoneIndex = SkeletalLodFindBoneIndex(SkeletalLodRawInfos, SkeletonRef.GetBoneName(SkeletonBoneIndex));
		if (MeshBoneIndex == INDEX_NONE)
		{
			return false;
		}
		do
		{
			// verify if parent name matches
			int32 ParentSkeletonBoneIndex = SkeletonRef.GetParentIndex(SkeletonBoneIndex);
			int32 ParentMeshBoneIndex = SkeletalLodGetParentIndex(SkeletalLodRawInfos, MeshBoneIndex);

			// if one of the parents doesn't exist, make sure both end. Otherwise fail.
			if ((ParentSkeletonBoneIndex == INDEX_NONE) || (ParentMeshBoneIndex == INDEX_NONE))
			{
				return (ParentSkeletonBoneIndex == ParentMeshBoneIndex);
			}

			// If parents are not named the same, fail.
			if (SkeletonRef.GetBoneName(ParentSkeletonBoneIndex) != SkeletalLodGetBoneName(SkeletalLodRawInfos, ParentMeshBoneIndex))
			{
				return false;
			}

			// move up
			SkeletonBoneIndex = ParentSkeletonBoneIndex;
			MeshBoneIndex = ParentMeshBoneIndex;
		} while (true);
	}

	void FSkeletonHelper::RecursiveBuildSkeletalSkeleton(const FString JoinToAddUid
		, const int32 ParentIndex
		, const UInterchangeBaseNodeContainer* BaseNodeContainer
		, TArray<FMeshBoneInfo>& SkeletalLodRawInfos
		, bool bConvertStaticToSkeletalActive
		, bool bImportSockets)
	{
		const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(JoinToAddUid));
		if (!SceneNode)
		{
			return;
		}

		if (!bConvertStaticToSkeletalActive && !SceneNode->IsA(UInterchangeJointNode::StaticClass()))
		{
			return;
		}

		if (bImportSockets && IsValidSocket(BaseNodeContainer, SceneNode))
		{
			return;
		}

		int32 JoinIndex = SkeletalLodRawInfos.Num();
		FMeshBoneInfo& Info = SkeletalLodRawInfos.AddZeroed_GetRef();
		Info.Name = *SceneNode->GetDisplayLabel();
		Info.ParentIndex = ParentIndex;
#if WITH_EDITORONLY_DATA
		Info.ExportName = Info.Name.ToString();
#endif
		//Iterate children
		const TArray<FString> ChildrenIds = BaseNodeContainer->GetNodeChildrenUids(JoinToAddUid);
		for (int32 ChildIndex = 0; ChildIndex < ChildrenIds.Num(); ++ChildIndex)
		{
			RecursiveBuildSkeletalSkeleton(ChildrenIds[ChildIndex], JoinIndex, BaseNodeContainer, SkeletalLodRawInfos, bConvertStaticToSkeletalActive, bImportSockets);
		}
	}
} //namespace UE::Interchange::Private

#endif //WITH_EDITOR