// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Mesh/InterchangeStaticMeshFactory.h"

#include "InterchangeAssetImportData.h"
#include "InterchangeCommonPipelineDataFactoryNode.h"
#include "InterchangeImportCommon.h"
#include "InterchangeImportLog.h"
#include "InterchangeManager.h"
#include "InterchangeMeshBundleNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangeSceneComponentNodes.h"
#include "InterchangeSceneNode.h"
#include "InterchangeSourceData.h"
#include "InterchangeStaticMeshFactoryNode.h"
#include "InterchangeStaticMeshLodDataNode.h"
#include "InterchangeTranslatorBase.h"
#include "Mesh/InterchangeMeshHelper.h"
#include "Mesh/InterchangeMeshPayloadInterface.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeSourceNode.h"

#include "Components.h"
#include "Engine/StaticMesh.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "MeshBudgetProjectSettings.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshCompiler.h"
#include "StaticMeshOperations.h"
#include "StaticMeshResources.h"
#if WITH_EDITOR
#include "Editor/UnrealEd/Private/GeomFitUtils.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeStaticMeshFactory)

#if WITH_EDITORONLY_DATA
#include "EditorFramework/AssetImportData.h"
#endif //WITH_EDITORONLY_DATA

static bool GInterchangeUseHashAsGuid = true;
static FAutoConsoleVariableRef CCvarInterchangeUseHashAsGuid(
	TEXT("Interchange.FeatureFlags.Import.UseHashAsGuid"),
	GInterchangeUseHashAsGuid,
	TEXT("Whether to enable the bUseHashAsGuid option when importing Static Meshes. Useful to prevent recomputing content already in cache and improve import time."),
	ECVF_Default);

static bool GInterchangeMarkAsNotHavingNavigationData = false;
static FAutoConsoleVariableRef CCvarInterchangeMarkAsNotHavingNavigationData(
	TEXT("Interchange.FeatureFlags.Import.MarkAsNotHavingNavigationData"),
	GInterchangeMarkAsNotHavingNavigationData,
	TEXT("If true, newly imported Static Meshes will have \"Has Navigation Data\" option disabled and will be ignored during navmesh generation."),
	ECVF_Default);

static bool GInterchangeNeverNeedsCookedCollisionData = false;
static FAutoConsoleVariableRef CCvarInterchangeNeverNeedsCookedCollisionData(
	TEXT("Interchange.FeatureFlags.Import.NeverNeedsCookedCollisionData"),
	GInterchangeNeverNeedsCookedCollisionData,
	TEXT("If true, imported Static Meshes will have \"Never Needs Cooked Collision Data\" option enabled and will prevent them from having a Complex Collision. Only used if Import Collisions setting is disabled."),
	ECVF_Default);

UClass* UInterchangeStaticMeshFactory::GetFactoryClass() const
{
	return UStaticMesh::StaticClass();
}

void UInterchangeStaticMeshFactory::CreatePayloadTasks(const FImportAssetObjectParams& Arguments, bool bAsync, TArray<TSharedPtr<UE::Interchange::FInterchangeTaskBase>>& PayloadTasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeStaticMeshFactory::CreatePayloadTasks);

	UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode = Cast<UInterchangeStaticMeshFactoryNode>(Arguments.AssetNode);
	if (StaticMeshFactoryNode == nullptr)
	{
		return;
	}

	TArray<FString> LodDataUniqueIds;
	StaticMeshFactoryNode->GetLodDataUniqueIds(LodDataUniqueIds);

	const int32 LodCount = FMath::Min(LodDataUniqueIds.Num(), MAX_STATIC_MESH_LODS);

	ensure(LodDataUniqueIds.Num() >= LodCount);

	const IInterchangeMeshPayloadInterface* MeshTranslatorPayloadInterface = Cast<IInterchangeMeshPayloadInterface>(Arguments.Translator);
	if (!MeshTranslatorPayloadInterface)
	{
		UE_LOGF(LogInterchangeImport, Error, "Cannot import static mesh. The translator does not implement IInterchangeMeshPayloadInterface.");
		return;
	}

	FTransform GlobalOffsetTransform = FTransform::Identity;
	bool bBakeMeshes = false;
	bool bBakePivotMeshes = false;
	if (UInterchangeCommonPipelineDataFactoryNode* CommonPipelineDataFactoryNode = UInterchangeCommonPipelineDataFactoryNode::GetUniqueInstance(Arguments.NodeContainer))
	{
		CommonPipelineDataFactoryNode->GetCustomGlobalOffsetTransform(GlobalOffsetTransform);
		CommonPipelineDataFactoryNode->GetBakeMeshes(bBakeMeshes);
		if (!bBakeMeshes)
		{
			CommonPipelineDataFactoryNode->GetBakePivotMeshes(bBakePivotMeshes);
		}
	}

	PayloadsPerLodIndex.Reserve(LodCount);
	int32 CurrentLodIndex = 0;

	UE::Interchange::FAttributeStorage PayloadAttributes;
	UInterchangeMeshFactoryNode::CopyPayloadKeyStorageAttributes(StaticMeshFactoryNode, PayloadAttributes);

	for (int32 LodIndex = 0; LodIndex < LodCount; ++LodIndex)
	{
		FString LodUniqueId = LodDataUniqueIds[LodIndex];
		const UInterchangeStaticMeshLodDataNode* LodDataNode = Cast<UInterchangeStaticMeshLodDataNode>(Arguments.NodeContainer->GetNode(LodUniqueId));
		if (!LodDataNode)
		{
			UE_LOGF(LogInterchangeImport, Warning, "Invalid LOD when importing StaticMesh asset %ls.", *Arguments.AssetName);
			continue;
		}

		FLodPayloads& LodPayloads = PayloadsPerLodIndex.FindOrAdd(LodIndex);

		auto AddMeshPayloads = [&Arguments, MeshTranslatorPayloadInterface, bBakeMeshes, bBakePivotMeshes, &GlobalOffsetTransform, &PayloadTasks, bAsync, &PayloadAttributes, LodCount]
			(const TArray<FInterchangeLODMeshData>& MeshNodeUidTransformsPairs, TMap<FInterchangeMeshPayLoadKey, UE::Interchange::FMeshPayload>& PayloadPerKey)
			{
				PayloadPerKey.Reserve(MeshNodeUidTransformsPairs.Num());

				for (const FInterchangeLODMeshData& MeshNodeUidTransformsPair : MeshNodeUidTransformsPairs)
				{
					const UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(Arguments.NodeContainer->GetNode(MeshNodeUidTransformsPair.MeshUid));
					
					if (!ensure(MeshNode))
					{
						UE_LOGF(LogInterchangeImport, Warning, "Invalid LOD mesh reference when importing StaticMesh asset %ls.", *Arguments.AssetName);
						continue;
					}

					TOptional<FInterchangeMeshPayLoadKey> OptionalPayLoadKey = MeshNode->GetPayLoadKey();
					if (!ensure(OptionalPayLoadKey.IsSet()))
					{
						UE_LOGF(LogInterchangeImport, Warning, "Empty LOD mesh reference payload when importing StaticMesh asset %ls.", *Arguments.AssetName);
						continue;
					}

					FInterchangeMeshPayLoadKey& PayLoadKey = OptionalPayLoadKey.GetValue();
					
					FInterchangeMeshPayLoadKey GlobalPayLoadKey = PayLoadKey;
					GlobalPayLoadKey.UniqueId += FInterchangeMeshPayLoadKey::GetTransformString(MeshNodeUidTransformsPair.Transform);
					if (!PayloadPerKey.Contains(GlobalPayLoadKey))
					{
						UE::Interchange::FMeshPayload& Payload = PayloadPerKey.FindOrAdd(GlobalPayLoadKey);
						Payload.Transform = MeshNodeUidTransformsPair.Transform;
						Payload.MeshName = PayLoadKey.UniqueId;
						PayloadAttributes.RegisterAttribute(UE::Interchange::FAttributeKey{ UE::Interchange::MeshPayload::Attributes::MeshGlobalTransform }, MeshNodeUidTransformsPair.Transform);
						TSharedPtr<UE::Interchange::FInterchangeTaskLambda, ESPMode::ThreadSafe> TaskGetMeshPayload = MakeShared<UE::Interchange::FInterchangeTaskLambda, ESPMode::ThreadSafe>(bAsync ? UE::Interchange::EInterchangeTaskThread::AsyncThread : UE::Interchange::EInterchangeTaskThread::GameThread
							, [&Payload, MeshTranslatorPayloadInterface, PayLoadKey, PayloadAttributes]()
							{
								TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeStaticMeshFactory::GetMeshPayloadDataTask);
								if (ensure(!Payload.PayloadData.IsSet()))
								{
									Payload.PayloadData = MeshTranslatorPayloadInterface->GetMeshPayloadData(PayLoadKey, PayloadAttributes);
								}
							});
						PayloadTasks.Add(TaskGetMeshPayload);
					}
				}
			};

		TArray<FInterchangeLODMeshData> LODMeshDataArray;
		LodDataNode->GetLODMeshDataArray(LODMeshDataArray);
		AddMeshPayloads(LODMeshDataArray, LodPayloads.MeshPayloadPerKey);

		if (LodIndex == 0)
		{
			TArray<FInterchangeLODMeshData> BoxColliderMeshDataArray;
			LodDataNode->GetBoxCollisionsMeshDataArray(BoxColliderMeshDataArray);
			AddMeshPayloads(BoxColliderMeshDataArray, LodPayloads.CollisionBoxPayloadPerKey);

			TArray<FInterchangeLODMeshData> SphereColliderMeshDataArray;
			LodDataNode->GetSphereCollisionsMeshDataArray(SphereColliderMeshDataArray);
			AddMeshPayloads(SphereColliderMeshDataArray, LodPayloads.CollisionSpherePayloadPerKey);

			TArray<FInterchangeLODMeshData> CapsuleColliderMeshDataArray;
			LodDataNode->GetCapsuleCollisionsMeshDataArray(CapsuleColliderMeshDataArray);
			AddMeshPayloads(CapsuleColliderMeshDataArray, LodPayloads.CollisionCapsulePayloadPerKey);

			TArray<FInterchangeLODMeshData> ConvexColliderMeshDataArray;
			LodDataNode->GetConvexCollisionsMeshDataArray(ConvexColliderMeshDataArray);
			AddMeshPayloads(ConvexColliderMeshDataArray, LodPayloads.CollisionConvexPayloadPerKey);
		}
	}
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeStaticMeshFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeStaticMeshFactory::BeginImportAsset_GameThread);

	//We must ensure we use the same settings until the import is finish, EditorUtilities->IsRuntimeOrPIE() can return a different
	//value during an asynchronous import
	ImportAssetObjectData.bIsAppGame = false;
	if (UInterchangeEditorUtilitiesBase* EditorUtilities = UInterchangeManager::GetInterchangeManager().GetEditorUtilities())
	{
		ImportAssetObjectData.bIsAppGame = EditorUtilities->IsRuntimeOrPIE();
	}

	FImportAssetResult ImportAssetResult;
	UStaticMesh* StaticMesh = nullptr;
	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	const UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode = Cast<UInterchangeStaticMeshFactoryNode>(Arguments.AssetNode);
	if (StaticMeshFactoryNode == nullptr)
	{
		return ImportAssetResult;
	}

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (StaticMeshFactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}
	}

	// create a new static mesh or overwrite existing asset, if possible
	if (!ExistingAsset)
	{
		StaticMesh = NewObject<UStaticMesh>(Arguments.Parent, *Arguments.AssetName, RF_Public | RF_Standalone);

		if (GInterchangeMarkAsNotHavingNavigationData)
		{
			StaticMesh->MarkAsNotHavingNavigationData();
		}
	}
	else
	{
		//This is a reimport, we are just re-updating the source data
		StaticMesh = Cast<UStaticMesh>(ExistingAsset);

		// Clear the render data on the existing static mesh from the game thread so that we're ready to update it
		if (StaticMesh && StaticMesh->AreRenderingResourcesInitialized())
		{
			const bool bInvalidateLighting = true;
			const bool bRefreshBounds = true;
			FStaticMeshComponentRecreateRenderStateContext RecreateRenderStateContext(StaticMesh, bInvalidateLighting, bRefreshBounds);
			StaticMesh->ReleaseResources();
			StaticMesh->ReleaseResourcesFence.Wait();
			
			StaticMesh->SetRenderData(nullptr);
		}
	}
	
	if (!StaticMesh)
	{
		if (!Arguments.ReimportObject)
		{
			UE_LOGF(LogInterchangeImport, Warning, "Could not create StaticMesh asset %ls.", *Arguments.AssetName);
		}
		return ImportAssetResult;
	}

	// create the BodySetup on the game thread
	if (!ExistingAsset)
	{
		StaticMesh->CreateBodySetup();
	}
	
#if WITH_EDITOR
	if (!ImportAssetObjectData.bIsAppGame)
	{
		StaticMesh->PreEditChange(nullptr);
	}
#endif // WITH_EDITOR

	ImportAssetResult.ImportedObject = StaticMesh;
	return ImportAssetResult;
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeStaticMeshFactory::ImportAsset_Async(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeStaticMeshFactory::ImportAsset_Async);

	FImportAssetResult ImportAssetResult;
	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode = Cast<UInterchangeStaticMeshFactoryNode>(Arguments.AssetNode);
	if (StaticMeshFactoryNode == nullptr)
	{
		return ImportAssetResult;
	}

	UObject* StaticMeshObject = UE::Interchange::FFactoryCommon::AsyncFindObject(StaticMeshFactoryNode, GetFactoryClass(), Arguments.Parent, Arguments.AssetName);
	const bool bReimport = Arguments.ReimportObject && StaticMeshObject;

	if (!StaticMeshObject)
	{
		UE_LOGF(LogInterchangeImport, Error, "Could not import the StaticMesh asset %ls because the asset does not exist.", *Arguments.AssetName);
		return ImportAssetResult;
	}

	UStaticMesh* StaticMesh = Cast<UStaticMesh>(StaticMeshObject);
	if (!ensure(StaticMesh))
	{
		UE_LOGF(LogInterchangeImport, Error, "Could not cast to StaticMesh asset %ls.", *Arguments.AssetName);
		return ImportAssetResult;
	}

	ensure(!StaticMesh->AreRenderingResourcesInitialized());

	TArray<FString> LodDataUniqueIds;
	StaticMeshFactoryNode->GetLodDataUniqueIds(LodDataUniqueIds);

	const int32 LodCount = FMath::Min(LodDataUniqueIds.Num(), MAX_STATIC_MESH_LODS);
	if (LodCount != LodDataUniqueIds.Num())
	{
		const int32 LodCountDiff = LodDataUniqueIds.Num() - MAX_STATIC_MESH_LODS;
		UE_LOGF(LogInterchangeImport, Warning, "Reached the maximum number of LODs for a Static Mesh (%d) - discarding %d LOD meshes.", MAX_STATIC_MESH_LODS, LodCountDiff);
	}
	ensure(LodDataUniqueIds.Num() >= LodCount);

#if WITH_EDITOR
	const int32 PrevLodCount = StaticMesh->GetNumSourceModels();
	const int32 FinalLodCount = FMath::Max(PrevLodCount, LodCount);
	StaticMesh->SetNumSourceModels(FinalLodCount);
#endif // WITH_EDITOR

	// If we are reimporting, cache the existing vertex colors so they can be optionally reapplied after reimport
	TMap<FVector3f, FColor> ExisitingVertexColorData;
	if (bReimport)
	{
		StaticMesh->GetVertexColorData(ExisitingVertexColorData);
	}

	bool bKeepSectionsSeparate = false;
	StaticMeshFactoryNode->GetCustomKeepSectionsSeparate(bKeepSectionsSeparate);

	
	//Call the mesh helper to create the missing material and to use the unmatched existing slot with the unmatch import slot
	{
		using namespace UE::Interchange::Private::MeshHelper;
		TMap<FString, FString> SlotMaterialDependencies;
		StaticMeshFactoryNode->GetSlotMaterialDependencies(SlotMaterialDependencies);
		StaticMeshFactorySetupAssetMaterialArray(StaticMesh->GetStaticMaterials(), SlotMaterialDependencies, Arguments.NodeContainer, bReimport);
	}

	TArray<FMeshDescription>& LodMeshDescriptions = ImportAssetObjectData.LodMeshDescriptions;
	LodMeshDescriptions.SetNum(LodCount);

	bool bImportCollision = false;
	EInterchangeMeshCollision Collision = EInterchangeMeshCollision::None;
	bool bImportedCustomCollision = false;
	int32 CurrentLodIndex = 0;
	for (int32 LodIndex = 0; LodIndex < LodCount; ++LodIndex)
	{
		if (!PayloadsPerLodIndex.Contains(LodIndex))
		{
			UE_LOGF(LogInterchangeImport, Error, "LOD %d do not have any valid payload to create a mesh when importing StaticMesh asset %ls.", LodIndex, *Arguments.AssetName);
			continue;
		}

		FString LodUniqueId = LodDataUniqueIds[LodIndex];
		const UInterchangeStaticMeshLodDataNode* LodDataNode = Cast<UInterchangeStaticMeshLodDataNode>(Arguments.NodeContainer->GetNode(LodUniqueId));
		if (!LodDataNode)
		{
			UE_LOGF(LogInterchangeImport, Warning, "Invalid LOD when importing StaticMesh asset %ls.", *Arguments.AssetName);
			continue;
		}

		// Add the lod mesh data to the static mesh
		FMeshDescription& LodMeshDescription = LodMeshDescriptions[CurrentLodIndex];

		FStaticMeshOperations::FAppendSettings AppendSettings;
		for (int32 ChannelIdx = 0; ChannelIdx < FStaticMeshOperations::FAppendSettings::MAX_NUM_UV_CHANNELS; ++ChannelIdx)
		{
			AppendSettings.bMergeUVChannels[ChannelIdx] = true;
		}
		
		// Merge the SourceObjectName attribute, if it exists on both the target and source.
		AppendSettings.bMergeObjectName = true;

		// Fill the lod mesh description using all combined mesh parts
		FLodPayloads* LodPayloads = &PayloadsPerLodIndex.FindChecked(LodIndex);

		// Just move the mesh description from the first valid payload then append the rest
		bool bFirstValidMoved = false;

		for (TPair<FInterchangeMeshPayLoadKey, UE::Interchange::FMeshPayload>& KeyAndPayload : LodPayloads->MeshPayloadPerKey)
		{
			const TOptional<UE::Interchange::FMeshPayloadData>& LodMeshPayload = KeyAndPayload.Value.PayloadData;
			if (!LodMeshPayload.IsSet())
			{
				UE_LOGF(LogInterchangeImport, Warning, "Invalid static mesh payload key for StaticMesh asset %ls.", *Arguments.AssetName);
				continue;
			}

			if (!bFirstValidMoved)
			{
				FMeshDescription& MeshDescription = const_cast<FMeshDescription&>(LodMeshPayload->MeshDescription);
				if (MeshDescription.IsEmpty())
				{
					continue;
				}
				LodMeshDescription = MoveTemp(MeshDescription); // This is emptying the mesh description inside PayloadsPerLodIndex
				bFirstValidMoved = true;
			}
			else
			{
				if (LodMeshPayload->MeshDescription.IsEmpty())
				{
					continue;
				}
				if (bKeepSectionsSeparate)
				{
					AppendSettings.PolygonGroupsDelegate = FAppendPolygonGroupsDelegate::CreateLambda([](const FMeshDescription& SourceMesh, FMeshDescription& TargetMesh, PolygonGroupMap& RemapPolygonGroup)
						{
							UE::Interchange::Private::MeshHelper::RemapPolygonGroups(SourceMesh, TargetMesh, RemapPolygonGroup);
						});
				}
				FStaticMeshOperations::AppendMeshDescription(LodMeshPayload->MeshDescription, LodMeshDescription, AppendSettings);
			}
		}

		// Manage vertex color
		// Replace -> do nothing, we want to use the translated source data
		// Ignore -> remove vertex color from import data (when we re-import, ignore have to put back the current mesh vertex color)
		// Override -> replace the vertex color by the override color
		// @todo: new mesh description attribute for painted vertex colors?
		{
			FStaticMeshAttributes Attributes(LodMeshDescription);
			TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
			bool bReplaceVertexColor = false;
			StaticMeshFactoryNode->GetCustomVertexColorReplace(bReplaceVertexColor);
			if (!bReplaceVertexColor)
			{
				bool bIgnoreVertexColor = false;
				StaticMeshFactoryNode->GetCustomVertexColorIgnore(bIgnoreVertexColor);
				if (bIgnoreVertexColor)
				{
					for (const FVertexInstanceID& VertexInstanceID : LodMeshDescription.VertexInstances().GetElementIDs())
					{
						//If we have old vertex color (reimport), we want to keep it if the option is ignore
						if (ExisitingVertexColorData.Num() > 0)
						{
							const FVector3f& VertexPosition = LodMeshDescription.GetVertexPosition(LodMeshDescription.GetVertexInstanceVertex(VertexInstanceID));
							const FColor* PaintedColor = ExisitingVertexColorData.Find(VertexPosition);
							if (PaintedColor)
							{
								// A matching color for this vertex was found
								VertexInstanceColors[VertexInstanceID] = FVector4f(FLinearColor(*PaintedColor));
							}
							else
							{
								//Flush the vertex color
								VertexInstanceColors[VertexInstanceID] = FVector4f(FLinearColor(FColor::White));
							}
						}
						else
						{
							//Flush the vertex color
							VertexInstanceColors[VertexInstanceID] = FVector4f(FLinearColor(FColor::White));
						}
					}
				}
				else
				{
					FColor OverrideVertexColor;
					if (StaticMeshFactoryNode->GetCustomVertexColorOverride(OverrideVertexColor))
					{
						for (const FVertexInstanceID& VertexInstanceID : LodMeshDescription.VertexInstances().GetElementIDs())
						{
							VertexInstanceColors[VertexInstanceID] = FVector4f(FLinearColor(OverrideVertexColor));
						}
					}
				}
			}
		}

		// Import collision geometry
		if (CurrentLodIndex == 0)
		{
			LodDataNode->GetImportCollision(bImportCollision);
			LodDataNode->GetImportCollisionType(Collision);
			if(bImportCollision)
			{
				if (bReimport)
				{
					//Let's clean only the imported collisions first in order 
					// to store the previous editor-generated collisions to re-generate them later in the Game Thread with their properties
					ImportAssetObjectData.AggregateGeom = StaticMesh->GetBodySetup()->AggGeom;
					StaticMesh->GetBodySetup()->AggGeom.EmptyElements();
				}

				bool bForceGeneration = false;
				if (!LodDataNode->GetForceCollisionPrimitiveGeneration(bForceGeneration))
				{
					bForceGeneration = false;
				}

				using namespace UE::Interchange::Private::MeshHelper;
				bImportedCustomCollision |= ImportBoxCollision(Arguments, LodPayloads->CollisionBoxPayloadPerKey, StaticMesh, bForceGeneration);
				bImportedCustomCollision |= ImportCapsuleCollision(Arguments, LodPayloads->CollisionCapsulePayloadPerKey, StaticMesh);
				bImportedCustomCollision |= ImportSphereCollision(Arguments, LodPayloads->CollisionSpherePayloadPerKey, StaticMesh, bForceGeneration);
				bImportedCustomCollision |= ImportConvexCollision(Arguments, LodPayloads->CollisionConvexPayloadPerKey, StaticMesh, LodDataNode);
			}
		}

		CurrentLodIndex++;
	}

#if WITH_EDITOR
	{
		// Default to AutoComputeLODScreenSizes in case the attribute is not set.
		bool bAutoComputeLODScreenSize = true;
		StaticMeshFactoryNode->GetCustomAutoComputeLODScreenSizes(bAutoComputeLODScreenSize);

		TArray<float> LodScreenSizes;
		StaticMeshFactoryNode->GetLODScreenSizes(LodScreenSizes);

		const bool bIsAReimport = Arguments.ReimportObject != nullptr;
		SetupSourceModelsSettings(*StaticMesh, LodMeshDescriptions, bAutoComputeLODScreenSize, LodScreenSizes, PrevLodCount, FinalLodCount, bIsAReimport);

		// SetupSourceModelsSettings can change the destination lightmap UV index
		// Make sure the destination lightmap UV index on the factory node takes
		// in account the potential change
		int32 FactoryDstLightmapIndex;
		if (StaticMeshFactoryNode->GetCustomDstLightmapIndex(FactoryDstLightmapIndex) && StaticMesh->GetLightMapCoordinateIndex() > FactoryDstLightmapIndex)
		{
			StaticMeshFactoryNode->SetCustomDstLightmapIndex(StaticMesh->GetLightMapCoordinateIndex());
		}
	}
#endif // WITH_EDITOR

	ImportAssetObjectData.bImportCollision = bImportCollision;
	ImportAssetObjectData.Collision = Collision;
	ImportAssetObjectData.bImportedCustomCollision = bImportedCustomCollision;

	// Getting the file Hash will cache it into the source data
	Arguments.SourceData->GetFileContentHash();

	BuildFromMeshDescriptions(*StaticMesh);

	ImportAssetResult.ImportedObject = StaticMeshObject;
	return ImportAssetResult;
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeStaticMeshFactory::EndImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeStaticMeshFactory::EndImportAsset_GameThread);

	FImportAssetResult ImportAssetResult;
	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode = Cast<UInterchangeStaticMeshFactoryNode>(Arguments.AssetNode);
	if (StaticMeshFactoryNode == nullptr)
	{
		return ImportAssetResult;
	}

	// create an asset if it doesn't exist
	UObject* ExistingAsset = StaticFindObject(nullptr, Arguments.Parent, *Arguments.AssetName);

	const bool bReimport = Arguments.ReimportObject && ExistingAsset;

	UStaticMesh* StaticMesh = Cast<UStaticMesh>(ExistingAsset);
	if (!ensure(StaticMesh))
	{
		UE_LOGF(LogInterchangeImport, Error, "Could not create StaticMesh asset %ls.", *Arguments.AssetName);
		return ImportAssetResult;
	}

	if (ImportAssetObjectData.bIsAppGame)
	{
		if (!Arguments.ReimportObject)
		{
			// Apply all StaticMeshFactoryNode custom attributes to the static mesh asset
			StaticMeshFactoryNode->ApplyAllCustomAttributeToObject(StaticMesh);
		}

		ImportAssetResult.ImportedObject = StaticMesh;
		return ImportAssetResult;
	}

	for (int32 LodIndex = 0; LodIndex < ImportAssetObjectData.LodMeshDescriptions.Num(); ++LodIndex)
	{
		// Add the lod mesh data to the static mesh
		FMeshDescription& LodMeshDescription = ImportAssetObjectData.LodMeshDescriptions[LodIndex];
		if (LodMeshDescription.IsEmpty())
		{
			//All the valid Mesh description are at the beginning of the array
			break;
		}

		// Build section info map from materials
		FStaticMeshConstAttributes StaticMeshAttributes(LodMeshDescription);
		TPolygonGroupAttributesRef<const FName> SlotNames = StaticMeshAttributes.GetPolygonGroupMaterialSlotNames();
#if WITH_EDITOR
		if (bReimport)
		{
			//Match the existing section info map data

			//First find the old mesh description polygon groups name that match with the imported mesh description polygon groups name.
			//Copy the data
			const int32 PreviousSectionCount = StaticMesh->GetSectionInfoMap().GetSectionNumber(LodIndex);
			TMap<FPolygonGroupID, FPolygonGroupID> ImportedToOldPolygonGroupMatch;
			ImportedToOldPolygonGroupMatch.Reserve(LodMeshDescription.PolygonGroups().Num());
			if (StaticMesh->IsMeshDescriptionValid(LodIndex))
			{
				//Match incoming mesh description with the old mesh description
				FMeshDescription& OldMeshDescription = *StaticMesh->GetMeshDescription(LodIndex);
				FStaticMeshConstAttributes OldStaticMeshAttributes(OldMeshDescription);
				TPolygonGroupAttributesRef<const FName> OldSlotNames = OldStaticMeshAttributes.GetPolygonGroupMaterialSlotNames();
				for (FPolygonGroupID PolygonGroupID : LodMeshDescription.PolygonGroups().GetElementIDs())
				{
					for (FPolygonGroupID OldPolygonGroupID : OldMeshDescription.PolygonGroups().GetElementIDs())
					{
						if (SlotNames[PolygonGroupID] == OldSlotNames[OldPolygonGroupID])
						{
							ImportedToOldPolygonGroupMatch.FindOrAdd(PolygonGroupID) = OldPolygonGroupID;
							break;
						}
					}
				}
			}
			//Create a new set of mesh section info for this lod
			TArray<FMeshSectionInfo> NewSectionInfoMapData;
			NewSectionInfoMapData.Reserve(LodMeshDescription.PolygonGroups().Num());
			for (FPolygonGroupID PolygonGroupID : LodMeshDescription.PolygonGroups().GetElementIDs())
			{
				if (FPolygonGroupID* OldPolygonGroupID = ImportedToOldPolygonGroupMatch.Find(PolygonGroupID))
				{
					if (StaticMesh->GetSectionInfoMap().IsValidSection(LodIndex, OldPolygonGroupID->GetValue()))
					{
						NewSectionInfoMapData.Add(StaticMesh->GetSectionInfoMap().Get(LodIndex, OldPolygonGroupID->GetValue()));
					}
				}
				else
				{
					//This is an unmatched section, its either added or we did not recover the name
					int32 MaterialSlotIndex = StaticMesh->GetMaterialIndexFromImportedMaterialSlotName(SlotNames[PolygonGroupID]);
					//Missing material slot should have been added before
					if (MaterialSlotIndex == INDEX_NONE)
					{
						MaterialSlotIndex = 0;
					}
					NewSectionInfoMapData.Add(FMeshSectionInfo(MaterialSlotIndex));
				}
			}

			//Clear all section for this LOD
			for (int32 PreviousSectionIndex = 0; PreviousSectionIndex < PreviousSectionCount; ++PreviousSectionIndex)
			{
				StaticMesh->GetSectionInfoMap().Remove(LodIndex, PreviousSectionIndex);
			}
			//Recreate the new section info map
			for (int32 NewSectionIndex = 0; NewSectionIndex < NewSectionInfoMapData.Num(); ++NewSectionIndex)
			{
				StaticMesh->GetSectionInfoMap().Set(LodIndex, NewSectionIndex, NewSectionInfoMapData[NewSectionIndex]);
			}
		}
		else
#endif // WITH_EDITOR
		{
			int32 SectionIndex = 0;
			for (FPolygonGroupID PolygonGroupID : LodMeshDescription.PolygonGroups().GetElementIDs())
			{
				int32 MaterialSlotIndex = StaticMesh->GetMaterialIndexFromImportedMaterialSlotName(SlotNames[PolygonGroupID]);

				// If no material was found with this slot name, fill out a blank slot instead.
				if (MaterialSlotIndex == INDEX_NONE)
				{
					MaterialSlotIndex = StaticMesh->GetStaticMaterials().Emplace(UMaterial::GetDefaultMaterial(MD_Surface), SlotNames[PolygonGroupID]);
#if !WITH_EDITOR
					StaticMesh->GetStaticMaterials()[MaterialSlotIndex].UVChannelData = FMeshUVChannelInfo(1.f);
#endif
				}

#if WITH_EDITOR
				FMeshSectionInfo Info = StaticMesh->GetSectionInfoMap().Get(LodIndex, SectionIndex);
				Info.MaterialIndex = MaterialSlotIndex;
				StaticMesh->GetSectionInfoMap().Remove(LodIndex, SectionIndex);
				StaticMesh->GetSectionInfoMap().Set(LodIndex, SectionIndex, Info);
#endif

				SectionIndex++;
			}
		}
	}

	CommitMeshDescriptions(*StaticMesh);

	UE::Interchange::Private::MeshHelper::ImportSockets(Arguments, StaticMesh, StaticMeshFactoryNode);

	if (!Arguments.ReimportObject)
	{
		// Apply all StaticMeshFactoryNode custom attributes to the static mesh asset
		StaticMeshFactoryNode->ApplyAllCustomAttributeToObject(StaticMesh);
	}
#if WITH_EDITOR
	else
	{
		//Apply the re import strategy 
		UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(StaticMesh->GetAssetImportData());
		UInterchangeFactoryBaseNode* PreviousNode = nullptr;
		if (InterchangeAssetImportData)
		{
			PreviousNode = InterchangeAssetImportData->GetStoredFactoryNode(InterchangeAssetImportData->NodeUniqueID);
		}

		UInterchangeFactoryBaseNode* CurrentNode = UInterchangeFactoryBaseNode::DuplicateWithObject(StaticMeshFactoryNode, StaticMesh);
		UE::Interchange::FFactoryCommon::ApplyReimportStrategyToAsset(StaticMesh, PreviousNode, CurrentNode, StaticMeshFactoryNode);

		//Reorder the Hires mesh description in the same order has the lod 0 mesh description
		if (StaticMesh->IsHiResMeshDescriptionValid())
		{
			FMeshDescription* HiresMeshDescription = StaticMesh->GetHiResMeshDescription();
			FMeshDescription* Lod0MeshDescription = StaticMesh->GetMeshDescription(0);
			if (HiresMeshDescription && Lod0MeshDescription)
			{
				StaticMesh->ModifyHiResMeshDescription();
				FString MaterialNameConflictMsg = TEXT("[Asset ") + StaticMesh->GetPathName() + TEXT("] Nanite high-resolution import has material names that differ from the LOD 0 material name. Your Nanite high-resolution mesh should use the same material names the LOD 0 uses to ensure the sections can be remapped in the same order.");
				FString MaterialCountConflictMsg = TEXT("[Asset ") + StaticMesh->GetPathName() + TEXT("] Nanite high-resolution import doesn't have the same material count as LOD 0. Your Nanite high-resolution mesh should have the same number of materials as LOD 0.");
				FStaticMeshOperations::ReorderMeshDescriptionPolygonGroups(*Lod0MeshDescription, *HiresMeshDescription, MaterialNameConflictMsg, MaterialCountConflictMsg);
				StaticMesh->CommitHiResMeshDescription();
			}
		}
	}

	//Let's now re-generate the previous collisions with their properties, only the extents will be updated	
	if(bReimport)
	{
		if (!StaticMesh->GetBodySetup())
		{
			StaticMesh->CreateBodySetup();
		}
		//If we do not have any imported collision, we put back the original collision body setup
		if (StaticMesh->GetBodySetup()->AggGeom.GetElementCount() == 0)
		{
			StaticMesh->GetBodySetup()->AggGeom = ImportAssetObjectData.AggregateGeom;
		}
		else
		{
			//If there is some collision, we remove the original imported collision and add any editor generated collision
			ImportAssetObjectData.AggregateGeom.EmptyImportedElements();

			for (FKBoxElem& BoxElem : ImportAssetObjectData.AggregateGeom.BoxElems)
			{
				int32 Index = GenerateBoxAsSimpleCollision(StaticMesh, false);
				FKBoxElem& NewBoxElem = StaticMesh->GetBodySetup()->AggGeom.BoxElems[Index];
				//Copy element
				NewBoxElem = BoxElem;
			}

			for (FKSphereElem& SphereElem : ImportAssetObjectData.AggregateGeom.SphereElems)
			{
				int32 Index = GenerateSphereAsSimpleCollision(StaticMesh, false);
				FKSphereElem& NewSphereElem = StaticMesh->GetBodySetup()->AggGeom.SphereElems[Index];
				//Copy element
				NewSphereElem = SphereElem;
			}

			for (FKSphylElem& CapsuleElem : ImportAssetObjectData.AggregateGeom.SphylElems)
			{
				int32 Index = GenerateSphylAsSimpleCollision(StaticMesh, false);
				FKSphylElem& NewCapsuleElem = StaticMesh->GetBodySetup()->AggGeom.SphylElems[Index];
				//Copy element
				NewCapsuleElem = CapsuleElem;
			}

			for (FKConvexElem& ConvexElem : ImportAssetObjectData.AggregateGeom.ConvexElems)
			{
				int32 Index = GenerateKDopAsSimpleCollision(StaticMesh, TArray<FVector>(KDopDir18, sizeof(KDopDir18) / sizeof(FVector)), false);
				FKConvexElem& NewConvexElem = StaticMesh->GetBodySetup()->AggGeom.ConvexElems[Index];
				//Copy element
				NewConvexElem = ConvexElem;
			}
		}
	}

	if(ImportAssetObjectData.bImportCollision)
	{
		if(!ImportAssetObjectData.bImportedCustomCollision && ImportAssetObjectData.Collision != EInterchangeMeshCollision::None)
		{
			// Don't generate collisions if the mesh already has one of the requested type, otherwise it will continue to create collisions,
			// it can happen in the case of an import, and then importing the same file without deleting the asset in the content browser (different from a reimport)
			bool bHasBoxCollision = !StaticMesh->GetBodySetup()->AggGeom.BoxElems.IsEmpty();
			bool bHasSphereCollision = !StaticMesh->GetBodySetup()->AggGeom.SphereElems.IsEmpty();
			bool bHasCapsuleCollision = !StaticMesh->GetBodySetup()->AggGeom.SphylElems.IsEmpty();
			bool bHasConvexCollision = !StaticMesh->GetBodySetup()->AggGeom.ConvexElems.IsEmpty();

			constexpr bool bUpdateRendering = false;
			switch(ImportAssetObjectData.Collision)
			{
			case EInterchangeMeshCollision::Box:
				if(!bHasBoxCollision)
				{
					GenerateBoxAsSimpleCollision(StaticMesh, bUpdateRendering);
				}
				break;
			case EInterchangeMeshCollision::Sphere:
				if(!bHasSphereCollision)
				{
					GenerateSphereAsSimpleCollision(StaticMesh, bUpdateRendering);
				}
				break;
			case EInterchangeMeshCollision::Capsule:
				if(!bHasCapsuleCollision)
				{
					GenerateSphylAsSimpleCollision(StaticMesh, bUpdateRendering);
				}
				break;
			case EInterchangeMeshCollision::Convex10DOP_X:
				if(!bHasConvexCollision)
				{
					GenerateKDopAsSimpleCollision(StaticMesh, TArray<FVector>(KDopDir10X, sizeof(KDopDir10X) / sizeof(FVector)), bUpdateRendering);
				}
				break;
			case EInterchangeMeshCollision::Convex10DOP_Y:
				if(!bHasConvexCollision)
				{
					GenerateKDopAsSimpleCollision(StaticMesh, TArray<FVector>(KDopDir10Y, sizeof(KDopDir10Y) / sizeof(FVector)), bUpdateRendering);

				}
				break;
			case EInterchangeMeshCollision::Convex10DOP_Z:
				if(!bHasConvexCollision)
				{
					GenerateKDopAsSimpleCollision(StaticMesh, TArray<FVector>(KDopDir10Z, sizeof(KDopDir10Z) / sizeof(FVector)), bUpdateRendering);
				}
				break;
			case EInterchangeMeshCollision::Convex18DOP:
				if(!bHasConvexCollision)
				{
					GenerateKDopAsSimpleCollision(StaticMesh, TArray<FVector>(KDopDir18, sizeof(KDopDir18) / sizeof(FVector)), bUpdateRendering);
				}
				break;
			case EInterchangeMeshCollision::Convex26DOP:
				if(!bHasConvexCollision)
				{
					GenerateKDopAsSimpleCollision(StaticMesh, TArray<FVector>(KDopDir26, sizeof(KDopDir26) / sizeof(FVector)), bUpdateRendering);
				}
				break;
			default:
				break;
			}
		}
#if WITH_EDITORONLY_DATA
		else
		{
			StaticMesh->SetCustomizedCollision(true);
		}
#endif // WITH_EDITORONLY_DATA
	}
	else
	{
		if (GInterchangeNeverNeedsCookedCollisionData)
		{
			// this will prevent the complex collision mesh from being generated
			StaticMesh->GetBodySetup()->bNeverNeedsCookedCollisionData = true;
		}
	}

	//Lod group need to use the static mesh API and cannot use the apply delegate
	if (!Arguments.ReimportObject)
	{
		FName LodGroup = NAME_None;
		if (StaticMeshFactoryNode->GetCustomLODGroup(LodGroup) && LodGroup != NAME_None)
		{
			constexpr bool bRebuildImmediately = false;
			constexpr bool bAllowModify = false;
			StaticMesh->SetLODGroup(LodGroup, bRebuildImmediately, bAllowModify);
		}
	}
	FMeshBudgetProjectSettingsUtils::SetLodGroupForStaticMesh(StaticMesh);

	if (bReimport)
	{
		UStaticMesh::RemoveUnusedMaterialSlots(StaticMesh);
	}

#if WITH_EDITORONLY_DATA
	// Clear the mesh descriptions to free memory, we don't need them anymore since all the meshes have been committed at this point.
	StaticMesh->ClearMeshDescriptions();
#endif // WITH_EDITORONLY_DATA
#endif // WITH_EDITOR

	ImportAssetResult.ImportedObject = StaticMesh;
	return ImportAssetResult;
}

void UInterchangeStaticMeshFactory::CommitMeshDescriptions(UStaticMesh& StaticMesh)
{
#if WITH_EDITOR
	if (ImportAssetObjectData.bIsAppGame)
	{
		return;
	}

	TArray<FMeshDescription> LodMeshDescriptions = MoveTemp(ImportAssetObjectData.LodMeshDescriptions);

	UStaticMesh::FCommitMeshDescriptionParams CommitMeshDescriptionParams;
	CommitMeshDescriptionParams.bMarkPackageDirty = false; // Marking packages dirty isn't thread-safe

	if (GInterchangeUseHashAsGuid)
	{
		CommitMeshDescriptionParams.bUseHashAsGuid = true;
	}

	for (int32 LodIndex = 0; LodIndex < LodMeshDescriptions.Num(); ++LodIndex)
	{
		FMeshDescription* StaticMeshDescription = StaticMesh.CreateMeshDescription(LodIndex);
		check(StaticMeshDescription);
		*StaticMeshDescription = MoveTemp(LodMeshDescriptions[LodIndex]);

		StaticMesh.CommitMeshDescription(LodIndex, CommitMeshDescriptionParams);
	}
#endif
}

void UInterchangeStaticMeshFactory::BuildFromMeshDescriptions(UStaticMesh& StaticMesh)
{
	if (!ImportAssetObjectData.bIsAppGame)
	{
		return;
	}

	TArray<FMeshDescription> LodMeshDescriptions = MoveTemp(ImportAssetObjectData.LodMeshDescriptions);
	TArray<const FMeshDescription*> MeshDescriptionPointers;
	MeshDescriptionPointers.Reserve(LodMeshDescriptions.Num());

	for (const FMeshDescription& MeshDescription : LodMeshDescriptions)
	{
		MeshDescriptionPointers.Add(&MeshDescription);
	}

	UStaticMesh::FBuildMeshDescriptionsParams BuildMeshDescriptionsParams;
	BuildMeshDescriptionsParams.bUseHashAsGuid = true;
	BuildMeshDescriptionsParams.bMarkPackageDirty = false;
	BuildMeshDescriptionsParams.bBuildSimpleCollision = false;
	// Do not commit since we only need the render data and commit is slow
	BuildMeshDescriptionsParams.bCommitMeshDescription = false;
	BuildMeshDescriptionsParams.bFastBuild = true;
	// For the time being at runtime collision is set to complex one
	// TODO: Revisit pipeline options for collision. bImportCollision is not enough.
	BuildMeshDescriptionsParams.bAllowCpuAccess = ImportAssetObjectData.Collision != EInterchangeMeshCollision::None;
	StaticMesh.bAllowCPUAccess = BuildMeshDescriptionsParams.bAllowCpuAccess;

	StaticMesh.BuildFromMeshDescriptions(MeshDescriptionPointers, BuildMeshDescriptionsParams);
	
	// TODO: Expand support for different collision types
	if (ensure(StaticMesh.GetRenderData()))
	{
		if (ImportAssetObjectData.Collision != EInterchangeMeshCollision::None && !ImportAssetObjectData.bImportedCustomCollision)
		{
			if (StaticMesh.GetBodySetup() == nullptr)
			{
				StaticMesh.CreateBodySetup();
			}

			StaticMesh.GetBodySetup()->CollisionTraceFlag = ECollisionTraceFlag::CTF_UseComplexAsSimple;
		}
	}	
}

#if WITH_EDITORONLY_DATA
void UInterchangeStaticMeshFactory::SetupSourceModelsSettings(UStaticMesh& StaticMesh, const TArray<FMeshDescription>& LodMeshDescriptions, bool bAutoComputeLODScreenSizes, const TArray<float>& LodScreenSizes, int32 PreviousLodCount, int32 FinalLodCount, bool bIsAReimport)
{
	// Default LOD Screen Size
	constexpr int32 LODIndex = 0;
	float PreviousLODScreenSize = UStaticMesh::ComputeLODScreenSize(LODIndex);

	// No change during reimport
	if (!bIsAReimport)
	{
		// If no values are provided, then force AutoCompute
		if (LodScreenSizes.IsEmpty())
		{
			bAutoComputeLODScreenSizes = true;
		}
		StaticMesh.SetAutoComputeLODScreenSize(bAutoComputeLODScreenSizes);
	}
	
	for (int32 LodIndex = 0; LodIndex < FinalLodCount; ++LodIndex)
	{
		FStaticMeshSourceModel& SrcModel = StaticMesh.GetSourceModel(LodIndex);

		if (!bIsAReimport && !bAutoComputeLODScreenSizes)
		{
			if (LodScreenSizes.IsValidIndex(LodIndex))
			{
				SrcModel.ScreenSize = LodScreenSizes[LodIndex]; 
			}
			else
			{
				SrcModel.ScreenSize = UStaticMesh::ComputeLODScreenSize(LodIndex, PreviousLODScreenSize);
			}
			PreviousLODScreenSize = SrcModel.ScreenSize.Default;
		}

		// Make sure that mesh descriptions for added LODs are kept as is when the mesh is built
		if (LodIndex >= PreviousLodCount)
		{
			SrcModel.ResetReductionSetting();
		}

		if (!bIsAReimport && LodMeshDescriptions.IsValidIndex(LodIndex))
		{
			FStaticMeshConstAttributes StaticMeshAttributes(LodMeshDescriptions[LodIndex]);
			const int32 NumUVChannels = StaticMeshAttributes.GetVertexInstanceUVs().IsValid() ? StaticMeshAttributes.GetVertexInstanceUVs().GetNumChannels() : 1;
			const int32 FirstOpenUVChannel = NumUVChannels >= MAX_MESH_TEXTURE_COORDS_MD ? 1 : NumUVChannels;

			SrcModel.BuildSettings.DstLightmapIndex = FirstOpenUVChannel;

			if (LodIndex == 0)
			{
				StaticMesh.SetLightMapCoordinateIndex(FirstOpenUVChannel);
			}
		}
	}
}
#endif // WITH_EDITORONLY_DATA

/* This function is call in the completion task on the main thread, use it to call main thread post creation step for your assets */
void UInterchangeStaticMeshFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeStaticMeshFactory::SetupObject_GameThread);

	check(IsInGameThread());
	Super::SetupObject_GameThread(Arguments);

	// TODO: make sure this works at runtime
#if WITH_EDITORONLY_DATA
	if (ensure(Arguments.ImportedObject && Arguments.SourceData))
	{
		// We must call the Update of the asset source file in the main thread because UAssetImportData::Update execute some delegate we do not control
		UStaticMesh* StaticMesh = CastChecked<UStaticMesh>(Arguments.ImportedObject);

		UAssetImportData* ImportDataPtr = StaticMesh->GetAssetImportData();
		UE::Interchange::FFactoryCommon::FUpdateImportAssetDataParameters UpdateImportAssetDataParameters(StaticMesh, ImportDataPtr, Arguments.SourceData, Arguments.NodeUniqueID, Arguments.NodeContainer, Arguments.OriginalPipelines, Arguments.Translator);
		ImportDataPtr = UE::Interchange::FFactoryCommon::UpdateImportAssetData(UpdateImportAssetDataParameters);
		StaticMesh->SetAssetImportData(ImportDataPtr);

#if WITH_EDITOR
		// Create nanite assembly from all LOD0 payloads 
		if (const UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode = Cast<const UInterchangeStaticMeshFactoryNode>(Arguments.NodeContainer->GetFactoryNode(Arguments.NodeUniqueID)))
		{
			auto ForEachLOD0MeshPayload = [this](TFunctionRef<void(const UE::Interchange::FMeshPayloadData* MeshPayloadData)> IterationLambda)
				{
					constexpr int32 LOD0 = 0;
					if (const FLodPayloads* LodPayloads = PayloadsPerLodIndex.Find(LOD0))
					{
						for (const TPair<FInterchangeMeshPayLoadKey, UE::Interchange::FMeshPayload>& KeyAndPayload : LodPayloads->MeshPayloadPerKey)
						{
							const TOptional<UE::Interchange::FMeshPayloadData>& MeshPayload = KeyAndPayload.Value.PayloadData;
							if (const UE::Interchange::FMeshPayloadData* MeshPayloadData = MeshPayload.GetPtrOrNull())
							{
								IterationLambda(MeshPayloadData);
							}
						}
					}
				};

			// Two passes here to avoid Nanite assembly builder construction for the common case of zero nanite assemblies.

			int32 NumActiveNaniteAssemblyPayloads = 0;
			ForEachLOD0MeshPayload([&NumActiveNaniteAssemblyPayloads](const UE::Interchange::FMeshPayloadData* const MeshPayloadData)
				{
					if (MeshPayloadData->NaniteAssemblyDescription.IsSet())
					{
						NumActiveNaniteAssemblyPayloads++;
					}
				});

			if (NumActiveNaniteAssemblyPayloads > 0)
			{
				using namespace UE::Interchange::Private::MeshHelper;

				FInterchangeNaniteAssemblyBuilder Builder = FInterchangeNaniteAssemblyBuilder::Create(StaticMesh, StaticMeshFactoryNode, Arguments.NodeContainer);

				ForEachLOD0MeshPayload([&Builder](const UE::Interchange::FMeshPayloadData* const MeshPayloadData)
					{
						Builder.AddDescription(MeshPayloadData->NaniteAssemblyDescription);
					});
			}
		}
#endif //WITH_EDITOR

		// Payloads have been processed, so remove
		PayloadsPerLodIndex.Empty();
	}
#endif // WITH_EDITORONLY_DATA
}

void UInterchangeStaticMeshFactory::BuildObject_GameThread(const FSetupObjectParams& Arguments, bool& OutPostEditchangeCalled)
{
	check(IsInGameThread());
	OutPostEditchangeCalled = false;
#if WITH_EDITOR
	if (Arguments.ImportedObject)
	{
		if (UStaticMesh* StaticMesh = CastChecked<UStaticMesh>(Arguments.ImportedObject))
		{
			//Start an async build of the staticmesh
			UStaticMesh::FBuildParameters BuildParameters;
			BuildParameters.bInSilent = true;
			BuildParameters.bInRebuildUVChannelData = true;
			BuildParameters.bInEnforceLightmapRestrictions = true;
			StaticMesh->Build(BuildParameters);
		}
	}
#endif
}

bool UInterchangeStaticMeshFactory::GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const
{
#if WITH_EDITORONLY_DATA
	if (const UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
	{
		return UE::Interchange::FFactoryCommon::GetSourceFilenames(StaticMesh->GetAssetImportData(), OutSourceFilenames);
	}
#endif

	return false;
}

bool UInterchangeStaticMeshFactory::SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const
{
#if WITH_EDITORONLY_DATA
	if (const UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
	{
		return UE::Interchange::FFactoryCommon::SetSourceFilename(StaticMesh->GetAssetImportData(), SourceFilename, SourceIndex);
	}
#endif

	return false;
}

void UInterchangeStaticMeshFactory::BackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
	{
		UE::Interchange::FFactoryCommon::BackupSourceData(StaticMesh->GetAssetImportData());
	}
#endif
}

void UInterchangeStaticMeshFactory::ReinstateSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
	{
		UE::Interchange::FFactoryCommon::ReinstateSourceData(StaticMesh->GetAssetImportData());
	}
#endif
}

void UInterchangeStaticMeshFactory::ClearBackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
	{
		UE::Interchange::FFactoryCommon::ClearBackupSourceData(StaticMesh->GetAssetImportData());
	}
#endif
}
