// Copyright Epic Games, Inc. All Rights Reserved. 
#include "Scene/InterchangeActorFactory.h"

#include "InterchangeActorFactoryNode.h"
#include "InterchangeCameraFactoryNode.h"
#include "InterchangeGroomBindingFactoryNode.h"
#include "InterchangeGroomComponentFactoryNode.h"
#include "InterchangeGroomFactoryNode.h"
#include "InterchangeSceneComponentFactoryNodes.h"
#include "InterchangeStaticMeshFactoryNode.h"
#include "Scene/InterchangeActorHelper.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"

#include "Components/ActorComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "GroomComponent.h"

#if WITH_EDITORONLY_DATA
#include "Engine/World.h"
#include "Layers/LayersSubsystem.h"
#include "Layers/Layer.h"
#include "Editor/EditorEngine.h"

extern UNREALED_API UEditorEngine* GEditor;
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeActorFactory)

UClass* UInterchangeActorFactory::GetFactoryClass() const
{
	return AActor::StaticClass();
}

void UInterchangeActorFactory::ExecuteResetObjectProperties(const UInterchangeBaseNodeContainer* BaseNodeContainer, UInterchangeFactoryBaseNode* FactoryNode, UObject* ImportedObject)
{
	using namespace UE::Interchange;

	FImportSceneObjectsParams TempSceneObjectParmeters;
	TempSceneObjectParmeters.FactoryNode = FactoryNode;
	TempSceneObjectParmeters.NodeContainer = BaseNodeContainer;
	TempSceneObjectParmeters.ReimportObject = ImportedObject;
	
	if (AActor* ImportedActor = ActorHelper::SpawnFactoryActor(TempSceneObjectParmeters))
	{
		if (UInterchangeActorFactoryNode* ActorFactoryNode = Cast<UInterchangeActorFactoryNode>(FactoryNode))
		{
			if (UObject* ObjectToUpdate = ProcessActor(*ImportedActor, *ActorFactoryNode, *BaseNodeContainer, TempSceneObjectParmeters))
			{
				if (USceneComponent* RootComponent = ImportedActor->GetRootComponent())
				{
					UActorComponent* ActorComponent = Cast<UActorComponent>(ObjectToUpdate);
					if (ActorComponent)
					{
#if WITH_EDITOR
						ActorComponent->PreEditChange(nullptr);
#endif
						ActorComponent->UnregisterComponent();
					}

					// Cache mobility value to allow application of transform
					EComponentMobility::Type CachedMobility = RootComponent->Mobility;
					RootComponent->SetMobility(EComponentMobility::Type::Movable);
					
					ApplyAllCustomAttributesToObject(TempSceneObjectParmeters, *ImportedActor, ObjectToUpdate);

					// Restore mobility value
					if (CachedMobility != EComponentMobility::Type::Movable)
					{
						RootComponent->SetMobility(CachedMobility);
					}
					
					if (ActorComponent)
					{
						ActorComponent->RegisterComponent();
#if WITH_EDITOR
						ActorComponent->PostEditChange();
#endif
					}
					
					return;
				}
			}
		}
	}

	Super::ExecuteResetObjectProperties(BaseNodeContainer, FactoryNode, ImportedObject);	
}


namespace UE::Interchange::Private
{
	//Per Actor (important for Name uniqueness tracker).
	struct FComponentProcessHelper
	{
		AActor* Owner;
		const UInterchangeBaseNodeContainer* NodeContainer;
		TMap<FString, uint64> NameUniquenessTracker;

		FComponentProcessHelper() = delete;
		FComponentProcessHelper(AActor* InOwner, const UInterchangeBaseNodeContainer* InNodeContainer)
			: Owner(InOwner)
			, NodeContainer(InNodeContainer)
		{
		}

		USceneComponent* CreateSceneComponent(UClass* ComponentClass, USceneComponent* ParentComponent, const FString& DisplayName)
		{
			FString UniqueDisplayName = DisplayName;
			if (NameUniquenessTracker.Contains(UniqueDisplayName))
			{
				UniqueDisplayName += TEXT("_") + FString::FromInt(++NameUniquenessTracker[UniqueDisplayName]);
			}
			else
			{
				NameUniquenessTracker.Add(UniqueDisplayName, 0);
			}

			USceneComponent* SceneComponent = NewObject<USceneComponent>(Owner, ComponentClass, *UniqueDisplayName);
			SceneComponent->SetupAttachment(ParentComponent);
			SceneComponent->RegisterComponent();
			Owner->AddInstanceComponent(SceneComponent);
			return SceneComponent;
		};

		void ProcessComponents(const TArray<FString>& ComponentUids, USceneComponent* ParentComponent, EComponentMobility::Type MobilityToSet)
		{
			for (const FString& ComponentUid : ComponentUids)
			{
				if (const UInterchangeSceneComponentFactoryNode* SceneComponentFactoryNode = Cast<const UInterchangeSceneComponentFactoryNode>(NodeContainer->GetNode(ComponentUid)))
				{
					UClass* ComponentClass = SceneComponentFactoryNode->GetObjectClass();
					FString Name = SceneComponentFactoryNode->GetDisplayLabel();

					USceneComponent* CreatedSceneComponent = CreateSceneComponent(ComponentClass, ParentComponent, Name);

					if (const UInterchangeInstancedStaticMeshComponentFactoryNode* ISMComponentNode = Cast<const UInterchangeInstancedStaticMeshComponentFactoryNode >(NodeContainer->GetNode(ComponentUid)))
					{
						if (UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(CreatedSceneComponent))
						{
							TArray<FTransform> InstanceTransforms;
							ISMComponentNode->GetInstanceTransforms(InstanceTransforms);
							for (const FTransform& InstanceTransform : InstanceTransforms)
							{
								ISMComponent->AddInstance(InstanceTransform);
							}

							FString AssetInstanceUid;
							ISMComponentNode->GetCustomInstancedAssetUid(AssetInstanceUid);

							if (const UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode = Cast<const UInterchangeStaticMeshFactoryNode>(NodeContainer->GetNode(AssetInstanceUid)))
							{
								FSoftObjectPath ReferenceObject;
								StaticMeshFactoryNode->GetCustomReferenceObject(ReferenceObject);
								if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(ReferenceObject.TryLoad()))
								{
									ISMComponent->SetStaticMesh(StaticMesh);
								}
							}
						}
					}
					else if (const UInterchangeGroomComponentFactoryNode* GroomComponentFactoryNode = Cast<const UInterchangeGroomComponentFactoryNode>(SceneComponentFactoryNode))
					{
						if (UGroomComponent* GroomComponent = Cast<UGroomComponent>(CreatedSceneComponent))
						{
							FString AssetUid;
							GroomComponentFactoryNode->GetGroomDependencyUid(AssetUid);
							if (const UInterchangeGroomFactoryNode* GroomFactoryNode = Cast<const UInterchangeGroomFactoryNode>(NodeContainer->GetNode(AssetUid)))
							{
								FSoftObjectPath ReferenceObject;
								GroomFactoryNode->GetCustomReferenceObject(ReferenceObject);
								if (UGroomAsset* GroomAsset = Cast<UGroomAsset>(ReferenceObject.TryLoad()))
								{
									GroomComponent->SetGroomAsset(GroomAsset);
								}
							}
							GroomComponentFactoryNode->GetGroomBindingDependencyUid(AssetUid);
							if (const UInterchangeGroomBindingFactoryNode* GroomBindingFactoryNode = Cast<const UInterchangeGroomBindingFactoryNode>(NodeContainer->GetNode(AssetUid)))
							{
								FSoftObjectPath ReferenceObject;
								GroomBindingFactoryNode->GetCustomReferenceObject(ReferenceObject);
								if (UGroomBindingAsset* GroomBindingAsset = Cast<UGroomBindingAsset>(ReferenceObject.TryLoad()))
								{
									GroomComponent->SetBindingAsset(GroomBindingAsset);
								}
							}
						}
					}

					//General setups:
					{
						FTransform LocalTransform;
						SceneComponentFactoryNode->GetCustomLocalTransform(LocalTransform);

						CreatedSceneComponent->SetMobility(EComponentMobility::Type::Movable); //so that RelativeTransform can be set
						CreatedSceneComponent->SetRelativeTransform(LocalTransform);
						CreatedSceneComponent->SetMobility(MobilityToSet);

						bool bComponentVisibility = true;
						if (SceneComponentFactoryNode->GetCustomComponentVisibility(bComponentVisibility))
						{
							CreatedSceneComponent->SetVisibility(bComponentVisibility);
						}

						TArray<FString> ChildrenComponentUids;
						SceneComponentFactoryNode->GetComponentUids(ChildrenComponentUids);

						ProcessComponents(ChildrenComponentUids, CreatedSceneComponent, MobilityToSet);
					}
				}
				else
				{
					ensureMsgf(false, TEXT("Unexpected Component class type in ComponentUids."));
				}
			}
		}
	};
}


UObject* UInterchangeActorFactory::ImportSceneObject_GameThread(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams)
{
	using namespace UE::Interchange;

	UInterchangeActorFactoryNode* FactoryNode = Cast<UInterchangeActorFactoryNode>(CreateSceneObjectsParams.FactoryNode);
	if (!ensure(FactoryNode) || !CreateSceneObjectsParams.NodeContainer)
	{
		return nullptr;
	}

	FText OutReason;
	AActor* SpawnedActor = ActorHelper::SpawnFactoryActor(CreateSceneObjectsParams, OutReason);

	if (SpawnedActor)
	{
		if (UObject* ObjectToUpdate = ProcessActor(*SpawnedActor, *FactoryNode, *CreateSceneObjectsParams.NodeContainer, CreateSceneObjectsParams))
		{
			if (USceneComponent* RootComponent = SpawnedActor->GetRootComponent())
			{
				// Cache mobility value to allow application of transform
				EComponentMobility::Type CachedMobility = RootComponent->Mobility;
				RootComponent->SetMobility(EComponentMobility::Type::Movable);

				ApplyAllCustomAttributesToObject(CreateSceneObjectsParams, *SpawnedActor, ObjectToUpdate);

				TArray<FString> ComponentUids;
				FactoryNode->GetComponentUids(ComponentUids);

				UE::Interchange::Private::FComponentProcessHelper ComponentProcessHelper(SpawnedActor, CreateSceneObjectsParams.NodeContainer);
				ComponentProcessHelper.ProcessComponents(ComponentUids, RootComponent, CachedMobility);
				
				// Restore mobility value
				if (CachedMobility != EComponentMobility::Type::Movable)
				{
					RootComponent->SetMobility(CachedMobility);
				}
			}
		}

		ProcessTags(FactoryNode, SpawnedActor);

		ProcessLayerNames(FactoryNode, SpawnedActor);
	}
	else if (!OutReason.IsEmpty())
	{
		UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
		Message->SourceAssetName = CreateSceneObjectsParams.SourceData->GetFilename();
		Message->DestinationAssetName = CreateSceneObjectsParams.ObjectName;
		Message->AssetType = FactoryNode->GetObjectClass();
		Message->Text = OutReason;
	}

	return SpawnedActor;
}

UObject* UInterchangeActorFactory::ProcessActor(AActor& SpawnedActor, const UInterchangeActorFactoryNode& /*FactoryNode*/, const UInterchangeBaseNodeContainer& /*NodeContainer*/, const FImportSceneObjectsParams& /*Params*/)
{
	return SpawnedActor.GetRootComponent();
}

void UInterchangeActorFactory::ProcessTags(UInterchangeActorFactoryNode* FactoryNode, AActor* SpawnedActor)
{
	TArray<FString> TagsArray;
	FactoryNode->GetTags(TagsArray);

	TSet<FString> Tags(TagsArray);
	TSet<FName> AlreadySetTags(SpawnedActor->Tags);

	for (const FString& Tag : Tags)
	{
		FName TagName(Tag);
		if (!AlreadySetTags.Contains(TagName))
		{
			SpawnedActor->Tags.Add(TagName);
		}
	}
}

void UInterchangeActorFactory::ProcessLayerNames(UInterchangeActorFactoryNode* FactoryNode, AActor* SpawnedActor)
{
	TArray<FString> LayerNamesArray;
	FactoryNode->GetLayerNames(LayerNamesArray);

	TSet<FString> LayerNames(LayerNamesArray);
#if WITH_EDITORONLY_DATA
	AddUniqueLayersToWorld(SpawnedActor->GetWorld(), LayerNames);
#endif

	TSet<FName> AlreadySetLayerNames(SpawnedActor->Layers);

	for (const FString& LayerNameString : LayerNames)
	{
		FName LayerName(LayerNameString);
		if (!AlreadySetLayerNames.Contains(LayerName))
		{
			SpawnedActor->Layers.Add(FName(LayerName));
		}
	}
}

#if WITH_EDITORONLY_DATA
void UInterchangeActorFactory::AddUniqueLayersToWorld(UWorld* World, const TSet<FString>& LayerNames)
{
	if (!World || !IsValidChecked(World) || World->IsUnreachable() || LayerNames.Num() == 0)
	{
		return;
	}

	TSet< FName > ExistingLayers;
	for (ULayer* Layer : World->Layers)
	{
		ExistingLayers.Add(Layer->GetLayerName());
	}

	int32 NumberOfExistingLayers = World->Layers.Num();

	ULayersSubsystem* LayersSubsystem = GEditor ? GEditor->GetEditorSubsystem<ULayersSubsystem>() : nullptr;
	for (const FString& LayerNameString : LayerNames)
	{
		FName LayerName(LayerNameString);

		if (!ExistingLayers.Contains(LayerName))
		{
			// Use the ILayers if we are adding the layers to the currently edited world
			if (LayersSubsystem && GWorld && World == GWorld.GetReference())
			{
				LayersSubsystem->CreateLayer(LayerName);
			}
			else
			{
				ULayer* NewLayer = NewObject<ULayer>(World, NAME_None, RF_Transactional);
				if (!ensure(NewLayer != NULL))
				{
					continue;
				}

				World->Layers.Add(NewLayer);

				NewLayer->SetLayerName(LayerName);
				NewLayer->SetVisible(true);
			}
		}
	}

	if (NumberOfExistingLayers != World->Layers.Num())
	{
		World->Modify();
	}
}
#endif

void UInterchangeActorFactory::ApplyAllCustomAttributesToObject(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams, AActor& SpawnedActor, UObject* ObjectToUpdate)
{
	using namespace UE::Interchange;
	ActorHelper::ApplyAllCustomAttributes(CreateSceneObjectsParams, *ObjectToUpdate);
}
