// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Scene/InterchangeActorHelper.h"

#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "InterchangeActorFactoryNode.h"
#include "InterchangeAssetUserData.h"
#include "InterchangeImportCommon.h"
#include "InterchangeImportReset.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeMeshActorFactoryNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangeSceneImportAsset.h"
#include "InterchangeSceneNode.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "Components/MeshComponent.h"
#include "Engine/World.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Misc/SecureHash.h"

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#endif

namespace UE::Interchange::ActorHelper
{
	AActor* GetSpawnedParentActor(const UInterchangeBaseNodeContainer* NodeContainer, const UInterchangeActorFactoryNode* FactoryNode)
	{
		AActor* ParentActor = nullptr;

		if (const UInterchangeFactoryBaseNode* ParentFactoryNode = NodeContainer->GetFactoryNode(FactoryNode->GetParentUid()))
		{
			FSoftObjectPath ReferenceObject;
			ParentFactoryNode->GetCustomReferenceObject(ReferenceObject);
			ParentActor = Cast<AActor>(ReferenceObject.TryLoad());
		}

		return  ParentActor;
	}

	AActor* SpawnFactoryActor(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams, FText& OutReason)
	{
		OutReason = FText::GetEmpty();
		UInterchangeActorFactoryNode* FactoryNode = Cast<UInterchangeActorFactoryNode>(CreateSceneObjectsParams.FactoryNode);
		const UInterchangeBaseNodeContainer* NodeContainer = CreateSceneObjectsParams.NodeContainer;

		if (!FactoryNode || !NodeContainer)
		{
			return nullptr;
		}

		FActorSpawnParameters SpawnParameters;
		SpawnParameters.Name = FName(CreateSceneObjectsParams.ObjectName);
		SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
		SpawnParameters.OverrideLevel = CreateSceneObjectsParams.Level;
		SpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;

		AActor* ParentActor = GetSpawnedParentActor(NodeContainer, FactoryNode);
		UWorld* const World = [&SpawnParameters, &ParentActor]()
		{
			if (SpawnParameters.OverrideLevel)
			{
				return SpawnParameters.OverrideLevel->GetWorld();
			}

			UWorld* DefaultWorld = nullptr;

			if (ParentActor)
			{
				DefaultWorld = ParentActor->GetWorld();
			}
#if WITH_EDITOR
			if (DefaultWorld == nullptr)
			{
				UEditorEngine* EditorEngine = Cast<UEditorEngine>(GEngine);
				if (GIsEditor && EditorEngine != nullptr)
				{
					DefaultWorld = EditorEngine->GetEditorWorldContext().World();
				}
			}
#endif
			if (DefaultWorld == nullptr && GEngine)
			{
				DefaultWorld = GEngine->GetWorld();
			}

			return DefaultWorld;
		}();

		if (!World)
		{
			return nullptr;
		}

		UClass* ActorClass = FactoryNode->GetObjectClass();
		AActor* SpawnedActor = Cast<AActor>(CreateSceneObjectsParams.ReimportObject);
		if (SpawnedActor)
		{
			if (SpawnedActor->GetClass() == ActorClass)
			{
				// TODO: Check whether parenting is the same
			}
			else
			{
				SpawnedActor = nullptr;
			}
		}
		// The related actor has been deleted. Check on reimport policy
		else if (CreateSceneObjectsParams.ReimportFactoryNode && !FactoryNode->ShouldForceNodeReimport())
		{
			// if reimport policy does not prioritize new content, do not recreate the actor
			return nullptr;
		}

		// #interchange_levelinstance_rework:
		// Check whether or not to create the actor if it will be added to a level related to a level instance
		// Use case: User is importing into level with level instance or packed actor enabled and the targeted
		// level already exists and they have indicated to skip its update/replacement
		FString LevelFactoryNodeUid;
		if (FactoryNode->GetCustomLevelUid(LevelFactoryNodeUid))
		{
			if (const UInterchangeFactoryBaseNode* LevelFactoryNode = NodeContainer->GetFactoryNode(LevelFactoryNodeUid))
			{
				if (LevelFactoryNode->ShouldSkipNodeImport())
				{
					OutReason = NSLOCTEXT("Interchange", "SpawnFactoryActor_Wrong_Level", "Cannot create or update an actor when targeted level exists and should be skipped.");
					return nullptr;
				}
			}
		}

		if (!SpawnedActor)
		{
			SpawnedActor = World->SpawnActor<AActor>(ActorClass, SpawnParameters);
		}

		if (SpawnedActor)
		{
#if WITH_EDITOR
			SpawnedActor->SetActorLabel(FactoryNode->GetDisplayLabel());
#endif
			if (!SpawnedActor->GetRootComponent())
			{
				USceneComponent* RootComponent = NewObject<USceneComponent>(SpawnedActor, USceneComponent::GetDefaultSceneRootVariableName(), RF_Transactional);
				SpawnedActor->SetRootComponent(RootComponent);
				SpawnedActor->AddInstanceComponent(RootComponent);
			}

			if (USceneComponent* RootComponent = SpawnedActor->GetRootComponent())
			{
				uint8 Mobility;
				if (FactoryNode->GetCustomMobility(Mobility))
				{
					//Make sure we don't have a mobility that's more restrictive than our parent mobility, as that wouldn't be a valid setup.
					EComponentMobility::Type TargetMobility = (EComponentMobility::Type)Mobility;

					if (ParentActor && ParentActor->GetRootComponent())
					{
						TargetMobility = (EComponentMobility::Type)FMath::Max((uint8)Mobility, (uint8)ParentActor->GetRootComponent()->Mobility);
					}

					RootComponent->SetMobility(TargetMobility);
				}
			}

			if (ParentActor && !SpawnedActor->IsAttachedTo(ParentActor))
			{
				SpawnedActor->AttachToActor(ParentActor, FAttachmentTransformRules::KeepRelativeTransform);
			}
		}

		return SpawnedActor;
	}
	
	AActor* SpawnFactoryActor(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams)
	{
		FText OutReason;
		return SpawnFactoryActor(CreateSceneObjectsParams, OutReason);
	}

	const UInterchangeFactoryBaseNode* FindAssetInstanceFactoryNode(const UInterchangeBaseNodeContainer* NodeContainer, const UInterchangeFactoryBaseNode* ActorFactoryNode)
	{
		TArray<FString> ActorTargetNodes;
		ActorFactoryNode->GetTargetNodeUids(ActorTargetNodes);
		const UInterchangeSceneNode* SceneNode = ActorTargetNodes.IsEmpty() ? nullptr : Cast<UInterchangeSceneNode>(NodeContainer->GetNode(ActorTargetNodes[0]));
		if (!SceneNode)
		{
			return nullptr;
		}

		FString AssetInstanceUid;
		SceneNode->GetCustomAssetInstanceUid(AssetInstanceUid);
		const UInterchangeBaseNode* AssetNode = NodeContainer->GetNode(AssetInstanceUid);
		if (!AssetNode)
		{
			return nullptr;
		}

		TArray<FString> AssetTargetNodeIds;
		AssetNode->GetTargetNodeUids(AssetTargetNodeIds);
		return AssetTargetNodeIds.IsEmpty() ? nullptr : NodeContainer->GetFactoryNode(AssetTargetNodeIds[0]);
	}

	void ApplySlotMaterialDependencies(const UInterchangeBaseNodeContainer& NodeContainer, const UInterchangeMeshActorFactoryNode& FactoryNode, UMeshComponent& MeshComponent, const UInterchangeMeshActorFactoryNode* PreviousFactoryNode)
	{
		const EReimportStrategyFlags ReimportStrategyFlags = FactoryNode.GetReimportStrategyFlags();
		if (PreviousFactoryNode && ReimportStrategyFlags == EReimportStrategyFlags::ApplyNoProperties)
		{
			// This is a reimport but factory node should be ignored, return
			return;
		}

		// Get material slots from imported factory node
		TMap<FString, FString> SlotMaterialDependencies;
		FactoryNode.GetSlotMaterialDependencies(SlotMaterialDependencies);
		if (SlotMaterialDependencies.IsEmpty())
		{
			if (PreviousFactoryNode && ReimportStrategyFlags == EReimportStrategyFlags::ApplyPipelineProperties)
			{
				// Imported factory node does not have material overrides and factory node prevails, remove the existing ones
				if (MeshComponent.HasOverrideMaterials())
				{
					MeshComponent.EmptyOverrideMaterials();
				}
			}

			return;
		}

		auto GetMaterialPath = [&NodeContainer](const FString& MaterialUid) -> FSoftObjectPath
			{
				const UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode = Cast<UInterchangeBaseMaterialFactoryNode>(NodeContainer.GetNode(MaterialUid));
				if (MaterialFactoryNode && MaterialFactoryNode->IsEnabled())
				{
					FSoftObjectPath ReferenceObject;
					MaterialFactoryNode->GetCustomReferenceObject(ReferenceObject);
					if (ReferenceObject.IsValid())
					{
						return ReferenceObject;
					}
				}

				// Fallback: the dependency value may be a content path to an existing material rather than a node UID. 
				// Verify via ResolveObject as this may run off the game thread
				FSoftObjectPath ContentPath(MaterialUid);
				if (ContentPath.IsValid() && ContentPath.ResolveObject())
				{
					return ContentPath;
				}

				return FSoftObjectPath();
			};

		if (PreviousFactoryNode && ReimportStrategyFlags == EReimportStrategyFlags::ApplyEditorChangedProperties)
		{
			using namespace UE::Interchange;
			// Retrieve the InterchangeSceneImportAsset object from the last import
			UInterchangeSceneImportAsset* SceneImportAsset = nullptr;
			if (AActor* Actor = MeshComponent.GetOwner())
			{
				if (USceneComponent* RootComponent = Actor->GetRootComponent())
				{
					UInterchangeAssetUserData* AssetUserData = RootComponent->GetAssetUserData<UInterchangeAssetUserData>();
					if (ensure(AssetUserData && AssetUserData->MetaData.Contains(InterchangeReset::Constants::SceneImportAssetPathKey)))
					{
						const FSoftObjectPath SceneImportAssetPath(AssetUserData->MetaData[InterchangeReset::Constants::SceneImportAssetPathKey]);
						SceneImportAsset = Cast<UInterchangeSceneImportAsset>(SceneImportAssetPath.TryLoad());
					}
				}
			}

			if (SceneImportAsset)
			{
				TMap<FString, FString> PreviousSlotMaterialDependencies;
				PreviousFactoryNode->GetSlotMaterialDependencies(PreviousSlotMaterialDependencies);

				for (TPair<FString, FString>& SlotMaterialDependency : SlotMaterialDependencies)
				{
					FSoftObjectPath MaterialPath = GetMaterialPath(SlotMaterialDependency.Value);
					if (!MaterialPath.IsValid())
					{
						continue;
					}

					FName MaterialSlotName = *SlotMaterialDependency.Key;
					const int32 MaterialSlotIndex = MeshComponent.GetMaterialIndex(MaterialSlotName);
					if (MaterialSlotIndex == INDEX_NONE)
					{
						// Cannot apply override as slot does not exist, continue 
						continue;
					}

					if (!PreviousSlotMaterialDependencies.Contains(SlotMaterialDependency.Key))
					{
						// New material slot, but it did not existed in last import, skip override
						continue;
					}

					FSoftObjectPath PreviousMaterialPath;
					const FString PreviousMaterialUid = PreviousSlotMaterialDependencies[SlotMaterialDependency.Key];
					if(const UInterchangeFactoryBaseNode* PreviousMaterialFactoryNode = SceneImportAsset->GetFactoryNodeFromUniqueID(PreviousMaterialUid))
					{
						PreviousMaterialFactoryNode->GetCustomReferenceObject(PreviousMaterialPath);
						
						if (!PreviousMaterialPath.IsValid())
						{
							continue;
						}
					}
					else
					{
						FSoftObjectPath ContentPath(PreviousMaterialUid);
						if (ContentPath.IsValid())
						{
							PreviousMaterialPath = ContentPath;
						}
					}

					const FSoftObjectPath MeshMaterialPath(MeshComponent.GetMaterial(MaterialSlotIndex));
					if (MeshMaterialPath.IsValid() && MeshMaterialPath == PreviousMaterialPath)
					{
						UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(MaterialPath.TryLoad());
						if (!MaterialInterface)
						{
							MaterialInterface = UMaterial::GetDefaultMaterial(MD_Surface);
						}

						MeshComponent.SetMaterial(MaterialSlotIndex, MaterialInterface);
					}
				}

				return;
			}
			// Else, proceed as if PreviousFactoryNode was null
		}

		// Apply material slots to mesh component

		// It is an import or a reimport where the factory node prevails, apply material overrides
		// i.e. !PreviousFactoryNode || ReimportStrategyFlags == EReimportStrategyFlags::ApplyPipelineProperties
		
		// Discard existing material overrides
		if (MeshComponent.HasOverrideMaterials())
		{
			MeshComponent.EmptyOverrideMaterials();
		}

		// Apply factory node's overrides
		for (TPair<FString, FString>& SlotMaterialDependency : SlotMaterialDependencies)
		{
			FSoftObjectPath ReferenceObject = GetMaterialPath(SlotMaterialDependency.Value);
			if (!ReferenceObject.IsValid())
			{
				continue;
			}

			FName MaterialSlotName = *SlotMaterialDependency.Key;
			UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(ReferenceObject.TryLoad());
			if (!MaterialInterface)
			{
				MaterialInterface = UMaterial::GetDefaultMaterial(MD_Surface);
			}

			int32 MaterialSlotIndex = MeshComponent.GetMaterialIndex(MaterialSlotName);
			if (MaterialSlotIndex != INDEX_NONE)
			{
				MeshComponent.SetMaterial(MaterialSlotIndex, MaterialInterface);
			}
		}
	}

	void ApplyAllCustomAttributes(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams, UObject& ObjectToUpdate)
	{
		if (CreateSceneObjectsParams.ReimportObject && (ObjectToUpdate.GetOuter() == CreateSceneObjectsParams.ReimportObject))
		{
				UInterchangeFactoryBaseNode* CurrentNode = UInterchangeFactoryBaseNode::DuplicateWithObject(CreateSceneObjectsParams.FactoryNode, &ObjectToUpdate);

				FFactoryCommon::ApplyReimportStrategyToAsset(&ObjectToUpdate, CreateSceneObjectsParams.ReimportFactoryNode, CurrentNode, CreateSceneObjectsParams.FactoryNode);
		}
		else
		{
			CreateSceneObjectsParams.FactoryNode->ApplyAllCustomAttributeToObject(&ObjectToUpdate);
		}
	}

	void AddInterchangeAssetUserDataToActor(AActor* Actor, const UInterchangeSceneImportAsset* SceneImportAsset, const UInterchangeFactoryBaseNode* FactoryNode)
	{
		if (!Actor || !SceneImportAsset || !FactoryNode)
		{
			return;
		}

		const FSoftObjectPath SceneImportAssetPath((const UObject*)SceneImportAsset);
		if (USceneComponent* RootComponent = Actor->GetRootComponent())
		{
			// This component may already have AssetUserData provided by UInterchangeGenericAssetsPipeline::AddMetaData(), so
			// we have to be careful as AddAssetUserData() will remove it when adding a new instance of the same class
			UInterchangeAssetUserData* AssetUserData = RootComponent->GetAssetUserData<UInterchangeAssetUserData>();
			if (!AssetUserData)
			{
				AssetUserData = NewObject<UInterchangeAssetUserData>(Actor);
				RootComponent->AddAssetUserData(AssetUserData);
			}

			AssetUserData->MetaData.Add(UE::Interchange::InterchangeReset::Constants::SceneImportAssetPathKey, SceneImportAssetPath.ToString());
			AssetUserData->MetaData.Add(UE::Interchange::InterchangeReset::Constants::FactoryNodeUidPathKey, FactoryNode->GetUniqueID());
		}
	}
	
	void AddInterchangeLevelAssetUserDataToWorld(UWorld* World, const UInterchangeSceneImportAsset* SceneImportAsset)
	{
		if (!World || !SceneImportAsset)
		{
			return;
		}

		if (AWorldSettings* WorldSettings = World->GetWorldSettings())
		{
			UInterchangeLevelAssetUserData* AssetUserData = NewObject<UInterchangeLevelAssetUserData>(World);
			const FSoftObjectPath SceneImportAssetPath((const UObject*)SceneImportAsset);
			AssetUserData->SceneImportPaths.Add(SceneImportAssetPath.ToString());
			WorldSettings->AddAssetUserData(AssetUserData);
		}
	}
}