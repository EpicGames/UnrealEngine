// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/InterchangeCameraActorFactory.h"

#include "InterchangeCameraFactoryNode.h"
#include "Scene/InterchangeActorFactory.h"
#include "Scene/InterchangeActorHelper.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeCameraActorFactory)

namespace UE::InterchangeCameraActorFactory::Private
{
	// For the camera actor types we get two components each: The root component is a default scene component, and the actual camera
	// component is a child of the scene component. We want to place all scene component stuff (mostly transform) on the root
	// component, and all the camera stuff on the camera component. This agrees with how the actor/root component is bound on
	// LevelSequences, and is likely what users expect because when you place a camera actor on the level and move it around, you always
	// affect the root component transform

	template <class T>
	void ApplyAllCameraCustomAttributes(
		const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams,
		T* CameraFactoryNode,
		USceneComponent* RootSceneComponent,
		USceneComponent* ChildCameraComponent
	)
	{
		using namespace UE::Interchange;

		if (!RootSceneComponent || !ChildCameraComponent)
		{
			return;
		}

		UInterchangeBaseNodeContainer* const NodeContainer = const_cast<UInterchangeBaseNodeContainer* const>(CreateSceneObjectsParams.NodeContainer);

		// Create a temp factory node so we don't modify our existing nodes with our changes
		T* FactoryNodeCopy = NewObject<T>(NodeContainer, NAME_None);
		NodeContainer->SetupAndReplaceFactoryNode(
			FactoryNodeCopy,
			CameraFactoryNode->GetUniqueID(),
			CameraFactoryNode->GetDisplayLabel(),
			CameraFactoryNode->GetNodeContainerType(),
			CameraFactoryNode->GetUniqueID());

		{
			UInterchangeFactoryBase::FImportSceneObjectsParams ParamsCopy{ CreateSceneObjectsParams };
			ParamsCopy.FactoryNode = FactoryNodeCopy;

			// Apply exclusively camera stuff to the CineCameraComponent
			FactoryNodeCopy->CopyWithObject(CameraFactoryNode, ChildCameraComponent);
			FactoryNodeCopy->RemoveCustomAttributesForClass(USceneComponent::StaticClass());
			ActorHelper::ApplyAllCustomAttributes(ParamsCopy, *ChildCameraComponent);

			// Apply exclusively scene component stuff to the root SceneComponent
			FactoryNodeCopy->CopyWithObject(CameraFactoryNode, RootSceneComponent);
			FactoryNodeCopy->RemoveCustomAttributesForClass(UCineCameraComponent::StaticClass());
			ActorHelper::ApplyAllCustomAttributes(ParamsCopy, *RootSceneComponent);
		}
		NodeContainer->ReplaceFactoryNode(FactoryNodeCopy->GetUniqueID(), CameraFactoryNode);
	}
}

UClass* UInterchangeCineCameraActorFactory::GetFactoryClass() const
{
	return ACineCameraActor::StaticClass();
}

UObject* UInterchangeCineCameraActorFactory::ProcessActor(AActor& SpawnedActor, const UInterchangeActorFactoryNode& /*FactoryNode*/, const UInterchangeBaseNodeContainer& /*NodeContainer*/, const FImportSceneObjectsParams& /*Params*/)
{
	ACineCameraActor* CineCameraActor = Cast<ACineCameraActor>(&SpawnedActor);

	return CineCameraActor ? CineCameraActor->GetCineCameraComponent() : nullptr;
}

void UInterchangeCineCameraActorFactory::ApplyAllCustomAttributesToObject(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams, AActor& SpawnedActor, UObject* ObjectToUpdate)
{
	using namespace UE::InterchangeCameraActorFactory::Private;

	if (UCineCameraComponent* CameraComponent = Cast<UCineCameraComponent>(ObjectToUpdate))
	{
		if (UInterchangePhysicalCameraFactoryNode* FactoryNode = Cast<UInterchangePhysicalCameraFactoryNode>(CreateSceneObjectsParams.FactoryNode))
		{
			if (USceneComponent* RootComponent = SpawnedActor.GetRootComponent())
			{
				ApplyAllCameraCustomAttributes(CreateSceneObjectsParams, FactoryNode, RootComponent, CameraComponent);
				return;
			}
		}
	}

	Super::ApplyAllCustomAttributesToObject(CreateSceneObjectsParams, SpawnedActor, ObjectToUpdate);
}


UClass* UInterchangeCameraActorFactory::GetFactoryClass() const
{
	return ACameraActor::StaticClass();
}

UObject* UInterchangeCameraActorFactory::ProcessActor(AActor& SpawnedActor, const UInterchangeActorFactoryNode& /*FactoryNode*/, const UInterchangeBaseNodeContainer& /*NodeContainer*/, const FImportSceneObjectsParams& /*Params*/)
{
	ACameraActor* CameraActor = Cast<ACameraActor>(&SpawnedActor);

	return CameraActor ? CameraActor->GetCameraComponent() : nullptr;
}

void UInterchangeCameraActorFactory::ApplyAllCustomAttributesToObject(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams, AActor& SpawnedActor, UObject* ObjectToUpdate)
{
	using namespace UE::InterchangeCameraActorFactory::Private;

	if (UCameraComponent* CameraComponent = Cast<UCameraComponent>(ObjectToUpdate))
	{
		if (UInterchangeStandardCameraFactoryNode* FactoryNode = Cast<UInterchangeStandardCameraFactoryNode>(CreateSceneObjectsParams.FactoryNode))
		{
			if (USceneComponent* RootComponent = SpawnedActor.GetRootComponent())
			{
				ApplyAllCameraCustomAttributes(CreateSceneObjectsParams, FactoryNode, RootComponent, CameraComponent);
				return;
			}
		}
	}

	Super::ApplyAllCustomAttributesToObject(CreateSceneObjectsParams, SpawnedActor, ObjectToUpdate);
}
