// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/InterchangeDecalActorFactory.h"

#include "InterchangeDecalActorFactoryNode.h"
#include "InterchangeDecalMaterialFactoryNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "Engine/DecalActor.h"
#include "Components/DecalComponent.h"

#include "Misc/PackageName.h"
#include "InterchangeImportLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeDecalActorFactory)

UClass* UInterchangeDecalActorFactory::GetFactoryClass() const
{
	return ADecalActor::StaticClass();
}

UObject* UInterchangeDecalActorFactory::ProcessActor(AActor& SpawnedActor, const UInterchangeActorFactoryNode& FactoryNode, const UInterchangeBaseNodeContainer& NodeContainer, const FImportSceneObjectsParams& Params)
{
	ADecalActor* DecalActor = Cast<ADecalActor>(&SpawnedActor);
	if (DecalActor)
	{
		if (const UInterchangeDecalActorFactoryNode* DecalActorFactoryNode = Cast<UInterchangeDecalActorFactoryNode>(&FactoryNode))
		{
			FString DecalMaterialPath;
			if (DecalActorFactoryNode->GetCustomDecalMaterialPathName(DecalMaterialPath))
			{
				UMaterialInterface* DecalMaterial = nullptr;
				if (FPackageName::IsValidObjectPath(DecalMaterialPath))
				{
					const FSoftObjectPath MaterialPath(DecalMaterialPath);
					DecalMaterial = Cast<UMaterialInterface>(MaterialPath.TryLoad());
				}
				else
				{
					if (UInterchangeDecalMaterialFactoryNode* MaterialFactoryNode = Cast<UInterchangeDecalMaterialFactoryNode>(NodeContainer.GetFactoryNode(DecalMaterialPath)))
					{
						FSoftObjectPath ReferenceObject;
						if (MaterialFactoryNode->GetCustomReferenceObject(ReferenceObject))
						{
							DecalMaterial = Cast<UMaterialInterface>(ReferenceObject.TryLoad());
						}
					}
				}

				if (DecalMaterial)
				{
					DecalActor->SetDecalMaterial(DecalMaterial);
				}
				else
				{
					const FText Message = FText::Format(NSLOCTEXT("DecalActorImport", "NoMaterial", "No valid decal material found. Make sure that the DecalMaterialPath is valid: {0}"), FText::FromString(DecalMaterialPath));
#if WITH_EDITOR
					LogMessage<UInterchangeResultError_Generic>(Params, Message, SpawnedActor.GetActorLabel());
#else
					LogMessage<UInterchangeResultError_Generic>(Params, Message, TEXT(""));
#endif
				}
			}
		}
	}

	return DecalActor ? DecalActor->GetDecal() : nullptr;
}
