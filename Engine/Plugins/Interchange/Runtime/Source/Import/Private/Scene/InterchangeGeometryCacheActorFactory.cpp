// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Scene/InterchangeGeometryCacheActorFactory.h"

#include "InterchangeGeometryCacheFactoryNode.h"
#include "InterchangeMeshActorFactoryNode.h"
#include "InterchangeSceneNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Scene/InterchangeActorHelper.h"

#include "GeometryCacheComponent.h"
#include "GeometryCache.h"
#include "GeometryCacheActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGeometryCacheActorFactory)

UClass* UInterchangeGeometryCacheActorFactory::GetFactoryClass() const
{
	return AGeometryCacheActor::StaticClass();
}

UObject* UInterchangeGeometryCacheActorFactory::ProcessActor(AActor& SpawnedActor, const UInterchangeActorFactoryNode& FactoryNode, const UInterchangeBaseNodeContainer& NodeContainer, const FImportSceneObjectsParams& /*Params*/)
{
	using namespace UE::Interchange;

	AGeometryCacheActor* GeometryCacheActor = Cast<AGeometryCacheActor>(&SpawnedActor);
	if (!GeometryCacheActor)
	{
		return nullptr;
	}

	UGeometryCacheComponent* GeometryCacheComponent = GeometryCacheActor->GetGeometryCacheComponent();
	if (!GeometryCacheComponent)
	{
		return nullptr;
	}

	const UInterchangeMeshActorFactoryNode* MeshActorFactoryNode = Cast<UInterchangeMeshActorFactoryNode>(&FactoryNode);
	if (!MeshActorFactoryNode)
	{
		return nullptr;
	}

	if (const UInterchangeFactoryBaseNode* MeshNode = ActorHelper::FindAssetInstanceFactoryNode(&NodeContainer, &FactoryNode))
	{
		GeometryCacheComponent->UnregisterComponent();

		ApplyGeometryCacheToComponent(MeshNode, GeometryCacheComponent, NodeContainer, MeshActorFactoryNode);

		GeometryCacheComponent->RegisterComponent();
	}

	return GeometryCacheComponent;
}

void UInterchangeGeometryCacheActorFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	if (AGeometryCacheActor* GeometryCacheActor = Cast<AGeometryCacheActor>(Arguments.ImportedObject))
	{
		if (UGeometryCacheComponent* GeometryCacheComponent = GeometryCacheActor->GetGeometryCacheComponent())
		{
			TArray<FString> TargetNodeUids;
			Arguments.FactoryNode->GetTargetNodeUids(TargetNodeUids);
			if (TargetNodeUids.Num() == 0)
			{
				return;
			}

			FString SceneNodeUid = TargetNodeUids[0];

			const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(Arguments.NodeContainer->GetNode(SceneNodeUid));
			if (!SceneNode)
			{
				return;
			}
			FString AssetUid;
			SceneNode->GetCustomAssetInstanceUid(AssetUid);

			const FString AssetFactoryUid = TEXT("Factory_GeometryCache_") + AssetUid;
			const UInterchangeGeometryCacheFactoryNode* GeometryCacheFactoryNode = Cast<UInterchangeGeometryCacheFactoryNode>(Arguments.NodeContainer->GetFactoryNode(AssetFactoryUid));
			if (GeometryCacheFactoryNode)
			{
				UInterchangeMeshActorFactoryNode* ActorFactoryNode = Cast<UInterchangeMeshActorFactoryNode>(Arguments.FactoryNode);
				ApplyGeometryCacheToComponent(GeometryCacheFactoryNode, GeometryCacheComponent, *Arguments.NodeContainer, ActorFactoryNode);
			}
		}
	}
}

UObject* UInterchangeGeometryCacheActorFactory::ImportSceneObject_GameThread(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams)
{
	// Cache the previous factory node on reimport
	PreviousFactoryNode = CreateSceneObjectsParams.bIsReimport ? CreateSceneObjectsParams.ReimportFactoryNode : nullptr;
	return Super::ImportSceneObject_GameThread(CreateSceneObjectsParams);
}

void UInterchangeGeometryCacheActorFactory::ExecuteResetObjectProperties(const UInterchangeBaseNodeContainer* BaseNodeContainer, UInterchangeFactoryBaseNode* FactoryNode, UObject* ImportedObject)
{
	Super::ExecuteResetObjectProperties(BaseNodeContainer, FactoryNode, ImportedObject);
	if (AGeometryCacheActor* GeometryCacheActor = Cast<AGeometryCacheActor>(ImportedObject))
	{
		if (UInterchangeMeshActorFactoryNode* MeshActorFactoryNode = Cast<UInterchangeMeshActorFactoryNode>(FactoryNode))
		{
			if (UGeometryCacheComponent* GeometryCacheComponent = GeometryCacheActor->GetGeometryCacheComponent())
			{
				using namespace UE::Interchange;
				FString InstancedAssetFactoryNodeUid;
				if (MeshActorFactoryNode->GetCustomInstancedAssetFactoryNodeUid(InstancedAssetFactoryNodeUid))
				{
					if (const UInterchangeFactoryBaseNode* MeshNode = Cast<UInterchangeFactoryBaseNode>(BaseNodeContainer->GetNode(InstancedAssetFactoryNodeUid)))
					{
						FSoftObjectPath ReferenceObject;
						MeshNode->GetCustomReferenceObject(ReferenceObject);
						if (UGeometryCache* GeometryCache = Cast<UGeometryCache>(ReferenceObject.TryLoad()))
						{
							if (GeometryCache != GeometryCacheComponent->GetGeometryCache())
							{
								GeometryCacheComponent->SetGeometryCache(GeometryCache);
							}
						}
					}
				}

				GeometryCacheComponent->EmptyOverrideMaterials();
				ActorHelper::ApplySlotMaterialDependencies(*BaseNodeContainer, *MeshActorFactoryNode, *GeometryCacheComponent);
			}
		}
	}
}

void UInterchangeGeometryCacheActorFactory::ApplyGeometryCacheToComponent(const UInterchangeFactoryBaseNode* MeshNode, UGeometryCacheComponent* GeometryCacheComponent, const UInterchangeBaseNodeContainer& NodeContainer, const UInterchangeMeshActorFactoryNode* MeshFactoryNode)
{
	FSoftObjectPath ReferenceObject;
	MeshNode->GetCustomReferenceObject(ReferenceObject);

	if (UGeometryCache* GeometryCache = Cast<UGeometryCache>(ReferenceObject.TryLoad()))
	{
		if (GeometryCache != GeometryCacheComponent->GetGeometryCache())
		{
			GeometryCacheComponent->SetGeometryCache(GeometryCache);
		}

		// Apply the materials from the mesh node to the geometry cache component
		if (MeshFactoryNode)
		{
			UE::Interchange::ActorHelper::ApplySlotMaterialDependencies(NodeContainer, *MeshFactoryNode, *GeometryCacheComponent, Cast<UInterchangeMeshActorFactoryNode>(PreviousFactoryNode));
		}
	}
}
