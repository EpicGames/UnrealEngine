// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/InterchangeHeterogeneousVolumeActorFactory.h"

#include "InterchangeHeterogeneousVolumeActorFactoryNode.h"
#include "InterchangeImportLog.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Scene/InterchangeActorHelper.h"

#include "Components/HeterogeneousVolumeComponent.h"
#include "CoreMinimal.h"
#include "Materials/MaterialInterface.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeHeterogeneousVolumeActorFactory)

UClass* UInterchangeHeterogeneousVolumeActorFactory::GetFactoryClass() const
{
	return AHeterogeneousVolume::StaticClass();
}

UObject* UInterchangeHeterogeneousVolumeActorFactory::ProcessActor(
	AActor& SpawnedActor,
	const UInterchangeActorFactoryNode& FactoryNode,
	const UInterchangeBaseNodeContainer& NodeContainer,
	const FImportSceneObjectsParams& Params
)
{
	AHeterogeneousVolume* VolumeActor = Cast<AHeterogeneousVolume>(&SpawnedActor);
	if (!VolumeActor)
	{
		return nullptr;
	}

	UHeterogeneousVolumeComponent* VolumeComponent = Cast<UHeterogeneousVolumeComponent>(VolumeActor->GetRootComponent());
	if (!VolumeComponent)
	{
		return nullptr;
	}

	const UInterchangeHeterogeneousVolumeActorFactoryNode* ActorFactoryNode = Cast<UInterchangeHeterogeneousVolumeActorFactoryNode>(&FactoryNode);
	if (!ActorFactoryNode)
	{
		return nullptr;
	}

	// Check for a referenced material
	UMaterialInterface* ReferencedMaterial = nullptr;
	if (FString MaterialFactoryNodeUid; ActorFactoryNode->GetCustomVolumetricMaterialUid(MaterialFactoryNodeUid))
	{
		if (UInterchangeFactoryBaseNode* MaterialFactoryNode = NodeContainer.GetFactoryNode(MaterialFactoryNodeUid))
		{
			FSoftObjectPath ReferencedObject;
			MaterialFactoryNode->GetCustomReferenceObject(ReferencedObject);

			ReferencedMaterial = Cast<UMaterialInterface>(ReferencedObject.TryLoad());
			if (!ReferencedMaterial)
			{
				UE_LOGF(
					LogInterchangeImport,
					Warning,
					"Failed to find material '%ls' referenced by heterogeneous volume actor factory node '%ls' ('%ls')",
					*ReferencedObject.ToString(),
					*ActorFactoryNode->GetUniqueID(),
					*ActorFactoryNode->GetDisplayLabel()
				);
			}
		}
	}
	if (ReferencedMaterial)
	{
		ReferencedMaterial->PostLoad();

		// The component is hard-coded to handle only one material (it will check for index == 0)
		const int ElementIndex = 0;
		VolumeComponent->SetMaterial(0, ReferencedMaterial);
	}

	return VolumeComponent;
}
