// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/InterchangeLevelFactory.h"

#include "Algo/TopologicalSort.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "InterchangeAssetUserData.h"
#include "InterchangeEditorUtilitiesBase.h"
#include "InterchangeImportCommon.h"
#include "InterchangeImportLog.h"
#include "InterchangeLevelFactoryNode.h"
#include "InterchangeLevelInstanceActorFactoryNode.h"
#include "InterchangeManager.h"
#include "InterchangeResult.h"
#include "InterchangeSceneImportAsset.h"
#include "InterchangeSourceData.h"
#include "InterchangeTranslatorBase.h"
#include "InterchangeVariantSetNode.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "Misc/App.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "UObject/Class.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterShape.h"
#include "WorldPartition/WorldPartitionEditorLoaderAdapter.h"
#include "WorldPartition/WorldPartition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeLevelFactory)

#define LOCTEXT_NAMESPACE "InterchangeLevelFactory"

namespace UE::Interchange::Private::InterchangeLevelFactory
{
	const UInterchangeLevelFactoryNode* GetFactoryNode(const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments, UClass* TargetClass)
	{
		if (!Arguments.NodeContainer || !Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(TargetClass))
		{
			return nullptr;
		}

		return Cast<UInterchangeLevelFactoryNode>(Arguments.AssetNode);
	}

	UWorld* FindOrCreateWorldAsset(const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments, UClass* TargetClass)
	{
		if (Arguments.ReimportObject && Arguments.ReimportObject->GetClass()->IsChildOf(TargetClass))
		{
			return Cast<UWorld>(Arguments.ReimportObject);
		}

		// Check whether map already exists. It should
		UPackage* ParentPackage = Cast<UPackage>(Arguments.Parent);
		if (!ensure(ParentPackage))
		{
			return nullptr;
		}

		// #interchange_levelinstance_rework
		// If this is not a reimport and the level already exists, it means the user has accepted to import over
		// Destroy all imported actors from existing world, equivalent to overwriting source data in static mesh for instance
		// Otherwise new actors will be added to the world instead of being updated
		if (!Arguments.ReimportObject)
		{
			const FTopLevelAssetPath TargetPath(*ParentPackage->GetPathName(), *Arguments.AssetName);
			if (UWorld* ExistingWorld = Cast<UWorld>(FSoftObjectPath(TargetPath).TryLoad()))
			{
				TSet<AActor*> ActorsToDelete;
				for (ULevel* Level : ExistingWorld->GetLevels())
				{
					for (AActor* Actor : Level->Actors)
					{
						if (!IsValid(Actor) || ActorsToDelete.Contains(Actor))
						{
							continue;
						}

						if (USceneComponent* RootComponent = Actor->GetRootComponent())
						{
							// Only delete actors created from a previous import
							if (RootComponent->GetAssetUserData<UInterchangeAssetUserData>())
							{
								ActorsToDelete.Add(Actor);

								// All parent actors were part of the import. Add them
								AActor* ParentActor = Actor->GetAttachParentActor();
								while (IsValid(ParentActor))
								{
									ActorsToDelete.Add(ParentActor);
									ParentActor = ParentActor->GetAttachParentActor();
								}
							}
						}
					}
				}

				// Sort actors from leaf to root to avoid reassignment of child actors
				// which generate multiple unnecessary editor's notifications when there is a re-parenting
				TArray<AActor*> SortedActors = ActorsToDelete.Array();
				ActorsToDelete.Empty();

				TMap<AActor*, TArray<AActor*>> ParentToChildren;
				for (AActor* Actor : SortedActors)
				{
					if (!IsValid(Actor))
					{
						continue;
					}

					AActor* ParentActor = Actor->GetAttachParentActor();
					if (ParentActor)
					{
						TArray<AActor*>& Children = ParentToChildren.FindOrAdd(ParentActor);
						Children.Add(Actor);
					}
				}

				auto FindDependencies = [&ParentToChildren](AActor* Actor) -> TArray<AActor*>
					{
						if (TArray<AActor*>* Children = ParentToChildren.Find(Actor))
						{
							return *Children;
						}

						return {};
					};

				Algo::TopologicalSort(SortedActors, FindDependencies);

				// Delete actors
				for (AActor* Actor : SortedActors)
				{
					if (!IsValid(Actor))
					{
						continue;
					}

					ExistingWorld->EditorDestroyActor(Actor, true);

					// Actor actual deletion can be delayed. Rename to avoid future name collision
					// Call UObject::Rename directly on actor to avoid AActor::Rename which unnecessarily unregister and re-register components
					Actor->UObject::Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
				}

				return ExistingWorld;
			}
		}

		UInterchangeLevelFactoryNode* LevelFactoryNode = Cast<UInterchangeLevelFactoryNode>(Arguments.AssetNode);
		if (!LevelFactoryNode)
		{
			return nullptr;
		}

		// Map creation can only occur in the game thread
		if (!ensure(IsInGameThread()))
		{
			return nullptr;
		}

		bool bCreateWorldPartition = false;
		LevelFactoryNode->GetCustomCreateWorldPartitionLevel(bCreateWorldPartition);

		// Create a new world.
		constexpr bool bAddToRoot = false;
		constexpr bool bEnableWorldPartitionStreaming = false;
		constexpr bool bInformEngineOfWorld = false;
		bool bIsRuntimeOrPIE = false;
		if (UInterchangeEditorUtilitiesBase* EditorUtilities = UInterchangeManager::GetInterchangeManager().GetEditorUtilities())
		{
			bIsRuntimeOrPIE = EditorUtilities->IsRuntimeOrPIE();
		}

		// Those are the init values taken from the default in UWorld::CreateWorld + CreateWorldPartition.
		UWorld::InitializationValues InitValues = UWorld::InitializationValues()
			.ShouldSimulatePhysics(false)
			.EnableTraceCollision(true)
			.CreateNavigation(!bIsRuntimeOrPIE)
			.CreateAISystem(!bIsRuntimeOrPIE)
			.CreateWorldPartition(bCreateWorldPartition)
			.EnableWorldPartitionStreaming(bEnableWorldPartitionStreaming);

		UWorld* NewWorld = UWorld::CreateWorld(EWorldType::Inactive, bInformEngineOfWorld, *Arguments.AssetName, ParentPackage, bAddToRoot, ERHIFeatureLevel::Num, &InitValues);
		if (ensure(NewWorld))
		{
			NewWorld->SetFlags(RF_Public | RF_Standalone);
			return NewWorld;
		}

		return nullptr;
	}
}


UClass* UInterchangeLevelFactory::GetFactoryClass() const
{
	return UWorld::StaticClass();
}

UObject* UInterchangeLevelFactory::GetObjectToReimport(UObject* ReimportObject, const UInterchangeFactoryBaseNode& FactoryNode, const FString& PackageName, const FString& AssetName, const FString& SubPathString)
{
	// #interchange_levelinstance_rework
	// The code below only works when associated with the UInterchangeGenericLevelPipeline and
	// the current implementation with only one level instance actor

	// If this is a level reimport, try to find the level of the previous import
	if (ReimportObject && FactoryNode.IsA<UInterchangeLevelFactoryNode>())
	{
		if (UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(ReimportObject))
		{
			// If reimporting with the same file, the InterchangeSceneImportAsset should be able to return the level of the previous import
			if (UObject* ObjectToReimport = SceneImportAsset->GetSceneObject(PackageName, AssetName, SubPathString))
			{
				return ObjectToReimport;
			}

			// If reimporting with a new file, return the level of the previous import if any
			TArray<const UInterchangeFactoryBaseNode*> FacoryNodes = SceneImportAsset->GetFactoryNodesOfClass(UInterchangeLevelFactoryNode::StaticClass());
			ensure(FacoryNodes.Num() < 2);

			if (FacoryNodes.Num() == 1)
			{
				const UInterchangeLevelFactoryNode* LevelFactoryNode = Cast<UInterchangeLevelFactoryNode>(FacoryNodes[0]);
				
				if (ensure(LevelFactoryNode))
				{
					FSoftObjectPath ObjectPath;
					ensure(LevelFactoryNode->GetCustomReferenceObject(ObjectPath));

					UObject* ObjectToReimport = ObjectPath.TryLoad();
					ensure(ObjectToReimport);

					return ObjectToReimport;
				}

			}
		}
	}

	return Super::GetObjectToReimport(ReimportObject, FactoryNode, PackageName, AssetName, SubPathString);
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeLevelFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeLevelFactory::BeginImportAsset_GameThread);

	using namespace UE::Interchange::Private::InterchangeLevelFactory;

	UClass* TargetClass = GetFactoryClass();

	if (GetFactoryNode(Arguments, TargetClass) == nullptr)
	{
		return {};
	}

	UWorld* WorldAsset = FindOrCreateWorldAsset(Arguments, TargetClass);

	if (!WorldAsset)
	{
		UE_LOGF(LogInterchangeImport, Warning, "Could not create world asset %ls", *Arguments.AssetName);
	}
	//Getting the file Hash will cache it into the source data
	else if (ensure(Arguments.SourceData))
	{
		Arguments.SourceData->GetFileContentHash();
	}

#if WITH_EDITOR
	UInterchangeEditorUtilitiesBase* EditorUtilities = UInterchangeManager::GetInterchangeManager().GetEditorUtilities();
	if ((!EditorUtilities || !EditorUtilities->IsRuntimeOrPIE()) && WorldAsset)
	{
		WorldAsset->PreEditChange(nullptr);
	}
#endif //WITH_EDITOR

	FImportAssetResult Result;
	Result.ImportedObject = WorldAsset;

	return Result;
}

void UInterchangeLevelFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeLevelFactory::SetupObject_GameThread);
	Super::SetupObject_GameThread(Arguments);
	if (!ensure(IsInGameThread()))
	{
		return;
	}

	UWorld* World = Cast<UWorld>(Arguments.ImportedObject);
	UInterchangeLevelFactoryNode* FactoryNode = Cast<UInterchangeLevelFactoryNode>(Arguments.FactoryNode);
	if (ensure(FactoryNode && World && Arguments.SourceData))
	{
		/** Apply all FactoryNode custom attributes to the level sequence asset */
		FactoryNode->ApplyAllCustomAttributeToObject(World);

		//The scene import data object will add them self to this world has a sub object
	}
}


#undef LOCTEXT_NAMESPACE
