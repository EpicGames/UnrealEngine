// Copyright Epic Games, Inc. All Rights Reserved. 
#include "Scene/InterchangeLevelInstanceActorFactory.h"

#include "Engine/Blueprint.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "InterchangeEditorUtilitiesBase.h"
#include "InterchangeImportLog.h"
#include "InterchangeLevelFactoryNode.h"
#include "InterchangeLevelInstanceActorFactoryNode.h"
#include "InterchangeManager.h"
#include "InterchangeSceneImportAsset.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "PackedLevelActor/PackedLevelActor.h"
#include "PackedLevelActor/PackedLevelActorBuilder.h"
#include "Scene/InterchangeActorHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeLevelInstanceActorFactory)

UClass* UInterchangeLevelInstanceActorFactory::GetFactoryClass() const
{
	return ALevelInstance::StaticClass();
}

UObject* UInterchangeLevelInstanceActorFactory::GetObjectToReimport(UObject* ReimportObject, const UInterchangeFactoryBaseNode& FactoryNode, const FString& PackageName, const FString& AssetName, const FString& SubPathString)
{
	// #interchange_levelinstance_rework
	// The code below only works when associated with the UInterchangeGenericLevelPipeline and
	// the current implementation with only one LevelInstance actor

	// If this is a level reimport, try to find the level instance of the previous import
	if (ReimportObject && FactoryNode.IsA<UInterchangeLevelInstanceActorFactoryNode>())
	{
		if (UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(ReimportObject))
		{
			// If reimporting with the same file, the InterchangeSceneImportAsset should be able to return the level of the previous import
			if (UObject* ObjectToReimport = SceneImportAsset->GetSceneObject(PackageName, AssetName, SubPathString))
			{
				return ObjectToReimport;
			}

			TArray<const UInterchangeFactoryBaseNode*> FacoryNodes = SceneImportAsset->GetFactoryNodesOfClass(UInterchangeLevelInstanceActorFactoryNode::StaticClass());
			ensure(FacoryNodes.Num() < 2);

			// If reimporting with a new file, return the LevelInstance actor of the previous import if any
			if (FacoryNodes.Num() == 1)
			{
				const UInterchangeLevelInstanceActorFactoryNode* ActorFactoryNode = Cast<UInterchangeLevelInstanceActorFactoryNode>(FacoryNodes[0]);

				if (ensure(ActorFactoryNode))
				{
					FSoftObjectPath ObjectPath;
					ensure(ActorFactoryNode->GetCustomReferenceObject(ObjectPath));

					return ObjectPath.TryLoad();
				}

			}
		}
	}

	return Super::GetObjectToReimport(ReimportObject, FactoryNode, PackageName, AssetName, SubPathString);
}

UObject* UInterchangeLevelInstanceActorFactory::ImportSceneObject_GameThread(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams)
{
#if !WITH_EDITOR
	return nullptr;
#else

	using namespace UE::Interchange;
	
	const bool bIsReimport = CreateSceneObjectsParams.bIsReimport || CreateSceneObjectsParams.ReimportObject || CreateSceneObjectsParams.ReimportFactoryNode;

	UInterchangeLevelInstanceActorFactoryNode* FactoryNode = Cast<UInterchangeLevelInstanceActorFactoryNode>(CreateSceneObjectsParams.FactoryNode);
	if (!ensure(FactoryNode) || !CreateSceneObjectsParams.NodeContainer)
	{
		return nullptr;
	}
	
	FString ReferenceLevelFactoryNodeUid;
	if (!FactoryNode->GetCustomLevelReference(ReferenceLevelFactoryNodeUid))
	{
		UE_LOGF(LogInterchangeImport, Warning, "UInterchangeLevelInstanceActorFactory::ImportSceneObject_GameThread: Actor level instance do not reference any level factory node.");
		return nullptr;
	}

	UInterchangeLevelFactoryNode* ReferenceLevelFactoryNode = Cast<UInterchangeLevelFactoryNode>(CreateSceneObjectsParams.NodeContainer->GetFactoryNode(ReferenceLevelFactoryNodeUid));
	if (!ReferenceLevelFactoryNode)
	{
		UE_LOGF(LogInterchangeImport, Warning, "UInterchangeLevelInstanceActorFactory::ImportSceneObject_GameThread: Reference Level factory node is invalid.");
		return nullptr;
	}

	FSoftObjectPath ReferenceWorldPath;
	if (!ReferenceLevelFactoryNode->GetCustomReferenceObject(ReferenceWorldPath))
	{
		UE_LOGF(LogInterchangeImport, Warning, "UInterchangeLevelInstanceActorFactory::ImportSceneObject_GameThread: Reference Level factory node do not point on a valid UWorld.");
		return nullptr;
	}

	UWorld* ReferenceWorld = Cast<UWorld>(ReferenceWorldPath.TryLoad());
	if (!ReferenceWorld)
	{
		UE_LOGF(LogInterchangeImport, Warning, "UInterchangeLevelInstanceActorFactory::ImportSceneObject_GameThread: Cannot load a valid Reference UWorld.");
		return nullptr;
	}

	// #interchange_levelinstance_rework
	// If the reference world is loaded in the editor, do not create a level instance
	// This scenario needs to be properly implemented
	if (bIsReimport && ReferenceWorld == (UWorld*)GWorld)
	{
		UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
		Message->SourceAssetName = CreateSceneObjectsParams.SourceData ? CreateSceneObjectsParams.SourceData->GetFilename() : TEXT("");
		Message->DestinationAssetName = CreateSceneObjectsParams.ObjectName;
		Message->AssetType = ReferenceLevelFactoryNode->GetObjectClass();
		Message->Text = NSLOCTEXT("ImportSceneObject_GameThread", "SkipLevelInstance", "Reimporting a level instance in the level it references. No LevelInstance will be created or updated.");
		
		return nullptr;
	}

	UWorld* TargetWorld = CreateSceneObjectsParams.Level ? CreateSceneObjectsParams.Level->GetWorld() : GWorld->GetCurrentLevel()->GetWorld();
	ensure (TargetWorld);

	const bool bIsPackedActor = FactoryNode->GetObjectClass()->IsChildOf<APackedLevelActor>();

	FString LevelInstanceActorName = FactoryNode->GetDisplayLabel();

	FActorSpawnParameters SpawnParameters;
	SpawnParameters.Name = FName(CreateSceneObjectsParams.ObjectName);
	SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
	SpawnParameters.OverrideLevel = CreateSceneObjectsParams.Level;
	SpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;

	UClass* LevelInstanceClass = bIsPackedActor ? APackedLevelActor::StaticClass() : ALevelInstance::StaticClass();

	ALevelInstance* LevelInstanceActor = nullptr;
	//Packed level actor need a blueprint between the reference level and the actor
	if (bIsPackedActor)
	{
		auto FindExisitingAsset = [](const FString& AssetPackagePath, const FString& AssetName)->UObject*
			{
				UPackage* Pkg = FindPackage(nullptr, *AssetPackagePath);
				if (!Pkg)
				{
					//Try to load the package from disk
					Pkg = LoadPackage(nullptr, *AssetPackagePath, LOAD_NoWarn | LOAD_Quiet);
				}
				if (Pkg)
				{
					return StaticFindObject(nullptr, Pkg, *AssetName);
				}
				return nullptr;
			};
		const FString ReferenceWorldPackageDir = FPaths::GetPath(ReferenceWorldPath.GetLongPackageName());
		const FString BPAssetName = FPackedLevelActorBuilder::GetPackedBPPrefix() + ReferenceWorldPath.GetAssetName();
		const FString BPPackageName = FString::Format(TEXT("{0}/{1}"), { ReferenceWorldPackageDir , BPAssetName });
	
		UBlueprint* PackedActorBP = Cast<UBlueprint>(FindExisitingAsset(BPPackageName, BPAssetName));
		TSharedPtr<FPackedLevelActorBuilder> PackedLevelActorBuilder = FPackedLevelActorBuilder::CreateDefaultBuilder();

		if (!PackedActorBP)
		{
			// #interchange_level_instance_revisit
			//Reference world must be saved if we want to create the packed actor
			if (UInterchangeEditorUtilitiesBase* EditorUtilities = UInterchangeManager::GetInterchangeManager().GetEditorUtilities())
			{
				if (!EditorUtilities->SaveAsset(ReferenceWorld))
				{
					UE_LOGF(LogInterchangeImport, Warning, "UInterchangeLevelInstanceActorFactory::ImportSceneObject_GameThread: Cannot save the packed actor's (%ls) referenced world (%ls)", *FactoryNode->GetDisplayLabel(), *ReferenceWorld->GetName());
				}
			}

			constexpr bool bCompileTrue = true;
			const FString BPAssetPath = FString::Format(TEXT("{0}.{1}"), { BPPackageName , BPAssetName });
			PackedActorBP = PackedLevelActorBuilder->CreatePackedLevelActorBlueprint(TSoftObjectPtr<UBlueprint>(FSoftObjectPath(BPAssetPath)), ReferenceWorld, bCompileTrue);
		}

		if (ensure(PackedActorBP))
		{
			TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = UInterchangeManager::GetInterchangeManager().GetAsyncHelper(CreateSceneObjectsParams.AsyncHelperUid);
			if (AsyncHelper.IsValid())
			{
				constexpr int32 SourceIndexZero = 0;
				PackedActorBP->SetInternalFlags(EInternalObjectFlags::Async);
				FImportAsyncHelper::FImportedObjectInfo& AssetInfo = AsyncHelper->AddDefaultImportedAssetGetRef(SourceIndexZero);

				//We need a valid factory node to have a valid DefaultImportedAsset entry. Its mandatory to register all created asset
				//Currently we do not have a blueprint factory node we can simply use the base factory node class.
				UInterchangeBaseNodeContainer* MutableNodeContainer = const_cast<UInterchangeBaseNodeContainer*>(CreateSceneObjectsParams.NodeContainer);
				UInterchangeFactoryBaseNode* BpFactoryNode = NewObject<UInterchangeFactoryBaseNode>(MutableNodeContainer);
				FString UniqueId = FGuid::NewGuid().ToString(EGuidFormats::Base36Encoded);
				MutableNodeContainer->SetupNode(BpFactoryNode, UniqueId, PackedActorBP->GetName(), EInterchangeNodeContainerType::FactoryData);
				BpFactoryNode->SetCustomReferenceObject(PackedActorBP);

				AssetInfo.SetImportedObject(PackedActorBP);
				AssetInfo.Factory = this;
				AssetInfo.FactoryNode = BpFactoryNode;
				AssetInfo.bIsReimport = false;
			}

			LevelInstanceActor = Cast<ALevelInstance>(CreateSceneObjectsParams.ReimportObject);
			if(!LevelInstanceActor)
			{
				LevelInstanceActor = TargetWorld->SpawnActor<APackedLevelActor>(PackedActorBP->GeneratedClass, SpawnParameters);
			}
		}

		if (!LevelInstanceActor)
		{
			UE_LOGF(LogInterchangeImport, Warning, "UInterchangeLevelInstanceActorFactory::ImportSceneObject_GameThread: Failed to create packed level blueprint.");
			return nullptr;
		}
	}
	else
	{
		LevelInstanceActor = Cast<ALevelInstance>(CreateSceneObjectsParams.ReimportObject);
		if(!LevelInstanceActor)
		{
			//simple Level Instance actor simply need to spawn the actor, the reference level is already setup
			LevelInstanceActor = TargetWorld->SpawnActor<ALevelInstance>(ALevelInstance::StaticClass(), SpawnParameters);
		}
	}

	if (!LevelInstanceActor)
	{
		UE_LOGF(LogInterchangeImport, Warning, "UInterchangeLevelInstanceActorFactory::ImportSceneObject_GameThread: Cannot create/Find a level instance actor in the target world.");
		return nullptr;
	}

	AActor* ParentActor = UE::Interchange::ActorHelper::GetSpawnedParentActor(CreateSceneObjectsParams.NodeContainer, FactoryNode);
	if (ParentActor || !LevelInstanceActor->IsAttachedTo(ParentActor))
	{
		LevelInstanceActor->AttachToActor(ParentActor, FAttachmentTransformRules::KeepWorldTransform);
	}

	FactoryNode->SetCustomReferenceObject(LevelInstanceActor);

	return LevelInstanceActor;
#endif //!WITH_EDITOR
}
