// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/InterchangeLightActorFactory.h"
#include "Components/LightComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/RectLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "InterchangeTextureLightProfileFactoryNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Scene/InterchangeActorHelper.h"

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "InterchangeLightFactoryNode.h"
#include "Engine/Light.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeLightActorFactory)

UClass* UInterchangeLightActorFactory::GetFactoryClass() const
{
	return ALight::StaticClass();
}

UObject* UInterchangeLightActorFactory::ProcessActor(AActor& SpawnedActor, const UInterchangeActorFactoryNode& FactoryNode, const UInterchangeBaseNodeContainer& NodeContainer, const FImportSceneObjectsParams& /*Params*/)
{
	if (ALight* LightActor = Cast<ALight>(&SpawnedActor))
	{
		if (ULightComponent* LightComponent = LightActor->GetLightComponent())
		{
			LightComponent->UnregisterComponent();

			if (const UInterchangeLightFactoryNode* LightFactoryNode = Cast<UInterchangeLightFactoryNode>(&FactoryNode))
			{
				if (FString IESTexture; LightFactoryNode->GetCustomIESTexture(IESTexture))
				{
					FString IESFactoryTextureId = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(IESTexture);
					if (const UInterchangeTextureLightProfileFactoryNode* TextureFactoryNode = Cast<UInterchangeTextureLightProfileFactoryNode>(NodeContainer.GetNode(IESFactoryTextureId)))
					{
						FSoftObjectPath ReferenceObject;
						TextureFactoryNode->GetCustomReferenceObject(ReferenceObject);
						if (UTextureLightProfile* Texture = Cast<UTextureLightProfile>(ReferenceObject.TryLoad()))
						{
							LightComponent->SetIESTexture(Texture);
							
							if (bool bUseIESBrightness; LightFactoryNode->GetCustomUseIESBrightness(bUseIESBrightness))
							{
								LightComponent->SetUseIESBrightness(bUseIESBrightness);
							}
							
							if (float IESBrightnessScale; LightFactoryNode->GetCustomIESBrightnessScale(IESBrightnessScale))
							{
								LightComponent->SetIESBrightnessScale(IESBrightnessScale);
							}
							
							if (FRotator Rotation; LightFactoryNode->GetCustomRotation(Rotation))
							{
								// Apply Ies rotation to light orientation
								FQuat NewLightRotation = LightComponent->GetRelativeRotation().Quaternion() * Rotation.Quaternion();
								LightComponent->SetRelativeRotation(NewLightRotation);
							}
						}
					}
				}
			}

			static const auto CVarDefaultLightUnits = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultFeature.LightUnits"));
			ELightUnits DefaultUnits = (ELightUnits)CVarDefaultLightUnits->GetValueOnAnyThread();

			// Replicate the code in UActorFactorySpotLight::PostSpawnActor and analogues so we end up with the same light units that a user would get
			// on their components when placing a light actor manually onto the level
			if (USpotLightComponent* SpotLightComponent = Cast<USpotLightComponent>(LightComponent))
			{
				SpotLightComponent->Intensity *= UPointLightComponent::GetUnitsConversionFactor(SpotLightComponent->IntensityUnits, DefaultUnits, SpotLightComponent->GetCosHalfConeAngle());
				SpotLightComponent->IntensityUnits = DefaultUnits;
			}
			else if (UPointLightComponent* PointLightComponent = Cast<UPointLightComponent>(LightComponent))
			{
				PointLightComponent->Intensity *= UPointLightComponent::GetUnitsConversionFactor(PointLightComponent->IntensityUnits, DefaultUnits, -1.f);
				PointLightComponent->IntensityUnits = DefaultUnits;
			}
			else if (URectLightComponent* RectLightComponent = Cast<URectLightComponent>(LightComponent))
			{
				// Passing .5 as CosHalfConeAngle  since URectLightComponent::SetLightBrightness() lumens conversion use only PI
				RectLightComponent->Intensity *= URectLightComponent::GetUnitsConversionFactor(RectLightComponent->IntensityUnits, DefaultUnits, .5f);
				RectLightComponent->IntensityUnits = DefaultUnits;
			}

			return LightComponent;
		}
	}

	return nullptr;
}
