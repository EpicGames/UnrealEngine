// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/InterchangeSceneImportAssetFactory.h"

#include "InterchangeAssetImportData.h"
#include "InterchangeImportCommon.h"
#include "InterchangeImportLog.h"
#include "InterchangeSceneImportAsset.h"
#include "InterchangeSceneImportAssetFactoryNode.h"
#include "InterchangeResult.h"
#include "InterchangeSourceData.h"
#include "InterchangeTranslatorBase.h"
#include "InterchangeVariantSetNode.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSceneImportAssetFactory)

#define LOCTEXT_NAMESPACE "InterchangeSceneImportAssetFactory"

namespace UE::Interchange::Private::InterchangeSceneImportAssetFactory
{
	const UInterchangeSceneImportAssetFactoryNode* GetFactoryNode(const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments, UClass* TargetClass)
	{
		if (!Arguments.NodeContainer || !Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(TargetClass))
		{
			return nullptr;
		}

		const UInterchangeSceneImportAssetFactoryNode* FactoryNode = Cast<UInterchangeSceneImportAssetFactoryNode>(Arguments.AssetNode);
		if (FactoryNode == nullptr)
		{
			return nullptr;
		}

		return FactoryNode;
	}

	UObject* FindOrCreateAsset(const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments, UClass* TargetClass)
	{
		UObject* TargetAsset = Arguments.ReimportObject ? Arguments.ReimportObject : StaticFindObject(nullptr, Arguments.Parent, *Arguments.AssetName);

		// Create a new asset or return existing asset, if possible
		if (!TargetAsset)
		{
			check(IsInGameThread());
			TargetAsset = NewObject<UObject>(Arguments.Parent, TargetClass, *Arguments.AssetName, RF_Public | RF_Standalone);
		}
		else if (!TargetAsset->GetClass()->IsChildOf(TargetClass))
		{
			TargetAsset = nullptr;
		}

		return TargetAsset;
	}
}

UClass* UInterchangeSceneImportAssetFactory::GetFactoryClass() const
{
	return UInterchangeSceneImportAsset::StaticClass();
}

bool UInterchangeSceneImportAssetFactory::GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const
{
#if WITH_EDITORONLY_DATA
	if (const UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(Object))
	{
		return UE::Interchange::FFactoryCommon::GetSourceFilenames(SceneImportAsset->AssetImportData, OutSourceFilenames);
	}
#endif

	return false;
}

bool UInterchangeSceneImportAssetFactory::SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const
{
#if WITH_EDITORONLY_DATA
	if (const UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(Object))
	{
		return UE::Interchange::FFactoryCommon::SetSourceFilename(SceneImportAsset->AssetImportData, SourceFilename, SourceIndex);
	}
#endif

	return false;
}

void UInterchangeSceneImportAssetFactory::BackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(Object))
	{
		UE::Interchange::FFactoryCommon::BackupSourceData(SceneImportAsset->AssetImportData);
	}
#endif
}

void UInterchangeSceneImportAssetFactory::ReinstateSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(Object))
	{
		UE::Interchange::FFactoryCommon::ReinstateSourceData(SceneImportAsset->AssetImportData);
	}
#endif
}

void UInterchangeSceneImportAssetFactory::ClearBackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(Object))
	{
		UE::Interchange::FFactoryCommon::ClearBackupSourceData(SceneImportAsset->AssetImportData);
	}
#endif
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeSceneImportAssetFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSceneImportAssetFactory::BeginImportAsset_GameThread);
	using namespace UE::Interchange::Private::InterchangeSceneImportAssetFactory;

	UClass* TargetClass = GetFactoryClass();

	if (GetFactoryNode(Arguments, TargetClass) == nullptr)
	{
		return {};
	}

	UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(FindOrCreateAsset(Arguments, TargetClass));

	if (!SceneImportAsset)
	{
		UE_LOGF(LogInterchangeImport, Warning, "Could not create InterchangeSceneImportAsset asset %ls", *Arguments.AssetName);
	}
	//Getting the file Hash will cache it into the source data
	else if (ensure(Arguments.SourceData))
	{
		Arguments.SourceData->GetFileContentHash();
	}

	FImportAssetResult Result;
	Result.ImportedObject = SceneImportAsset;

	return Result;
}

void UInterchangeSceneImportAssetFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSceneImportAssetFactory::SetupObject_GameThread);
	check(IsInGameThread());
	Super::SetupObject_GameThread(Arguments);

	UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(Arguments.ImportedObject);
	UInterchangeSceneImportAssetFactoryNode* FactoryNode = Cast<UInterchangeSceneImportAssetFactoryNode>(Arguments.FactoryNode);
	if (ensure(FactoryNode && SceneImportAsset && Arguments.SourceData))
	{
		/** Apply all FactoryNode custom attributes to the level sequence asset */
		FactoryNode->ApplyAllCustomAttributeToObject(SceneImportAsset);

#if WITH_EDITORONLY_DATA
		using namespace UE::Interchange;

		FFactoryCommon::FUpdateImportAssetDataParameters Parameters(SceneImportAsset, SceneImportAsset->AssetImportData, Arguments.SourceData, Arguments.NodeUniqueID, Arguments.NodeContainer, Arguments.OriginalPipelines, Arguments.Translator);
		SceneImportAsset->AssetImportData = Cast<UInterchangeAssetImportData>(FFactoryCommon::UpdateImportAssetData(Parameters));
#endif
	}
}

void UInterchangeSceneImportAssetFactory::FinalizeObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSceneImportAssetFactory::FinalizeObject_GameThread);
	check(IsInGameThread());
	Super::FinalizeObject_GameThread(Arguments);

	UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(Arguments.ImportedObject);
	if (!ensure(SceneImportAsset && Arguments.FactoryNode))
	{
		return;
	}


#if WITH_EDITOR
	SceneImportAsset->RegisterWorldRenameCallbacks();
#endif
}

#undef LOCTEXT_NAMESPACE
