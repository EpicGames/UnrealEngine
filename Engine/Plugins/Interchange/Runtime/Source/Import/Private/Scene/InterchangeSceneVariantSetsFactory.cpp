// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/InterchangeSceneVariantSetsFactory.h"

#include "InterchangeImportLog.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeResult.h"
#include "InterchangeSourceData.h"
#include "InterchangeTranslatorBase.h"
#include "InterchangeSceneVariantSetsFactoryNode.h"
#include "InterchangeVariantSetNode.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "GameFramework/Actor.h"
#include "LevelVariantSets.h"
#include "Materials/MaterialInterface.h"
#include "PropertyValue.h"
#include "Variant.h"
#include "VariantObjectBinding.h"
#include "VariantSet.h"
#if WITH_EDITOR
#include "VariantManager.h"
#include "VariantManagerModule.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSceneVariantSetsFactory)

#define LOCTEXT_NAMESPACE "InterchangeSceneVariantSetsFactory"

#if WITH_EDITOR
namespace UE::Interchange::Private
{
	class FLevelVariantSetsHelper
	{
	public:
		FLevelVariantSetsHelper(ULevelVariantSets& InLevelSequence, UInterchangeSceneVariantSetsFactoryNode& InFactoryNode, const UInterchangeBaseNodeContainer& InNodeContainer, const IInterchangeVariantSetPayloadInterface& InPayloadInterface)
			: LevelVariantSets(InLevelSequence)
			, FactoryNode(InFactoryNode)
			, NodeContainer(InNodeContainer)
			, PayloadInterface(InPayloadInterface)
		{
		}

		void PopulateLevelVariantSets();
		TSharedPtr<FVariantManager> GetVariantManager() { return VariantManager; }

		TArray<FText> Warnings;

	private:
		void AddVariantSet(const UInterchangeVariantSetNode& VariantSetNode);
		void AddVariant(const FVariant& Variant, UVariantSet& VariantSet);
		void AddBinding(const FVariantBinding& Binding, UVariant& Variant);
		void AddPropertyCapture(const FVariantPropertyCaptureData& PropertyCapture, UVariantObjectBinding& ObjectBinding);

		AActor* GetActor(const FString& ActorNodeUid);

	private:
		ULevelVariantSets& LevelVariantSets;
		UInterchangeSceneVariantSetsFactoryNode& FactoryNode;
		const UInterchangeBaseNodeContainer& NodeContainer;
		const IInterchangeVariantSetPayloadInterface& PayloadInterface;
		TSharedPtr<FVariantManager> VariantManager;
	};

	void FLevelVariantSetsHelper::PopulateLevelVariantSets()
	{
		if (FactoryNode.GetCustomVariantSetUidCount() == 0)
		{
			return;
		}

		IVariantManagerModule& VariantManagerModule = FModuleManager::LoadModuleChecked<IVariantManagerModule>("VariantManager");
		VariantManager = VariantManagerModule.CreateVariantManager(&LevelVariantSets);
		if (!ensure(VariantManager))
		{
			return;
		}

		TArray<FString> VariantSetUids;
		FactoryNode.GetCustomVariantSetUids(VariantSetUids);

		for (const FString& VariantSetUid : VariantSetUids)
		{
			if (const UInterchangeVariantSetNode* VariantSetNode = Cast<UInterchangeVariantSetNode>(NodeContainer.GetNode(VariantSetUid)))
			{
				AddVariantSet(*VariantSetNode);
			}
		}
	}

	void FLevelVariantSetsHelper::AddVariantSet(const UInterchangeVariantSetNode& VariantSetNode)
	{
		FString PayloadKey;
		if (!VariantSetNode.GetCustomVariantsPayloadKey(PayloadKey))
		{
			return;
		}

		FString DisplayText;
		VariantSetNode.GetCustomDisplayText(DisplayText);

		const TOptional<UE::Interchange::FVariantSetPayloadData> PayloadData = PayloadInterface.GetVariantSetPayloadData(PayloadKey);
		if (!PayloadData.IsSet())
		{
			UE_LOGF(LogInterchangeImport, Warning, "No payload for variant set %ls.", *DisplayText);
			return;
		}

		UVariantSet* VariantSet = VariantManager->CreateVariantSet(&LevelVariantSets);

		VariantSet->SetDisplayText(FText::FromString(DisplayText));

		for (const FVariant& Variant : PayloadData->Variants)
		{
			AddVariant(Variant, *VariantSet);
		}
	}

	void FLevelVariantSetsHelper::AddVariant(const FVariant& VariantData, UVariantSet& VariantSet)
	{

		UVariant* Variant = VariantManager->CreateVariant(&VariantSet);
		Variant->SetDisplayText(FText::FromString(VariantData.DisplayText));

		for (const FVariantBinding& Binding : VariantData.Bindings)
		{
			AddBinding(Binding, *Variant);
		}
	}

	void FLevelVariantSetsHelper::AddBinding(const FVariantBinding& Binding, UVariant& Variant)
	{
		// Get targeted actor exists
		AActor* Actor = GetActor(Binding.TargetUid);
		if (!Actor)
		{
			UE_LOGF(LogInterchangeImport, Warning, "Cannot find actor for variant %ls.", *Variant.GetDisplayText().ToString());
			return;
		}

		TArray<UVariantObjectBinding*> Bindings = VariantManager->CreateObjectBindings({ Actor }, { &Variant });
		if (Bindings.Num() != 1 || Bindings[0] == nullptr)
		{
			return;
		}

		UVariantObjectBinding& ObjectBinding = *(Bindings[0]);
		for (const FVariantPropertyCaptureData& PropertyCapture : Binding.Captures)
		{
			AddPropertyCapture(PropertyCapture, ObjectBinding);
		}
	}

	void FLevelVariantSetsHelper::AddPropertyCapture(const FVariantPropertyCaptureData& PropertyCapture, UVariantObjectBinding& ObjectBinding)
	{
		using namespace UE::Interchange;

		static_assert((uint8)EVariantPropertyCaptureCategory::Undefined == (uint8)EPropertyValueCategory::Undefined, "INVALID_ENUM_VALUE");
		static_assert((uint8)EVariantPropertyCaptureCategory::Generic == (uint8)EPropertyValueCategory::Generic, "INVALID_ENUM_VALUE");
		static_assert((uint8)EVariantPropertyCaptureCategory::RelativeLocation == (uint8)EPropertyValueCategory::RelativeLocation, "INVALID_ENUM_VALUE");
		static_assert((uint8)EVariantPropertyCaptureCategory::RelativeRotation == (uint8)EPropertyValueCategory::RelativeRotation, "INVALID_ENUM_VALUE");
		static_assert((uint8)EVariantPropertyCaptureCategory::RelativeScale3D == (uint8)EPropertyValueCategory::RelativeScale3D, "INVALID_ENUM_VALUE");
		static_assert((uint8)EVariantPropertyCaptureCategory::Visibility == (uint8)EPropertyValueCategory::Visibility, "INVALID_ENUM_VALUE");
		static_assert((uint8)EVariantPropertyCaptureCategory::Material == (uint8)EPropertyValueCategory::Material, "INVALID_ENUM_VALUE");
		static_assert((uint8)EVariantPropertyCaptureCategory::Color == (uint8)EPropertyValueCategory::Color, "INVALID_ENUM_VALUE");
		static_assert((uint8)EVariantPropertyCaptureCategory::Option == (uint8)EPropertyValueCategory::Option, "INVALID_ENUM_VALUE");

		if (PropertyCapture.Buffer.IsSet())
		{
			const TArray<uint8>& RecordedData = *PropertyCapture.Buffer;

			UPropertyValue* PropertyValue = nullptr;

			if (EnumHasAnyFlags(PropertyCapture.Category, EVariantPropertyCaptureCategory::Visibility))
			{
				TArray<UPropertyValue*> PropertyValues = VariantManager->CreateVisibilityPropertyCaptures({ &ObjectBinding });
				if (PropertyValues.Num() == 1)
				{
					PropertyValue = PropertyValues[0];
				}
			}
			else if (EnumHasAnyFlags(PropertyCapture.Category, EVariantPropertyCaptureCategory::RelativeLocation))
			{
				TArray<UPropertyValue*> PropertyValues = VariantManager->CreateLocationPropertyCaptures({ &ObjectBinding });
				if (PropertyValues.Num() == 1)
				{
					PropertyValue = PropertyValues[0];
				}
			}
			else if (EnumHasAnyFlags(PropertyCapture.Category, EVariantPropertyCaptureCategory::RelativeRotation))
			{
				TArray<UPropertyValue*> PropertyValues = VariantManager->CreateRotationPropertyCaptures({ &ObjectBinding });
				if (PropertyValues.Num() == 1)
				{
					PropertyValue = PropertyValues[0];
				}
			}
			else if (EnumHasAnyFlags(PropertyCapture.Category, EVariantPropertyCaptureCategory::RelativeScale3D))
			{
				TArray<UPropertyValue*> PropertyValues = VariantManager->CreateScale3DPropertyCaptures({ &ObjectBinding });
				if (PropertyValues.Num() == 1)
				{
					PropertyValue = PropertyValues[0];
				}
			}

			if (PropertyValue)
			{
				PropertyValue->SetRecordedData(RecordedData.GetData(), RecordedData.Num());
			}
		}
		else if (PropertyCapture.ObjectUid.IsSet())
		{
			const FString TargetFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(*PropertyCapture.ObjectUid);

			if (EnumHasAnyFlags(PropertyCapture.Category, EVariantPropertyCaptureCategory::Material))
			{
				if (const UInterchangeBaseMaterialFactoryNode* TargetFactoryNode = Cast<UInterchangeBaseMaterialFactoryNode>(NodeContainer.GetNode(TargetFactoryNodeUid)))
				{
					FSoftObjectPath ReferenceObject;
					TargetFactoryNode->GetCustomReferenceObject(ReferenceObject);
					if (const UMaterialInterface* TargetMaterial = Cast<UMaterialInterface>(ReferenceObject.TryLoad()))
					{
						// FVariantManager::CreateMaterialPropertyCaptures returns an array of the newly found properties
						// Therefore, if a mesh actor has more than 1 material and a binding has more than 1 material variant
						// The second call to FVariantManager::CreateMaterialPropertyCaptures will return an empty array.
						// Consequently, we have to check if material properties has not been extracted yet
						TArray<UPropertyValue*> PropertyValues;
						{
							const TArray<UPropertyValue*>& ExistingPropertyValues = ObjectBinding.GetCapturedProperties();

							for (const UPropertyValue* PropertyValue : ExistingPropertyValues)
							{
								if (EnumHasAnyFlags(PropertyValue->GetPropCategory(), EPropertyValueCategory::Material))
								{
									PropertyValues.Add(const_cast<UPropertyValue*>(PropertyValue));
								}
							}

							// Material properties have not been asked for yet.
							if (PropertyValues.Num() == 0)
							{
								PropertyValues = VariantManager->CreateMaterialPropertyCaptures({ &ObjectBinding });
							}
						}

						if (PropertyValues.Num() == 1)
						{
							PropertyValues[0]->SetRecordedData((uint8*)&TargetMaterial, sizeof(UMaterialInterface*));
						}
						else
						{
							for (UPropertyValue* PropertyValue : PropertyValues)
							{
								if (PropertyValue && PropertyValue->GetFullDisplayString() == PropertyCapture.PropertyPath)
								{
									PropertyValue->SetRecordedData((uint8*)&TargetMaterial, sizeof(UMaterialInterface*));
									break;
								}
							}
						}
					}
					else
					{
						UE_LOGF(LogInterchangeImport, Warning, "Could not find material '%ls' when creating variant asset.", *TargetFactoryNode->GetDisplayLabel());
					}
				}
				else
				{
					UE_LOGF(LogInterchangeImport, Warning, "Could not find factory when creating variant assets.");
				}
			}
		}
	}

	AActor* FLevelVariantSetsHelper::GetActor(const FString& ActorNodeUid)
	{
		const FString ActorFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(ActorNodeUid);
		const UInterchangeFactoryBaseNode* ActorFactoryNode = Cast<UInterchangeFactoryBaseNode>(NodeContainer.GetNode(ActorFactoryNodeUid));

		if (ActorFactoryNode )
		{
			FSoftObjectPath ReferenceObject;
			ActorFactoryNode->GetCustomReferenceObject(ReferenceObject);
			return ReferenceObject.IsValid() ? Cast<AActor>(ReferenceObject.TryLoad()) : nullptr;
		}

		return nullptr;
	}
} //namespace UE::Interchange::Private
#endif //WITH_EDITOR

UClass* UInterchangeSceneVariantSetsFactory::GetFactoryClass() const
{
	return ULevelVariantSets::StaticClass();
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeSceneVariantSetsFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSceneVariantSetsFactory::BeginImportAsset_GameThread);
	FImportAssetResult ImportAssetResult;
#if WITH_EDITOR && WITH_EDITORONLY_DATA
	auto CannotReimportMessage = [&Arguments, this]()
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName = Arguments.SourceData->GetFilename();
		Message->DestinationAssetName = Arguments.AssetName;
		Message->AssetType = ULevelVariantSets::StaticClass();
		Message->Text = LOCTEXT("CreateEmptyAssetUnsupportedReimport", "Reimport of ULevelVariantSets is not supported.");
		Arguments.AssetNode->SetSkipNodeImport();
	};

	if (Arguments.ReimportObject)
	{
		CannotReimportMessage();
		return ImportAssetResult;
	}

	if (!Cast<IInterchangeVariantSetPayloadInterface>(Arguments.Translator))
	{
		UE_LOGF(LogInterchangeImport, Error, "Cannot import LevelVariantSets. The translator does not implement the IInterchangeVariantSetPayloadInterface interface.");
		return ImportAssetResult;
	}

	ULevelVariantSets* LevelVariantSets = nullptr;
	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return ImportAssetResult;
	}

	const UInterchangeSceneVariantSetsFactoryNode* FactoryNode = Cast<UInterchangeSceneVariantSetsFactoryNode>(Arguments.AssetNode);
	if (FactoryNode == nullptr)
	{
		return ImportAssetResult;
	}

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (FactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}
	}

	// create a new material or overwrite existing asset, if possible
	if (!ExistingAsset)
	{
		LevelVariantSets = NewObject<ULevelVariantSets>(Arguments.Parent, *Arguments.AssetName, RF_Public | RF_Standalone);
	}
	else
	{
		CannotReimportMessage();
		return ImportAssetResult;
		// TODO put back the cast when the re-import will be supported
		// This is a reimport, we are just re-updating the source data
		//LevelVariantSets = Cast<ULevelVariantSets>(ExistingAsset);
	}

	if (!LevelVariantSets)
	{
		UE_LOGF(LogInterchangeImport, Warning, "Could not create LevelVariantSets asset %ls.", *Arguments.AssetName);
		return ImportAssetResult;
	}

	ImportAssetResult.ImportedObject = ImportObjectSourceData(Arguments, LevelVariantSets);
#endif

	return ImportAssetResult;
}

UObject* UInterchangeSceneVariantSetsFactory::ImportObjectSourceData(const FImportAssetObjectParams& Arguments, ULevelVariantSets* LevelVariantSets)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSceneVariantSetsFactory::ImportObjectSourceData);
#if WITH_EDITOR && WITH_EDITORONLY_DATA
	// TODO: Can we import ULevelVariantSets at runtime
	using namespace UE::Interchange;


	if (!LevelVariantSets || !Arguments.NodeContainer || !Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		return nullptr;
	}

	UInterchangeSceneVariantSetsFactoryNode* FactoryNode = Cast<UInterchangeSceneVariantSetsFactoryNode>(Arguments.AssetNode);
	if (!FactoryNode)
	{
		return nullptr;
	}

	Translator = Arguments.Translator;
	const IInterchangeVariantSetPayloadInterface* LevelVariantSetsPayloadInterface = Cast<IInterchangeVariantSetPayloadInterface>(Arguments.Translator);
	if (!LevelVariantSetsPayloadInterface)
	{
		UE_LOGF(LogInterchangeImport, Error, "Cannot import LevelVariantSets. The translator does not implement the IInterchangeVariantSetPayloadInterface interface.");
		return nullptr;
	}

	const UClass* LevelVariantSetsClass = FactoryNode->GetObjectClass();
	check(LevelVariantSetsClass && LevelVariantSetsClass->IsChildOf(GetFactoryClass()));

	Private::FLevelVariantSetsHelper Helper(*LevelVariantSets, *FactoryNode, *Arguments.NodeContainer, *LevelVariantSetsPayloadInterface);
	Helper.PopulateLevelVariantSets();

	/** Apply all FactoryNode custom attributes to the level sequence asset */
	FactoryNode->ApplyAllCustomAttributeToObject(LevelVariantSets);

	//Getting the file Hash will cache it into the source data
	Arguments.SourceData->GetFileContentHash();
#endif

	//The interchange completion task (call in the GameThread after the factories pass), will call PostEditChange which will trig another asynchronous system that will build all material in parallel
	return LevelVariantSets;
}

void UInterchangeSceneVariantSetsFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSceneVariantSetsFactory::SetupObject_GameThread);
	check(IsInGameThread());
	Super::SetupObject_GameThread(Arguments);

	// TODO: Need to add an AssetImportData property to ULevelVariantSets
}

#undef LOCTEXT_NAMESPACE
