// Copyright Epic Games, Inc. All Rights Reserved. 
#include "Scene/InterchangeSkeletalMeshActorFactory.h"

#include "InterchangeMeshActorFactoryNode.h"
#include "InterchangeAnimSequenceFactoryNode.h"
#include "Scene/InterchangeActorHelper.h"
#include "InterchangeSceneNode.h"
#include "InterchangeSkeletalMeshFactoryNode.h"

#include "Animation/SkeletalMeshActor.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetAsyncCompileUtils.h"

#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSkeletalMeshActorFactory)


UObject* UInterchangeSkeletalMeshActorFactory::ProcessActor(AActor& SpawnedActor, const UInterchangeActorFactoryNode& /*FactoryNode*/, const UInterchangeBaseNodeContainer& /*NodeContainer*/, const FImportSceneObjectsParams& Params)
{
	ASkeletalMeshActor* SkeletalMeshActor = Cast<ASkeletalMeshActor>(&SpawnedActor);

	if (!SkeletalMeshActor)
	{
		return nullptr;
	}

	if (USkeletalMeshComponent* SkeletalMeshComponent = SkeletalMeshActor->GetSkeletalMeshComponent())
	{
		USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		if (SkeletalMesh)
		{
#if WITH_EDITOR
			//If we are importing the skeletal mesh, we need to use a async build scope to unregister the component to avoid a deadlock
			FSkinnedAssetAsyncBuildScope AsyncBuildScope(SkeletalMesh);
#endif
			SkeletalMeshComponent->UnregisterComponent();
		}
		else
		{
			SkeletalMeshComponent->UnregisterComponent();
		}

		return SkeletalMeshComponent;
	}

	return nullptr;
}

void UInterchangeSkeletalMeshActorFactory::ExecuteResetObjectProperties(const UInterchangeBaseNodeContainer* BaseNodeContainer, UInterchangeFactoryBaseNode* FactoryNode, UObject* ImportedObject)
{
	Super::ExecuteResetObjectProperties(BaseNodeContainer, FactoryNode, ImportedObject);
	if (ASkeletalMeshActor* SkeletalMeshActor = Cast<ASkeletalMeshActor>(ImportedObject))
	{
		if (UInterchangeMeshActorFactoryNode* MeshActorFactoryNode = Cast<UInterchangeMeshActorFactoryNode>(FactoryNode))
		{
			if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(SkeletalMeshActor->GetSkeletalMeshComponent()))
			{
				using namespace UE::Interchange;
				FString InstancedAssetFactoryNodeUid;
				if (MeshActorFactoryNode->GetCustomInstancedAssetFactoryNodeUid(InstancedAssetFactoryNodeUid))
				{
					if (const UInterchangeFactoryBaseNode* MeshNode = Cast<UInterchangeFactoryBaseNode>(BaseNodeContainer->GetNode(InstancedAssetFactoryNodeUid)))
					{
						FSoftObjectPath ReferenceObject;
						MeshNode->GetCustomReferenceObject(ReferenceObject);
						if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(ReferenceObject.TryLoad()))
						{
							if (SkeletalMesh != SkeletalMeshComponent->GetSkeletalMeshAsset())
							{
								SkeletalMeshComponent->SetSkeletalMeshAsset(SkeletalMesh);
							}
						}
					}
					else
					{
						// TODO: Warn that new mesh has not been applied
					}
				}

				SkeletalMeshComponent->EmptyOverrideMaterials();
				ActorHelper::ApplySlotMaterialDependencies(*BaseNodeContainer, *MeshActorFactoryNode, *SkeletalMeshComponent);
			}
		}
	}
}

UClass* UInterchangeSkeletalMeshActorFactory::GetFactoryClass() const
{
	return ASkeletalMeshActor::StaticClass();
}

void UInterchangeSkeletalMeshActorFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	if (ASkeletalMeshActor* SkeletalMeshActor = Cast<ASkeletalMeshActor>(Arguments.ImportedObject))
	{
		if (USkeletalMeshComponent* SkeletalMeshComponent = SkeletalMeshActor->GetSkeletalMeshComponent())
		{
			TArray<FString> TargetNodeUids;
			Arguments.FactoryNode->GetTargetNodeUids(TargetNodeUids);
			const UInterchangeSkeletalMeshFactoryNode* SkeletalMeshFactoryNode = TargetNodeUids.IsEmpty() ? nullptr : Cast<UInterchangeSkeletalMeshFactoryNode>(Arguments.NodeContainer->GetFactoryNode(TargetNodeUids[0]));
			if (SkeletalMeshFactoryNode)
			{
				FSoftObjectPath ReferenceObject;
				SkeletalMeshFactoryNode->GetCustomReferenceObject(ReferenceObject);
				if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(ReferenceObject.TryLoad()))
				{
					SkeletalMeshComponent->SetSkeletalMeshAsset(SkeletalMesh);

					if (const UInterchangeMeshActorFactoryNode* MeshActorFactoryNode = Cast<UInterchangeMeshActorFactoryNode>(Arguments.FactoryNode))
					{
						UE::Interchange::ActorHelper::ApplySlotMaterialDependencies(*Arguments.NodeContainer, *MeshActorFactoryNode, *SkeletalMeshComponent, Cast<UInterchangeMeshActorFactoryNode>(PreviousFactoryNode));

						FString AnimationAssetUidToPlay;
						if (MeshActorFactoryNode->GetCustomAnimationAssetUidToPlay(AnimationAssetUidToPlay))
						{
							const FString AnimSequenceFactoryNodeUid = TEXT("\\AnimSequence") + AnimationAssetUidToPlay;
							if (const UInterchangeAnimSequenceFactoryNode* AnimSequenceFactoryNode = Cast<UInterchangeAnimSequenceFactoryNode>(Arguments.NodeContainer->GetFactoryNode(AnimSequenceFactoryNodeUid)))
							{
								FSoftObjectPath AnimSequenceObject;
								if (AnimSequenceFactoryNode->GetCustomReferenceObject(AnimSequenceObject))
								{
									if (UAnimSequence* AnimSequence = Cast<UAnimSequence>(AnimSequenceObject.TryLoad()))
									{
										SkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationSingleNode);
										SkeletalMeshComponent->AnimationData.AnimToPlay = AnimSequence;
										SkeletalMeshComponent->AnimationData.bSavedLooping = false;
										SkeletalMeshComponent->AnimationData.bSavedPlaying = false;
										SkeletalMeshComponent->SetAnimation(AnimSequence);
										
									}
								}
							}
						}
					}
				}
			}
		}
	}
}