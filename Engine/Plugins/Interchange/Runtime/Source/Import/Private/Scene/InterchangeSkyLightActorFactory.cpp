// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/InterchangeSkyLightActorFactory.h"

#include "InterchangeTextureCubeFactoryNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Scene/InterchangeActorHelper.h"

#include "Components/SkyLightComponent.h"
#include "CoreMinimal.h"
#include "Engine/SkyLight.h"
#include "InterchangeLightFactoryNode.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSkyLightActorFactory)

UClass* UInterchangeSkyLightActorFactory::GetFactoryClass() const
{
	return ASkyLight::StaticClass();
}

void UInterchangeSkyLightActorFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	Super::SetupObject_GameThread(Arguments);

	if (!ensure(Arguments.NodeContainer))
	{
		return;
	}

	if (ASkyLight* SkyLightActor = Cast<ASkyLight>(Arguments.ImportedObject))
	{
		if (USkyLightComponent* SkyLightComponent = SkyLightActor->GetLightComponent())
		{
			SkyLightComponent->UnregisterComponent();

			if (const UInterchangeSkyLightFactoryNode* SkyLightFactoryNode = Cast<UInterchangeSkyLightFactoryNode>(Arguments.FactoryNode))
			{
				EInterchangeSkyLightSourceType SourceType;
				if (SkyLightFactoryNode->GetCustomSourceType(SourceType))
				{
					static_assert((int)EInterchangeSkyLightSourceType::CapturedScene == (int)ESkyLightSourceType::SLS_CapturedScene);
					static_assert((int)EInterchangeSkyLightSourceType::SpecifiedCubemap == (int)ESkyLightSourceType::SLS_SpecifiedCubemap);
					SkyLightComponent->SourceType = static_cast<ESkyLightSourceType>(SourceType);
				}

				FString TextureFactoryNodeUid;
				if (SkyLightFactoryNode->GetCustomCubemapDependency(TextureFactoryNodeUid) && !TextureFactoryNodeUid.IsEmpty())
				{
					UInterchangeTextureCubeFactoryNode* TextureCubeFactoryNode = Cast<UInterchangeTextureCubeFactoryNode>(
						Arguments.NodeContainer->GetFactoryNode(TextureFactoryNodeUid)
					);

					if (TextureCubeFactoryNode)
					{
						FSoftObjectPath ReferenceObject;
						TextureCubeFactoryNode->GetCustomReferenceObject(ReferenceObject);
						if (UTextureCube* Texture = Cast<UTextureCube>(ReferenceObject.TryLoad()))
						{
							// We must assign the cubemap from SetupObject_GameThread instead of ProcessActor, as this call
							// will invalidate the reflection capture and get FScene::CaptureOrUploadReflectionCapture() to try and
							// read the UTextureCube::GetResource() immediately (or on the next frame?). By the time ProcessActor
							// runs however, we haven't run UpdateResource() on the UTextureCube yet, so this process would have
							// failed and produced a warning
							SkyLightComponent->SetCubemap(Texture);
						}
					}
				}
			}
		}
	}
}

UObject* UInterchangeSkyLightActorFactory::ProcessActor(
	AActor& SpawnedActor,
	const UInterchangeActorFactoryNode& FactoryNode,
	const UInterchangeBaseNodeContainer& NodeContainer,
	const FImportSceneObjectsParams& Params
)
{
	if (ASkyLight* SkyLightActor = Cast<ASkyLight>(&SpawnedActor))
	{
		if (USkyLightComponent* SkyLightComponent = SkyLightActor->GetLightComponent())
		{
			return SkyLightComponent;
		}
	}

	return nullptr;
}
