// Copyright Epic Games, Inc. All Rights Reserved. 
#include "Scene/InterchangeStaticMeshActorFactory.h"

#include "InterchangeMeshActorFactoryNode.h"
#include "InterchangeStaticMeshFactoryNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Scene/InterchangeActorHelper.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeStaticMeshActorFactory)

UClass* UInterchangeStaticMeshActorFactory::GetFactoryClass() const
{
	return AStaticMeshActor::StaticClass();
}

UObject* UInterchangeStaticMeshActorFactory::ProcessActor(AActor& SpawnedActor, const UInterchangeActorFactoryNode& FactoryNode, const UInterchangeBaseNodeContainer& NodeContainer, const FImportSceneObjectsParams& /*Params*/)
{
	using namespace UE::Interchange;

	AStaticMeshActor* StaticMeshActor = Cast<AStaticMeshActor>(&SpawnedActor);

	if (!StaticMeshActor)
	{
		return nullptr;
	}

	if (UStaticMeshComponent* StaticMeshComponent = StaticMeshActor->GetStaticMeshComponent())
	{
		bool bHasGeometricTransform = false;
		FTransform GeometricTransform;
		const UInterchangeMeshActorFactoryNode* MeshActorFactoryNode = Cast<UInterchangeMeshActorFactoryNode>(&FactoryNode);
		if (MeshActorFactoryNode)
		{
			if (MeshActorFactoryNode->GetCustomGeometricTransform(GeometricTransform))
			{
				bHasGeometricTransform = true;
			}
		}

		if (bHasGeometricTransform)
		{
			UStaticMeshComponent* GeometricTransformMeshComponent = nullptr;
			if (StaticMeshComponent->GetNumChildrenComponents() > 0)
			{
				USceneComponent* ChildSceneComponent = StaticMeshComponent->GetChildComponent(0);
				if (ChildSceneComponent->IsA(UStaticMeshComponent::StaticClass()))
				{
					AActor* ParentActor = ChildSceneComponent->GetAttachParentActor();
					if (ParentActor == StaticMeshActor)
					{
						GeometricTransformMeshComponent = Cast<UStaticMeshComponent>(ChildSceneComponent);
					}
				}
			}

			if (GeometricTransformMeshComponent == nullptr)
			{
				StaticMeshActor->UnregisterAllComponents();

				GeometricTransformMeshComponent = NewObject<UStaticMeshComponent>(StaticMeshActor->GetRootComponent(), TEXT("GeometricTransform"));

#if WITH_EDITORONLY_DATA
				GeometricTransformMeshComponent->bVisualizeComponent = true;
#endif
				StaticMeshActor->AddInstanceComponent(GeometricTransformMeshComponent);

				if (const UInterchangeFactoryBaseNode* MeshNode = ActorHelper::FindAssetInstanceFactoryNode(&NodeContainer, &FactoryNode))
				{
					ApplyStaticMeshToComponent(MeshNode, GeometricTransformMeshComponent, NodeContainer, MeshActorFactoryNode);
				}

				GeometricTransformMeshComponent->SetupAttachment(StaticMeshComponent);

				StaticMeshActor->RegisterAllComponents();

				EComponentMobility::Type MobilityToSet = StaticMeshComponent->Mobility;
				GeometricTransformMeshComponent->SetMobility(EComponentMobility::Type::Movable); //so that RelativeTransform can be set
				GeometricTransformMeshComponent->SetRelativeTransform(GeometricTransform);
				GeometricTransformMeshComponent->SetMobility(MobilityToSet);

				return StaticMeshComponent;
			}
		}

		if (const UInterchangeFactoryBaseNode* MeshNode = ActorHelper::FindAssetInstanceFactoryNode(&NodeContainer, &FactoryNode))
		{
			StaticMeshComponent->UnregisterComponent();

			ApplyStaticMeshToComponent(MeshNode, StaticMeshComponent, NodeContainer, MeshActorFactoryNode);

			StaticMeshComponent->RegisterComponent();
		}
		else
		{
			// TODO: Warn that new mesh has not been applied
		}

		return StaticMeshComponent;
	}

	return nullptr;
}

void UInterchangeStaticMeshActorFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	if (AStaticMeshActor* StaticMeshActor = Cast<AStaticMeshActor>(Arguments.ImportedObject))
	{
		if (UStaticMeshComponent* StaticMeshComponent = StaticMeshActor->GetStaticMeshComponent())
		{
			TArray<FString> TargetNodeUids;
			Arguments.FactoryNode->GetTargetNodeUids(TargetNodeUids);
			const UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode = TargetNodeUids.IsEmpty() ? nullptr : Cast<UInterchangeStaticMeshFactoryNode>(Arguments.NodeContainer->GetFactoryNode(TargetNodeUids[0]));
			if (StaticMeshFactoryNode)
			{
				UInterchangeMeshActorFactoryNode* ActorFactoryNode = Cast<UInterchangeMeshActorFactoryNode>(Arguments.FactoryNode);

				ApplyStaticMeshToComponent(StaticMeshFactoryNode, StaticMeshComponent, *Arguments.NodeContainer, ActorFactoryNode);

				FString SubstituteUID;
				if (!StaticMeshComponent->GetStaticMesh() && StaticMeshFactoryNode->GetCustomSubstituteUID(SubstituteUID))
				{
					StaticMeshFactoryNode = Cast<UInterchangeStaticMeshFactoryNode>(Arguments.NodeContainer->GetFactoryNode(SubstituteUID));
					if (StaticMeshFactoryNode)
					{
						ApplyStaticMeshToComponent(StaticMeshFactoryNode, StaticMeshComponent, *Arguments.NodeContainer, ActorFactoryNode);
					}
				}
			}
		}
	}
}

void UInterchangeStaticMeshActorFactory::ExecuteResetObjectProperties(const UInterchangeBaseNodeContainer* BaseNodeContainer, UInterchangeFactoryBaseNode* FactoryNode, UObject* ImportedObject)
{
	Super::ExecuteResetObjectProperties(BaseNodeContainer, FactoryNode, ImportedObject);
	if (AStaticMeshActor* StaticMeshActor = Cast<AStaticMeshActor>(ImportedObject))
	{
		if (UInterchangeMeshActorFactoryNode* MeshActorFactoryNode = Cast<UInterchangeMeshActorFactoryNode>(FactoryNode))
		{
			if (UStaticMeshComponent* StaticMeshComponent = StaticMeshActor->GetStaticMeshComponent())
			{
				using namespace UE::Interchange;
				FString InstancedAssetFactoryNodeUid;
				if (MeshActorFactoryNode->GetCustomInstancedAssetFactoryNodeUid(InstancedAssetFactoryNodeUid))
				{
					if (const UInterchangeFactoryBaseNode* MeshNode = Cast<UInterchangeFactoryBaseNode>(BaseNodeContainer->GetNode(InstancedAssetFactoryNodeUid)))
					{
						FSoftObjectPath ReferenceObject;
						MeshNode->GetCustomReferenceObject(ReferenceObject);
						if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(ReferenceObject.TryLoad()))
						{
							if (StaticMesh != StaticMeshComponent->GetStaticMesh())
							{
								StaticMeshComponent->SetStaticMesh(StaticMesh);
							}
						}
					}
					else
					{
						// TODO: Warn that new mesh has not been applied
					}
				}

				StaticMeshComponent->EmptyOverrideMaterials();
				ActorHelper::ApplySlotMaterialDependencies(*BaseNodeContainer, *MeshActorFactoryNode, *StaticMeshComponent);
			}
		}
	}
}

void UInterchangeStaticMeshActorFactory::ApplyStaticMeshToComponent(const UInterchangeFactoryBaseNode* MeshNode, UStaticMeshComponent* StaticMeshComponent, const UInterchangeBaseNodeContainer& NodeContainer, const UInterchangeMeshActorFactoryNode* MeshFactoryNode)
{
	FSoftObjectPath ReferenceObject;
	MeshNode->GetCustomReferenceObject(ReferenceObject);
	
	if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(ReferenceObject.TryLoad()))
	{
		if (StaticMesh != StaticMeshComponent->GetStaticMesh())
		{
			StaticMeshComponent->SetStaticMesh(StaticMesh);
		}

		// UE-306557: At this point, bounds might not be calculated yet but the navigation system will try to add it to the Octree, causing an Empty Bounds warning.
		// To avoid this warning, we manually calculate the bounds here.
		StaticMesh->CalculateExtendedBounds();

		// Apply the materials from the mesh node to the static mesh component
		if (MeshFactoryNode)
		{
			UE::Interchange::ActorHelper::ApplySlotMaterialDependencies(NodeContainer, *MeshFactoryNode, *StaticMeshComponent, Cast<UInterchangeMeshActorFactoryNode>(PreviousFactoryNode));
		}
	}
}
