// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpecularProfile/InterchangeSpecularProfileFactory.h"
#include "InterchangeSourceData.h"
#include "InterchangeSpecularProfileFactoryNode.h"
#include "InterchangeTexture2DFactoryNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "Engine/SpecularProfile.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSpecularProfileFactory)

#define LOCTEXT_NAMESPACE "InterchangeSpecularProfileFactory"

UClass* UInterchangeSpecularProfileFactory::GetFactoryClass() const
{
	return USpecularProfile::StaticClass();
}

EInterchangeFactoryAssetType UInterchangeSpecularProfileFactory::GetFactoryAssetType()
{
	return EInterchangeFactoryAssetType::Materials;
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeSpecularProfileFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSpecularProfileFactory::BeginImportAsset_GameThread);

	Super::BeginImportAsset_GameThread(Arguments);

	FImportAssetResult ImportAssetResult;

	auto CannotReimportMessage = [&Arguments, this]()
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName = Arguments.SourceData->GetFilename();
		Message->DestinationAssetName = Arguments.AssetName;
		Message->AssetType = GetFactoryClass();
		Message->Text = LOCTEXT("CreateEmptyAssetUnsupportedReimport", "Re-import of USpecularProfile not supported yet.");
		Arguments.AssetNode->SetSkipNodeImport();
	};

	if (Arguments.ReimportObject)
	{
		CannotReimportMessage();
		return ImportAssetResult;
	}

	auto CouldNotCreateSpecularProfileLog = [this, &Arguments, &ImportAssetResult](const FText& Info)
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName = Arguments.SourceData->GetFilename();
		Message->DestinationAssetName = Arguments.AssetName;
		Message->AssetType = GetFactoryClass();
		Message->Text = FText::Format(LOCTEXT("SpecProfileFactory_CouldNotCreateSpecProfile", "Could not create Specular Profile asset %s. Reason: %s"), FText::FromString(Arguments.AssetName), Info);
		ImportAssetResult.bIsFactorySkipAsset = true;
	};

	const FText MissMatchClassText = LOCTEXT("SpecProfileFactory_CouldNotCreateSpecProfile_MissMatchClass", "Mismatch between Interchange specular profile factory node class and factory class.");
	
	if (!Arguments.AssetNode || !Arguments.AssetNode->GetObjectClass()->IsChildOf(GetFactoryClass()))
	{
		CouldNotCreateSpecularProfileLog(MissMatchClassText);
		return ImportAssetResult;
	}

	const UInterchangeSpecularProfileFactoryNode* FactoryNode = Cast<UInterchangeSpecularProfileFactoryNode>(Arguments.AssetNode);
	if (!FactoryNode)
	{
		CouldNotCreateSpecularProfileLog(LOCTEXT("SpecProfileFactory_CouldNotCreateSpecProfile_CannotCastFactoryNode", "Cannot cast Interchange factory node to UInterchangeSpecularProfileFactoryNode."));
		return ImportAssetResult;
	}

	const UClass* SpecularProfileClass = FactoryNode->GetObjectClass();
	if (!ensure(SpecularProfileClass && SpecularProfileClass->IsChildOf(GetFactoryClass())))
	{
		CouldNotCreateSpecularProfileLog(MissMatchClassText);
		return ImportAssetResult;
	}

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (FactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}
	}

	// create a new specular profile or overwrite existing asset, if possible
	USpecularProfile* SpecularProfile = nullptr;
	if (!ExistingAsset)
	{
		SpecularProfile = NewObject<USpecularProfile>(Arguments.Parent, *Arguments.AssetName, RF_Public | RF_Standalone);
	}
	else
	{
		// This is a reimport or an override, we are just re-updating the source data

		//TODO: put back the Cast when the SpecularProfile will support re-import
		//SpecularProfile = Cast<USpecularProfile>(ExistingAsset);
		CannotReimportMessage();
		return ImportAssetResult;
	}

	if (!SpecularProfile)
	{
		CouldNotCreateSpecularProfileLog(LOCTEXT("SpecProfileFactory_CouldNotCreateSpecProfile_SpecularProfileCreationFail", "Specular Profile creation failed"));
		return ImportAssetResult;
	}

	ImportAssetResult.ImportedObject = SpecularProfile;

	if(ESpecularProfileFormat Format; FactoryNode->GetCustomFormat(Format))
	{
		SpecularProfile->Settings.Format = Format;
	}

	if (FString TextureUid; FactoryNode->GetCustomTexture(TextureUid))
	{
		FString TextureFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TextureUid);
		if (const UInterchangeTexture2DFactoryNode* TextureFactoryNode = Cast<UInterchangeTexture2DFactoryNode>(Arguments.NodeContainer->GetNode(TextureFactoryNodeUid)))
		{
			if(FSoftObjectPath ReferenceObject; TextureFactoryNode->GetCustomReferenceObject(ReferenceObject))
			{
				if (UTexture2D* Texture = Cast<UTexture2D>(ReferenceObject.TryLoad()))
				{
					SpecularProfile->Settings.Texture = Texture;
				}
			}
		}
	}

	return ImportAssetResult;
}
#undef LOCTEXT_NAMESPACE