// Copyright Epic Games, Inc. All Rights Reserved.

#include "Texture/InterchangeBlockedTexturePayloadData.h"

#include "Async/ParallelFor.h"

namespace UE::Interchange
{
	namespace Private::BlockedPayload
	{
		int64 ComputeBlockSize(const FTextureSourceBlock& Block, ETextureSourceFormat Format)
		{
			int64 Size = 0;
			int64 Width = Block.SizeX;
			int64 Height = Block.SizeY;
			for (int32 CurrentMip = 0; CurrentMip < Block.NumMips; ++CurrentMip)
			{
				Size += Width * Height * FTextureSource::GetBytesPerPixel(Format) * Block.NumSlices;
				Width  = FMath::Max(Width>>1,1);
				Height = FMath::Max(Height>>1,1);
			}
			return Size;
		}
	}

	int64 FImportBlockedImage::ComputeBufferSize() const
	{
		int64 Size = 0;
		for (const FTextureSourceBlock& Block : BlocksData)
		{
			Size += Private::BlockedPayload::ComputeBlockSize(Block, Format);
		}

		return Size;
	}

	bool FImportBlockedImage::IsValid() const
	{
		return Format != TSF_Invalid
			&& BlocksData.Num() > 0
			&& RawData.GetSize() == ComputeBufferSize()
			;
	}

	bool FImportBlockedImage::InitDataSharedAmongBlocks(const FImportImage& Image)
	{
		if (Image.IsValid())
		{
			Format = Image.Format;
			CompressionSettings = Image.CompressionSettings;
			bSRGB = Image.bSRGB;
			//don't set MipGenSettings from Image 0
			//MipGenSettings = Image.MipGenSettings;
			return true;
		}

		return false;
	}

	bool FImportBlockedImage::InitBlockFromImage(int32 BlockX, int32 BlockY, const FImportImage& Image)
	{
		if (!Image.IsValid())
		{
			return false;
		}

		if (Image.Format != Format)
		{
			// The images must be of the same format
			return false;
		}

		FTextureSourceBlock& Block = BlocksData.AddDefaulted_GetRef();
		Block.BlockX = BlockX;
		Block.BlockY = BlockY;
		Block.SizeX = Image.SizeX;
		Block.SizeY = Image.SizeY;
		Block.NumSlices = 1;
		Block.NumMips = Image.NumMips;

		return true;
	}

	bool FImportBlockedImage::MigrateDataFromImagesToRawData(TArray<FImportImage>& Images)
	{
		RawData = FUniqueBuffer::Alloc(ComputeBufferSize());

		int64 ImageSize = 0;
		TArray<int64> BlocksOffset;
		BlocksOffset.Reserve(Images.Num());
		
		if (BlocksData.Num() != Images.Num())
		{
			return false;
		}

		int32 BlockIndex = 0;
		int64 CurrentBlockOffset = 0;
		for (const FImportImage& Image : Images)
		{
			if (!Image.IsValid())
			{
				return false;
			}

			if (Image.Format != Format)
			{
				// The images must be of the same format
				return false;
			}

			BlocksOffset.Add(CurrentBlockOffset);
			const int64 BlockSize = Private::BlockedPayload::ComputeBlockSize(BlocksData[BlockIndex], Format);
			CurrentBlockOffset += BlockSize;

			if (BlockSize != Image.RawData.GetSize())
			{
				return false;
			}

			++BlockIndex;
		}

		uint8* StartPtr = static_cast<uint8*>(RawData.GetData());
		ParallelFor(Images.Num(),[this, StartPtr, &Images, &BlocksOffset](int32 Index)
			{
				int64 BlockSize = Private::BlockedPayload::ComputeBlockSize(BlocksData[Index], Format);
				FMemory::Memcpy(StartPtr + BlocksOffset[Index], static_cast<uint8*>(Images[Index].RawData.GetData()), BlockSize);
			}
			, IsInGameThread() ? EParallelForFlags::Unbalanced : EParallelForFlags::BackgroundPriority | EParallelForFlags::Unbalanced
			);


		return true;
	}
}