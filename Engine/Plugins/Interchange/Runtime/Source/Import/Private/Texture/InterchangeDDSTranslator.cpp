// Copyright Epic Games, Inc. All Rights Reserved.

#include "Texture/InterchangeDDSTranslator.h"

#include "DDSFile.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "HAL/FileManager.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "ImageCoreUtils.h"
#include "InterchangeImportLog.h"
#include "InterchangeTextureCubeNode.h"
#include "InterchangeTextureNode.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Serialization/Archive.h"
#include "Texture/TextureTranslatorUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeDDSTranslator)

static bool GInterchangeEnableDDSImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnableDDSImport(
	TEXT("Interchange.FeatureFlags.Import.DDS"),
	GInterchangeEnableDDSImport,
	TEXT("Whether DDS support is enabled."),
	ECVF_Default);

namespace UE::Interchange::Private::InterchangeDDSTranslator
{
	bool LoadDDSHeaderFromFile(TArray64<uint8>& OutHeader, const FString& Filename)
	{
		// The file is close when the archive is destroyed
		TUniquePtr<FArchive> FileReaderArchive(IFileManager::Get().CreateFileReader(*Filename));

		if (FileReaderArchive)
		{
			const int64 SizeOfFile = FileReaderArchive->TotalSize();
			const int64 MinimalHeaderSize = UE::DDS::GetDDSHeaderMinimalSize();

			// If the file is not bigger than the smallest header possible then clearly the file is not valid as a dds file.
			if (SizeOfFile > MinimalHeaderSize)
			{
				const int64 MaximalHeaderSize = UE::DDS::GetDDSHeaderMaximalSize();
				const int64 BytesToRead = FMath::Min(SizeOfFile, MaximalHeaderSize);

				OutHeader.Reset(BytesToRead);
				OutHeader.AddUninitialized(BytesToRead);
				FileReaderArchive->Serialize(OutHeader.GetData(), OutHeader.Num());

				return true;
			}
		}

		return false;
	}

	TUniquePtr<UE::DDS::FDDSFile> CreateDDSFileReader(const FString& Filename, bool bHeaderOnly)
	{
		TArray64<uint8> DDSSourceData;

		if (!FPaths::FileExists(Filename))
		{
			UE_LOGF(LogInterchangeImport, Error, "Failed to import DDS, cannot open file. [%ls]", *Filename);
			return {};
		}

		if (bHeaderOnly)
		{
			LoadDDSHeaderFromFile(DDSSourceData, Filename);
			// no error checking?
		}
		else if (!FFileHelper::LoadFileToArray(DDSSourceData, *Filename))
		{
			UE_LOGF(LogInterchangeImport, Error, "Failed to import DDS, cannot load file content into an array. [%ls]", *Filename);
			return {};
		}

		UE::DDS::EDDSError Error;
		const UE::DDS::EDDSReadMipMode MipReadMode = bHeaderOnly ? UE::DDS::EDDSReadMipMode::HeaderOnly : UE::DDS::EDDSReadMipMode::Full;
		TUniquePtr<UE::DDS::FDDSFile> DDS = TUniquePtr<UE::DDS::FDDSFile>(UE::DDS::FDDSFile::CreateFromDDSInMemory(DDSSourceData.GetData(), DDSSourceData.Num(), &Error, MipReadMode));
		if (!DDS.IsValid())
		{
			// NotADds is okay/expected , we try this on all image buffers
			// IoError means buffer is too small to be a DDS
			check(Error != UE::DDS::EDDSError::OK);
			if (Error != UE::DDS::EDDSError::NotADds && Error != UE::DDS::EDDSError::IoError)
			{
				UE_LOGF(LogInterchangeImport, Warning, "Failed to load DDS (Error=%d) [%ls]", (int)Error, *Filename);
			}
			return {};
		}

		if (!bHeaderOnly)
		{
			// this function is called 4 times with bHeaderOnly=true, then once with bHeaderOnly=false
			// do noisy warnings only in the last call so they aren't repeated

			// We require a complete mip chain or a single mip.
			if (DDS->MipCount > 1)
			{
				if (DDS->MipCount != FImageCoreUtils::GetMipCountFromDimensions(DDS->Width, DDS->Height, DDS->Depth, DDS->IsValidTextureVolume()))
				{
					UE_LOGF(LogInterchangeImport, Warning, "Imported DDS files must either be a single base mip, or a full mip chain, not a partial mip chain (found %d expected %d) [%ls]", DDS->MipCount, FImageCoreUtils::GetMipCountFromDimensions(DDS->Width, DDS->Height, DDS->Depth, DDS->IsValidTextureVolume()), *Filename);
				}
			}

			// change X8 formats to A8 :	
			DDS->ConvertRGBXtoRGBA();
			// change RGBA8 to BGRA8 before DXGIFormatGetClosestRawFormat :
			DDS->ConvertChannelOrder(UE::DDS::EChannelOrder::BGRA);
		}

		return DDS;
	}

	uint32 GetDDSMipCount(const TUniquePtr<UE::DDS::FDDSFile>& DDS)
	{
		uint32 MipCount = (uint32)DDS->MipCount;

		if (MipCount > MAX_TEXTURE_MIP_COUNT)
		{
			// resolution can be above MAX_TEXTURE_MIP_COUNT for VT
			UE_LOGF(LogInterchangeImport, Warning, "DDS exceeds MAX_TEXTURE_MIP_COUNT");
			MipCount = MAX_TEXTURE_MIP_COUNT;
		}

		return MipCount;
	}

	bool DDSPayloadIsSRGB(const TUniquePtr<UE::DDS::FDDSFile>& DDS, ERawImageFormat::Type RawFormat)
	{
		bool bSRGB = false;

		if (ERawImageFormat::GetFormatNeedsGammaSpace(RawFormat))
		{
			if (DDS->CreateFlags & UE::DDS::FDDSFile::CREATE_FLAG_WAS_D3D9)
			{
				// no SRGB info in Dx9 format
				// assume SRGB yes
				bSRGB = true;
			}
			else if (UE::DDS::DXGIFormatHasLinearAndSRGBForm(DDS->DXGIFormat))
			{
				// Dx10 file with format that has linear/srgb pair
				//	( _UNORM when _UNORM_SRGB)
				bSRGB = UE::DDS::DXGIFormatIsSRGB(DDS->DXGIFormat);
			}
			else
			{
				// Dx10 format that doesn't have linear/srgb pairs

				// R8G8_UNORM and R8_UNORM have no _SRGB pair
				// so no way to clearly indicate SRGB or Linear for them
				// assume SRGB yes
				bSRGB = true;
			}
		}

		return bSRGB;
	}

	template < typename PayloadType >
	void SetupPayload(const TUniquePtr<UE::DDS::FDDSFile>& DDS, PayloadType& PayloadData, ERawImageFormat::Type RawFormat, const int32 MipCount)
	{
		PayloadData.bSRGB = DDSPayloadIsSRGB(DDS, RawFormat);

		if (ERawImageFormat::IsHDR(RawFormat))
		{
			PayloadData.CompressionSettings = TC_HDR;
			ensure(PayloadData.bSRGB == false);
		}

		if (MipCount > 1)
		{
			// if the source has mips we keep the mips by default, unless the user changes that
			PayloadData.MipGenSettings = TMGS_LeaveExistingMips;
		}
	}
}

TArray<FString> UInterchangeDDSTranslator::GetSupportedFormats() const
{
	if (GInterchangeEnableDDSImport || GIsAutomationTesting)
	{
		TArray<FString> Formats{ TEXT("dds;DirectDraw Surface") };
		return Formats;
	}
	else
	{
		return TArray<FString>{};
	}
}

bool UInterchangeDDSTranslator::CanImportSourceData(const UInterchangeSourceData* InSourceData) const
{
	using namespace UE::Interchange::Private::InterchangeDDSTranslator;

	if (!UInterchangeTranslatorBase::CanImportSourceData(InSourceData))
	{
		return false;
	}

	FString Filename = InSourceData->GetFilename();

	if (!FPaths::FileExists(Filename))
	{
		return false;
	}

	constexpr bool bHeaderOnly = true;
	TUniquePtr<UE::DDS::FDDSFile> DDS = CreateDDSFileReader(Filename, bHeaderOnly);

	if (!DDS)
	{
		return false;
	}

	// @@ check IsImageImportPossible here?

	return DDS->IsValidTexture2D() || DDS->IsValidTextureCube() || DDS->IsValidTextureArray() || DDS->IsValidTextureVolume();
}

bool UInterchangeDDSTranslator::Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const
{
	using namespace UE::Interchange::Private::InterchangeDDSTranslator;

	/*
	 * DDS file can also be a cube map so we have to open the file and see if its a valid 2D texture.
	 */
	FString Filename = GetSourceData()->GetFilename();
	if (!FPaths::FileExists(Filename))
	{
		return false;
	}

	constexpr bool bHeaderOnly = true;
	TUniquePtr<UE::DDS::FDDSFile> DDS = CreateDDSFileReader(Filename, bHeaderOnly);

	if (!DDS)
	{
		return false;
	}

	if (DDS->IsValidTexture2D())
	{
		return UE::Interchange::FTextureTranslatorUtilities::Generic2DTextureTranslate(GetSourceData(), BaseNodeContainer);
	}

	if (DDS->IsValidTextureCube())
	{
		if (DDS->ArraySize > 6)
		{
			return UE::Interchange::FTextureTranslatorUtilities::GenericTextureCubeArrayTranslate(GetSourceData(), BaseNodeContainer);
		}
		else
		{
			return UE::Interchange::FTextureTranslatorUtilities::GenericTextureCubeTranslate(GetSourceData(), BaseNodeContainer);
		}
	}

	if (DDS->IsValidTextureArray())
	{
		return UE::Interchange::FTextureTranslatorUtilities::GenericTexture2DArrayTranslate(GetSourceData(), BaseNodeContainer);
	}

	if (DDS->IsValidTextureVolume())
	{
		return UE::Interchange::FTextureTranslatorUtilities::GenericVolumeTextureTranslate(GetSourceData(), BaseNodeContainer);
	}

	return false;
}

TOptional<UE::Interchange::FImportImage> UInterchangeDDSTranslator::GetTexturePayloadData(const FString& /*PayloadKey*/, TOptional<FString>& /*AlternateTexturePath*/) const
{
	using namespace UE::Interchange::Private::InterchangeDDSTranslator;
	using namespace UE::Interchange;

	if (!FTextureTranslatorUtilities::IsTranslatorValid(*this, TEXT("DDS")))
	{
		return {};
	}

	const FString Filename = GetSourceData()->GetFilename();
	constexpr bool bHeaderOnly = false;
	TUniquePtr<UE::DDS::FDDSFile> DDS = CreateDDSFileReader(Filename, bHeaderOnly);

	if (!DDS || !DDS->IsValidTexture2D())
	{
		FText ErrorMessage = FText::Format(NSLOCTEXT("InterchangeDDSTranslator", "UnsupportedFormat", "Failed to import DDS, unsupported format. [{0}]"), FText::FromString(Filename));
		FTextureTranslatorUtilities::LogError(*this, MoveTemp(ErrorMessage));
		return TOptional<UE::Interchange::FImportImage>();
	}

	// map format to RawFormat and ETextureSourceFormat
	ERawImageFormat::Type RawFormat = UE::DDS::DXGIFormatGetClosestRawFormat(DDS->DXGIFormat);
	if (RawFormat == ERawImageFormat::Invalid)
	{
		FText ErrorMessage = FText::Format(NSLOCTEXT("InterchangeDDSTranslator", "InvalidDXGIFormat", "DDS DXGIFormat not supported : {0} : {1}"), (int)DDS->DXGIFormat, FText::FromString(UE::DDS::DXGIFormatGetName(DDS->DXGIFormat)));
		FTextureTranslatorUtilities::LogError(*this, MoveTemp(ErrorMessage));
		return {};
	}

	ETextureSourceFormat TSFormat = FImageCoreUtils::ConvertToTextureSourceFormat(RawFormat);

	const uint32 MipCount = GetDDSMipCount(DDS);

	UE::Interchange::FImportImage PayloadData;
	PayloadData.Init2DWithParams(
		DDS->Width,
		DDS->Height,
		MipCount,
		TSFormat,
		DDSPayloadIsSRGB(DDS, RawFormat)
	);

	int64 MipOffset = 0;

	for (uint32 MipIndex = 0; MipIndex < MipCount; ++MipIndex)
	{
		const uint32 MipWidth = FMath::Max(DDS->Width >> MipIndex, 1u);
		const uint32 MipHeight = FMath::Max(DDS->Height >> MipIndex, 1u);

		FImageView DestMipData((void*)(reinterpret_cast<const uint8*>(PayloadData.RawData.GetData()) + MipOffset),
			MipWidth, MipHeight, RawFormat);

		if (!DDS->GetMipImage(DestMipData, MipIndex))
		{
			FTextureTranslatorUtilities::LogError(*this, NSLOCTEXT("InterchangeDDSTranslator", "FailedPixelFormat", "DDS could not convert pixel format"));
			return {};
		}

		MipOffset += PayloadData.GetMipSize(MipIndex);
	}

	SetupPayload(DDS, PayloadData, RawFormat, MipCount);

	return PayloadData;
}

TOptional<UE::Interchange::FImportSlicedImage> UInterchangeDDSTranslator::GetSlicedTexturePayloadData(const FString& /*PayloadKey*/, TOptional<FString>& /*AlternateTexturePath*/) const
{
	using namespace UE::Interchange;
	using namespace UE::Interchange::Private::InterchangeDDSTranslator;

	if (!FTextureTranslatorUtilities::IsTranslatorValid(*this, TEXT("DDS")))
	{
		return {};
	}

	const FString Filename = GetSourceData()->GetFilename();
	constexpr bool bHeaderOnly = false;
	TUniquePtr<UE::DDS::FDDSFile> DDS = CreateDDSFileReader(Filename, bHeaderOnly);

	if (!DDS)
	{
		FText ErrorMessage = FText::Format(NSLOCTEXT("InterchangeDDSTranslator", "UnsupportedFormat", "Failed to import DDS, unsupported format. [{0}]"), FText::FromString(Filename));
		FTextureTranslatorUtilities::LogError(*this, MoveTemp(ErrorMessage));
		return {};
	}

	uint32 NumSlices = 1;

	if (DDS->IsValidTextureCube() || DDS->IsValidTextureArray())
	{
		NumSlices = DDS->ArraySize;
	}
	else if (DDS->IsValidTextureVolume())
	{
		NumSlices = DDS->Depth;
	}
	else
	{
		UE_LOGF(LogInterchangeImport, Error, "Failed to import DDS, unsupported format. [%ls]", *Filename);
		return TOptional<UE::Interchange::FImportSlicedImage>();
	}

	// map format to RawFormat and ETextureSourceFormat
	ERawImageFormat::Type RawFormat = UE::DDS::DXGIFormatGetClosestRawFormat(DDS->DXGIFormat);
	if (RawFormat == ERawImageFormat::Invalid)
	{
		FText ErrorMessage = FText::Format(NSLOCTEXT("InterchangeDDSTranslator", "InvalidDXGIFormat", "DDS DXGIFormat not supported : {0} : {1}"), (int)DDS->DXGIFormat, FText::FromString(UE::DDS::DXGIFormatGetName(DDS->DXGIFormat)));
		FTextureTranslatorUtilities::LogError(*this, MoveTemp(ErrorMessage));
		return {};
	}

	ETextureSourceFormat TSFormat = FImageCoreUtils::ConvertToTextureSourceFormat(RawFormat);

	const uint32 MipCount = GetDDSMipCount(DDS);
	const bool bIsVolume = (DDS->Dimension == 3);

	UE::Interchange::FImportSlicedImage PayloadData;

	if (bIsVolume)
	{
		PayloadData.InitVolume(
			DDS->Width,
			DDS->Height,
			NumSlices,
			MipCount,
			TSFormat,
			DDSPayloadIsSRGB(DDS, RawFormat)
		);
	}
	else
	{
		PayloadData.Init(
			DDS->Width,
			DDS->Height,
			NumSlices,
			MipCount,
			TSFormat,
			DDSPayloadIsSRGB(DDS, RawFormat)
		);
	}

	for (uint32 MipIndex = 0; MipIndex < MipCount; ++MipIndex)
	{
		const uint32 MipWidth = FMath::Max(DDS->Width >> MipIndex, 1u);
		const uint32 MipHeight = FMath::Max(DDS->Height >> MipIndex, 1u);
		const uint32 MipNumSlices = bIsVolume ? FMath::Max(NumSlices >> MipIndex, 1u) : NumSlices;

		FImageView DestMipData(PayloadData.GetMipData(MipIndex),
			MipWidth,
			MipHeight,
			MipNumSlices,
			RawFormat,
			PayloadData.bSRGB ? EGammaSpace::sRGB : EGammaSpace::Linear
		);

		if (bIsVolume)
		{
			check(DDS->Mips.Num() == DDS->MipCount);
			check(DDS->Mips[MipIndex].Depth == MipNumSlices);

			if (!DDS->GetMipImage(DestMipData, MipIndex))
			{
				FTextureTranslatorUtilities::LogError(*this, NSLOCTEXT("InterchangeDDSTranslator", "InvalidPixelFormat", "DDS could not convert pixel format"));
				return {};
			}
		}
		else
		{
			// DDS->Mips[] contains both mips and arrays
			check(DDS->Mips.Num() == DDS->MipCount * MipNumSlices);

			for (uint32 SliceIndex = 0; SliceIndex < MipNumSlices; ++SliceIndex)
			{
				FImageView DestSliceData = DestMipData.GetSlice(SliceIndex);

				// DDS Mips[] array has whole mip chain of each slice, then next slice
				// we have the opposite (all slices of top mip first, then next mip)
				int DDSMipIndex = SliceIndex * DDS->MipCount + MipIndex;

				if (!DDS->GetMipImage(DestSliceData, DDSMipIndex))
				{
					FTextureTranslatorUtilities::LogError(*this, NSLOCTEXT("InterchangeDDSTranslator", "FailedPixelFormat", "DDS could not convert pixel format"));
					return {};
				}
			}
		}
	}

	SetupPayload(DDS, PayloadData, RawFormat, MipCount);

	return PayloadData;
}



