// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Texture/InterchangeIESTranslator.h"

#include "Engine/Texture.h"
#include "Engine/TextureLightProfile.h"
#include "IESConverter.h"
#include "InterchangeImportLog.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Texture/TextureTranslatorUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeIESTranslator)

#define LOCTEXT_NAMESPACE "InterchangeIESTranslator"

static bool GInterchangeEnableIESImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnableIESImport(
	TEXT("Interchange.FeatureFlags.Import.IES"),
	GInterchangeEnableIESImport,
	TEXT("Whether IES support is enabled."),
	ECVF_Default);

TArray<FString> UInterchangeIESTranslator::GetSupportedFormats() const
{
	if (GInterchangeEnableIESImport || GIsAutomationTesting)
	{
		TArray<FString> Formats{ TEXT("ies;IES light profile") };
		return Formats;
	}
	else
	{
		return TArray<FString>{};
	}
}

bool UInterchangeIESTranslator::Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const
{
	return UE::Interchange::FTextureTranslatorUtilities::GenericTextureLightProfileTranslate(GetSourceData(), BaseNodeContainer);
}

TOptional<UE::Interchange::FImportLightProfile> UInterchangeIESTranslator::GetLightProfilePayloadData(const FString& /*PayloadKey*/, TOptional<FString>& /*AlternateTexturePath*/) const
{
	using namespace UE::Interchange;

	TArray64<uint8> SourceDataBuffer;
	if (!FTextureTranslatorUtilities::LoadSourceBuffer(*this, TEXT("IES"), SourceDataBuffer))
	{
		return {};
	}

	return GetLightProfilePayloadData(SourceDataBuffer.GetData(), SourceDataBuffer.Num());
}

TOptional<UE::Interchange::FImportLightProfile> UInterchangeIESTranslator::GetLightProfilePayloadData(const uint8* Buffer, uint32 BufferLength) const
{
	UE::Interchange::FImportLightProfile Payload;

	Payload.SourceDataBuffer.SetNum(BufferLength);

	FPlatformMemory::Memcpy(Payload.SourceDataBuffer.GetData(), Buffer, BufferLength);

	FIESConverter IESConverter(Payload.SourceDataBuffer.GetData(), Payload.SourceDataBuffer.Num());

	if (IESConverter.IsValid())
	{
		Payload.Init2DWithParams(
			IESConverter.GetWidth(),
			IESConverter.GetHeight(),
			TSF_RGBA16F,
			false
		);

		Payload.CompressionSettings = TC_HDR;
		Payload.Brightness = IESConverter.GetBrightness();
		Payload.TextureMultiplier = IESConverter.GetMultiplier();


		const TArray<uint8>& RawData = IESConverter.GetRawData();;

		FPlatformMemory::Memcpy(Payload.RawData.GetData(), RawData.GetData(), Payload.RawData.GetSize());

		return Payload;
	}
	else
	{
		UE::Interchange::FTextureTranslatorUtilities::LogError(*this, FText::Format(LOCTEXT("IESImportFailure_Generic", "IES import failed: \"{0}\""), FText::FromString(IESConverter.GetError())));
	}

	return {};
}

#undef LOCTEXT_NAMESPACE