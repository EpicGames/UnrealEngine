// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Texture/InterchangeImageWrapperTranslator.h"

#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "ImageWrapperOutputTypes.h"
#include "ImageCoreUtils.h"
#include "InterchangeImportLog.h"
#include "InterchangeTextureNode.h"
#include "Memory/SharedBuffer.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Texture/TextureTranslatorUtilities.h"
#include "TextureImportUtils.h"
#include "TextureImportUserSettings.h"
#include "TgaImageSupport.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeImageWrapperTranslator)

static bool GInterchangeEnablePNGImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnablePNGImport(
	TEXT("Interchange.FeatureFlags.Import.PNG"),
	GInterchangeEnablePNGImport,
	TEXT("Whether PNG support is enabled."),
	ECVF_Default);

static bool GInterchangeEnableBMPImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnableBMPImport(
	TEXT("Interchange.FeatureFlags.Import.BMP"),
	GInterchangeEnableBMPImport,
	TEXT("Whether BMP support is enabled."),
	ECVF_Default);

static bool GInterchangeEnableEXRImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnableEXRImport(
	TEXT("Interchange.FeatureFlags.Import.EXR"),
	GInterchangeEnableEXRImport,
	TEXT("Whether OpenEXR support is enabled."),
	ECVF_Default);

static bool GInterchangeEnableHDRImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnableHDRImport(
	TEXT("Interchange.FeatureFlags.Import.HDR"),
	GInterchangeEnableHDRImport,
	TEXT("Whether HDR support is enabled."),
	ECVF_Default);

#if WITH_LIBTIFF
static bool GInterchangeEnableTIFFImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnableTIFFImport(
	TEXT("Interchange.FeatureFlags.Import.TIFF"),
	GInterchangeEnableTIFFImport,
	TEXT("Whether TIFF support is enabled."),
	ECVF_Default);
#endif

// TODO: should this still be a feature that can be turned off? 
static bool GInterchangeEnableMipMapImageImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnableMipMapImageImport(
	TEXT("Interchange.FeatureFlags.Import.MipMapImage"),
	GInterchangeEnableMipMapImageImport,
	TEXT("Whether Mip Mapped Image support is enabled."),
	ECVF_Default);

static bool GInterchangeEnableTGAImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnableTGAImport(
	TEXT("Interchange.FeatureFlags.Import.TGA"),
	GInterchangeEnableTGAImport,
	TEXT("Whether TGA support is enabled."),
	ECVF_Default);


namespace UE::Interchange::ImageWrapperTranslator::Private
{
	static bool SupportsMipMapsAndMetaData(EImageFormat ImageFormat)
	{
		switch (ImageFormat)
		{
		case EImageFormat::TIFF:
			return true;
		default:
			return false;
		}
	}
}

UInterchangeImageWrapperTranslator::UInterchangeImageWrapperTranslator()
{
	// construction of CDO is not thread safe,
	//  so ensure it is done on game thread before using it from threads
	// that is guaranteed because the module loader inits all CDOs

	PNGInfill = UE::TextureUtilitiesCommon::GetPNGInfillSetting();
}

TArray<FString> UInterchangeImageWrapperTranslator::GetSupportedFormats() const
{
	TArray<FString> Formats;
	Formats.Reserve(7);

	if (GInterchangeEnablePNGImport || GIsAutomationTesting)
	{
		Formats.Emplace(TEXT("png;Portable Network Graphic"));
	}

	if (GInterchangeEnableBMPImport || GIsAutomationTesting)
	{
		Formats.Emplace(TEXT("bmp;Bitmap image"));
	}

	if (GInterchangeEnableEXRImport || GIsAutomationTesting)
	{
		Formats.Emplace(TEXT("exr;OpenEXR image"));
	}

	if (GInterchangeEnableHDRImport || GIsAutomationTesting)
	{
		Formats.Emplace(TEXT("hdr;High Dynamic Range image"));
	}

#if WITH_LIBTIFF
	if (GInterchangeEnableTIFFImport || GIsAutomationTesting)
	{
		Formats.Emplace(TEXT("tif;Tag Image File Format"));
		Formats.Emplace(TEXT("tiff;Tag Image File Format"));
		Formats.Emplace(TEXT("tx;Tag Image File Format"));
	}
#endif

	if (GInterchangeEnableTGAImport || GIsAutomationTesting)
	{
		Formats.Emplace(TEXT("tga;Targa image"));
	}

	return Formats;
}

bool UInterchangeImageWrapperTranslator::Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const
{
	return UE::Interchange::FTextureTranslatorUtilities::Generic2DTextureTranslate(GetSourceData(), BaseNodeContainer);
}

TOptional<UE::Interchange::FImportImage> UInterchangeImageWrapperTranslator::GetTexturePayloadData(const FString& /*PayloadKey*/, TOptional<FString>& /*AlternateTexturePath*/) const
{
	using namespace UE::Interchange;

	if (!FTextureTranslatorUtilities::IsTranslatorValid(*this, TEXT("ImageWrapper")))
	{
		return {};
	}

	TArray64<uint8> SourceDataBuffer;
	FString Filename = GetSourceData()->GetFilename();

	if (!FFileHelper::LoadFileToArray(SourceDataBuffer, *Filename))
	{
		UE_LOGF(LogInterchangeImport, Error, "Failed to import Texture, cannot load file content into an array. [%ls]", *Filename);
		return TOptional<UE::Interchange::FImportImage>();
	}

	return GetTexturePayloadDataFromBuffer(SourceDataBuffer);
}

TOptional<UE::Interchange::FImportImage> UInterchangeImageWrapperTranslator::GetTexturePayloadDataFromBuffer(const TArray64<uint8>& SourceDataBuffer) const
{
	const uint8* Buffer = SourceDataBuffer.GetData();
	const uint8* BufferEnd = Buffer + SourceDataBuffer.Num();

	const int64 Length = BufferEnd - Buffer;

	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));

	EImageFormat ImageFormat = ImageWrapperModule.DetectImageFormat(Buffer, Length);


	UE::Interchange::FImportImage PayloadData;

	if (ImageFormat != EImageFormat::Invalid)
	{
		using namespace UE::Interchange::ImageWrapperTranslator::Private;

		if (GInterchangeEnableMipMapImageImport && SupportsMipMapsAndMetaData(ImageFormat))
		{
			FDecompressedImageOutput DecompressedImage;
			if (ImageWrapperModule.DecompressImage(Buffer, Length, DecompressedImage))
			{
				ETextureSourceFormat TextureFormat = FImageCoreUtils::ConvertToTextureSourceFormat(DecompressedImage.MipMapImage.Format);
				bool bSRGB = DecompressedImage.MipMapImage.GammaSpace != EGammaSpace::Linear;
				
				constexpr int32 MipLevel = 0;
				FImageView MipZeroImageView = DecompressedImage.MipMapImage.GetMipImage(MipLevel);

				PayloadData.Init2DWithParams(
					MipZeroImageView.SizeX,
					MipZeroImageView.SizeY,
					TextureFormat,
					bSRGB,
					false
				);

				PayloadData.RawData = MakeUniqueBufferFromArray(MoveTemp(DecompressedImage.MipMapImage.RawData));

				if (ERawImageFormat::IsHDR(DecompressedImage.MipMapImage.Format))
				{
					PayloadData.CompressionSettings = TC_HDR;
					check(bSRGB == false);
				}

				// Format Specific Settings
				if (ImageFormat == EImageFormat::TIFF)
				{
					PayloadData.NumMips = DecompressedImage.MipMapImage.GetMipCount();
					PayloadData.MipGenSettings = PayloadData.NumMips > 1 ? TextureMipGenSettings::TMGS_LeaveExistingMips : TextureMipGenSettings::TMGS_FromTextureGroup;
				}

				// Meta Data
				const bool bHasVendorString = !DecompressedImage.ApplicationVendor.IsEmpty();
				const bool bHasNameString = !DecompressedImage.ApplicationName.IsEmpty();
				const bool bHasVersionString = !DecompressedImage.ApplicationVersion.IsEmpty();

				if (bHasVendorString || bHasNameString || bHasVersionString)
				{
					UE::Interchange::FTextureCreatorApplicationMetadata Metadata;
					Metadata.ApplicationVendor = DecompressedImage.ApplicationVendor;
					Metadata.ApplicationName = DecompressedImage.ApplicationName;
					Metadata.ApplicationVersion = DecompressedImage.ApplicationVersion;
					PayloadData.TextureCreatorApplicationMetadata = Metadata;
				}
			}
		}
		else
		{

			// Generic ImageWrapper loader :
			// for PNG,EXR,BMP,TGA :
			FImage LoadedImage;
			if (ImageWrapperModule.DecompressImage(Buffer, Length, LoadedImage))
			{
				ETextureSourceFormat TextureFormat = FImageCoreUtils::ConvertToTextureSourceFormat(LoadedImage.Format);
				bool bSRGB = LoadedImage.GammaSpace != EGammaSpace::Linear;

				PayloadData.Init2DWithParams(
					LoadedImage.SizeX,
					LoadedImage.SizeY,
					TextureFormat,
					bSRGB,
					false
				);

				PayloadData.RawData = MakeUniqueBufferFromArray(MoveTemp(LoadedImage.RawData));

				if (ERawImageFormat::IsHDR(LoadedImage.Format))
				{
					PayloadData.CompressionSettings = TC_HDR;
					check(bSRGB == false);
				}

				// do per-format processing to match legacy behavior :

				if (ImageFormat == EImageFormat::PNG)
				{
					if (PNGInfill != ETextureImportPNGInfill::Never)
					{
						bool bDoOnComplexAlphaNotJustBinaryTransparency = (PNGInfill == ETextureImportPNGInfill::Always);

						// Replace the pixels with 0.0 alpha with a color value from the nearest neighboring color which has a non-zero alpha
						UE::TextureUtilitiesCommon::FillZeroAlphaPNGData(PayloadData.SizeX, PayloadData.SizeY, PayloadData.Format, reinterpret_cast<uint8*>(PayloadData.RawData.GetData()), bDoOnComplexAlphaNotJustBinaryTransparency);
					}
				}
				else if (ImageFormat == EImageFormat::TGA)
				{
					const FTGAFileHeader* TGA = (FTGAFileHeader*)Buffer;

					if (TGA->ColorMapType == 1 && TGA->ImageTypeCode == 1 && TGA->BitsPerPixel == 8)
					{
						// Notes: The Scaleform GFx exporter (dll) strips all font glyphs into a single 8-bit texture.
						// The targa format uses this for a palette index; GFx uses a palette of (i,i,i,i) so the index
						// is also the alpha value.
						//
						// We store the image as PF_G8, where it will be used as alpha in the Glyph shader.

						// ?? check or convert? or neither?
						//check( TextureFormat == TSF_G8 );

						PayloadData.CompressionSettings = TC_Grayscale;
					}
					else if (TGA->ColorMapType == 0 && TGA->ImageTypeCode == 3 && TGA->BitsPerPixel == 8)
					{
						// standard grayscale images

						// ?? check or convert? or neither?
						//check( TextureFormat == TSF_G8 );

						PayloadData.CompressionSettings = TC_Grayscale;
					}

					if (PayloadData.CompressionSettings == TC_Grayscale && TGA->ImageTypeCode == 3)
					{
						// default grayscales to linear as they wont get compression otherwise and are commonly used as masks
						PayloadData.bSRGB = false;
					}
				}
			}
		}
	}

	return PayloadData;
}

