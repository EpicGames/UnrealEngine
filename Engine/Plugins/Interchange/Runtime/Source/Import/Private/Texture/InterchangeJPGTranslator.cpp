// Copyright Epic Games, Inc. All Rights Reserved. 
#include "Texture/InterchangeJPGTranslator.h"

#include "Algo/Find.h"
#include "Containers/StaticArray.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "ImageCoreUtils.h"
#include "InterchangeImportLog.h"
#include "InterchangeTextureNode.h"
#include "Memory/SharedBuffer.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Texture/TextureTranslatorUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeJPGTranslator)

static bool GInterchangeEnableJPGImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnableJPGImport(
	TEXT("Interchange.FeatureFlags.Import.JPG"),
	GInterchangeEnableJPGImport,
	TEXT("Whether JPG support is enabled."),
	ECVF_Default);

TArray<FString> UInterchangeJPGTranslator::GetSupportedFormats() const
{
	TArray<FString> Formats;

	if (GInterchangeEnableJPGImport || GIsAutomationTesting)
	{
		Formats.Reserve(2);

		Formats.Add(TEXT("jpg;JPEG image"));
		Formats.Add(TEXT("jpeg;JPEG image"));
	}

	return Formats;
}

bool UInterchangeJPGTranslator::Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const
{
	return UE::Interchange::FTextureTranslatorUtilities::Generic2DTextureTranslate(GetSourceData(), BaseNodeContainer);
}

TOptional<UE::Interchange::FImportImage> UInterchangeJPGTranslator::GetTexturePayloadData(const FString& /*PayloadKey*/, TOptional<FString>& AlternateTexturePath) const
{
	using namespace UE::Interchange;

	TArray64<uint8> SourceDataBuffer;
	if (!FTextureTranslatorUtilities::LoadSourceBuffer(*this, TEXT("JPEG"), SourceDataBuffer))
	{
		return {};
	}

	const bool bImportRaw = false;
	return GetTexturePayloadImplementation(MoveTemp(SourceDataBuffer), bImportRaw);

}

bool UInterchangeJPGTranslator::SupportCompressedTexturePayloadData() const
{
	return true;
}

TOptional<UE::Interchange::FImportImage> UInterchangeJPGTranslator::GetCompressedTexturePayloadData(const FString& /*PayloadKey*/, TOptional<FString>& /*AlternateTexturePath*/) const
{
	using namespace UE::Interchange;

	TArray64<uint8> SourceDataBuffer;
	if (!FTextureTranslatorUtilities::LoadSourceBuffer(*this, TEXT("JPEG"), SourceDataBuffer))
	{
		return {};
	}

	const bool bImportRaw = true;
	return GetTexturePayloadImplementation(MoveTemp(SourceDataBuffer), bImportRaw);
}

TOptional<UE::Interchange::FImportImage> UInterchangeJPGTranslator::GetTexturePayloadImplementation(TArray64<uint8>&& SourceDataBuffer, bool bShouldImportRaw) const
{
	using namespace UE::Interchange;

	const uint8* Buffer = SourceDataBuffer.GetData();
	const uint8* BufferEnd = Buffer + SourceDataBuffer.Num();

	const int64 Length = BufferEnd - Buffer;

	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));

	//
	// JPG
	//
	TSharedPtr<IImageWrapper> JpegImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::JPEG);
	if (!JpegImageWrapper.IsValid() || !JpegImageWrapper->SetCompressed(Buffer, Length))
	{
		FTextureTranslatorUtilities::LogError(*this, NSLOCTEXT("InterchangeJPEGTranslator", "DecodingFailed", "Failed to decode JPEG."));
		return TOptional<UE::Interchange::FImportImage>();
	}

	// Select the texture's source format
	ERawImageFormat::Type RawFormat = JpegImageWrapper->GetClosestRawImageFormat();
	check( RawFormat != ERawImageFormat::Invalid );
	ETextureSourceFormat TextureFormat = FImageCoreUtils::ConvertToTextureSourceFormat(RawFormat);

	UE::Interchange::FImportImage PayloadData;

	const bool bShouldAllocateRawDataBuffer = false;

	PayloadData.Init2DWithParams(
		JpegImageWrapper->GetWidth(),
		JpegImageWrapper->GetHeight(),
		TextureFormat,
		JpegImageWrapper->GetSRGB(),
		bShouldAllocateRawDataBuffer
	);

	TArray64<uint8> RawData;
	if (bShouldImportRaw)
	{
		PayloadData.RawData = MakeUniqueBufferFromArray(MoveTemp(SourceDataBuffer));
		PayloadData.RawDataCompressionFormat = ETextureSourceCompressionFormat::TSCF_JPEG;
	}
	else if (JpegImageWrapper->GetRaw(RawData))
	{
		PayloadData.RawData = MakeUniqueBufferFromArray(MoveTemp(RawData));
	}
	else
	{
		FTextureTranslatorUtilities::LogError(*this, NSLOCTEXT("InterchangeJPEGTranslator", "DecodingFailed", "Failed to decode JPEG."));
		return TOptional<UE::Interchange::FImportImage>();
	}

	return PayloadData;
}

