// Copyright Epic Games, Inc. All Rights Reserved.

#include "Texture/InterchangePSDTranslator.h"

#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "ImageCoreUtils.h"
#include "InterchangeImportLog.h"
#include "InterchangeTextureNode.h"
#include "Math/GuardedInt.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Texture/TextureTranslatorUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangePSDTranslator)

static bool GInterchangeEnablePSDImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnablePSDImport(
	TEXT("Interchange.FeatureFlags.Import.PSD"),
	GInterchangeEnablePSDImport,
	TEXT("Whether PSD support is enabled."),
	ECVF_Default);

//////////////////////////////////////////////////////////////////////////
// PSD helper local function
namespace UE
{
	namespace Interchange
	{
		namespace Private
		{
#pragma pack(push,1)
			struct FPSDFileHeader
			{
				int32	Signature;		// 8BPS
				int16	Version;		// Version
				int16	nChannels;		// Number of Channels (3=RGB) (4=RGBA)
				int32	Height;			// Number of Image Rows
				int32	Width;			// Number of Image Columns
				int16	Depth;			// Number of Bits per Channel
				int16	Mode;			// Image Mode (0=Bitmap)(1=Grayscale)(2=Indexed)(3=RGB)(4=CYMK)(7=Multichannel)
				uint8	Pad[6];			// Padding

				/**
				 * @return Whether file has a valid signature
				 */
				bool IsValid() const
				{
					// Fail on bad signature
					return (Signature == 0x38425053);
				}

				/**
				 * @return Whether file has a supported version
				 */
				bool IsSupported() const
				{
					// Fail on bad version
					if (Version != 1)
						return false;

					// Fail on anything other than 1, 3 or 4 channels
					if ((nChannels != 1) && (nChannels != 3) && (nChannels != 4))
						return false;

					// Fail on anything other than 8 Bits/channel or 16 Bits/channel  
					if ((Depth != 8) && (Depth != 16))
						return false;

					// Fail on anything other than Grayscale and RGB
					// We can add support for indexed later if needed.
					if (Mode != 1 && Mode != 3)
						return false;

					return true;
				}
			};
#pragma pack(pop)

			static uint32 ReadNetwork32(const void* Ptr)
			{
				const uint8* Data = (const uint8*)Ptr;
				return ((uint32)Data[0] << 24) +
					((uint32)Data[1] << 16) +
					((uint32)Data[2] << 8) +
					((uint32)Data[3] << 0);
			}
			static uint16 ReadNetwork16(const void* Ptr)
			{
				const uint8* Data = (const uint8*)Ptr;
				return ((uint16)Data[0] << 8) +
					((uint16)Data[1] << 0);
			}

			static bool SkipSection(FMemoryView& InOutView)
			{
				// PSD has several 32 bit sized sections we ignore.
				if (InOutView.GetSize() < 4)
				{
					return false;
				}

				uint32 SectionSize = ReadNetwork32(InOutView.GetData());
				InOutView.RightChopInline(SectionSize + 4);
				return InOutView.GetSize() != 0;
			}

			static bool ReadUInt16(FMemoryView& InOutView, uint16& OutValue)
			{
				if (InOutView.GetSize() < 2)
				{
					OutValue = 0;
					return false;
				}
				OutValue = ReadNetwork16(InOutView.GetData());
				InOutView.RightChopInline(2);
				return true;
			}

			// Decodes the raw data for a row - this is the same independent of the scanline format.
			static bool DecodeRLERow(const uint8* RowSource, uint16 RowSourceBytes, uint8* OutputScanlineData, uint64 OutputScanlineDatasize)
			{
				uint64 OutputByte = 0;
				uint32 SourceByteIndex = 0;
				while (SourceByteIndex < RowSourceBytes)
				{
					int8 code = (int8)RowSource[SourceByteIndex];
					SourceByteIndex++;

					// Is it a repeat?
					if (code == -128) // nop code for alignment.
					{
					}
					else if (code < 0)
					{
						int32 Count = -(int32)code + 1;
						if (OutputByte + Count > OutputScanlineDatasize) // OutputScanlineDatasize originates as int32 + int8 can't overflow a uint64
						{
							return false;
						}

						if (SourceByteIndex >= RowSourceBytes)
						{
							return false;
						}
						uint8 Value = RowSource[SourceByteIndex];
						SourceByteIndex++;

						FMemory::Memset(OutputScanlineData + OutputByte, Value, Count);
						OutputByte += Count;
					}
					// Must be a run of literals then
					else
					{
						int32 Count = (int32)code + 1;
						if (SourceByteIndex + Count > RowSourceBytes) // int32 + int8, can't overflow.
						{
							return false;
						}

						FMemory::Memcpy(OutputScanlineData + OutputByte, RowSource + SourceByteIndex, Count);
						SourceByteIndex += Count;
						OutputByte += Count;
					}
				}

				// Confirm that we decoded the right number of bytes
				return OutputByte == OutputScanlineDatasize;
			}

			static bool ReadData(FMutableMemoryView& Output, FMemoryView Buffer, FPSDFileHeader& Info)
			{
				// Double check to make sure this is a valid request
				if (!Info.IsValid() || !Info.IsSupported())
				{
					return false;
				}

				if (Buffer.GetSize() <= sizeof(FPSDFileHeader) ||
					FImageCoreUtils::IsImageImportPossible(Info.Width, Info.Height) == false)
				{
					return false;
				}

				FMemoryView Current = Buffer;
				Current.RightChopInline(sizeof(FPSDFileHeader));

				FGuardedInt64 GuardedPixelCount = FGuardedInt64(Info.Width) * Info.Height;
				if (GuardedPixelCount.InvalidOrLessOrEqual(0))
				{
					return false;
				}

				if (Info.Depth != 8 && Info.Depth != 16)
				{
					return false;
				}
				if (Info.nChannels != 1 && Info.nChannels != 3 && Info.nChannels != 4)
				{
					return false;
				}

				FGuardedInt64 OutputBytesNeeded = GuardedPixelCount * 4 * (Info.Depth / 8);
				if (OutputBytesNeeded.InvalidOrGreaterThan(Output.GetSize()))
				{
					return false;
				}

				// Skip Clut, Image Resource Section, and Layer/Mask Section
				if (SkipSection(Current) == false || //-V501
					SkipSection(Current) == false ||
					SkipSection(Current) == false)
				{
					return false;
				}

				// Get Compression Type
				uint16 CompressionType = 0;
				if (ReadUInt16(Current, CompressionType) == false)
				{
					return false;
				}
				if (CompressionType != 0 && CompressionType != 1)
				{
					return false;
				}

				// Overflow checked above.
				uint64 UncompressedScanLineSizePerChannel = Info.Width * (Info.Depth / 8);
				uint64 OutputScanLineSize = UncompressedScanLineSizePerChannel * 4;
				uint64 UncompressedPlaneSize = UncompressedScanLineSizePerChannel * Info.Height;

				// For copying alpha when the source doesn't have alpha information.
				uint8 OpaqueAlpha[2] = {255, 255};

				const uint16* RLERowTable[4] = {};
				const uint8* RLEPlaneSource[4] = {};
				TArray<uint8> RLETempScanlines[4];

				if (CompressionType == 1)
				{
					uint64 RowTableBytesPerChannel = Info.Height * sizeof(uint16);
					if (Current.GetSize() < RowTableBytesPerChannel * Info.nChannels)
					{
						return false;
					}

					FMemoryView RowTableSource = Current;
					RowTableSource.LeftInline(RowTableBytesPerChannel * Info.nChannels);
					Current.RightChopInline(RowTableBytesPerChannel * Info.nChannels);

					// We want the row tables for each plane, which means we have to decode them.
					uint64 CurrentOffset = 0;
					for (uint64 Plane = 0; Plane < (uint64)Info.nChannels; Plane++)
					{
						RLETempScanlines[Plane].AddUninitialized(UncompressedScanLineSizePerChannel);
						RLERowTable[Plane] = (const uint16*)RowTableSource.GetData();
						RowTableSource.RightChopInline(RowTableBytesPerChannel);

						// Save off where this plane's source is.
						RLEPlaneSource[Plane] = (const uint8*)Current.GetData() + CurrentOffset;
						for (uint64 Row = 0; Row < Info.Height; Row++)
						{
							// can't overflow: we're adding 16-bit uints and Info.Height is bounded by a 32-bit value, so sum fits in 48 bits
							CurrentOffset += ReadNetwork16(&RLERowTable[Plane][Row]);
						}

						// Now that we know the size, verify we have it.
						if (Current.GetSize() < CurrentOffset)
						{
							return false;
						}
					}
				}
				else
				{
					if (Current.GetSize() < UncompressedPlaneSize * Info.nChannels) // overflow checked on function entry.
					{
						return false;
					}
				}

				uint8* OutputScanline = (uint8*)Output.GetData();
				for (uint64 Row = 0; Row < (uint64)Info.Height; Row++, OutputScanline += OutputScanLineSize)
				{
					const uint8* SourceScanLine[4] = {};
					uint64 AlphaMask = ~0ULL;

					// Init the source scanlines from the file data. For RLE we decode into a temp buffer,
					// otherwise we read directly. File size has already been validated.
					if (CompressionType == 0)
					{
						SourceScanLine[0] = (const uint8*)Current.GetData() + Row * UncompressedScanLineSizePerChannel;

						for (uint16 Channel = 1; Channel < Info.nChannels; Channel++)
						{
							SourceScanLine[Channel] = SourceScanLine[Channel-1] + UncompressedPlaneSize;
						}
					}
					else
					{
						for (uint16 Channel = 0; Channel < Info.nChannels; Channel++)
						{
							if (DecodeRLERow(RLEPlaneSource[Channel], ReadNetwork16(&RLERowTable[Channel][Row]), RLETempScanlines[Channel].GetData(), RLETempScanlines[Channel].Num()) == false)
							{
								return false;
							}
							RLEPlaneSource[Channel] += ReadNetwork16(&RLERowTable[Channel][Row]);
							SourceScanLine[Channel] = RLETempScanlines[Channel].GetData();
						}
					}

					// If we don't have all 4 channels, set up the scanlines to valid data.
					if (Info.nChannels == 1)
					{
						SourceScanLine[1] = SourceScanLine[0];
						SourceScanLine[2] = SourceScanLine[0];
						SourceScanLine[3] = OpaqueAlpha;
						AlphaMask = 0;
					}
					else if (Info.nChannels == 3)
					{
						SourceScanLine[3] = OpaqueAlpha;
						AlphaMask = 0;
					}
					else if (Info.nChannels == 4)
					{
						AlphaMask = ~0ULL;
					}

					// Do the plane interleaving.
					if (Info.Depth == 8)
					{
						FColor* ScanLine8 = (FColor*)OutputScanline;
						for (uint64 X = 0; X < Info.Width; X++)
						{
							ScanLine8[X].R = SourceScanLine[0][X];
							ScanLine8[X].G = SourceScanLine[1][X];
							ScanLine8[X].B = SourceScanLine[2][X];
							ScanLine8[X].A = SourceScanLine[3][X & AlphaMask];
						}
					}
					else if (Info.Depth == 16)
					{
						const uint16* SourceScanLineR16 = (const uint16*)SourceScanLine[0];
						const uint16* SourceScanLineG16 = (const uint16*)SourceScanLine[1];
						const uint16* SourceScanLineB16 = (const uint16*)SourceScanLine[2];
						const uint16* SourceScanLineA16 = (const uint16*)SourceScanLine[3];
						uint16* ScanLine16 = (uint16*)OutputScanline;
						for (uint64 X = 0; X < Info.Width; X++)
						{
							ScanLine16[4*X + 0] = ReadNetwork16(&SourceScanLineR16[X]);
							ScanLine16[4*X + 1] = ReadNetwork16(&SourceScanLineG16[X]);
							ScanLine16[4*X + 2] = ReadNetwork16(&SourceScanLineB16[X]);
							ScanLine16[4*X + 3] = ReadNetwork16(&SourceScanLineA16[X & AlphaMask]);
						}
					}
				} // end each row.

				// Success!
				return true;
			}

			static void GetPSDHeader(const uint8* Buffer, FPSDFileHeader& Info)
			{
				Info.Signature = (int32)ReadNetwork32(Buffer + 0);
				Info.Version = (int16)ReadNetwork16(Buffer + 4);
				Info.nChannels = (int16)ReadNetwork16(Buffer + 12);
				Info.Height = (int32)ReadNetwork32(Buffer + 14);
				Info.Width = (int32)ReadNetwork32(Buffer + 18);
				Info.Depth = (int16)ReadNetwork16(Buffer + 22);
				Info.Mode = (int16)ReadNetwork16(Buffer + 24);
			}

		}//ns Private
	}//ns Interchange
}//ns UE

TArray<FString> UInterchangePSDTranslator::GetSupportedFormats() const
{
	if (GInterchangeEnablePSDImport || GIsAutomationTesting)
	{
		TArray<FString> Formats{ TEXT("psd;Photoshop Document") };
		return Formats;
	}
	else
	{
		return TArray<FString>{};
	}
}

bool UInterchangePSDTranslator::Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const
{
	return UE::Interchange::FTextureTranslatorUtilities::Generic2DTextureTranslate(GetSourceData(), BaseNodeContainer);
}

TOptional<UE::Interchange::FImportImage> UInterchangePSDTranslator::GetTexturePayloadData(const FString& /*PayloadKey*/, TOptional<FString>& /*AlternateTexturePath*/) const
{
	using namespace UE::Interchange;

	TArray64<uint8> SourceDataBuffer;
	if (!FTextureTranslatorUtilities::LoadSourceBuffer(*this, TEXT("PSD"), SourceDataBuffer))
	{
		return {};
	}

	//
	// PSD File
	//
	UE::Interchange::Private::FPSDFileHeader PSDHeader;
	if (SourceDataBuffer.Num() < sizeof(UE::Interchange::Private::FPSDFileHeader))
	{
		return TOptional<UE::Interchange::FImportImage>();
	}

	UE::Interchange::Private::GetPSDHeader(SourceDataBuffer.GetData(), PSDHeader);

	if (!PSDHeader.IsValid())
	{
		return TOptional<UE::Interchange::FImportImage>();
	}

	if (!PSDHeader.IsSupported())
	{
		FTextureTranslatorUtilities::LogError(*this, NSLOCTEXT("InterchangePSDTranslator", "UnsupportedFormat", "Format of this PSD is not supported. Only Grayscale and RGBColor PSD images are currently supported, in 8-bit or 16-bit."));
		return TOptional<UE::Interchange::FImportImage>();
	}

	// Select the texture's source format
	ETextureSourceFormat TextureFormat = TSF_Invalid;
	if (PSDHeader.Depth == 8)
	{
		TextureFormat = TSF_BGRA8;
	}
	else if (PSDHeader.Depth == 16)
	{
		TextureFormat = TSF_RGBA16;
	}

	if (TextureFormat == TSF_Invalid)
	{
		FTextureTranslatorUtilities::LogError(*this, NSLOCTEXT("InterchangePSDTranslator", "UnsupportedPixelFormat", "This PSD file contains data in an unsupported pixel format."));
		return TOptional<UE::Interchange::FImportImage>();
	}

	UE::Interchange::FImportImage PayloadData;

	// The PSD is supported. Load it up.        
	PayloadData.Init2DWithParams(
		PSDHeader.Width,
		PSDHeader.Height,
		TextureFormat,
		true,
		true
	);

	FMutableMemoryView Output(PayloadData.RawData.GetData(), PayloadData.RawData.GetSize());
	if (!UE::Interchange::Private::ReadData(Output, MakeMemoryView(SourceDataBuffer), PSDHeader))
	{
		FTextureTranslatorUtilities::LogError(*this, NSLOCTEXT("InterchangePSDTranslator", "FailedRead", "Failed to read this PSD."));
		return TOptional<UE::Interchange::FImportImage>();
	}

	return PayloadData;
}

