// Copyright Epic Games, Inc. All Rights Reserved.

#include "Texture/InterchangeSlicedTexturePayloadData.h"

#include "Math/GuardedInt.h"
#include "CoreMinimal.h"

void UE::Interchange::FImportSlicedImage::Init(int32 InSizeX, int32 InSizeY, int32 InNumSlice, int32 InNumMips, ETextureSourceFormat InFormat, bool InSRGB)
{
	NumSlice = InNumSlice;

	FImportImage::Init2DWithParams(InSizeX, InSizeY, InNumMips, InFormat, InSRGB);
}

void UE::Interchange::FImportSlicedImage::InitVolume(int32 InSizeX, int32 InSizeY, int32 InNumSlice, int32 InNumMips, ETextureSourceFormat InFormat, bool InSRGB)
{
	bIsVolume = true;
	Init(InSizeX, InSizeY, InNumSlice, InNumMips, InFormat, InSRGB);
}

const uint8* UE::Interchange::FImportSlicedImage::GetMipData(int32 InMipIndex, int32 InSliceIndex) const
{
	const int32 SliceFactor = bIsVolume ? 1 : NumSlice;

	int64 Offset = 0;
	int32 MipIndex;
	for (MipIndex = 0; MipIndex < InMipIndex; ++MipIndex)
	{
		Offset += GetMipSize(MipIndex) * SliceFactor;
	}

	if (InSliceIndex != INDEX_NONE)
	{ 
		check(!bIsVolume);
		Offset += GetMipSize(MipIndex) * InSliceIndex;
	}

	check(RawData.GetSize() > uint64(Offset));

	return static_cast<const uint8*>(RawData.GetData()) + Offset;
}

uint8* UE::Interchange::FImportSlicedImage::GetMipData(int32 InMipIndex, int32 InSliceIndex)
{
	const uint8* ConstBufferPtr = static_cast<const FImportSlicedImage*>(this)->GetMipData(InMipIndex, InSliceIndex);
	return const_cast<uint8*>(ConstBufferPtr);
}

int64 UE::Interchange::FImportSlicedImage::GetMipSize(int32 InMipIndex) const
{
	const int64 BytesPerPixel = FTextureSource::GetBytesPerPixel(Format);

	int32 MipSizeX = FMath::Max<int32>(SizeX >> InMipIndex, 1);
	int32 MipSizeY = FMath::Max<int32>(SizeY >> InMipIndex, 1);
	int32 MipSizeZ = bIsVolume ? FMath::Max(NumSlice >> InMipIndex, 1) : 1;

	FGuardedInt64 MipSize = FGuardedInt64(MipSizeX) * MipSizeY * MipSizeZ * BytesPerPixel;
	checkf(MipSize.IsValid(), TEXT("Invalid (overflowing) mip sizes made it in to FImportSlicedImage::GetMipSize! Check import locations for mip size validation"));

	return MipSize.Get(0);
}

int64 UE::Interchange::FImportSlicedImage::ComputeBufferSize() const
{
	const int32 SliceFactor = bIsVolume ? 1 : NumSlice;
	return FImportImage::ComputeBufferSize() * SliceFactor;
}

bool UE::Interchange::FImportSlicedImage::IsValid() const
{
	return NumSlice > 0 && FImportImage::IsValid();
}

