// Copyright Epic Games, Inc. All Rights Reserved.

#include "Texture/InterchangeTextureFactory.h"

#include "Async/ParallelFor.h"
#include "Async/TaskGraphInterfaces.h"
#include "Async/TaskGraphInterfaces.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture2DArray.h"
#include "Engine/TextureCube.h"
#include "Engine/TextureDefines.h"
#include "Engine/TextureLightProfile.h"
#include "HAL/FileManagerGeneric.h"
#include "ImageCoreUtils.h"
#include "ImageUtils.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeImportCommon.h"
#include "InterchangeImportLog.h"
#include "InterchangeResult.h"
#include "InterchangeTexture2DArrayFactoryNode.h"
#include "InterchangeTexture2DArrayNode.h"
#include "InterchangeTexture2DFactoryNode.h"
#include "InterchangeTexture2DNode.h"
#include "InterchangeTextureCubeArrayFactoryNode.h"
#include "InterchangeTextureCubeArrayNode.h"
#include "InterchangeTextureCubeFactoryNode.h"
#include "InterchangeTextureCubeNode.h"
#include "InterchangeTextureFactoryNode.h"
#include "InterchangeTextureLightProfileFactoryNode.h"
#include "InterchangeTextureLightProfileNode.h"
#include "InterchangeTextureNode.h"
#include "InterchangeTranslatorBase.h"
#include "InterchangeVolumeTextureFactoryNode.h"
#include "InterchangeVolumeTextureNode.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreStats.h"
#include "Misc/Paths.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeSourceNode.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Serialization/EditorBulkData.h"
#include "Texture/InterchangeBlockedTexturePayloadInterface.h"
#include "Texture/InterchangeJPGTranslator.h"
#include "Texture/InterchangeUEJPEGTranslator.h"
#include "Texture/InterchangeSlicedTexturePayloadInterface.h"
#include "Texture/InterchangeTextureLightProfilePayloadInterface.h"
#include "Texture/InterchangeTexturePayloadInterface.h"
#include "TextureCompiler.h"
#include "TextureImportSettings.h"
#include "TextureImportUtils.h"
#include "TextureResource.h"
#include "UDIMUtilities.h"
#include "UObject/ObjectMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeTextureFactory)

#if WITH_EDITORONLY_DATA

#include "EditorFramework/AssetImportData.h"

#endif //WITH_EDITORONLY_DATA

#define LOCTEXT_NAMESPACE "InterchangeTextureFactory"

namespace UE::Interchange::Private::InterchangeTextureFactory
{
	class FScopedTranslatorsAndSourceData
	{
	public:
		FScopedTranslatorsAndSourceData(const UInterchangeTranslatorBase& BaseTranslator, uint32 NumRequired)
		{
			UObject* TransientPackage = GetTransientPackage();

			TranslatorsAndSourcesData.Reserve(NumRequired);

			UClass* TranslatorClass = BaseTranslator.GetClass();
			for (uint32 Index = 0; Index < NumRequired; Index++)
			{
				TranslatorsAndSourcesData.Emplace(
					NewObject<UInterchangeTranslatorBase>(TransientPackage, TranslatorClass, NAME_None)
					, NewObject<UInterchangeSourceData>(TransientPackage, UInterchangeSourceData::StaticClass(), NAME_None)
				);
			}

			for (TPair<UInterchangeTranslatorBase*, UInterchangeSourceData*>& TranslatorAndSourceData : TranslatorsAndSourcesData)
			{
				TranslatorAndSourceData.Key->SetResultsContainer(BaseTranslator.Results);
			}
		};

		~FScopedTranslatorsAndSourceData()
		{
			for (TPair<UInterchangeTranslatorBase*, UInterchangeSourceData*>& TranslatorAndSourceData : TranslatorsAndSourcesData)
			{
				TranslatorAndSourceData.Key->ClearInternalFlags(EInternalObjectFlags::Async);
				TranslatorAndSourceData.Value->ClearInternalFlags(EInternalObjectFlags::Async);
			}
		};

		TPair<UInterchangeTranslatorBase*, UInterchangeSourceData*>& operator[](uint32 Index)
		{
			return TranslatorsAndSourcesData[Index];
		}

		const TPair<UInterchangeTranslatorBase*, UInterchangeSourceData*>& operator[](uint32 Index) const
		{
			return TranslatorsAndSourcesData[Index];
		}

		uint32 Num() const
		{
			return TranslatorsAndSourcesData.Num();
		}

	private:
		TArray<TPair<UInterchangeTranslatorBase*, UInterchangeSourceData*>> TranslatorsAndSourcesData;
	};

	class FScopedClearAsyncFlag
	{
	public:
		FScopedClearAsyncFlag(UObject* InObject)
			: Object(nullptr)
		{
			if (InObject && InObject->HasAnyInternalFlags(EInternalObjectFlags::Async))
			{
				Object = InObject;
			}
		}

		~FScopedClearAsyncFlag()
		{
			if (Object)
			{
				Object->ClearInternalFlags(EInternalObjectFlags::Async);
			}
		}

	private:
		UObject* Object;
	};

	/**
	 * Return the supported class if the node is one otherwise return nullptr
	 */
	UClass* GetSupportedFactoryNodeClass(const UInterchangeFactoryBaseNode* AssetNode)
	{
		UClass* TextureCubeFactoryClass = UInterchangeTextureCubeFactoryNode::StaticClass();
		UClass* TextureCubeArrayFactoryClass = UInterchangeTextureCubeArrayFactoryNode::StaticClass();
		UClass* Texture2DFactoryClass = UInterchangeTexture2DFactoryNode::StaticClass();
		UClass* Texture2DArrayFactoryClass = UInterchangeTexture2DArrayFactoryNode::StaticClass();
		UClass* TextureLightProfileFactoryClass = UInterchangeTextureLightProfileFactoryNode::StaticClass();
		UClass* VolumeTextureFactoryClass = UInterchangeVolumeTextureFactoryNode::StaticClass();
		UClass* AssetClass = AssetNode->GetClass();
#if USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_STRUCTARRAY
		if (AssetClass->IsChildOf(Texture2DArrayFactoryClass))
		{
			return Texture2DArrayFactoryClass;
		}
		else if (AssetClass->IsChildOf(TextureCubeFactoryClass))
		{
			return TextureCubeFactoryClass;
		}
		else if (AssetClass->IsChildOf(TextureCubeArrayFactoryClass))
		{
			return TextureCubeArrayFactoryClass;
		}
		else if (AssetClass->IsChildOf(TextureLightProfileFactoryClass))
		{
			return TextureLightProfileFactoryClass;
		}
		else if (AssetClass->IsChildOf(VolumeTextureFactoryClass))
		{
			return VolumeTextureFactoryClass;
		}
		else if (AssetClass->IsChildOf(Texture2DFactoryClass))
		{
			return Texture2DFactoryClass;
		}
#else
		while (AssetClass)
		{
			if (AssetClass == TextureCubeFactoryClass
				|| AssetClass == TextureCubeArrayFactoryClass
				|| AssetClass == Texture2DFactoryClass
				|| AssetClass == Texture2DArrayFactoryClass
				|| AssetClass == TextureLightProfileFactoryClass
				|| AssetClass == VolumeTextureFactoryClass)
			{
				return AssetClass;
			}

			AssetClass = AssetClass->GetSuperClass();
		}
#endif

		return nullptr;
	}

	using FTextureFactoryNodeVariant = TVariant<FEmptyVariantState
		, UInterchangeTexture2DFactoryNode*
		, UInterchangeTextureCubeFactoryNode*
		, UInterchangeTextureCubeArrayFactoryNode*
		, UInterchangeTexture2DArrayFactoryNode*
		, UInterchangeTextureLightProfileFactoryNode*
		, UInterchangeVolumeTextureFactoryNode*>;

	FTextureFactoryNodeVariant GetAsTextureFactoryNodeVariant(UInterchangeFactoryBaseNode* AssetNode, UClass* SupportedFactoryNodeClass)
	{
		static_assert(TVariantSize_V<FTextureFactoryNodeVariant> == 7, "Please update the code below and this assert to reflect the change to the variant type.");

		if (AssetNode)
		{
			if (!SupportedFactoryNodeClass)
			{
				SupportedFactoryNodeClass = GetSupportedFactoryNodeClass(AssetNode);
			}

			if (SupportedFactoryNodeClass == UInterchangeTexture2DFactoryNode::StaticClass())
			{
				return FTextureFactoryNodeVariant(TInPlaceType<UInterchangeTexture2DFactoryNode*>(), static_cast<UInterchangeTexture2DFactoryNode*>(AssetNode));
			}

			if (SupportedFactoryNodeClass == UInterchangeTextureCubeFactoryNode::StaticClass())
			{
				return FTextureFactoryNodeVariant(TInPlaceType<UInterchangeTextureCubeFactoryNode*>(), static_cast<UInterchangeTextureCubeFactoryNode*>(AssetNode));
			}

			if (SupportedFactoryNodeClass == UInterchangeTextureCubeArrayFactoryNode::StaticClass())
			{
				return FTextureFactoryNodeVariant(TInPlaceType<UInterchangeTextureCubeArrayFactoryNode*>(), static_cast<UInterchangeTextureCubeArrayFactoryNode*>(AssetNode));
			}

			if (SupportedFactoryNodeClass == UInterchangeTexture2DArrayFactoryNode::StaticClass())
			{
				return FTextureFactoryNodeVariant(TInPlaceType<UInterchangeTexture2DArrayFactoryNode*>(), static_cast<UInterchangeTexture2DArrayFactoryNode*>(AssetNode));
			}

			if (SupportedFactoryNodeClass == UInterchangeTextureLightProfileFactoryNode::StaticClass())
			{
				return FTextureFactoryNodeVariant(TInPlaceType<UInterchangeTextureLightProfileFactoryNode*>(), static_cast<UInterchangeTextureLightProfileFactoryNode*>(AssetNode));
			}

			if (SupportedFactoryNodeClass == UInterchangeVolumeTextureFactoryNode::StaticClass())
			{
				return FTextureFactoryNodeVariant(TInPlaceType<UInterchangeVolumeTextureFactoryNode*>(), static_cast<UInterchangeVolumeTextureFactoryNode*>(AssetNode));
			}
		}

		ensureMsgf(false
			, TEXT("Unknown factory node class (%s). To add support for a new texture type, either update this factory or register a new factory to Interchange.\n\
				However, if the factory node class is used to add functions for some custom attributes of a pipeline,\n\
				use static function libraries or use helper structs/objects instead.\n\
				Here is an example of what this could look like in C++:\n\n\
				\tif (UCustomAttributeInterface::HasInterface(TextureFactoryNode))\n\
				\t{\n\
					\t\tFString MyPath = UCustomAttributeInterface::GetCustomPath(TextureFactoryNode);\n\
					\t\t...\n\
				\t}\n\
				or\n\
				\tif (UCustomAttributeInterface* CustomInterface = UCustomAttributeInterface::GetInterface(TextureFactoryNode))\n\
				\t{\n\
					\t\tFString MyPath = CustomInterface->GetPath();\n\
					\t\t...\n\
				\t}\n\n")
			, *AssetNode->GetClass()->GetName()
			);

		return {};
	}

	UInterchangeTextureFactoryNode* GetTextureFactoryNodeFromVariant(const FTextureFactoryNodeVariant& FactoryNodeVariant)
	{
		static_assert(TVariantSize_V<FTextureFactoryNodeVariant> == 7, "Please update the code below and this assert to reflect the change to the variant type.");

		if (UInterchangeTexture2DFactoryNode* const* TextureFactoryNode = FactoryNodeVariant.TryGet<UInterchangeTexture2DFactoryNode*>())
		{
			return *TextureFactoryNode;
		}

		if (UInterchangeTextureCubeFactoryNode* const* TextureCubeFactoryNode = FactoryNodeVariant.TryGet<UInterchangeTextureCubeFactoryNode*>())
		{
			return *TextureCubeFactoryNode;
		}

		if (UInterchangeTextureCubeArrayFactoryNode* const* TextureCubeArrayFactoryNode = FactoryNodeVariant.TryGet<UInterchangeTextureCubeArrayFactoryNode*>())
		{
			return *TextureCubeArrayFactoryNode;
		}

		if (UInterchangeTexture2DArrayFactoryNode* const* Texture2DArrayFactoryNode = FactoryNodeVariant.TryGet<UInterchangeTexture2DArrayFactoryNode*>())
		{
			return *Texture2DArrayFactoryNode;
		}

		if (UInterchangeTextureLightProfileFactoryNode* const* TextureLightProfileFactoryNode = FactoryNodeVariant.TryGet<UInterchangeTextureLightProfileFactoryNode*>())
		{
			return *TextureLightProfileFactoryNode;
		}

		if (UInterchangeVolumeTextureFactoryNode* const* VolumeTextureFactoryNode = FactoryNodeVariant.TryGet<UInterchangeVolumeTextureFactoryNode*>())
		{
			return *VolumeTextureFactoryNode;
		}

		return nullptr;
	}

	using FTextureNodeVariant = TVariant<FEmptyVariantState
		, const UInterchangeTexture2DNode*
		, const UInterchangeTextureCubeNode*
		, const UInterchangeTextureCubeArrayNode*
		, const UInterchangeTexture2DArrayNode*
		, const UInterchangeTextureLightProfileNode*
		, const UInterchangeVolumeTextureNode*>;

	FTextureNodeVariant GetTextureNodeVariantFromFactoryVariant(const FTextureFactoryNodeVariant& FactoryVariant, const UInterchangeBaseNodeContainer* NodeContainer)
	{
		static_assert(TVariantSize_V<FTextureFactoryNodeVariant> == 7, "Please update the code below and this assert to reflect the change to the variant type.");

		static_assert(TVariantSize_V<FTextureNodeVariant> == 7, "Please update the code below and this assert to reflect the change to the variant type.");

		FString TextureNodeUniqueID;

		if (UInterchangeTexture2DFactoryNode* const* Texture2DFactoryNode = FactoryVariant.TryGet<UInterchangeTexture2DFactoryNode*>())
		{
			(*Texture2DFactoryNode)->GetCustomTranslatedTextureNodeUid(TextureNodeUniqueID);
		}
		else if (UInterchangeTextureCubeFactoryNode* const* TextureCubeFactoryNode = FactoryVariant.TryGet<UInterchangeTextureCubeFactoryNode*>())
		{
			(*TextureCubeFactoryNode)->GetCustomTranslatedTextureNodeUid(TextureNodeUniqueID);
		}
		else if (UInterchangeTextureCubeArrayFactoryNode* const* TextureCubeArrayFactoryNode = FactoryVariant.TryGet<UInterchangeTextureCubeArrayFactoryNode*>())
		{
			(*TextureCubeArrayFactoryNode)->GetCustomTranslatedTextureNodeUid(TextureNodeUniqueID);
		}
		else if (UInterchangeTexture2DArrayFactoryNode* const* Texture2DArrayFactoryNode = FactoryVariant.TryGet<UInterchangeTexture2DArrayFactoryNode*>())
		{
			(*Texture2DArrayFactoryNode)->GetCustomTranslatedTextureNodeUid(TextureNodeUniqueID);
		}
		else if (UInterchangeTextureLightProfileFactoryNode* const* TextureLightProfileFactoryNode = FactoryVariant.TryGet<UInterchangeTextureLightProfileFactoryNode*>())
		{
			(*TextureLightProfileFactoryNode)->GetCustomTranslatedTextureNodeUid(TextureNodeUniqueID);
		}
		else if (UInterchangeVolumeTextureFactoryNode* const* VolumeTextureFactoryNode = FactoryVariant.TryGet<UInterchangeVolumeTextureFactoryNode*>())
		{
			(*VolumeTextureFactoryNode)->GetCustomTranslatedTextureNodeUid(TextureNodeUniqueID);
		}

		if (const UInterchangeBaseNode* TranslatedNode = NodeContainer->GetNode(TextureNodeUniqueID))
		{
			if (const UInterchangeTextureCubeNode* TextureCubeTranslatedNode = Cast<UInterchangeTextureCubeNode>(TranslatedNode))
			{
				return FTextureNodeVariant(TInPlaceType<const UInterchangeTextureCubeNode*>(), TextureCubeTranslatedNode);
			}

			if (const UInterchangeTextureCubeArrayNode* TextureCubeArrayTranslatedNode = Cast<UInterchangeTextureCubeArrayNode>(TranslatedNode))
			{
				return FTextureNodeVariant(TInPlaceType<const UInterchangeTextureCubeArrayNode*>(), TextureCubeArrayTranslatedNode);
			}

			if (const UInterchangeTexture2DArrayNode* Texture2DArrayTranslatedNode = Cast<UInterchangeTexture2DArrayNode>(TranslatedNode))
			{
				return FTextureNodeVariant(TInPlaceType<const UInterchangeTexture2DArrayNode*>(), Texture2DArrayTranslatedNode);
			}

			if (const UInterchangeTextureLightProfileNode* TextureLightProfileTranslatedNode = Cast<UInterchangeTextureLightProfileNode>(TranslatedNode))
			{
				return FTextureNodeVariant(TInPlaceType<const UInterchangeTextureLightProfileNode*>(), TextureLightProfileTranslatedNode);
			}

			if (const UInterchangeTexture2DNode* TextureTranslatedNode = Cast<UInterchangeTexture2DNode>(TranslatedNode))
			{
				return FTextureNodeVariant(TInPlaceType<const UInterchangeTexture2DNode*>(), TextureTranslatedNode);
			}

			if (const UInterchangeVolumeTextureNode* TextureTranslatedNode = Cast<UInterchangeVolumeTextureNode>(TranslatedNode))
			{
				return FTextureNodeVariant(TInPlaceType<const UInterchangeVolumeTextureNode*>(), TextureTranslatedNode);
			}
		}

		return {};
	}

	bool HasPayloadKey(const FTextureNodeVariant& TextureNodeVariant)
	{
		static_assert(TVariantSize_V<FTextureNodeVariant> == 7, "Please update the code below and this assert to reflect the change to the variant type.");

		if (const UInterchangeTexture2DNode* const* TextureNode =  TextureNodeVariant.TryGet<const UInterchangeTexture2DNode*>())
		{
			return (*TextureNode)->GetPayLoadKey().IsSet();
		}

		if (const UInterchangeTextureCubeNode* const* TextureCubeNode = TextureNodeVariant.TryGet<const UInterchangeTextureCubeNode*>())
		{
			return (*TextureCubeNode)->GetPayLoadKey().IsSet();
		}

		if (const UInterchangeTextureCubeArrayNode* const* TextureCubeArrayNode = TextureNodeVariant.TryGet<const UInterchangeTextureCubeArrayNode*>())
		{
			return (*TextureCubeArrayNode)->GetPayLoadKey().IsSet();
		}

		if (const UInterchangeTexture2DArrayNode* const* Texture2DArrayNode = TextureNodeVariant.TryGet<const UInterchangeTexture2DArrayNode*>())
		{
			return (*Texture2DArrayNode)->GetPayLoadKey().IsSet();
		}

		if (const UInterchangeTextureLightProfileNode* const* TextureLightProfileNode = TextureNodeVariant.TryGet<const UInterchangeTextureLightProfileNode*>())
		{
			return (*TextureLightProfileNode)->GetPayLoadKey().IsSet();
		}

		if (const UInterchangeVolumeTextureNode* const* VolumeTextureNode = TextureNodeVariant.TryGet<const UInterchangeVolumeTextureNode*>())
		{
			return (*VolumeTextureNode)->GetPayLoadKey().IsSet();
		}

		return false;
	}

	TOptional<FString> GetPayloadKey(const FTextureNodeVariant& TextureNodeVariant)
	{
		static_assert(TVariantSize_V<FTextureNodeVariant> == 7, "Please update the code below and this assert to reflect the change to the variant type.");

		if (const UInterchangeTexture2DNode* const* TextureNode =  TextureNodeVariant.TryGet<const UInterchangeTexture2DNode*>())
		{
			return (*TextureNode)->GetPayLoadKey();
		}

		if (const UInterchangeTextureCubeNode* const* TextureCubeNode = TextureNodeVariant.TryGet<const UInterchangeTextureCubeNode*>())
		{
			return (*TextureCubeNode)->GetPayLoadKey();
		}

		if (const UInterchangeTextureCubeArrayNode* const* TextureCubeArrayNode = TextureNodeVariant.TryGet<const UInterchangeTextureCubeArrayNode*>())
		{
			return (*TextureCubeArrayNode)->GetPayLoadKey();
		}

		if (const UInterchangeTexture2DArrayNode* const* Texture2DArrayNode = TextureNodeVariant.TryGet<const UInterchangeTexture2DArrayNode*>())
		{
			return (*Texture2DArrayNode)->GetPayLoadKey();
		}

		if (const UInterchangeTextureLightProfileNode* const* TextureLightProfileNode = TextureNodeVariant.TryGet<const UInterchangeTextureLightProfileNode*>())
		{
			return (*TextureLightProfileNode)->GetPayLoadKey();
		}

		if (const UInterchangeVolumeTextureNode* const* VolumeTextureNode = TextureNodeVariant.TryGet<const UInterchangeVolumeTextureNode*>())
		{
			return (*VolumeTextureNode)->GetPayLoadKey();
		}

		return {};
	}

	TOptional<FImportBlockedImage> GetBlockedTexturePayloadDataFromSourceFiles(const UInterchangeSourceData* SourceData, const TMap<int32, FString>& InBlockSourcesData, const UInterchangeTranslatorBase* Translator)
	{
		check(!InBlockSourcesData.IsEmpty());

		// note if only one image in UDIM set is found,
		//	then SetupTexture2DSourceDataFromBulkData_Editor will wind up discarding the UDIM parse
		//	and instead import it as a single image.
		// eg. if you have something like "test_1024.bmp" it will not be treated as UDIM with offsets of X=3 and Y=2

		TArray<const TPair<int32, FString>*> UDIMsAndSourcesFileArray;
		UDIMsAndSourcesFileArray.Reserve(InBlockSourcesData.Num());
		int32 OrginalSourceDataIndex = INDEX_NONE;
		for (const TPair<int32, FString>& Pair : InBlockSourcesData)
		{
			if (OrginalSourceDataIndex == INDEX_NONE && SourceData->GetFilename() == Pair.Value)
			{
				OrginalSourceDataIndex = UDIMsAndSourcesFileArray.Num();
			}

			UDIMsAndSourcesFileArray.Add(&Pair);
		}

		if (OrginalSourceDataIndex != INDEX_NONE && 0 != OrginalSourceDataIndex)
		{
			// Move the original file to the front.
			UDIMsAndSourcesFileArray.Swap(0, OrginalSourceDataIndex);
		}

		FScopedTranslatorsAndSourceData TranslatorsAndSourceData(*Translator, UDIMsAndSourcesFileArray.Num());

		/**
			* Possible improvement notes.
			* If we are able at some point to extract the size and format of the textures from the translate step for some formats,
			* We could use those information to init the blocked image RawData and set the ImportsImages RawData to be a view into the blocked image.
			*/ 
		TArray<UE::Interchange::FImportImage> Images;
		Images.AddDefaulted(UDIMsAndSourcesFileArray.Num());

		ParallelFor(UDIMsAndSourcesFileArray.Num(), [&TranslatorsAndSourceData, &UDIMsAndSourcesFileArray, &Images](int32 Index)
			{
				TPair<UInterchangeTranslatorBase*,UInterchangeSourceData*>& TranslatorAndSourceData = TranslatorsAndSourceData[Index];
				UInterchangeTranslatorBase* Translator = TranslatorAndSourceData.Key;
				IInterchangeTexturePayloadInterface* TextureTranslator = CastChecked<IInterchangeTexturePayloadInterface>(TranslatorAndSourceData.Key);
				UInterchangeSourceData* SourceDataForBlock = TranslatorAndSourceData.Value;

				const TPair<int32, FString>& UDIMAndFilename = *UDIMsAndSourcesFileArray[Index];
				const int32 CurrentUDIM = UDIMAndFilename.Key;
				SourceDataForBlock->SetFilename(UDIMAndFilename.Value);
				
				TOptional<UE::Interchange::FImportImage> Payload;
				if (Translator->CanImportSourceData(SourceDataForBlock))
				{
					TOptional<FString> DummyAlternateTexturePath;
					Translator->SourceData = SourceDataForBlock;
					Payload = TextureTranslator->GetTexturePayloadData(FString(), DummyAlternateTexturePath);
				}

				if (Payload.IsSet())
				{
					UE::Interchange::FImportImage& Image = Payload.GetValue();
					// Consume the image
					Images[Index] = MoveTemp(Image);
				}
				else
				{
					// Todo Capture the error message from the translator?
				}

				// Let the translator release its data.
				Translator->ImportFinish();
			}
			, EParallelForFlags::Unbalanced | EParallelForFlags::BackgroundPriority);

		
		// remove any that failed to import:
		for (int32 Index = Images.Num()-1; Index >= 0; --Index)
		{
			// we will crash later if an image has Format == TSF_Invalid
			if ( ! Images[Index].IsValid() )
			{
				Images.RemoveAt(Index);
				UDIMsAndSourcesFileArray.RemoveAt(Index);
			}
		}

		if ( Images.Num() == 0 )
		{
			return {};
		}

		// InitDataSharedAmongBlocks will not work unless Images[0] is valid :
		check( Images[0].IsValid() );
		check( Images.Num() == UDIMsAndSourcesFileArray.Num() );

		UE::Interchange::FImportBlockedImage BlockedImage;
		// If the image that triggered the import is not valid. We shouldn't import the rest of the images into the UDIM.
		if (BlockedImage.InitDataSharedAmongBlocks(Images[0]))
		{
			BlockedImage.BlocksData.Reserve(Images.Num());

			bool bMismatchedFormats = false;
			bool bMismatchedGammaSpace = false;
			for (int32 Index = 0; Index < Images.Num(); ++Index)
			{
				if (BlockedImage.bSRGB != Images[Index].bSRGB)
				{
					bMismatchedGammaSpace = true;
					BlockedImage.bSRGB = false;
					BlockedImage.Format = TSF_RGBA32F;
				}

				if (BlockedImage.Format != Images[Index].Format)
				{
					bMismatchedFormats = true;
					BlockedImage.Format = FImageCoreUtils::GetCommonSourceFormat(BlockedImage.Format, Images[Index].Format);
				}
			}

			if ( BlockedImage.bSRGB && ! ERawImageFormat::GetFormatNeedsGammaSpace( FImageCoreUtils::ConvertToRawImageFormat(BlockedImage.Format) ) )
			{
				BlockedImage.bSRGB = false;
				bMismatchedGammaSpace = true;
			}

			if (bMismatchedGammaSpace || bMismatchedFormats)
			{
				UE_LOGF(LogInterchangeImport, Display, "Mismatched UDIM image %ls. Converting all to %ls/%ls ...", bMismatchedGammaSpace ? TEXT("gamma spaces") : TEXT("pixel formats"),
					ERawImageFormat::GetName(FImageCoreUtils::ConvertToRawImageFormat(BlockedImage.Format)), BlockedImage.bSRGB ? TEXT("sRGB") : TEXT("Linear"));

				for (UE::Interchange::FImportImage& Image : Images)
				{
					if (Image.bSRGB != BlockedImage.bSRGB || Image.Format != BlockedImage.Format)
					{
						check(Image.IsValid());

						ERawImageFormat::Type ImageRawFormat = FImageCoreUtils::ConvertToRawImageFormat(Image.Format);
						FImageView SourceImage(Image.RawData.GetData(), Image.SizeX, Image.SizeY, 1, ImageRawFormat, Image.bSRGB ? EGammaSpace::sRGB : EGammaSpace::Linear);
						check( SourceImage.GetImageSizeBytes() <= (int64)Image.RawData.GetSize() );
						
						ERawImageFormat::Type BlockedImageRawFormat = FImageCoreUtils::ConvertToRawImageFormat(BlockedImage.Format);
						FImage DestImage(Image.SizeX, Image.SizeY, BlockedImageRawFormat, BlockedImage.bSRGB ? EGammaSpace::sRGB : EGammaSpace::Linear);

						{
							FImage GrayScaleImage;
							switch (ImageRawFormat)
							{
							case ERawImageFormat::R16F:
								// falls through
							case ERawImageFormat::R32F:
							{
								GrayScaleImage.Init(Image.SizeX, Image.SizeY, ERawImageFormat::G16, Image.bSRGB ? EGammaSpace::sRGB : EGammaSpace::Linear);
								FImageCore::CopyImage(SourceImage, GrayScaleImage);
								SourceImage = GrayScaleImage;
								break;
							}
							default:
								break;
							}

							FImageCore::CopyImage(SourceImage, DestImage);
						}

						Image.RawData = MakeUniqueBufferFromArray(MoveTemp(DestImage.RawData));
						Image.Format = BlockedImage.Format;
						Image.bSRGB = BlockedImage.bSRGB;

						if ( Image.NumMips != 1 )
						{
							UE_LOGF(LogInterchangeImport, Warning, "UDIM Image had existing mips. They were discarded in the format change." );
							Image.NumMips = 1; // discard imported mips, they won't be used in UDIM VT anyway
							Image.MipGenSettings.Reset();
						}

						check(Image.IsValid());
					}
				}
			}

			bool bAllImagesSameNumMips = true;
			for (int32 Index = 0; Index < Images.Num(); ++Index)
			{
				int32 UDIMIndex = UDIMsAndSourcesFileArray[Index]->Key;
				int32 BlockX;
				int32 BlockY;
				UE::TextureUtilitiesCommon::ExtractUDIMCoordinates(UDIMIndex, BlockX, BlockY);

				if ( ! BlockedImage.InitBlockFromImage(BlockX, BlockY, Images[Index]) )
				{
					UE_LOGF(LogInterchangeImport, Warning, "UDIM InitBlockFromImage failed." );
					return { };
				}

				if ( Images[Index].NumMips != Images[0].NumMips )
				{
					bAllImagesSameNumMips = false;
				}
			}

			if (! BlockedImage.MigrateDataFromImagesToRawData(Images) )
			{
				UE_LOGF(LogInterchangeImport, Warning, "UDIM MigrateDataFromImagesToRawData failed." );
				return { };
			}
			
			// BlockedImage.MipGenSettings is note set from Images[].MipGenSettings
			//	instead see if all UDIM blocks have existing mips, and if so set LeaveExisting
			if ( Images[0].NumMips != 1 && bAllImagesSameNumMips )
			{
				BlockedImage.MipGenSettings = TextureMipGenSettings::TMGS_LeaveExistingMips;
			}

			return BlockedImage;
		}
		else
		{
			return { };
		}
	}

	FTexturePayloadVariant GetTexturePayload(const UInterchangeSourceData* SourceData, const FString& PayloadKey, const FTextureNodeVariant& TextureNodeVariant, const FTextureFactoryNodeVariant& FactoryNodeVariant, const UInterchangeTranslatorBase* Translator, TOptional<FString>& AlternateTexturePath)
	{
		static_assert(TVariantSize_V<FTextureFactoryNodeVariant> == 7, "Please update the code below and this assert to reflect the change to the variant type.");

		static_assert(TVariantSize_V<FTexturePayloadVariant> == 5, "Please update the code below and this assert to reflect the change to the variant type.");

		static_assert(TVariantSize_V<FTextureNodeVariant> == 7, "Please update the code below and this assert to reflect the change to the variant type.");

		// Standard texture 2D payload
		if (const UInterchangeTexture2DNode* const* TextureNode =  TextureNodeVariant.TryGet<const UInterchangeTexture2DNode*>())
		{
			TMap<int32, FString> BlockAndSourceDataFiles;
			bool bShoudImportCompressedImage = false;
			if (const UInterchangeTexture2DFactoryNode* const* Texture2DFactoryNode = FactoryNodeVariant.TryGet<UInterchangeTexture2DFactoryNode*>())
			{
				BlockAndSourceDataFiles = (*Texture2DFactoryNode)->GetSourceBlocks();
			}

			if (const UInterchangeTextureFactoryNode* TextureFactoryNode = GetTextureFactoryNodeFromVariant(FactoryNodeVariant))
			{
				TextureFactoryNode->GetCustomPreferCompressedSourceData(bShoudImportCompressedImage);
			}

			const IInterchangeTexturePayloadInterface* TextureTranslator = Cast<IInterchangeTexturePayloadInterface>(Translator);
			const IInterchangeBlockedTexturePayloadInterface* BlockedTextureTranslator = Cast<IInterchangeBlockedTexturePayloadInterface>(Translator);

			// If the translator implements both interfaces we need to decide which one to invoke
			const bool bInvokeBlockedInterface = (BlockedTextureTranslator && !TextureTranslator)
										   || (BlockedTextureTranslator && TextureTranslator && !BlockAndSourceDataFiles.IsEmpty());

			if (TextureTranslator && !bInvokeBlockedInterface)
			{
				if (BlockAndSourceDataFiles.IsEmpty())
				{
					if (Translator->GetClass() == UInterchangeJPGTranslator::StaticClass()
						|| Translator->GetClass() == UInterchangeUEJPEGTranslator::StaticClass())
					{
						// Honor setting from TextureImporter.RetainJpegFormat in Editor.ini if it exists (ideally we should deprecate this as it is confusing and probably not thread safe)
						const bool bShouldImportRawCache = bShoudImportCompressedImage;
						if (GConfig->GetBool(TEXT("TextureImporter"), TEXT("RetainJpegFormat"), bShoudImportCompressedImage, GEditorIni) && bShouldImportRawCache != bShoudImportCompressedImage)
						{
							UE_LOGF(LogInterchangeImport, Log, "JPEG file [%ls]: Pipeline setting 'bPreferCompressedSourceData' has been overridden by Editor setting 'RetainJpegFormat'.", *SourceData->GetFilename());
						}
					}

					if (bShoudImportCompressedImage && TextureTranslator->SupportCompressedTexturePayloadData())
					{
						return FTexturePayloadVariant(TInPlaceType<TOptional<FImportImage>>(), TextureTranslator->GetCompressedTexturePayloadData(PayloadKey, AlternateTexturePath));
					}
					else
					{
						return FTexturePayloadVariant(TInPlaceType<TOptional<FImportImage>>(), TextureTranslator->GetTexturePayloadData(PayloadKey, AlternateTexturePath));
					}
				}
				else
				{
					return FTexturePayloadVariant(TInPlaceType<TOptional<FImportBlockedImage>>(), GetBlockedTexturePayloadDataFromSourceFiles(SourceData, BlockAndSourceDataFiles, Translator));
				}
			}
			else if (bInvokeBlockedInterface)
			{
				return FTexturePayloadVariant(TInPlaceType<TOptional<FImportBlockedImage>>(), BlockedTextureTranslator->GetBlockedTexturePayloadData(PayloadKey, AlternateTexturePath));
			}
		}

		// Cube, array, cube array or volume texture payload 
		if (TextureNodeVariant.IsType<const UInterchangeTextureCubeNode*>() || TextureNodeVariant.IsType<const UInterchangeTexture2DArrayNode*>()
			|| TextureNodeVariant.IsType<const UInterchangeTextureCubeArrayNode*>() || TextureNodeVariant.IsType<const UInterchangeVolumeTextureNode*>())
		{
			if (const IInterchangeSlicedTexturePayloadInterface* SlicedTextureTranslator = Cast<IInterchangeSlicedTexturePayloadInterface>(Translator))
			{
				return FTexturePayloadVariant(TInPlaceType<TOptional<FImportSlicedImage>>(), SlicedTextureTranslator->GetSlicedTexturePayloadData(PayloadKey, AlternateTexturePath));
			}
		}

		// Light Profile
		if (TextureNodeVariant.IsType<const UInterchangeTextureLightProfileNode*>())
		{
			if (const IInterchangeTextureLightProfilePayloadInterface* LightProfileTranslator = Cast<IInterchangeTextureLightProfilePayloadInterface>(Translator))
			{
				return FTexturePayloadVariant(TInPlaceType<TOptional<FImportLightProfile>>(), LightProfileTranslator->GetLightProfilePayloadData(PayloadKey, AlternateTexturePath));
			}
		}

		return {};
	}

	void SetupTextureSourceDataFromBulkData(UTexture* Texture, const FImportImage& Image, UE::Serialization::FEditorBulkData::FSharedBufferWithID&& BufferAndId, bool bIsReimport);

#if WITH_EDITOR
	
	// "Nearest" is in log scale (eg. in ratio sense) not linear value difference
	//	RoundToNearestPowerOfTwo32(0) == 1
	static inline uint64 RoundToNearestPowerOfTwo32(const uint32 x)
	{
		uint64 up = FMath::RoundUpToPowerOfTwo64(x);
		uint64 dn = up>>1;
	
		// input arg is 32 bit so squares fit in 64
		return ( up*dn <= x*x ) ? up : dn;
	}

	// Image and BufferAndId will be read from and mutated if a resize is done
	void ResizeImportImageIfNeeded(UTexture* Texture, FImportImage& Image, int64 NumSlices, UE::Serialization::FEditorBulkData::FSharedBufferWithID & BufferAndId)
	{
		// Image.RawData ignored; use BufferAndId instead for payload bytes
		//	NumSlices not changed by import (no volumes here)
		Image.RawData.Reset();

		if ( Texture->VirtualTextureStreaming )
		{
			// may already be set for a reimport
			return;
		}

		int64 LimitPixelCount = GetDefault<UTextureImportSettings>()->GetAutoLimitPixelCount();
		if ( LimitPixelCount == 0 )
		{
			return;
		}

		int64 InitialPixelCount = (int64) Image.SizeX * Image.SizeY * NumSlices;
		if ( InitialPixelCount <= LimitPixelCount )
		{
			return;
		}

		// ensure resizing to 1x1xNumSlices will get us under LimitPixelCount
		int64 MinLimitPixelCount = 16384;
		check( NumSlices <= MinLimitPixelCount );
		if ( LimitPixelCount < MinLimitPixelCount )
		{
			UE_LOGF(LogInterchangeImport, Warning, "Tiny LimitPixelCount %lld changed to %lld .", LimitPixelCount, MinLimitPixelCount );
			LimitPixelCount = MinLimitPixelCount;
		}

		if ( Texture->GetTextureClass() == ETextureClass::TwoD && NumSlices == 1 && GetDefault<UTextureImportSettings>()->IsImportAutoVTEnabled() )
		{
			// see ShouldTextureBeVirtualByAutoImportSize
			Texture->VirtualTextureStreaming = true;
			return;
		}

		// texture is not VT and exceeds pixel count limit, do resize
		
		const int64 BytesPerPixel = FTextureSource::GetBytesPerPixel(Image.Format);

		if ( Image.NumMips > 1 )
		{
			// for existing-mips, handle by stepping down the existing mips first
			//	existing mips must either be full mip chains or just one
			
			check( Image.RawDataCompressionFormat == TSCF_None );

			int64 MipIndex = 0;
			int64 NewSizeX = Image.SizeX;
			int64 NewSizeY = Image.SizeY;
			int64 BytesDiscarded = 0;
			while( NewSizeX * NewSizeY * NumSlices > LimitPixelCount )
			{
				BytesDiscarded += NewSizeX * NewSizeY * NumSlices * BytesPerPixel;

				MipIndex ++;				
				NewSizeX = FMath::Max<int64>(NewSizeX>>1,1);
				NewSizeY = FMath::Max<int64>(NewSizeY>>1,1);
			}
			
			FSharedBuffer SrcBuffer = BufferAndId.GetPayload();
			FMemoryView SrcView = SrcBuffer.GetView();

			check( SrcView.GetSize() == Image.ComputeBufferSize() );

			if ( MipIndex < Image.NumMips )
			{
				// just truncate the large mips off the front of the buffer
				
				UE_LOG(LogInterchangeImport, Display, TEXT("Image %s larger than GetAutoLimitPixelCount.  Dropping existing mips: %d x %d -> %" INT64_FMT " x %" INT64_FMT " (%" INT64_FMT " slices)"),
					*Texture->GetName(),
					Image.SizeX,Image.SizeY,
					NewSizeX,NewSizeY,
					NumSlices);

				Image.SizeX = NewSizeX;
				Image.SizeY = NewSizeY;
				Image.NumMips -= MipIndex;

				check( (int64)SrcView.GetSize() > BytesDiscarded );
				SrcView = SrcView.RightChop( BytesDiscarded );
				
				check( SrcView.GetSize() == Image.ComputeBufferSize() );

				BufferAndId = UE::Serialization::FEditorBulkData::FSharedBufferWithID( FSharedBuffer::Clone(SrcView) );

				// done just by stepping down existing mips, no actual resizing needed
				return;
			}
			else
			{
				// didn't have enough mips to step down
				// just change to 1 mip only and resize from there
				
				UE_LOGF(LogInterchangeImport, Display, "Image %ls has existing mips but not a full set, discarding.",
					*Texture->GetName() );

				Image.NumMips = 1;
				SrcView = SrcView.LeftChop( Image.SizeX * Image.SizeY * NumSlices * BytesPerPixel );
				
				check( SrcView.GetSize() == Image.ComputeBufferSize() );

				BufferAndId = UE::Serialization::FEditorBulkData::FSharedBufferWithID( FSharedBuffer::Clone(SrcView) );

				// continue to resizing
			}
		}

		// since we are resizing anyway, may as well change to pow2 also
		// note that we change both dimensions to pow2
		//	 which can change the aspect ratio if the source was not square or pow2

		int64 LargerDim  = FMath::Max<int64>(Image.SizeX,Image.SizeY);
		int64 SmallerDim = FMath::Min<int64>(Image.SizeX,Image.SizeY);
		double OriginalDimRatio = (double) SmallerDim / LargerDim;

		LargerDim = FMath::RoundUpToPowerOfTwo64(LargerDim);

		for(;;)
		{
			LargerDim >>= 1;
			SmallerDim = RoundToNearestPowerOfTwo32( FMath::RoundToInt32( LargerDim * OriginalDimRatio) );

			if ( LargerDim * SmallerDim * NumSlices <= LimitPixelCount )
			{
				break;
			}
		}

		int64 NewSizeX,NewSizeY;
		if ( Image.SizeX >= Image.SizeY )
		{
			NewSizeX = LargerDim;
			NewSizeY = SmallerDim;
		}
		else
		{
			NewSizeX = SmallerDim;
			NewSizeY = LargerDim;
		}

		UE_LOGF(LogInterchangeImport, Display, "Image %ls larger than GetAutoLimitPixelCount.  Resizing %d x %d -> %lld x %lld (%lld slices)",
			*Texture->GetName(),
			Image.SizeX,Image.SizeY,
			NewSizeX,NewSizeY,
			NumSlices);

		check( Image.NumMips == 1 );
		
		int64 SrcBytesPerSlice = (int64) Image.SizeX * Image.SizeY * BytesPerPixel;
		int64 DstBytesPerSlice = (int64) NewSizeX * NewSizeY * BytesPerPixel;

		FUniqueBuffer DstBuffer = FUniqueBuffer::Alloc(DstBytesPerSlice * NumSlices);
		uint8 * DstBufferData = (uint8 *) DstBuffer.GetData();

		FSharedBuffer SrcBuffer = BufferAndId.GetPayload();
		
		const uint8 * SrcBufferData = (const uint8 *) SrcBuffer.GetData();
		int64 SrcBufferSize = SrcBuffer.GetSize();
		
		FImageView Src,Dst;

		Src.SizeX = Image.SizeX;
		Src.SizeY = Image.SizeY;
		Src.NumSlices = 1;
		Src.Format = FImageCoreUtils::ConvertToRawImageFormat(Image.Format);
		Src.GammaSpace = Image.bSRGB ? EGammaSpace::sRGB : EGammaSpace::Linear;

		Dst = Src;
		Dst.SizeX = NewSizeX;
		Dst.SizeY = NewSizeY;
		
		check( Src.GetImageSizeBytes() == SrcBytesPerSlice );
		check( Dst.GetImageSizeBytes() == DstBytesPerSlice );

		if ( Image.RawDataCompressionFormat != TSCF_None )
		{
			// only single 2d images can import compressed currently :
			check( NumSlices == 1 );

			// decompress image data
			FImage SrcImage;
			bool bOk = FImageUtils::DecompressImage(SrcBufferData,SrcBufferSize,SrcImage);
			if ( ! bOk )
			{
				UE_LOGF(LogInterchangeImport, Warning, "DecompressImage failed on [%ls].", *(Texture->GetName()) );
				return;
			}
			
			// read SrcImage , not "Src"

			Dst.RawData = DstBufferData;

			FImageCore::ResizeImage(SrcImage,Dst,FImageCore::EResizeImageFilter::Default);

			// change Image now holds uncompressed data:
			Image.RawDataCompressionFormat = TSCF_None;
		}
		else
		{
			// SrcBuffer contains raw images
			check( SrcBufferSize == SrcBytesPerSlice * NumSlices );
			if ( SrcBufferSize != SrcBytesPerSlice * NumSlices )
			{
				UE_LOGF(LogInterchangeImport, Warning, "Invalid SrcBufferSize on [%ls].", *(Texture->GetName()) );
				return;
			}

			// ResizeImage default edge mode is Clamp, so this is okay for cube map faces

			for(int64 SliceIndex=0;SliceIndex<NumSlices;SliceIndex++)
			{
				Src.RawData = (void *)( SrcBufferData + SrcBytesPerSlice * SliceIndex );
	
				Dst.RawData = DstBufferData + DstBytesPerSlice * SliceIndex;

				FImageCore::ResizeImage(Src,Dst,FImageCore::EResizeImageFilter::Default);
			}
		}

		SrcBuffer.Reset(); // release the FSharedBuffer ref

		// mutate Image:
		Image.SizeX = NewSizeX;
		Image.SizeY = NewSizeY;

		// mutate BufferAndId
		BufferAndId = UE::Serialization::FEditorBulkData::FSharedBufferWithID( DstBuffer.MoveToShared() );
	}

	void SetupTextureSourceDataFromBulkData_Editor(UTexture* Texture, FImportImage& Image, UE::Serialization::FEditorBulkData::FSharedBufferWithID&& BufferAndId, bool bIsReimport)
	{
		// Image has a .RawData but it is ignored here
		//	use BufferAndId for the pixel data instead

		ResizeImportImageIfNeeded(Texture,Image,1,BufferAndId);

		Texture->Source.InitWithCompressedSourceData(
			Image.SizeX,
			Image.SizeY,
			Image.NumMips,
			Image.Format,
			MoveTemp(BufferAndId),
			Image.RawDataCompressionFormat
		);

		if (!bIsReimport)
		{
			Texture->CompressionSettings = Image.CompressionSettings;
			
			Texture->SRGB = UE::TextureUtilitiesCommon::GetDefaultSRGB(Image.CompressionSettings,Image.Format,Image.bSRGB);

			//If the MipGenSettings was set by the translator, we must apply it before the build
			if (Image.MipGenSettings.IsSet())
			{
				// if the source has mips we keep the mips by default, unless the user changes that
				Texture->MipGenSettings = Image.MipGenSettings.GetValue();
			}
		}
	}

	void SetupTexture2DSourceDataFromBulkData_Editor(UTexture2D* Texture2D, const FImportBlockedImage& BlockedImage, UE::Serialization::FEditorBulkData::FSharedBufferWithID&& BufferAndId, bool bIsReimport)
	{
		if (BlockedImage.BlocksData.Num() > 1)
		{
			// note: ResizeImportImageIfNeeded not done
			//	no size limit for UDIM

			Texture2D->Source.InitBlocked(
				&BlockedImage.Format,
				BlockedImage.BlocksData.GetData(),
				/*InNumLayers=*/ 1,
				BlockedImage.BlocksData.Num(),
				MoveTemp(BufferAndId)
				);

			if (!bIsReimport)
			{
				Texture2D->CompressionSettings = BlockedImage.CompressionSettings;
				
				Texture2D->SRGB = UE::TextureUtilitiesCommon::GetDefaultSRGB(BlockedImage.CompressionSettings,BlockedImage.Format,BlockedImage.bSRGB);

				// UDIM/Blocked must be VT :
				Texture2D->VirtualTextureStreaming = true;

				if (BlockedImage.MipGenSettings.IsSet())
				{
					// if the source has mips we keep the mips by default, unless the user changes that
					Texture2D->MipGenSettings = BlockedImage.MipGenSettings.GetValue();
				}
			}
		}
		else
		{
			// only 1 image in UDIM set found
			//Import as a normal texture (not VT/not UDIM)
			FImportImage Image;
			Image.Format = BlockedImage.Format;
			Image.CompressionSettings = BlockedImage.CompressionSettings;
			Image.bSRGB = BlockedImage.bSRGB;
			Image.MipGenSettings = BlockedImage.MipGenSettings;

			const FTextureSourceBlock& Block = BlockedImage.BlocksData[0];
			Image.SizeX = Block.SizeX;
			Image.SizeY = Block.SizeY;
			Image.NumMips = Block.NumMips;
			// note any UDIM block X,Y offset is discarded

			SetupTextureSourceDataFromBulkData_Editor(Texture2D, Image, MoveTemp(BufferAndId), bIsReimport);
		}
	}

	void SetupTextureSourceDataFromBulkData_Editor(UTexture* Texture, FImportSlicedImage& SlicedImage, UE::Serialization::FEditorBulkData::FSharedBufferWithID&& BufferAndId,  bool bIsReimport)
	{
		bool bTextureIsVolume = Texture->GetTextureClass() == ETextureClass::Volume;
		if ( SlicedImage.bIsVolume )
		{
			check( bTextureIsVolume );

			// @@ no Volume resize currently
		}
		else
		{
			// NumSlice not changed by resize
			// FImportSlicedImage passed as FImportImage
			ResizeImportImageIfNeeded(Texture,SlicedImage,SlicedImage.NumSlice,BufferAndId);
		}

		Texture->Source.InitLayered(
			SlicedImage.SizeX,
			SlicedImage.SizeY,
			SlicedImage.NumSlice,
			1, // NumLayers
			SlicedImage.NumMips,
			&SlicedImage.Format,
			MoveTemp(BufferAndId)
			);

		if (!bIsReimport)
		{
			Texture->CompressionSettings = SlicedImage.CompressionSettings;
			
			Texture->SRGB = UE::TextureUtilitiesCommon::GetDefaultSRGB(SlicedImage.CompressionSettings,SlicedImage.Format,SlicedImage.bSRGB);

			if (SlicedImage.MipGenSettings.IsSet())
			{
				// if the source has mips we keep the mips by default, unless the user changes that
				Texture->MipGenSettings = SlicedImage.MipGenSettings.GetValue();
			}
		}
	}

	bool CanSetupTextureCubeSourceData(FTexturePayloadVariant& TexturePayload)
	{
		static_assert(TVariantSize_V<FTexturePayloadVariant> == 5, "Please update the code below and this assert to reflect the change to the variant type.");

		if (TOptional<FImportSlicedImage>* SlicedImage = TexturePayload.TryGet<TOptional<FImportSlicedImage>>())
		{
			if (SlicedImage->IsSet())
			{
				return (*SlicedImage)->IsValid() && (*SlicedImage)->NumSlice == 6;
			}
		}
		else if (TOptional<FImportImage>* Image = TexturePayload.TryGet<TOptional<FImportImage>>())
		{
			if (Image->IsSet())
			{
				return (*Image)->IsValid();
			}
		}
		else if (TOptional<FImportLightProfile>* LightProfile = TexturePayload.TryGet<TOptional<FImportLightProfile>>())
		{
			if (LightProfile->IsSet())
			{
				return (*LightProfile)->IsValid();
			}
		}

		return false;
	}

	void SetupTextureCubeSourceDataFromPayload(UTextureCube* TextureCube, FProcessedPayload& ProcessedPayload, bool bIsReimport)
	{
		if (TOptional<FImportSlicedImage>* OptionalSlicedImage = ProcessedPayload.SettingsFromPayload.TryGet<TOptional<FImportSlicedImage>>())
		{
			if (OptionalSlicedImage->IsSet())
			{
				FImportSlicedImage& SlicedImage = OptionalSlicedImage->GetValue();

				// Cube texture always have six slice
				if (SlicedImage.NumSlice == 6)
				{
					SetupTextureSourceDataFromBulkData_Editor(TextureCube, SlicedImage, MoveTemp(ProcessedPayload.PayloadAndId), bIsReimport);
				}
			}
		}
		else if (TOptional<FImportImage>* OptionalImage = ProcessedPayload.SettingsFromPayload.TryGet<TOptional<FImportImage>>())
		{
			if (OptionalImage->IsSet())
			{
				FImportImage& Image = OptionalImage->GetValue();
				SetupTextureSourceDataFromBulkData_Editor(TextureCube, Image, MoveTemp(ProcessedPayload.PayloadAndId), bIsReimport);
			}
		}
		else if (TOptional<FImportLightProfile>* OptionalLightProfile = ProcessedPayload.SettingsFromPayload.TryGet<TOptional<FImportLightProfile>>())
		{
			if (OptionalLightProfile->IsSet())
			{
				FImportLightProfile& LightProfile = OptionalLightProfile->GetValue();
				SetupTextureSourceDataFromBulkData_Editor(TextureCube, LightProfile, MoveTemp(ProcessedPayload.PayloadAndId), bIsReimport);
			}
		}
		else
		{
			// The TexturePayload should be validated before calling this function
			checkNoEntry();
		}
	}

	bool CanSetupTextureCubeArraySourceData(FTexturePayloadVariant& TexturePayload)
	{
		static_assert(TVariantSize_V<FTexturePayloadVariant> == 5, "Please update the code below and this assert to reflect the change to the variant type.");

		if (TOptional<FImportSlicedImage>* SlicedImage = TexturePayload.TryGet<TOptional<FImportSlicedImage>>())
		{
			if (SlicedImage->IsSet())
			{
				return (*SlicedImage)->IsValid() && (*SlicedImage)->NumSlice > 6;
			}
		}
		else if (TOptional<FImportImage>* Image = TexturePayload.TryGet<TOptional<FImportImage>>())
		{
			if (Image->IsSet())
			{
				return (*Image)->IsValid();
			}
		}
		else if (TOptional<FImportLightProfile>* LightProfile = TexturePayload.TryGet<TOptional<FImportLightProfile>>())
		{
			if (LightProfile->IsSet())
			{
				return (*LightProfile)->IsValid();
			}
		}

		return false;
	}

	void SetupTextureCubeArraySourceDataFromPayload(UTextureCubeArray* TextureCubeArray, FProcessedPayload& ProcessedPayload, bool bIsReimport)
	{
		if (TOptional<FImportSlicedImage>* OptionalSlicedImage = ProcessedPayload.SettingsFromPayload.TryGet<TOptional<FImportSlicedImage>>())
		{
			if (OptionalSlicedImage->IsSet())
			{
				FImportSlicedImage& SlicedImage = OptionalSlicedImage->GetValue();

				if (SlicedImage.NumSlice > 6)
				{
					SetupTextureSourceDataFromBulkData_Editor(TextureCubeArray, SlicedImage, MoveTemp(ProcessedPayload.PayloadAndId), bIsReimport);
				}
			}
		}
		else if (TOptional<FImportImage>* OptionalImage = ProcessedPayload.SettingsFromPayload.TryGet<TOptional<FImportImage>>())
		{
			if (OptionalImage->IsSet())
			{
				FImportImage& Image = OptionalImage->GetValue();
				SetupTextureSourceDataFromBulkData_Editor(TextureCubeArray, Image, MoveTemp(ProcessedPayload.PayloadAndId), bIsReimport);
			}
		}
		else if (TOptional<FImportLightProfile>* OptionalLightProfile = ProcessedPayload.SettingsFromPayload.TryGet<TOptional<FImportLightProfile>>())
		{
			if (OptionalLightProfile->IsSet())
			{
				FImportLightProfile& LightProfile = OptionalLightProfile->GetValue();
				SetupTextureSourceDataFromBulkData_Editor(TextureCubeArray, LightProfile, MoveTemp(ProcessedPayload.PayloadAndId), bIsReimport);
			}
		}
		else
		{
			// The TexturePayload should be validated before calling this function
			checkNoEntry();
		}
	}

	bool CanSetupTexture2DArraySourceData(FTexturePayloadVariant& TexturePayload)
	{
		static_assert(TVariantSize_V<FTexturePayloadVariant> == 5, "Please update the code below and this assert to reflect the change to the variant type.");

		if (TOptional<FImportSlicedImage>* SlicedImage = TexturePayload.TryGet<TOptional<FImportSlicedImage>>())
		{
			if (SlicedImage->IsSet())
			{
				return (*SlicedImage)->IsValid();
			}
		}
		else if (TOptional<FImportImage>* Image = TexturePayload.TryGet<TOptional<FImportImage>>())
		{
			if (Image->IsSet())
			{
				return (*Image)->IsValid();
			}
		}
		else if (TOptional<FImportLightProfile>* LightProfile = TexturePayload.TryGet<TOptional<FImportLightProfile>>())
		{
			if (LightProfile->IsSet())
			{
				return (*LightProfile)->IsValid();
			}
		}

		return false;
	}

	void SetupTexture2DArraySourceDataFromPayload(UTexture2DArray* Texture2DArray, FProcessedPayload& ProcessedPayload, bool bIsReimport)
	{
		if (TOptional<FImportSlicedImage>* OptionalSlicedImage = ProcessedPayload.SettingsFromPayload.TryGet<TOptional<FImportSlicedImage>>())
		{
			if (OptionalSlicedImage->IsSet())
			{
				FImportSlicedImage& SlicedImage = OptionalSlicedImage->GetValue();

				SetupTextureSourceDataFromBulkData_Editor(Texture2DArray, SlicedImage, MoveTemp(ProcessedPayload.PayloadAndId), bIsReimport);
			}
		}
		else if (TOptional<FImportImage>* OptionalImage = ProcessedPayload.SettingsFromPayload.TryGet<TOptional<FImportImage>>())
		{
			if (OptionalImage->IsSet())
			{
				FImportImage& Image = OptionalImage->GetValue();
				SetupTextureSourceDataFromBulkData_Editor(Texture2DArray, Image, MoveTemp(ProcessedPayload.PayloadAndId), bIsReimport);
			}
		}
		else if (TOptional<FImportLightProfile>* OptionalLightProfile = ProcessedPayload.SettingsFromPayload.TryGet<TOptional<FImportLightProfile>>())
		{
			if (OptionalLightProfile->IsSet())
			{
				FImportLightProfile& LightProfile = OptionalLightProfile->GetValue();
				SetupTextureSourceDataFromBulkData_Editor(Texture2DArray, LightProfile, MoveTemp(ProcessedPayload.PayloadAndId), bIsReimport);
			}
		}
		else
		{
			// The TexturePayload should be validated before calling this function
			checkNoEntry();
		}
	}

	bool CanSetupVolumeTextureSourceData(FTexturePayloadVariant& TexturePayload)
	{
		static_assert(TVariantSize_V<FTexturePayloadVariant> == 5, "Please update the code below and this assert to reflect the change to the variant type.");

		if (TOptional<FImportSlicedImage>* SlicedImage = TexturePayload.TryGet<TOptional<FImportSlicedImage>>())
		{
			if (SlicedImage->IsSet())
			{
				return (*SlicedImage)->IsValid();
			}
		}
		else if (TOptional<FImportImage>* Image = TexturePayload.TryGet<TOptional<FImportImage>>())
		{
			if (Image->IsSet())
			{
				return (*Image)->IsValid();
			}
		}
		else if (TOptional<FImportLightProfile>* LightProfile = TexturePayload.TryGet<TOptional<FImportLightProfile>>())
		{
			if (LightProfile->IsSet())
			{
				return (*LightProfile)->IsValid();
			}
		}

		return false;
	}

	void SetupVolumeTextureSourceDataFromPayload(UVolumeTexture* VolumeTexture, FProcessedPayload& ProcessedPayload, bool bIsReimport)
	{
		if (TOptional<FImportSlicedImage>* OptionalSlicedImage = ProcessedPayload.SettingsFromPayload.TryGet<TOptional<FImportSlicedImage>>())
		{
			if (OptionalSlicedImage->IsSet())
			{
				FImportSlicedImage& SlicedImage = OptionalSlicedImage->GetValue();

				SetupTextureSourceDataFromBulkData_Editor(VolumeTexture, SlicedImage, MoveTemp(ProcessedPayload.PayloadAndId), bIsReimport);
			}
		}
		else if (TOptional<FImportImage>* OptionalImage = ProcessedPayload.SettingsFromPayload.TryGet<TOptional<FImportImage>>())
		{
			if (OptionalImage->IsSet())
			{
				FImportImage& Image = OptionalImage->GetValue();
				SetupTextureSourceDataFromBulkData_Editor(VolumeTexture, Image, MoveTemp(ProcessedPayload.PayloadAndId), bIsReimport);
			}
		}
		else if (TOptional<FImportLightProfile>* OptionalLightProfile = ProcessedPayload.SettingsFromPayload.TryGet<TOptional<FImportLightProfile>>())
		{
			if (OptionalLightProfile->IsSet())
			{
				FImportLightProfile& LightProfile = OptionalLightProfile->GetValue();
				SetupTextureSourceDataFromBulkData_Editor(VolumeTexture, LightProfile, MoveTemp(ProcessedPayload.PayloadAndId), bIsReimport);
			}
		}
		else
		{
			// The TexturePayload should be validated before calling this function
			checkNoEntry();
		}
	}

	FGraphEventArray GenerateHashSourceFilesTasks(const UInterchangeSourceData* SourceData, TArray<FString>&& FilesToHash, TArray<FAssetImportInfo::FSourceFile>& OutSourceFiles)
	{
		// This hashing is quite slow (MD5 single threaded)
		// it is needed for the auto-reimport feature that detects changed files by comparing their hashes

		struct FHashSourceTaskBase
		{
			/**
				* Returns the name of the thread that this task should run on.
				*
				* @return Always run on any thread.
				*/
			ENamedThreads::Type GetDesiredThread()
			{
				return ENamedThreads::AnyBackgroundThreadNormalTask;
			}

			/**
				* Gets the task's stats tracking identifier.
				*
				* @return Stats identifier.
				*/
			TStatId GetStatId() const
			{
				return GET_STATID(STAT_TaskGraph_OtherTasks);
			}

			/**
				* Gets the mode for tracking subsequent tasks.
				*
				* @return Always track subsequent tasks.
				*/
			static ESubsequentsMode::Type GetSubsequentsMode()
			{
				return ESubsequentsMode::TrackSubsequents;
			}
		};

		FGraphEventArray TasksToDo;

		// We do the hashing of the source files after the import to avoid a bigger memory overhead.
		if (FilesToHash.IsEmpty())
		{
			struct FHashSingleSource : public FHashSourceTaskBase
			{
				FHashSingleSource(const UInterchangeSourceData* InSourceData)
					: SourceData(InSourceData)
				{}

				/**
				 * Performs the actual task.
				 *
				 * @param CurrentThread The thread that this task is executing on.
				 * @param MyCompletionGraphEvent The completion event.
				 */
				void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
				{
					if (SourceData)
					{ 
						//Getting the file Hash will cache it into the source data
						SourceData->GetFileContentHash();
					}
				}

			private:
				const UInterchangeSourceData* SourceData = nullptr;
			};
		
			TasksToDo.Add(TGraphTask<FHashSingleSource>::CreateTask().ConstructAndDispatchWhenReady(SourceData));
		}
		else
		{
			struct FHashMutipleSource : public FHashSourceTaskBase
			{
				FHashMutipleSource(FString&& InFileToHash, FAssetImportInfo::FSourceFile& OutSourceFile)
					: FileToHash(MoveTemp(InFileToHash))
					, SourceFile(OutSourceFile)
				{}

				/**
				 * Performs the actual task.
				 *
				 * @param CurrentThread The thread that this task is executing on.
				 * @param MyCompletionGraphEvent The completion event.
				 */
				void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
				{
					SourceFile.FileHash = FMD5Hash::HashFile(*FileToHash);
					SourceFile.Timestamp =IFileManager::Get().GetTimeStamp(*FileToHash);
					SourceFile.RelativeFilename = MoveTemp(FileToHash);
				}

			private:
				FString FileToHash;
				FAssetImportInfo::FSourceFile& SourceFile;
			};

			OutSourceFiles.AddDefaulted(FilesToHash.Num());

			for (int32 Index = 0; Index < FilesToHash.Num(); ++Index)
			{
				TasksToDo.Add(TGraphTask<FHashMutipleSource>::CreateTask().ConstructAndDispatchWhenReady(MoveTemp(FilesToHash[Index]), OutSourceFiles[Index]));
			}
		}

		return TasksToDo;
	}
#else // !WITH_EDITOR
	void SetupTexture2DSourceDataFromBulkData_Runtime(UTexture2D* Texture2D, const FImportImage& ImportImage, UE::Serialization::FEditorBulkData::FSharedBufferWithID&& BufferAndId)
	{
		Texture2D->SRGB = UE::TextureUtilitiesCommon::GetDefaultSRGB(Texture2D->CompressionSettings,ImportImage.Format,ImportImage.bSRGB);
		
		ERawImageFormat::Type SourcePixelRawFormat = FImageCoreUtils::ConvertToRawImageFormat(ImportImage.Format);

		ERawImageFormat::Type PixelFormatRawFormat;
		const EPixelFormat PixelFormat = FImageCoreUtils::GetPixelFormatForRawImageFormat(SourcePixelRawFormat, &PixelFormatRawFormat);

		UE::Serialization::FEditorBulkData BulkData;
		BulkData.UpdatePayload(MoveTemp(BufferAndId));
		TFuture<FSharedBuffer> Payload = BulkData.GetPayload();

		const EGammaSpace SourceGammaSpace = ImportImage.bSRGB ? EGammaSpace::sRGB : EGammaSpace::Linear;
		constexpr int32 NumSlices = 1;
		FImageView SourceImageView(const_cast<void*>(Payload.Get().GetData()), ImportImage.SizeX, ImportImage.SizeY, NumSlices, SourcePixelRawFormat, SourceGammaSpace);

		FImage DecompressedSourceImage;
		if (ImportImage.RawDataCompressionFormat != TSCF_None)
		{
			if (FImageUtils::DecompressImage(Payload.Get().GetData(), Payload.Get().GetSize(), DecompressedSourceImage))
			{
				SourceImageView = DecompressedSourceImage;
			}
		}

		{
			Texture2D->SetPlatformData(new FTexturePlatformData());
			Texture2D->GetPlatformData()->SizeX = ImportImage.SizeX;
			Texture2D->GetPlatformData()->SizeY = ImportImage.SizeY;
			Texture2D->GetPlatformData()->PixelFormat = PixelFormat;

			// Allocate first mipmap.
			int32 NumBlocksX = ImportImage.SizeX / GPixelFormats[PixelFormat].BlockSizeX;
			int32 NumBlocksY = ImportImage.SizeY / GPixelFormats[PixelFormat].BlockSizeY;
			FTexture2DMipMap* Mip = new FTexture2DMipMap(ImportImage.SizeX, ImportImage.SizeY);
			Texture2D->GetPlatformData()->Mips.Add(Mip);
			Mip->BulkData.Lock(LOCK_READ_WRITE);
			Mip->BulkData.Realloc(NumBlocksX * NumBlocksY * GPixelFormats[PixelFormat].BlockBytes);
			Mip->BulkData.Unlock();
		}

		{

			Texture2D->bNotOfflineProcessed = true;

			EGammaSpace MipGammaSpace;
			if ( Texture2D->SRGB && ERawImageFormat::GetFormatNeedsGammaSpace(PixelFormatRawFormat) )
			{
				MipGammaSpace = EGammaSpace::sRGB;
			}
			else
			{
				MipGammaSpace = EGammaSpace::Linear;
			}

			uint8* MipData = static_cast<uint8*>(Texture2D->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE));
			check(MipData != nullptr);
			int64 MipDataSize = Texture2D->GetPlatformData()->Mips[0].BulkData.GetBulkDataSize();

			FImageView MipImageView(MipData, ImportImage.SizeX, ImportImage.SizeY, NumSlices, PixelFormatRawFormat, MipGammaSpace);

			// copy into texture and convert if necessary :
			FImageCore::CopyImage(SourceImageView, MipImageView);

			Texture2D->GetPlatformData()->Mips[0].BulkData.Unlock();

			Texture2D->UpdateResource();
		}
	}
#endif // !WITH_EDITOR

	FSharedBuffer MoveRawDataToSharedBuffer(FTexturePayloadVariant& TexturePayload)
	{
		static_assert(TVariantSize_V<FTexturePayloadVariant> == 5, "Please update the code below and this assert to reflect the change to the variant type.");

		if (TOptional<FImportBlockedImage>* BlockedImage = TexturePayload.TryGet<TOptional<FImportBlockedImage>>())
		{
			return (*BlockedImage)->RawData.MoveToShared();
		}
		else if (TOptional<FImportImage>* Image = TexturePayload.TryGet<TOptional<FImportImage>>())
		{
			return (*Image)->RawData.MoveToShared();
		}
		else if (TOptional<FImportSlicedImage>* SlicedImage = TexturePayload.TryGet<TOptional<FImportSlicedImage>>())
		{
			return (*SlicedImage)->RawData.MoveToShared();
		}
		else if (TOptional<FImportLightProfile>* LightProfile = TexturePayload.TryGet<TOptional<FImportLightProfile>>())
		{
			return (*LightProfile)->RawData.MoveToShared();
		}

		// The TexturePayload should be validated before calling this function
		checkNoEntry();
		return {};
	}

	FProcessedPayload& FProcessedPayload::operator=(UE::Interchange::Private::InterchangeTextureFactory::FTexturePayloadVariant&& InPayloadVariant)
	{
		SettingsFromPayload = MoveTemp(InPayloadVariant);
		PayloadAndId = MoveRawDataToSharedBuffer(SettingsFromPayload);

		return *this;
	}

	bool FProcessedPayload::IsValid() const
	{
		if (SettingsFromPayload.IsType<FEmptyVariantState>())
		{
			return false;
		}

		return true;
	}


	TArray<FString> GetFilesToHash(const FTextureFactoryNodeVariant& TextureFactoryNodeVariant, const FTexturePayloadVariant& TexturePayload)
	{
		TArray<FString> FilesToHash;
		// Standard texture 2D payload
		if (const UInterchangeTexture2DFactoryNode* const* TextureNode = TextureFactoryNodeVariant.TryGet<UInterchangeTexture2DFactoryNode*>())
		{
			using namespace UE::Interchange;
			if (const TOptional<FImportBlockedImage>* OptionalBlockedPayload = TexturePayload.TryGet<TOptional<FImportBlockedImage>>())
			{
				if (OptionalBlockedPayload->IsSet())
				{
					const FImportBlockedImage& BlockImage = OptionalBlockedPayload->GetValue();
					TMap<int32, FString> BlockAndFiles = (*TextureNode)->GetSourceBlocks();
					FilesToHash.Reserve(BlockAndFiles.Num());
					for (const FTextureSourceBlock& BlockData : BlockImage.BlocksData)
					{
						if (FString* FilePath = BlockAndFiles.Find(UE::TextureUtilitiesCommon::GetUDIMIndex(BlockData.BlockX, BlockData.BlockY)))
						{
							FilesToHash.Add(*FilePath);
						}
					}
				}

			}
		}
		return FilesToHash;
	}

	void LogErrorInvalidPayload(UInterchangeFactoryBase& Factory, const UClass* TextureClass, const FString& SourceAssetName, const FString& DestinationAssetName)
	{
		UInterchangeResultError_Generic* ErrorMessage = Factory.AddMessage<UInterchangeResultError_Generic>();
		ErrorMessage->AssetType = TextureClass;
		ErrorMessage->SourceAssetName = SourceAssetName;
		ErrorMessage->DestinationAssetName = DestinationAssetName;
		ErrorMessage->Text = LOCTEXT("InvalidPayload", "Unable to retrieve the payload from the source file.");
	}

	bool IsTextureOptionalPayloadSet(const FTexturePayloadVariant& TexturePayloadVarient)
	{
		if (TexturePayloadVarient.IsType<FEmptyVariantState>())
		{
			return false;
		}

		if (TexturePayloadVarient.IsType<TOptional<FImportImage>>())
		{
			if (const TOptional<FImportImage>* Payload = TexturePayloadVarient.TryGet<TOptional<FImportImage>>())
			{
				return Payload->IsSet();
			}
		}

		if (TexturePayloadVarient.IsType<TOptional<FImportBlockedImage>>())
		{
			if (const TOptional<FImportBlockedImage>* Payload = TexturePayloadVarient.TryGet<TOptional<FImportBlockedImage>>())
			{
				return Payload->IsSet();
			}
		}

		if (TexturePayloadVarient.IsType<TOptional<FImportSlicedImage>>())
		{
			if (const TOptional<FImportSlicedImage>* Payload = TexturePayloadVarient.TryGet<TOptional<FImportSlicedImage>>())
			{
				return Payload->IsSet();
			}
		}

		if (TexturePayloadVarient.IsType<TOptional<FImportLightProfile>>())
		{
			if (const TOptional<FImportLightProfile>* Payload = TexturePayloadVarient.TryGet<TOptional<FImportLightProfile>>())
			{
				return Payload->IsSet();
			}
		}

		return false;
	}

	bool CanSetupTexture2DSourceData(FTexturePayloadVariant& TexturePayload)
	{
		static_assert(TVariantSize_V<FTexturePayloadVariant> == 5, "Please update the code below and this assert to reflect the change to the variant type.");

#if WITH_EDITOR
		if (TOptional<FImportBlockedImage>* BlockedImage = TexturePayload.TryGet<TOptional<FImportBlockedImage>>())
		{
			if (BlockedImage->IsSet())
			{
				return (*BlockedImage)->IsValid();
			}
		}
		else
#endif // WITH_EDITOR
		if (TOptional<FImportImage>* Image = TexturePayload.TryGet<TOptional<FImportImage>>())
		{
			if (Image->IsSet())
			{
				return (*Image)->IsValid();
			}
		}
		else if (TOptional<FImportLightProfile>* LightProfile = TexturePayload.TryGet<TOptional<FImportLightProfile>>())
		{
			if (LightProfile->IsSet())
			{
				return (*LightProfile)->IsValid();
			}
		}

		return false;
	}

	void SetupTextureSourceDataFromBulkData(UTexture* Texture, FImportImage& Image, UE::Serialization::FEditorBulkData::FSharedBufferWithID&& BufferAndId, bool bIsReimport)
	{
#if WITH_EDITOR
		SetupTextureSourceDataFromBulkData_Editor(Texture, Image, MoveTemp(BufferAndId), bIsReimport);
#else
		SetupTexture2DSourceDataFromBulkData_Runtime(Cast<UTexture2D>(Texture), Image, MoveTemp(BufferAndId));
#endif
	}

	void SetupTexture2DSourceDataFromPayload(UInterchangeFactoryBaseNode* FactorBaseNode, UTexture2D* Texture2D, FProcessedPayload& ProcessedPayload, bool bIsReimport)
	{
#if WITH_EDITOR
		if (TOptional<FImportBlockedImage>* BlockedImage = ProcessedPayload.SettingsFromPayload.TryGet<TOptional<FImportBlockedImage>>())
		{
			if (BlockedImage->IsSet())
			{
				SetupTexture2DSourceDataFromBulkData_Editor(Texture2D, BlockedImage->GetValue(), MoveTemp(ProcessedPayload.PayloadAndId), bIsReimport);
			}
		}
		else
#endif // WITH_EDITOR
		if (TOptional<FImportImage>* Image = ProcessedPayload.SettingsFromPayload.TryGet<TOptional<FImportImage>>())
		{
			if (Image->IsSet())
			{
				SetupTextureSourceDataFromBulkData(Texture2D, Image->GetValue(), MoveTemp(ProcessedPayload.PayloadAndId), bIsReimport);
			}
		}
		else if (TOptional<FImportLightProfile>* OptionalLightProfile = ProcessedPayload.SettingsFromPayload.TryGet<TOptional<FImportLightProfile>>())
		{
			if (OptionalLightProfile->IsSet())
			{
				FImportLightProfile& LightProfile = OptionalLightProfile->GetValue();

				if (UTextureLightProfile* TextureLightProfile = Cast<UTextureLightProfile>(Texture2D))
				{
					SetupTextureSourceDataFromBulkData(TextureLightProfile, LightProfile, MoveTemp(ProcessedPayload.PayloadAndId), bIsReimport);

					const UE::Interchange::FImportLightProfile& ImportLightProfile = OptionalLightProfile->GetValue();

					// The legacy factory updated these setting on each reimport (leave it as it is for 5.1 but we should look into moving that decision into the pipelines)
					TextureLightProfile->Brightness = ImportLightProfile.Brightness;
					TextureLightProfile->TextureMultiplier = ImportLightProfile.TextureMultiplier;

					if (UInterchangeTextureLightProfileFactoryNode* TextureProfileFactoryNode = Cast<UInterchangeTextureLightProfileFactoryNode>(FactorBaseNode))
					{
						TextureProfileFactoryNode->StoreIESSourceFileContents(ImportLightProfile.SourceDataBuffer);
					}
				}
				else
				{
					SetupTextureSourceDataFromBulkData(Texture2D, LightProfile, MoveTemp(ProcessedPayload.PayloadAndId), bIsReimport);
				}
			}
		}
		else
		{
			// The TexturePayload should be validated before calling this function
			checkNoEntry();
		}

#if WITH_EDITORONLY_DATA
		// The texture has been imported and has no editor specific changes applied so we clear the painted flag.
		Texture2D->bHasBeenPaintedInEditor = false;

		if ( UE::TextureUtilitiesCommon::ShouldTextureBeVirtualByAutoImportSize(Texture2D) )
		{
			Texture2D->VirtualTextureStreaming = true;
		}
#endif // WITH_EDITORONLY_DATA
	}

	void SetupTextureSourceDataFromPayload(UInterchangeFactoryBaseNode* FactorBaseNode, UTexture* Texture, FProcessedPayload& ProcessedPayload, bool bIsReimport)
	{
		if (UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
		{
			SetupTexture2DSourceDataFromPayload(FactorBaseNode, Texture2D, ProcessedPayload, bIsReimport);
		}
#if WITH_EDITOR
		else if (UTextureCube* TextureCube = Cast<UTextureCube>(Texture))
		{
			SetupTextureCubeSourceDataFromPayload(TextureCube, ProcessedPayload, bIsReimport);
		}
		else if (UTextureCubeArray* TextureCubeArray = Cast<UTextureCubeArray>(Texture))
		{
			SetupTextureCubeArraySourceDataFromPayload(TextureCubeArray, ProcessedPayload, bIsReimport);
		}
		else if (UTexture2DArray* Texture2DArray = Cast<UTexture2DArray>(Texture))
		{
			SetupTexture2DArraySourceDataFromPayload(Texture2DArray, ProcessedPayload, bIsReimport);
		}
		else if (UVolumeTexture* VolumeTexture = Cast<UVolumeTexture>(Texture))
		{
			SetupVolumeTextureSourceDataFromPayload(VolumeTexture, ProcessedPayload, bIsReimport);
		}
#endif // WITH_EDITOR
		else
		{
			// This should never happen.
			ensure(false);
		}

		// at this point the texture is mostly set up
		// NormalMapIdentification is not run yet
	}

	void ExtractCreatorApplicationMetadata(const FTexturePayloadVariant& TexturePayloadVarient, FTextureCreatorApplicationMetadata& OutTextureCreatorApplicationMetadata)
	{
		if (TexturePayloadVarient.IsType<FEmptyVariantState>())
		{
			return;
		}

		if (TexturePayloadVarient.IsType<TOptional<FImportImage>>())
		{
			if (const TOptional<FImportImage>* Payload = TexturePayloadVarient.TryGet<TOptional<FImportImage>>(); Payload->IsSet())
			{
				const FImportImage& ImagePayload = Payload->GetValue();
				if (ImagePayload.TextureCreatorApplicationMetadata.IsSet())
				{
					OutTextureCreatorApplicationMetadata = ImagePayload.TextureCreatorApplicationMetadata.GetValue();
				}
			}
		}

		if (TexturePayloadVarient.IsType<TOptional<FImportBlockedImage>>())
		{
			if (const TOptional<FImportBlockedImage>* Payload = TexturePayloadVarient.TryGet<TOptional<FImportBlockedImage>>(); Payload->IsSet())
			{
				const FImportBlockedImage& ImagePayload = Payload->GetValue();
				if (ImagePayload.TextureCreatorApplicationMetadata.IsSet())
				{
					OutTextureCreatorApplicationMetadata = ImagePayload.TextureCreatorApplicationMetadata.GetValue();
				}
			}
		}

		if (TexturePayloadVarient.IsType<TOptional<FImportSlicedImage>>())
		{
			if (const TOptional<FImportSlicedImage>* Payload = TexturePayloadVarient.TryGet<TOptional<FImportSlicedImage>>(); Payload->IsSet())
			{
				const FImportSlicedImage& ImagePayload = Payload->GetValue();
				if (ImagePayload.TextureCreatorApplicationMetadata.IsSet())
				{
					OutTextureCreatorApplicationMetadata = ImagePayload.TextureCreatorApplicationMetadata.GetValue();
				}
			}
		}
	}
 }

UClass* UInterchangeTextureFactory::GetFactoryClass() const
{
	return UTexture::StaticClass();
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeTextureFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeTextureFactory::BeginImportAsset_GameThread);
	using namespace  UE::Interchange::Private::InterchangeTextureFactory;
	FImportAssetResult ImportAssetResult;
	UTexture* Texture = nullptr;

	auto CouldNotCreateTextureLog = [this, &Arguments, &ImportAssetResult](const FText& Info)
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName = Arguments.SourceData->GetFilename();
		Message->DestinationAssetName = Arguments.AssetName;
		Message->AssetType = GetFactoryClass();
		Message->Text = FText::Format(LOCTEXT("TexFact_CouldNotCreateMat", "UInterchangeTextureFactory: Could not create texture asset {0}. Reason: {1}"), FText::FromString(Arguments.AssetName), Info);
		bSkipImport = true;
		ImportAssetResult.bIsFactorySkipAsset = true;
	};

	if (!Arguments.AssetNode)
	{
		CouldNotCreateTextureLog(LOCTEXT("TextureFactory_AssetNodeNull", "Asset node parameter is null."));
		return ImportAssetResult;
	}

	const UClass* TextureClass = Arguments.AssetNode->GetObjectClass();
	if (!TextureClass || !TextureClass->IsChildOf(UTexture::StaticClass()))
	{
		CouldNotCreateTextureLog(LOCTEXT("TextureFactory_NodeClassMissmatch", "Asset node parameter class doesn't derive from UTexture."));
		return ImportAssetResult;
	}

	UClass* SupportedFactoryNodeClass = GetSupportedFactoryNodeClass(Arguments.AssetNode);
	if (SupportedFactoryNodeClass == nullptr)
	{
		CouldNotCreateTextureLog(LOCTEXT("TextureFactory_NodeClassWrong", "Asset node parameter is not a UInterchangeTextureFactoryNode or UInterchangeTextureCubeFactoryNode."));
		return ImportAssetResult;
	}


	FTextureNodeVariant TextureNodeVariant = GetTextureNodeVariantFromFactoryVariant(GetAsTextureFactoryNodeVariant(Arguments.AssetNode, SupportedFactoryNodeClass), Arguments.NodeContainer);
	if (TextureNodeVariant.IsType<FEmptyVariantState>())
	{
		FText Info = FText::Format(LOCTEXT("TextureFactory_InvalidTextureTranslated", "Asset factory node ({0}) doesn't reference a valid texture translated node.")
			, FText::FromString(SupportedFactoryNodeClass->GetAuthoredName()));
		CouldNotCreateTextureLog(Info);
		return ImportAssetResult;
	}

	if (!HasPayloadKey(TextureNodeVariant))
	{
		CouldNotCreateTextureLog(LOCTEXT("TextureFactory_InvalidPayloadKey", "Texture translated node doesn't have a payload key."));
		return ImportAssetResult;
	}

	const bool bIsReimport = Arguments.ReimportObject != nullptr;

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (Arguments.AssetNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}
	}

	// create a new texture or overwrite existing asset, if possible
	if (!ExistingAsset)
	{
		Texture = NewObject<UTexture>(Arguments.Parent, TextureClass, *Arguments.AssetName, RF_Public | RF_Standalone);
		bCreatedAsset = true;

		if (UTextureLightProfile* LightProfile = Cast<UTextureLightProfile>(Texture))
		{
			LightProfile->AddressX = TA_Clamp;
			LightProfile->AddressY = TA_Clamp;
#if WITH_EDITORONLY_DATA
			LightProfile->MipGenSettings = TMGS_NoMipmaps;
#endif
			LightProfile->LODGroup = TEXTUREGROUP_IESLightProfile;
		}
	}
	else
	{
		Texture = Cast<UTexture>(ExistingAsset);
		bCreatedAsset = false;

		//We allow override of existing Textures only if the translator is a pure texture translator or the user directly ask to re-import this object
		if (!bIsReimport && Arguments.Translator->GetSupportedAssetTypes() != EInterchangeTranslatorAssetType::Textures)
		{
			//Do not override the material asset
			ImportAssetResult.bIsFactorySkipAsset = true;
			bSkipImport = true;
		}
	}

	if (!Texture)
	{
		CouldNotCreateTextureLog(LOCTEXT("TextureFactory_TextureCreateFail", "Texture creation failed."));
		return ImportAssetResult;
	}

	ImportAssetResult.ImportedObject = Texture;
	return ImportAssetResult;
}

// The payload fetching and the heavy operations are done here
UInterchangeFactoryBase::FImportAssetResult UInterchangeTextureFactory::ImportAsset_Async(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeTextureFactory::ImportAsset_Async);

	using namespace UE::Interchange::Private::InterchangeTextureFactory;
	FImportAssetResult ImportAssetResult;
	ImportAssetResult.bIsFactorySkipAsset = bSkipImport;

	auto ImportTextureErrorLog = [this, &Arguments](const FText& Info)
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName = Arguments.SourceData->GetFilename();
		Message->DestinationAssetName = Arguments.AssetName;
		Message->AssetType = GetFactoryClass();
		Message->Text = Info;
	};

	if (!Arguments.AssetNode)
	{
		ImportTextureErrorLog(LOCTEXT("TextureFactory_Async_AssetNodeNull", "UInterchangeTextureFactory: Asset node parameter is null."));
		return ImportAssetResult;
	}

	const UClass* TextureClass = Arguments.AssetNode->GetObjectClass();
	if (!TextureClass || !TextureClass->IsChildOf(UTexture::StaticClass()))
	{
		ImportTextureErrorLog(LOCTEXT("TextureFactory_Async_MissMatchClass", "UInterchangeTextureFactory: Asset node parameter class doesn't derive from UTexture."));
		return ImportAssetResult;
	}

	UClass* SupportedFactoryNodeClass = GetSupportedFactoryNodeClass(Arguments.AssetNode);
	if (SupportedFactoryNodeClass == nullptr)
	{
		ImportTextureErrorLog(LOCTEXT("TextureFactory_Async_NodeWrongClass", "UInterchangeTextureFactory: Asset node parameter is not a child of UInterchangeTextureFactoryNode."));
		return ImportAssetResult;
	}

	UObject* ExistingAsset = nullptr;
	FSoftObjectPath ReferenceObject;
	if (Arguments.AssetNode->GetCustomReferenceObject(ReferenceObject))
	{
		ExistingAsset = ReferenceObject.TryLoad();
	}

	//Do not override an asset we skip
	if (bSkipImport)
	{
		ImportAssetResult.ImportedObject = ExistingAsset;
		return ImportAssetResult;
	}

	FTextureFactoryNodeVariant TextureFactoryNodeVariant = GetAsTextureFactoryNodeVariant(Arguments.AssetNode, SupportedFactoryNodeClass);
	FTextureNodeVariant TextureNodeVariant = GetTextureNodeVariantFromFactoryVariant(TextureFactoryNodeVariant, Arguments.NodeContainer);
	if (TextureNodeVariant.IsType<FEmptyVariantState>())
	{
		FText Info = FText::Format(LOCTEXT("TextureFactory_Async_InvalidTextureTranslated", "UInterchangeTextureFactory: Asset factory node ({0}) doesn't reference a valid texture translated node.")
			, FText::FromString(SupportedFactoryNodeClass->GetAuthoredName()));
		ImportTextureErrorLog(Info);
		return ImportAssetResult;
	}

	const TOptional<FString>& PayLoadKey = GetPayloadKey(TextureNodeVariant);
	if (!PayLoadKey.IsSet())
	{
		ImportTextureErrorLog(LOCTEXT("TextureFactory_Async_InvalidPayloadKey", "UInterchangeTextureFactory: Texture translated node (UInterchangeTexture2DNode) doesn't have a payload key."));
		return ImportAssetResult;
	}

	AlternateTexturePath.Reset();
	FTexturePayloadVariant TexturePayload = GetTexturePayload(Arguments.SourceData, PayLoadKey.GetValue(), TextureNodeVariant, TextureFactoryNodeVariant, Arguments.Translator, AlternateTexturePath);

	if (!IsTextureOptionalPayloadSet(TexturePayload))
	{
		ImportTextureErrorLog(LOCTEXT("TextureFactory_Async_CannotRetrievePayload", "UInterchangeTextureFactory: Invalid translator couldn't retrieve a payload."));
		return ImportAssetResult;
	}

	UE::Interchange::FTextureCreatorApplicationMetadata OutTextureCreatorApplicationMetadata;
	ExtractCreatorApplicationMetadata(TexturePayload, OutTextureCreatorApplicationMetadata);
	if (UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(const_cast<UInterchangeBaseNodeContainer*>(Arguments.NodeContainer)))
	{
		using namespace UE::Interchange;
		if (!OutTextureCreatorApplicationMetadata.ApplicationVendor.IsEmpty())
		{
			SourceNode->SetExtraInformation(FSourceNodeExtraInfoStaticData::GetApplicationVendorExtraInfoKey(), OutTextureCreatorApplicationMetadata.ApplicationVendor);
		}

		if (!OutTextureCreatorApplicationMetadata.ApplicationName.IsEmpty())
		{
			SourceNode->SetExtraInformation(FSourceNodeExtraInfoStaticData::GetApplicationNameExtraInfoKey(), OutTextureCreatorApplicationMetadata.ApplicationName);
		}

		if (!OutTextureCreatorApplicationMetadata.ApplicationVersion.IsEmpty())
		{
			SourceNode->SetExtraInformation(FSourceNodeExtraInfoStaticData::GetApplicationVersionExtraInfoKey(), OutTextureCreatorApplicationMetadata.ApplicationVersion);
		}
	}

	UTexture* Texture = nullptr;
	// create a new texture or overwrite existing asset, if possible
	if (!ExistingAsset)
	{
		ImportTextureErrorLog(LOCTEXT("TextureFactory_Async_CannotCreateAsync", "UInterchangeTextureFactory: Could not create Texture asset outside of the game thread."));
		return ImportAssetResult;
	}
	else
	{
		Texture = Cast<UTexture>(ExistingAsset);
	}

	if (!Texture)
	{
		ImportTextureErrorLog(LOCTEXT("TextureFactory_Async_FailCreatingAsset", "UInterchangeTextureFactory: Could not create texture asset."));
		return ImportAssetResult;
	}

	// Check if the imported texture(s) has a valid resolution
	CheckForInvalidResolutions(TexturePayload, Arguments.SourceData, CastChecked<UInterchangeTextureFactoryNode>(Arguments.AssetNode));

	bool bCanSetup = false;
	// Check if the payload is valid for the Texture
	if (UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
	{
		bCanSetup = CanSetupTexture2DSourceData(TexturePayload);
	}
#if WITH_EDITOR
	else if (UTextureCube* TextureCube = Cast<UTextureCube>(Texture))
	{
		bCanSetup = CanSetupTextureCubeSourceData(TexturePayload);
	}
	else if (UTextureCubeArray* TextureCubeArray = Cast<UTextureCubeArray>(Texture))
	{
		bCanSetup = CanSetupTextureCubeArraySourceData(TexturePayload);
	}
	else if (UTexture2DArray* Texture2DArray = Cast<UTexture2DArray>(Texture))
	{
		bCanSetup = CanSetupTexture2DArraySourceData(TexturePayload);
	}
	else if (UVolumeTexture* VolumeTexture = Cast<UVolumeTexture>(Texture))
	{
		bCanSetup = CanSetupVolumeTextureSourceData(TexturePayload);
	}
#endif // WITH_EDITOR

	if (!bCanSetup)
	{
		LogErrorInvalidPayload(*this, Texture->GetClass(), Arguments.SourceData->GetFilename(), Texture->GetName());
		ImportAssetResult.ImportedObject = Arguments.ReimportObject ? Texture : nullptr;
		ImportAssetResult.bIsFactorySkipAsset = true;
		Arguments.AssetNode->SetEnabled(false);
		return ImportAssetResult;
	}

	FGraphEventArray TasksToDo;

#if WITH_EDITORONLY_DATA
	TasksToDo = GenerateHashSourceFilesTasks(Arguments.SourceData, GetFilesToHash(TextureFactoryNodeVariant, TexturePayload), SourceFiles);
#endif

	// Hash the payload while we hash the source files

	// This will hash the payload to generate a unique ID before passing it to the virtualized bulkdata
	ProcessedPayload = MoveTemp(TexturePayload);

	// Wait for the hashing task(s)
	ENamedThreads::Type NamedThread = IsInGameThread() ? ENamedThreads::GameThread : ENamedThreads::AnyThread;
	FTaskGraphInterface::Get().WaitUntilTasksComplete(TasksToDo, NamedThread);

	//The interchange completion task (call in the GameThread after the factories pass), will call PostEditChange which will trig another asynchronous system that will build all texture in parallel
	ImportAssetResult.ImportedObject = Texture;
	return ImportAssetResult;
}

//If the texture has a bad payload and its not a reimport we want to mark for garbage collect the newly created texture
UInterchangeFactoryBase::FImportAssetResult UInterchangeTextureFactory::EndImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeTextureFactory::ImportAsset_Async);

	using namespace UE::Interchange::Private::InterchangeTextureFactory;
	FImportAssetResult ImportAssetResult;
	ImportAssetResult.bIsFactorySkipAsset = bSkipImport;

	//We need a valid node
	if (!Arguments.AssetNode)
	{
		return ImportAssetResult;
	}

	UObject* ExistingAsset = nullptr;
	FSoftObjectPath ReferenceObject;
	if (Arguments.AssetNode->GetCustomReferenceObject(ReferenceObject))
	{
		ExistingAsset = ReferenceObject.TryLoad();
	}

	//We need a valid imported object
	if (!ExistingAsset)
	{
		return ImportAssetResult;
	}

	if (UTexture* Texture = Cast<UTexture>(ExistingAsset))
	{
		ImportAssetResult.ImportedObject = Texture;

		// We need to be careful to try and discard the asset only if we were the ones that originally created it. We don't want 
		// to discard an asset we just found
		if (bCreatedAsset && !ProcessedPayload.IsValid())
		{
			// The texture is not supported
			if (!Arguments.ReimportObject)
			{
				ImportAssetResult.ImportedObject = nullptr;
				// Not thread safe. So those should stay on the game thread.
				Texture->RemoveFromRoot();
				Texture->MarkAsGarbage();

				Arguments.AssetNode->SetCustomReferenceObject(nullptr);
			}
			Arguments.AssetNode->SetEnabled(false);
			bSkipImport = true;
			ImportAssetResult.bIsFactorySkipAsset = bSkipImport;
		}
	}
	
	return ImportAssetResult;
}

/* This function is call in the completion task on the main thread, use it to call main thread post creation step for your assets*/
void UInterchangeTextureFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeTextureFactory::SetupObject_GameThread);

	check(IsInGameThread());

	if (bSkipImport)
	{
		return;
	}

	UTexture* Texture = Cast<UTexture>(Arguments.ImportedObject);

	// Finish the import on the game thread by doing the setup on the texture here
	if (Texture && ProcessedPayload.IsValid())
	{
#if WITH_EDITOR
		Texture->PreEditChange(nullptr);
#endif
		using namespace UE::Interchange::Private::InterchangeTextureFactory;

		// Setup source data
		UInterchangeFactoryBaseNode* TextureFactoryNode = Arguments.FactoryNode;
		SetupTextureSourceDataFromPayload(TextureFactoryNode, Texture, ProcessedPayload, Arguments.bIsReimport);

#if WITH_EDITOR
		if (!Arguments.bIsReimport)
		{
			// Finalize texture properties, before ApplyAllCustomAttributeToObject :
			UE::TextureUtilitiesCommon::ApplyDefaultsForNewlyImportedTextures(Texture,Arguments.bIsReimport);

			/** Apply all TextureNode custom attributes to the texture asset */
			TextureFactoryNode->ApplyAllCustomAttributeToObject(Texture);
		}
		else
		{
			UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(Texture->AssetImportData);
			UInterchangeFactoryBaseNode* PreviousNode = nullptr;
			if (InterchangeAssetImportData)
			{
				PreviousNode = InterchangeAssetImportData->GetStoredFactoryNode(InterchangeAssetImportData->NodeUniqueID);
			}

			UInterchangeFactoryBaseNode* CurrentNode = UInterchangeFactoryBaseNode::DuplicateWithObject(TextureFactoryNode, Texture);
			//Apply reimport strategy
			UE::Interchange::FFactoryCommon::ApplyReimportStrategyToAsset(Texture, PreviousNode, CurrentNode, TextureFactoryNode);
			
			// ApplyDefaultsForNewlyImportedTextures after ApplyReimportStrategyToAsset
			//	so we can override values
			UE::TextureUtilitiesCommon::ApplyDefaultsForNewlyImportedTextures(Texture,Arguments.bIsReimport);
		}

		// Texture is mostly done setting properties now (exception: FTaskPipelinePostImport NormalmapIdentification)
		// PostEditChange will be called subseqently, in FTaskPreCompletion::DoTask
#endif // WITH_EDITOR
	}
	else
	{
		// The texture is not supported
		if (!Arguments.bIsReimport)
		{
			// Not thread safe. So those should stay on the game thread.
			Texture->RemoveFromRoot();
			Texture->MarkAsGarbage();
		}
	}

	Super::SetupObject_GameThread(Arguments);

	//TODO make sure this work at runtime
#if WITH_EDITORONLY_DATA
	if (Texture && Arguments.SourceData && ProcessedPayload.IsValid())
	{
		//We must call the Update of the asset source file in the main thread because UAssetImportData::Update execute some delegate we do not control
		UE::Interchange::FFactoryCommon::FSetImportAssetDataParameters SetImportAssetDataParameters(Texture
																						 , Texture->AssetImportData
																						 , Arguments.SourceData
																						 , Arguments.NodeUniqueID
																						 , Arguments.NodeContainer
																						 , Arguments.OriginalPipelines
																						 , Arguments.Translator);
		SetImportAssetDataParameters.SourceFiles = MoveTemp(SourceFiles);

		Texture->AssetImportData = UE::Interchange::FFactoryCommon::SetImportAssetData(SetImportAssetDataParameters);
		
		if (AlternateTexturePath.IsSet())
		{
			// A file different from the source data has been used to create the texture.
			// Set this file's path as the first file name
			Texture->AssetImportData->AddFileName(AlternateTexturePath.GetValue(), 0);
		}
	}
#endif
}

void UInterchangeTextureFactory::CheckForInvalidResolutions(UE::Interchange::Private::InterchangeTextureFactory::FTexturePayloadVariant& InPayloadVariant, const UInterchangeSourceData* SourceData, const UInterchangeTextureFactoryNode* TextureFactoryNode)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeTextureFactory::CheckForInvalidResolutions);
	using namespace UE::Interchange;

	auto AddErrorMessage = [this, &TextureFactoryNode](FString&& InSourceAssetName, const FText& ErrorMessage) -> UInterchangeResultError_Generic*
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName = MoveTemp(InSourceAssetName);
		Message->DestinationAssetName = TextureFactoryNode->GetAssetName();
		Message->AssetType = TextureFactoryNode->GetObjectClass();
		Message->Text = ErrorMessage;
		return Message;
	};


	bool bAllowNonPowerOfTwo = false;
	TextureFactoryNode->GetCustomAllowNonPowerOfTwo(bAllowNonPowerOfTwo);

	static_assert(TVariantSize_V<Private::InterchangeTextureFactory::FTexturePayloadVariant> == 5, "Please update the code below and this assert to reflect the change to the variant type.");

	FText ErrorMessage;
	if (TOptional<FImportBlockedImage>* BlockedImagePtr = InPayloadVariant.TryGet<TOptional<FImportBlockedImage>>())
	{
		if (BlockedImagePtr->IsSet())
		{
			FImportBlockedImage& BlockedImage = BlockedImagePtr->GetValue();

			for (int32 Index = 0; Index < BlockedImage.BlocksData.Num(); ++Index)
			{
				const FTextureSourceBlock& Block = BlockedImage.BlocksData[Index];
				if (!UE::TextureUtilitiesCommon::IsImportResolutionValid(Block.SizeX, Block.SizeY, bAllowNonPowerOfTwo, &ErrorMessage))
				{
					FString SourceFile;
					if (const UInterchangeTexture2DFactoryNode* Texture2DFactoryNode = Cast<UInterchangeTexture2DFactoryNode>(TextureFactoryNode))
					{
						Texture2DFactoryNode->GetSourceBlockByCoordinates(Block.BlockX, Block.BlockY, SourceFile);
					}
					if (SourceFile.IsEmpty())
					{
						SourceFile = SourceData->GetFilename();
					}
					UInterchangeResultError_Generic* Message = AddErrorMessage(MoveTemp(SourceFile), ErrorMessage);

					FString BlockMessage = FText::Format(LOCTEXT("BlockIndexInvalidResolutionError", "Invalid block (X : {0}, Y : {1}) \n"), Block.BlockX, Block.BlockY).ToString();
					Message->Text = FText::FromString(BlockMessage.Append(Message->Text.ToString()));

					BlockedImage.BlocksData.RemoveAtSwap(Index);
					--Index;
				}
			}

			if (BlockedImage.BlocksData.IsEmpty())
			{
				AddErrorMessage(FString(), LOCTEXT("UDIMHasNoValidBlock", "All blocks were invalid. The texture won't be imported."));

				// Remove the payload
				InPayloadVariant.Set<FEmptyVariantState>(FEmptyVariantState());
			}
		}
	}
	else if (TOptional<FImportImage>* ImagePtr = InPayloadVariant.TryGet<TOptional<FImportImage>>())
	{
		if (ImagePtr->IsSet())
		{
			const FImportImage& Image = ImagePtr->GetValue();
			if (UTextureCube::StaticClass() != TextureFactoryNode->GetObjectClass() && !UE::TextureUtilitiesCommon::IsImportResolutionValid(Image.SizeX, Image.SizeY, bAllowNonPowerOfTwo, &ErrorMessage))
			{
				AddErrorMessage(SourceData->GetFilename(), ErrorMessage);

				// Remove the payload
				InPayloadVariant.Set<FEmptyVariantState>(FEmptyVariantState());
			}
		}
	}
	else if (TOptional<FImportLightProfile>* LightProfilePtr = InPayloadVariant.TryGet<TOptional<FImportLightProfile>>())
	{
		if (LightProfilePtr->IsSet())
		{
			const FImportLightProfile& LightProfile = LightProfilePtr->GetValue();
			if (!UE::TextureUtilitiesCommon::IsImportResolutionValid(LightProfile.SizeX, LightProfile.SizeY, bAllowNonPowerOfTwo, &ErrorMessage))
			{
				AddErrorMessage(SourceData->GetFilename(), ErrorMessage);

				// Remove the payload
				InPayloadVariant.Set<FEmptyVariantState>(FEmptyVariantState());
			}
		}
	}
	else if (TOptional<FImportSlicedImage>* SlicedImagePtr = InPayloadVariant.TryGet<TOptional<FImportSlicedImage>>())
	{
		if (SlicedImagePtr->IsSet())
		{
			const FImportSlicedImage& SlicedImage = SlicedImagePtr->GetValue();

			if (!UE::TextureUtilitiesCommon::IsImportResolutionValid(SlicedImage.SizeX, SlicedImage.SizeY, bAllowNonPowerOfTwo, &ErrorMessage))
			{
				AddErrorMessage(SourceData->GetFilename(), ErrorMessage);

				// Remove the payload
				InPayloadVariant.Set<FEmptyVariantState>(FEmptyVariantState());
			}
		}
	}
}

bool UInterchangeTextureFactory::GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const
{
#if WITH_EDITORONLY_DATA
	if (const UTexture* Texture = Cast<UTexture>(Object))
	{
		return UE::Interchange::FFactoryCommon::GetSourceFilenames(Texture->AssetImportData.Get(), OutSourceFilenames);
	}
#endif

	return false;
}

bool UInterchangeTextureFactory::SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const
{
#if WITH_EDITORONLY_DATA
	if (const UTexture* Texture = Cast<UTexture>(Object))
	{
		return UE::Interchange::FFactoryCommon::SetSourceFilename(Texture->AssetImportData.Get(), SourceFilename, SourceIndex);
	}
#endif

	return false;
}

void UInterchangeTextureFactory::BackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UTexture* Texture = Cast<UTexture>(Object))
	{
		UE::Interchange::FFactoryCommon::BackupSourceData(Texture->AssetImportData.Get());
	}
#endif
}

void UInterchangeTextureFactory::ReinstateSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UTexture* Texture = Cast<UTexture>(Object))
	{
		UE::Interchange::FFactoryCommon::ReinstateSourceData(Texture->AssetImportData.Get());
	}
#endif
}

void UInterchangeTextureFactory::ClearBackupSourceData(const UObject* Object) const
{
#if WITH_EDITORONLY_DATA
	if (const UTexture* Texture = Cast<UTexture>(Object))
	{
		UE::Interchange::FFactoryCommon::ClearBackupSourceData(Texture->AssetImportData.Get());
	}
#endif
}

#undef LOCTEXT_NAMESPACE
