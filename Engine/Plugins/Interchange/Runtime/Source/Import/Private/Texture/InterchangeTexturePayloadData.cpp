// Copyright Epic Games, Inc. All Rights Reserved.

#include "Texture/InterchangeTexturePayloadData.h"

#include "CoreMinimal.h"
#include "Engine/Texture.h"
#include "HAL/IConsoleManager.h"
#include "Misc/MessageDialog.h"
#include "RHI.h"
#include "ImageCoreUtils.h"
#include "TextureImportUtils.h"

void UE::Interchange::FImportImage::Init2DWithParams(int32 InSizeX, int32 InSizeY, ETextureSourceFormat InFormat, bool bInSRGB, bool bShouldAllocateRawData)
{
	Init2DWithParams(InSizeX, InSizeY, 1, InFormat, bInSRGB, bShouldAllocateRawData);
}

void UE::Interchange::FImportImage::Init2DWithParams(int32 InSizeX, int32 InSizeY, int32 InNumMips, ETextureSourceFormat InFormat, bool bInSRGB, bool bShouldAllocateRawData)
{
	SizeX = InSizeX;
	SizeY = InSizeY;
	NumMips = InNumMips;
	Format = InFormat;
	bSRGB = bInSRGB;
	if (bShouldAllocateRawData)
	{
		RawData = FUniqueBuffer::Alloc(ComputeBufferSize());
	}
}

int64 UE::Interchange::FImportImage::GetMipSize(int32 InMipIndex) const
{
	check(InMipIndex >= 0);
	check(InMipIndex < NumMips);
	const int32 MipSizeX = FMath::Max(SizeX >> InMipIndex, 1);
	const int32 MipSizeY = FMath::Max(SizeY >> InMipIndex, 1);
	return (int64)MipSizeX * MipSizeY * FTextureSource::GetBytesPerPixel(Format);
}

int64 UE::Interchange::FImportImage::ComputeBufferSize() const
{
	int64 TotalSize = 0;
	for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
	{
		TotalSize += GetMipSize(MipIndex);
	}

	return TotalSize;
}

TArrayView64<uint8> UE::Interchange::FImportImage::GetArrayViewOfRawData()
{
	return TArrayView64<uint8>(static_cast<uint8*>(RawData.GetData()), RawData.GetSize());
}

bool UE::Interchange::FImportImage::IsValid() const
{
	bool bIsRawDataBufferValid = false;

	if (RawDataCompressionFormat == TSCF_None)
	{
		bIsRawDataBufferValid = ComputeBufferSize() == RawData.GetSize();
	}
	else
	{
		bIsRawDataBufferValid = !RawData.IsNull();
	}

	return SizeX > 0
		&& SizeY > 0
		&& NumMips > 0
		&& Format != TSF_Invalid
		&& bIsRawDataBufferValid;
}
