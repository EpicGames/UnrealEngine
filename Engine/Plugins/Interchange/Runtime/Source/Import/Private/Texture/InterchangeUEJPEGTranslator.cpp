// Copyright Epic Games, Inc. All Rights Reserved. 
#include "Texture/InterchangeUEJPEGTranslator.h"

#include "Algo/Find.h"
#include "Containers/StaticArray.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "ImageCoreUtils.h"
#include "InterchangeImportLog.h"
#include "InterchangeTextureNode.h"
#include "Memory/SharedBuffer.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Texture/TextureTranslatorUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeUEJPEGTranslator)

static bool GInterchangeEnableUEJPEGImport = true;
static FAutoConsoleVariableRef CCvarInterchangeEnableUEJPEGImport(
	TEXT("Interchange.FeatureFlags.Import.UEJPEG"),
	GInterchangeEnableUEJPEGImport,
	TEXT("Whether UEJPEG support is enabled."),
	ECVF_Default);

TArray<FString> UInterchangeUEJPEGTranslator::GetSupportedFormats() const
{
	TArray<FString> Formats;

	if (GInterchangeEnableUEJPEGImport || GIsAutomationTesting)
	{
		Formats.Reserve(3);

		Formats.Add(TEXT("uej;UEJpeg image"));
		bool bEnableUEJpeg = false;
		GConfig->GetBool(TEXT("TextureImporter"), TEXT("EnableUEJpeg"), bEnableUEJpeg, GEditorIni);
		if (bEnableUEJpeg)
		{
			Formats.Add(TEXT("jpg;JPEG image"));
			Formats.Add(TEXT("jpeg;JPEG image"));
		}
	}

	return Formats;
}

bool UInterchangeUEJPEGTranslator::Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const
{
	return UE::Interchange::FTextureTranslatorUtilities::Generic2DTextureTranslate(GetSourceData(), BaseNodeContainer);
}

TOptional<UE::Interchange::FImportImage> UInterchangeUEJPEGTranslator::GetTexturePayloadData(const FString& /*PayloadKey*/, TOptional<FString>& AlternateTexturePath) const
{
	using namespace UE::Interchange;

	TArray64<uint8> SourceDataBuffer;
	if (!FTextureTranslatorUtilities::LoadSourceBuffer(*this, TEXT("UEJPEG"), SourceDataBuffer))
	{
		return {};
	}

	const bool bImportRaw = false;
	return GetTexturePayloadImplementation(MoveTemp(SourceDataBuffer), bImportRaw);

}

bool UInterchangeUEJPEGTranslator::SupportCompressedTexturePayloadData() const
{
	return true;
}

TOptional<UE::Interchange::FImportImage> UInterchangeUEJPEGTranslator::GetCompressedTexturePayloadData(const FString& /*PayloadKey*/, TOptional<FString>& /*AlternateTexturePath*/) const
{
	using namespace UE::Interchange;

	TArray64<uint8> SourceDataBuffer;
	if (!FTextureTranslatorUtilities::LoadSourceBuffer(*this, TEXT("UEJPEG"), SourceDataBuffer))
	{
		return {};
	}

	const bool bImportRaw = true;
	return GetTexturePayloadImplementation(MoveTemp(SourceDataBuffer), bImportRaw);
}

TOptional<UE::Interchange::FImportImage> UInterchangeUEJPEGTranslator::GetTexturePayloadImplementation(TArray64<uint8>&& SourceDataBuffer, bool bShouldImportRaw) const
{
	using namespace UE::Interchange;

	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));

	//
	// UEJPEG
	//
	ETextureSourceCompressionFormat TscfFormat = ETextureSourceCompressionFormat::TSCF_UEJPEG; 
	TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::UEJPEG);
	if (!ImageWrapper.IsValid() || !ImageWrapper->SetCompressed(SourceDataBuffer.GetData(), SourceDataBuffer.Num()))
	{
		TscfFormat = ETextureSourceCompressionFormat::TSCF_JPEG; 
		ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::JPEG);
		if (!ImageWrapper.IsValid() || !ImageWrapper->SetCompressed(SourceDataBuffer.GetData(), SourceDataBuffer.Num()))
		{
			FTextureTranslatorUtilities::LogError(*this, NSLOCTEXT("InterchangeUEJPEGTranslator", "DecodingFailed", "Failed to decode UEJPEG."));
			return TOptional<UE::Interchange::FImportImage>();
		}
	}

	// Select the texture's source format
	ERawImageFormat::Type RawFormat = ImageWrapper->GetClosestRawImageFormat();
	check( RawFormat != ERawImageFormat::Invalid );
	ETextureSourceFormat TextureFormat = FImageCoreUtils::ConvertToTextureSourceFormat(RawFormat);

	UE::Interchange::FImportImage PayloadData;

	const bool bShouldAllocateRawDataBuffer = false;

	PayloadData.Init2DWithParams(
		ImageWrapper->GetWidth(),
		ImageWrapper->GetHeight(),
		TextureFormat,
		ImageWrapper->GetSRGB(),
		bShouldAllocateRawDataBuffer
	);

	TArray64<uint8> RawData;
	if (bShouldImportRaw)
	{
		PayloadData.RawData = MakeUniqueBufferFromArray(ImageWrapper->GetCompressed());
		PayloadData.RawDataCompressionFormat = TscfFormat;
	}
	else if (ImageWrapper->GetRaw(RawData))
	{
		PayloadData.RawData = MakeUniqueBufferFromArray(MoveTemp(RawData));
	}
	else
	{
		FTextureTranslatorUtilities::LogError(*this, NSLOCTEXT("InterchangeUEJPEGTranslator", "DecodingFailed", "Failed to decode UEJPEG."));
		return TOptional<UE::Interchange::FImportImage>();
	}

	return PayloadData;
}
