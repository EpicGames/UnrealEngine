// Copyright Epic Games, Inc. All Rights Reserved. 
#include "Texture/TextureTranslatorUtilities.h"

#include "CoreMinimal.h"
#include "InterchangeImportLog.h"
#include "InterchangeSourceData.h"
#include "InterchangeTexture2DArrayNode.h"
#include "InterchangeTexture2DNode.h"
#include "InterchangeTextureCubeArrayNode.h"
#include "InterchangeTextureCubeNode.h"
#include "InterchangeTextureLightProfileNode.h"
#include "InterchangeTextureNode.h"
#include "InterchangeTranslatorBase.h"
#include "InterchangeVolumeTextureNode.h"
#include "Misc/Paths.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#define LOCTEXT_NAMESPACE "InterchangeTextureTranslator"

namespace UE::Interchange::Private
{
	bool GenericTextureTranslate(const UInterchangeSourceData* SourceData, UInterchangeBaseNodeContainer& BaseNodeContainer, const TSubclassOf<UInterchangeTextureNode>& TextureNodeClass)
	{
		FString Filename = SourceData->GetFilename();
		FPaths::NormalizeFilename(Filename);
		if (!FPaths::FileExists(Filename))
		{
			return false;
		}

		UClass* Class = TextureNodeClass.Get();
		if (!ensure(Class))
		{
			return false;
		}

		FString DisplayLabel = FPaths::GetBaseFilename(Filename);
		FString NodeUID(Filename);
		UInterchangeTextureNode* TextureNode = NewObject<UInterchangeTextureNode>(&BaseNodeContainer, Class);
		if (!ensure(TextureNode))
		{
			return false;
		}

		BaseNodeContainer.SetupNode(TextureNode, NodeUID, DisplayLabel, EInterchangeNodeContainerType::TranslatedAsset);

		TextureNode->SetPayLoadKey(Filename);

		return true;
	}
}

bool UE::Interchange::FTextureTranslatorUtilities::Generic2DTextureTranslate(const UInterchangeSourceData* SourceData, UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	return Private::GenericTextureTranslate(SourceData, BaseNodeContainer, UInterchangeTexture2DNode::StaticClass());
}

bool UE::Interchange::FTextureTranslatorUtilities::GenericTextureCubeTranslate(const UInterchangeSourceData* SourceData, UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	return Private::GenericTextureTranslate(SourceData, BaseNodeContainer, UInterchangeTextureCubeNode::StaticClass());
}

bool UE::Interchange::FTextureTranslatorUtilities::GenericTextureCubeArrayTranslate(const UInterchangeSourceData* SourceData, UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	return Private::GenericTextureTranslate(SourceData, BaseNodeContainer, UInterchangeTextureCubeArrayNode::StaticClass());
}

bool UE::Interchange::FTextureTranslatorUtilities::GenericTexture2DArrayTranslate(const UInterchangeSourceData* SourceData, UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	return Private::GenericTextureTranslate(SourceData, BaseNodeContainer, UInterchangeTexture2DArrayNode::StaticClass());
}

bool UE::Interchange::FTextureTranslatorUtilities::GenericVolumeTextureTranslate(const UInterchangeSourceData* SourceData, UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	return Private::GenericTextureTranslate(SourceData, BaseNodeContainer, UInterchangeVolumeTextureNode::StaticClass());
}

bool UE::Interchange::FTextureTranslatorUtilities::GenericTextureLightProfileTranslate(const UInterchangeSourceData* SourceData, UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	return Private::GenericTextureTranslate(SourceData, BaseNodeContainer, UInterchangeTextureLightProfileNode::StaticClass());
}

void UE::Interchange::FTextureTranslatorUtilities::LogError(const UInterchangeTranslatorBase& TextureTranslator, FText&& ErrorText)
{
	UInterchangeResultError_Generic* ErrorMessage = TextureTranslator.AddMessage<UInterchangeResultError_Generic>();
	if (ensure(ErrorMessage))
	{
		ErrorMessage->AssetType = TextureTranslator.GetClass();

		if (TextureTranslator.GetSourceData())
		{
			const FString Filename = TextureTranslator.GetSourceData()->GetFilename();
			ErrorMessage->SourceAssetName = Filename;
			ErrorMessage->InterchangeKey = FPaths::GetBaseFilename(Filename);
		}
		else
		{
			ErrorMessage->InterchangeKey = TEXT("Undefined");
		}

		ErrorMessage->Text = MoveTemp(ErrorText);
	}
}

bool UE::Interchange::FTextureTranslatorUtilities::IsTranslatorValid(const UInterchangeTranslatorBase& TextureTranslator, const TCHAR* Format)
{
	const UInterchangeSourceData* SourceData = TextureTranslator.GetSourceData();

	if (!SourceData)
	{
		LogError(TextureTranslator, FText::Format(LOCTEXT("TextureImportFailure_BadData", "{0}: Failed to import, bad source data."), FText::FromString(Format)));
		return false;
	}

	FString Filename = SourceData->GetFilename();

	if (!FPaths::FileExists(Filename))
	{
		LogError(TextureTranslator, FText::Format(LOCTEXT("TextureImportFailure_OpenFile", "{0}: Failed to import texture asset, cannot open file [{1}]."), FText::FromString(Format), FText::FromString(Filename)));
		return false;
	}

	return true;
}

bool UE::Interchange::FTextureTranslatorUtilities::LoadSourceBuffer(const UInterchangeTranslatorBase& TextureTranslator, const TCHAR* Format, TArray64<uint8>& SourceDataBuffer)
{
	if (!IsTranslatorValid(TextureTranslator, Format))
	{
		return false;
	}

	if (!FFileHelper::LoadFileToArray(SourceDataBuffer, *TextureTranslator.GetSourceData()->GetFilename()))
	{
		LogError(TextureTranslator, FText::Format(LOCTEXT("TextureImportFailure_LoadBuffer", "Failed to import {0}, cannot load file content into an array."), FText::FromString(Format)));
		return false;
	}

	return !SourceDataBuffer.IsEmpty();
}

#undef LOCTEXT_NAMESPACE
