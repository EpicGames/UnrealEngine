// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "InterchangeSourceData.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

class UInterchangeTextureNode;
class UInterchangeTranslatorBase;

template <class T>
class TSubclassOf;

namespace UE::Interchange
{
	class FTextureTranslatorUtilities
	{
	public:
		static bool Generic2DTextureTranslate(const UInterchangeSourceData* SourceData, UInterchangeBaseNodeContainer& BaseNodeContainer);

		static bool GenericTextureCubeTranslate(const UInterchangeSourceData* SourceData, UInterchangeBaseNodeContainer& BaseNodeContainer);

		static bool GenericTextureCubeArrayTranslate(const UInterchangeSourceData* SourceData, UInterchangeBaseNodeContainer& BaseNodeContainer);

		static bool GenericTexture2DArrayTranslate(const UInterchangeSourceData* SourceData, UInterchangeBaseNodeContainer& BaseNodeContainer);

		static bool GenericVolumeTextureTranslate(const UInterchangeSourceData* SourceData, UInterchangeBaseNodeContainer& BaseNodeContainer);

		static bool GenericTextureLightProfileTranslate(const UInterchangeSourceData* SourceData, UInterchangeBaseNodeContainer& BaseNodeContainer);

		static bool IsTranslatorValid(const UInterchangeTranslatorBase& TextureTranslator, const TCHAR* Format);

		static bool LoadSourceBuffer(const UInterchangeTranslatorBase& TextureTranslator, const TCHAR* Format, TArray64<uint8>& SourceDataBuffer);

		static void LogError(const UInterchangeTranslatorBase& TextureTranslator, FText&& ErrorText);

	};
}//ns UE

