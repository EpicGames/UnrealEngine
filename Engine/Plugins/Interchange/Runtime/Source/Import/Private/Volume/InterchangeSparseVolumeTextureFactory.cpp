// Copyright Epic Games, Inc. All Rights Reserved.

#include "Volume/InterchangeSparseVolumeTextureFactory.h"

#include "InterchangeAssetImportData.h"
#include "InterchangeImportCommon.h"
#include "InterchangeResult.h"
#include "InterchangeSparseVolumeTextureFactoryNode.h"
#include "InterchangeTranslatorBase.h"
#include "InterchangeVolumeNode.h"
#include "Materials/MaterialInterface.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Volume/InterchangeVolumeDefinitions.h"
#include "Volume/InterchangeVolumePayloadData.h"
#include "Volume/InterchangeVolumePayloadInterface.h"

#include "Async/ParallelFor.h"
#include "Async/TaskGraphInterfaces.h"
#include "Components/HeterogeneousVolumeComponent.h"
#include "Misc/CoreStats.h"
#include "Misc/Paths.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Serialization/EditorBulkData.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "UDIMUtilities.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectIterator.h"
#if WITH_EDITORONLY_DATA
#include "EditorFramework/AssetImportData.h"
#endif	  // WITH_EDITORONLY_DATA

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSparseVolumeTextureFactory)

#define LOCTEXT_NAMESPACE "InterchangeSparseVolumeTextureFactory"

namespace UE::InterchangeSparseVolumeTextureFactory::Private
{
	// Returns the UInterchangeVolumeNodes referenced as targets by the provided UInterchangeSparseVolumeTextureFactoryNode
	TArray<const UInterchangeVolumeNode*> GetVolumeNodes(
		const UInterchangeSparseVolumeTextureFactoryNode* FactoryNode,
		const UInterchangeBaseNodeContainer* NodeContainer
	)
	{
		if (!FactoryNode || !NodeContainer)
		{
			return {};
		}

		TArray<const UInterchangeVolumeNode*> VolumeNodes;

		TArray<FString> TargetNodeUids;
		FactoryNode->GetTargetNodeUids(TargetNodeUids);

		for (int32 TargetNodeIndex = TargetNodeUids.Num() - 1; TargetNodeIndex >= 0; --TargetNodeIndex)
		{
			const UInterchangeBaseNode* BaseNode = NodeContainer->GetNode(TargetNodeUids[TargetNodeIndex]);
			if (const UInterchangeVolumeNode* VolumeNode = Cast<UInterchangeVolumeNode>(BaseNode))
			{
				VolumeNodes.Add(VolumeNode);
			}
		}

		return VolumeNodes;
	}

	// Returns the UInterchangeVolumeGridNodes referenced as dependencies by the provided UInterchangeVolumeNode
	TArray<const UInterchangeVolumeGridNode*> GetGridNodes(
		const UInterchangeVolumeNode* VolumeNode,
		const UInterchangeBaseNodeContainer* NodeContainer
	)
	{
		if (!VolumeNode || !NodeContainer)
		{
			return {};
		}

		TArray<FString> GridNodeUids;
		VolumeNode->GetCustomGridDependecies(GridNodeUids);

		TArray<const UInterchangeVolumeGridNode*> Result;
		Result.Reserve(GridNodeUids.Num());
		for (const FString& GridNodeUid : GridNodeUids)
		{
			Result.Add(Cast<UInterchangeVolumeGridNode>(NodeContainer->GetNode(GridNodeUid)));
		}

		return Result;
	}

#if WITH_EDITOR
	void HashSourceFiles(TArray<FAssetImportInfo::FSourceFile>& SourceFiles)
	{
		// References:
		// - GenerateHashSourceFilesTasks from InterchangeTextureFactory.cpp

		struct FHashTask
		{
			FHashTask(FAssetImportInfo::FSourceFile* InSourceFile)
				: SourceFile(InSourceFile)
			{
			}

			ENamedThreads::Type GetDesiredThread()
			{
				return ENamedThreads::AnyBackgroundThreadNormalTask;
			}

			TStatId GetStatId() const
			{
				return GET_STATID(STAT_TaskGraph_OtherTasks);
			}

			static ESubsequentsMode::Type GetSubsequentsMode()
			{
				return ESubsequentsMode::TrackSubsequents;
			}

			void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
			{
				SourceFile->FileHash = FMD5Hash::HashFile(*SourceFile->RelativeFilename);
				SourceFile->Timestamp = IFileManager::Get().GetTimeStamp(*SourceFile->RelativeFilename);
			}

		private:
			FAssetImportInfo::FSourceFile* SourceFile;
		};

		FGraphEventArray TasksToDo;
		for (FAssetImportInfo::FSourceFile& SourceFile : SourceFiles)
		{
			TasksToDo.Add(TGraphTask<FHashTask>::CreateTask().ConstructAndDispatchWhenReady(&SourceFile));
		}

		ENamedThreads::Type NamedThread = IsInGameThread() ? ENamedThreads::GameThread : ENamedThreads::AnyThread;
		FTaskGraphInterface::Get().WaitUntilTasksComplete(TasksToDo, NamedThread);
	}
#endif	  // WITH_EDITOR

	void FillSparseVolumeTextureWithPayloadData(USparseVolumeTexture* SparseVolumeTexture, TArray<FVolumePayload>& ProcessedPayloads)
	{
		UStreamableSparseVolumeTexture* Streamable = Cast<UStreamableSparseVolumeTexture>(SparseVolumeTexture);
		int32 NumPayloads = ProcessedPayloads.Num();
		if (!Streamable || NumPayloads < 1)
		{
			return;
		}

		bool bSucess = Streamable->BeginInitialize(NumPayloads);
		if (!bSucess)
		{
			return;
		}

		// Setup some temp structure to help us sort the animation frames, if any
		struct FPayloadAndIndex
		{
			FVolumePayload* Payload = nullptr;
			int32 FrameIndex;
		};

		TArray<FPayloadAndIndex> PayloadAndIndices;
		PayloadAndIndices.Reserve(ProcessedPayloads.Num());

		for (FVolumePayload& VolumePayload : ProcessedPayloads)
		{
			if (VolumePayload.PayloadFrameIndices.Num() == 0)
			{
				// No animation index --> Not animated, so pretend it's the earliest frame possible
				FPayloadAndIndex& NewEntry = PayloadAndIndices.Emplace_GetRef();
				NewEntry.Payload = &VolumePayload;
				NewEntry.FrameIndex = TNumericLimits<int32>::Lowest();
			}
			else
			{
				for (const int32 FrameIndex : VolumePayload.PayloadFrameIndices)
				{
					FPayloadAndIndex& NewEntry = PayloadAndIndices.Emplace_GetRef();
					NewEntry.Payload = &VolumePayload;
					NewEntry.FrameIndex = FrameIndex;
				}
			}
		}

		PayloadAndIndices.Sort(
			[](const FPayloadAndIndex& LHS, const FPayloadAndIndex& RHS)
			{
				if (LHS.FrameIndex == RHS.FrameIndex)
				{
					// Fallback compare on the volume node UID for consistency, or else the order could change for identical imports
					return LHS.Payload->VolumeNodeUid < RHS.Payload->VolumeNodeUid;
				}

				return LHS.FrameIndex < RHS.FrameIndex;
			}
		);

		// Note that we have no deduplication past this point: If a frame is repeated in the animated SVT, it will be added to the
		// SVT twice (we did only fetch the payload for it once though).
		//
		// We could maybe add this in the future, but it's probably a premature optimization that is just more prone to lead to confusion
		// than anything else. This because the way we compensate for removing those duplicated frames from the SVT is by playing the
		// unique frame multiple times on the LevelSequence. We can't help with that if the import is asset only (or if LevelSequences
		// aren't being imported, etc.), so it's probably best to just always add all the frames and make the animated SVT asset work
		// as intended on its own
		for (FPayloadAndIndex& PayloadAndIndex : PayloadAndIndices)
		{
			TOptional<UE::Interchange::FVolumePayloadData>& PayloadData = PayloadAndIndex.Payload->PayloadData;
			if (!PayloadData.IsSet())
			{
				continue;
			}

			bSucess = Streamable->AppendFrame(PayloadData->TextureData, PayloadData->Transform);
			if (!bSucess)
			{
				return;
			}
		}

		bSucess = Streamable->EndInitialize();
		if (!bSucess)
		{
			return;
		}
	}

	// We have to convert the assignment info we spread out into individual string attributes on the factory node back into an
	// index-based description of assignment info via FAssignmentInfo, which will ultimately be converted into an
	// FOpenVDBImportOptions by the translator when retrieving the payload, as that is what the OpenVDB utils expect.
	//
	// It's probably for the best though, as this can just live in here and instead users would only interact with the string attributes
	// on the factory nodes
	TOptional<UE::Interchange::Volume::FAssignmentInfo> GetAssignmentInfo(
		UInterchangeSparseVolumeTextureFactoryNode* FactoryNode,
		TArray<const UInterchangeVolumeNode*>& VolumeNodes,
		const UInterchangeBaseNodeContainer* NodeContainer
	)
	{
		using namespace UE::Interchange::Volume;

		if (!FactoryNode || !NodeContainer || VolumeNodes.Num() == 0)
		{
			return {};
		}

		FAssignmentInfo Result;
		Result.bIsSequence = VolumeNodes.Num() > 1;

		TArray<const UInterchangeVolumeGridNode*> OrderedGridNodes;
		{
			// The grid dependency order still matches the grid order in the OpenVDB file.
			//
			// If we're importing multiple volume nodes, it's implied that they're separate frames of the same animated SparseVolumeTexture,
			// and that means they should have the same grid and assignment, so picking any volume node should do
			TArray<FString> GridNodeUids;
			VolumeNodes[0]->GetCustomGridDependecies(GridNodeUids);

			OrderedGridNodes.Reserve(GridNodeUids.Num());
			for (const FString& GridNodeUid : GridNodeUids)
			{
				const UInterchangeVolumeGridNode* GridNode = Cast<UInterchangeVolumeGridNode>(NodeContainer->GetNode(GridNodeUid));
				ensure(GridNode);

				OrderedGridNodes.Add(GridNode);
			}
		}

		TMap<FString, int32> GridNameToIndex;
		GridNameToIndex.Reserve(OrderedGridNodes.Num());
		for (int32 Index = 0; Index < OrderedGridNodes.Num(); ++Index)
		{
			const UInterchangeVolumeGridNode* GridNode = OrderedGridNodes[Index];
			GridNameToIndex.Add(GridNode->GetDisplayLabel(), Index);
		}

		// Convert formats
		{
			EInterchangeSparseVolumeTextureFormat Format;

			Result.Attributes[0].Format = EInterchangeSparseVolumeTextureFormat::Float16;
			if (FactoryNode->GetCustomAttributesAFormat(Format))
			{
				Result.Attributes[0].Format = Format;
			}

			Result.Attributes[1].Format = EInterchangeSparseVolumeTextureFormat::Float16;
			if (FactoryNode->GetCustomAttributesBFormat(Format))
			{
				Result.Attributes[1].Format = Format;
			}
		}

		// Convert channels
		using GetterFunc = decltype(&UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesAChannelX);
		static const TArray<GetterFunc> AttributeChannelGetters = {
			&UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesAChannelX,
			&UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesAChannelY,
			&UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesAChannelZ,
			&UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesAChannelW,
			&UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesBChannelX,
			&UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesBChannelY,
			&UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesBChannelZ,
			&UInterchangeSparseVolumeTextureFactoryNode::GetCustomAttributesBChannelW,
		};
		for (int32 ChannelIndex = 0; ChannelIndex < AttributeChannelGetters.Num(); ++ChannelIndex)
		{
			// e.g. "temperature_6"
			FString GridNameAndComponentIndex;
			bool bGetResult = (FactoryNode->*AttributeChannelGetters[ChannelIndex])(GridNameAndComponentIndex);
			if (!bGetResult || GridNameAndComponentIndex.IsEmpty())
			{
				continue;
			}

			FString GridName;
			FString ComponentIndexStr;
			bool bSplit = GridNameAndComponentIndex.Split(
				GridNameAndComponentIndexSeparator,
				&GridName,
				&ComponentIndexStr,
				ESearchCase::CaseSensitive,
				ESearchDir::FromEnd
			);
			if (!bSplit || GridName.IsEmpty() || ComponentIndexStr.IsEmpty())
			{
				continue;
			}

			int32* FoundGridIndex = GridNameToIndex.Find(GridName);
			if (!FoundGridIndex)
			{
				continue;
			}

			int32 ComponentIndex = INDEX_NONE;
			bool bLexed = LexTryParseString(ComponentIndex, *ComponentIndexStr);
			if (!bLexed)
			{
				continue;
			}

			int32 TextureIndex = ChannelIndex / 4;
			int32 ChannelComponentIndex = ChannelIndex % 4;

			FTextureInfo& TextureInfo = Result.Attributes[TextureIndex];
			FComponentMapping& Mapping = TextureInfo.Mappings[ChannelComponentIndex];
			Mapping.SourceGridIndex = *FoundGridIndex;
			Mapping.SourceComponentIndex = ComponentIndex;
		}

		return Result;
	}

	// Computes the combined AABB bounds for all the volume grids in the provided nodes.
	// The OpenVDB utils need this when retrieving the final payload data.
	void ComputeExpandedVolumeBounds(
		TArray<const UInterchangeVolumeNode*>& InVolumeNodes,
		const UInterchangeBaseNodeContainer* InNodeContainer,
		FIntVector3& OutVolumeBoundsMin,
		FIntVector3& OutVolumeBoundsMax
	)
	{
		FIntVector3 VolumeBoundsMin = FIntVector3(INT32_MAX, INT32_MAX, INT32_MAX);
		FIntVector3 VolumeBoundsMax = FIntVector3(INT32_MIN, INT32_MIN, INT32_MIN);

		for (const UInterchangeVolumeNode* VolumeNode : InVolumeNodes)
		{
			for (const UInterchangeVolumeGridNode* GridNode : GetGridNodes(VolumeNode, InNodeContainer))
			{
				if (!GridNode)
				{
					continue;
				}

				FIntVector Min;
				if (GridNode->GetCustomGridActiveAABBMin(Min))
				{
					FIntVector3 GridMin{Min};
					VolumeBoundsMin.X = FMath::Min(VolumeBoundsMin.X, GridMin.X);
					VolumeBoundsMin.Y = FMath::Min(VolumeBoundsMin.Y, GridMin.Y);
					VolumeBoundsMin.Z = FMath::Min(VolumeBoundsMin.Z, GridMin.Z);
				}

				FIntVector Max;
				if (GridNode->GetCustomGridActiveAABBMax(Max))
				{
					FIntVector3 GridMax{Max};
					VolumeBoundsMax.X = FMath::Min(VolumeBoundsMax.X, GridMax.X);
					VolumeBoundsMax.Y = FMath::Min(VolumeBoundsMax.Y, GridMax.Y);
					VolumeBoundsMax.Z = FMath::Min(VolumeBoundsMax.Z, GridMax.Z);
				}
			}
		}

		OutVolumeBoundsMin = VolumeBoundsMin;
		OutVolumeBoundsMax = VolumeBoundsMax;
	}

	// Sanity check that all the provided volume nodes have the same grid arrangement, with the same number
	// of components and data types. If this passes, we can make a single animated SVT from those volume frames.
	bool CheckForGridConsistency(const TArray<const UInterchangeVolumeNode*>& VolumeNodes, const UInterchangeBaseNodeContainer* NodeContainer)
	{
		if (VolumeNodes.Num() < 2)
		{
			// A single volume is always "consistent"
			return true;
		}

		struct FGridInfo
		{
			EVolumeGridElementType GridType = EVolumeGridElementType::Unknown;
			int32 NumComponents = 0;
		};

		// Get info from the first volume
		TMap<FString, FGridInfo> GridNameToInfo;
		{
			const UInterchangeVolumeNode* FirstVolume = VolumeNodes[0];
			for (const UInterchangeVolumeGridNode* GridNode : GetGridNodes(FirstVolume, NodeContainer))
			{
				FString DisplayLabel = GridNode->GetDisplayLabel();

				int32 NumComponents = 0;
				GridNode->GetCustomNumComponents(NumComponents);

				EVolumeGridElementType GridType = EVolumeGridElementType::Unknown;
				GridNode->GetCustomElementType(GridType);

				if (GridNameToInfo.Contains(DisplayLabel))
				{
					// Grid names should be unique within a volume
					return false;
				}
				GridNameToInfo.Add(DisplayLabel, FGridInfo{GridType, NumComponents});
			}
		}

		// Compare it with the other volumes
		for (int32 Index = 0; Index < VolumeNodes.Num(); ++Index)
		{
			const UInterchangeVolumeNode* OtherVolume = VolumeNodes[Index];

			TArray<const UInterchangeVolumeGridNode*> GridNodes = GetGridNodes(OtherVolume, NodeContainer);
			if (GridNodes.Num() != GridNameToInfo.Num())
			{
				// Should have the same number of grids as the first volume
				return false;
			}

			for (const UInterchangeVolumeGridNode* GridNode : GridNodes)
			{
				FString DisplayLabel = GridNode->GetDisplayLabel();

				int32 NumComponents = 0;
				GridNode->GetCustomNumComponents(NumComponents);

				EVolumeGridElementType GridType = EVolumeGridElementType::Unknown;
				GridNode->GetCustomElementType(GridType);

				const FGridInfo* FoundInfo = GridNameToInfo.Find(DisplayLabel);
				if (!FoundInfo)
				{
					// Grid is not present on the first volume
					return false;
				}

				if (FoundInfo->NumComponents != NumComponents || FoundInfo->GridType != GridType)
				{
					// Grid is different from the corresponding grid of the first volume
					return false;
				}
			}
		}

		return true;
	}
}	 // namespace UE::InterchangeSparseVolumeTextureFactory::Private

UClass* UInterchangeSparseVolumeTextureFactory::GetFactoryClass() const
{
	return USparseVolumeTexture::StaticClass();
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeSparseVolumeTextureFactory::BeginImportAsset_GameThread(
	const FImportAssetObjectParams& Arguments
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSparseVolumeTextureFactory::BeginImportAsset_GameThread);

	using namespace UE::Interchange;
	using namespace UE::InterchangeSparseVolumeTextureFactory::Private;

	FImportAssetResult ImportAssetResult;
	USparseVolumeTexture* SparseVolumeTexture = nullptr;

	auto HandleFailure = [this, &Arguments, &ImportAssetResult](const FText& Info)
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName = Arguments.SourceData->GetFilename();
		Message->DestinationAssetName = Arguments.AssetName;
		Message->AssetType = GetFactoryClass();
		Message->Text = FText::Format(
			LOCTEXT(
				"SparseVolumeTextureFactory_Failure",
				"UInterchangeSparseVolumeTextureFactory: Could not create SparseVolumeTexture asset '{0}'. Reason: {1}"
			),
			FText::FromString(Arguments.AssetName),
			Info
		);
		bSkipImport = true;
		ImportAssetResult.bIsFactorySkipAsset = true;
	};

	UInterchangeSparseVolumeTextureFactoryNode* FactoryNode = Cast<UInterchangeSparseVolumeTextureFactoryNode>(Arguments.AssetNode);
	if (!FactoryNode)
	{
		HandleFailure(
			LOCTEXT("SparseVolumeTextureFactory_AssetNodeNull", "Asset node parameter is not an UInterchangeSparseVolumeTextureFactoryNode.")
		);
		return ImportAssetResult;
	}

	const UClass* ObjectClass = Arguments.AssetNode->GetObjectClass();
	if (!ObjectClass || !ObjectClass->IsChildOf(USparseVolumeTexture::StaticClass()))
	{
		HandleFailure(
			LOCTEXT("SparseVolumeTextureFactory_NodeClassMissmatch", "Asset node parameter class doesn't derive the USparseVolumeTexture class.")
		);
		return ImportAssetResult;
	}

	// Get the source translated nodes for this volume (could have multiple if this is an animated SVT and we have multiple frames)
	TArray<const UInterchangeVolumeNode*> VolumeNodes = GetVolumeNodes(FactoryNode, Arguments.NodeContainer);
	if (VolumeNodes.Num() == 0)
	{
		HandleFailure(LOCTEXT("SparseVolumeTextureFactory_NoVolumes", "Asset node parameter class doesn't target any UInterchangeVolumeNode."));
		return ImportAssetResult;
	}

	// If we're trying to import an animation, the available grids must be identical for each volume frame
	if (VolumeNodes.Num() > 1)
	{
		bool bConsistentGrids = CheckForGridConsistency(VolumeNodes, Arguments.NodeContainer);
		if (!bConsistentGrids)
		{
			HandleFailure(LOCTEXT(
				"SparseVolumeTextureFactory_InconsistentGrids",
				"UInterchangeVolumeNodes provided for animated SparseVolumeTexture import don't have consistent volume grids"
			));
			return ImportAssetResult;
		}
	}

	UObject* ExistingAsset = Arguments.ReimportObject;
	if (!ExistingAsset)
	{
		FSoftObjectPath ReferenceObject;
		if (FactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}

		// If we're not reimporting this asset directly, and are instead importing/reimporting some other asset type and just
		// ran into this asset as a "dependency" (e.g. importing USD and found a volume already in the folder, or reimporting
		// a material and found the SVT already in the folder), then mark this as a "FactorySkipAsset" and just reuse it.
		// With bSkipImport == true the further stages won't modify the asset, just carry it along
		if (ExistingAsset && ExistingAsset->IsA(USparseVolumeTexture::StaticClass())
			&& Arguments.Translator->GetSupportedAssetTypes() != EInterchangeTranslatorAssetType::Textures)
		{
			bSkipImport = true;
			ImportAssetResult.bIsFactorySkipAsset = true;
			ImportAssetResult.ImportedObject = ExistingAsset;
		}
	}

	// Actually create a new asset
	if (!bSkipImport)
	{
		// Pick between static and animated SVT
		//
		// Note that SVT assets cannot be reused for reimport: They have an internal InitState flag that only lets
		// them ever receive their "source data" exactly once. The legacy factory reimports by calling NewObject with the
		// same class, name and outer, which resets the SVT. We'll do this here too, we just have to be careful to
		// pick the same class as the existing asset if we have one (during reimports): We will crash if we try
		// overwriting an animated SVT asset with a static one, for example.
		UClass* ClassToUse = ExistingAsset ? ExistingAsset->GetClass() : nullptr;
		if (!ClassToUse)
		{
			// Check the FactoryNode's attribute to decide this, as that is what is affected by the pipeline settings
			FString AnimationID;
			if (!FactoryNode->GetCustomAnimationID(AnimationID) || AnimationID.IsEmpty())
			{
				ClassToUse = UStaticSparseVolumeTexture::StaticClass();
			}
			else
			{
				ClassToUse = UAnimatedSparseVolumeTexture::StaticClass();
			}
		}

		// If we have a mismatch we should return now before we stomp the existing asset for no reason.
		// Note that it's fine to try and stuff a single frame payload into an animated SVT: It will just be a one-frame-animation
		if (ExistingAsset && ClassToUse == UStaticSparseVolumeTexture::StaticClass())
		{
			// Check whether our current volume nodes actually describe an animation or not
			bool bHasAnimation = VolumeNodes.Num() > 1;
			if (!bHasAnimation)
			{
				int32 NumFrames = 0;
				for (const UInterchangeVolumeNode* VolumeNode : VolumeNodes)
				{
					TArray<int32> FrameIndicesForVolume;
					VolumeNode->GetCustomFrameIndicesInAnimation(FrameIndicesForVolume);

					NumFrames += FrameIndicesForVolume.Num();
				}
				if (NumFrames > 1)
				{
					bHasAnimation = true;
				}
			}

			if (bHasAnimation)
			{
				HandleFailure(LOCTEXT("SparseVolumeTextureFactory_Mismatch", "Cannot import a volume animation into a StaticSparseVolumeTexture."));
				return ImportAssetResult;
			}
		}

#if WITH_EDITOR
		// We have to collect all the info we can from the previous asset before we stomp it for reimport, as we will need
		// to apply it back to the new asset
		TStrongObjectPtr<UAssetImportData> OldAssetImportData = nullptr;
#endif	  // WITH_EDITOR
		TArray<TStrongObjectPtr<UAssetUserData>> OldAssetUserData;

		if (UStreamableSparseVolumeTexture* OriginalTexture = Cast<UStreamableSparseVolumeTexture>(Arguments.ReimportObject))
		{
#if WITH_EDITOR
			if (OriginalTexture->AssetImportData)
			{
				OldAssetImportData.Reset(OriginalTexture->AssetImportData);

				UObject* NewOuter = GetTransientPackage();
				FName NewName = MakeUniqueObjectName(NewOuter, OriginalTexture->AssetImportData->GetClass());
				ensure(OldAssetImportData->Rename(*NewName.ToString(), NewOuter, REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty));

				OriginalTexture->AssetImportData = nullptr;
			}
#endif	  // WITH_EDITOR

			OldAssetUserData.Reserve(OriginalTexture->AssetUserData.Num());
			for (UAssetUserData* UserData : OriginalTexture->AssetUserData)
			{
				OldAssetUserData.Emplace(UserData);

				UObject* NewOuter = GetTransientPackage();
				FName NewName = MakeUniqueObjectName(NewOuter, UserData->GetClass());
				ensure(UserData->Rename(*NewName.ToString(), NewOuter, REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty));
			}
		}

		SparseVolumeTexture = NewObject<USparseVolumeTexture>(Arguments.Parent, ClassToUse, *Arguments.AssetName, RF_Public | RF_Standalone);
		if (!SparseVolumeTexture)
		{
			HandleFailure(LOCTEXT("SparseVolumeTextureFactory_TextureCreateFail", "SparseVolumeTexture creation failed."));
			return ImportAssetResult;
		}

#if WITH_EDITOR
		// Let's set these back right away and pretend they were always there, as that's what Interchange will expect of a reimported asset
		if (UStreamableSparseVolumeTexture* NewTexture = Cast<UStreamableSparseVolumeTexture>(SparseVolumeTexture))
		{
			if (OldAssetImportData)
			{
				NewTexture->AssetImportData = OldAssetImportData.Get();

				UObject* NewOuter = NewTexture;
				FName NewName = MakeUniqueObjectName(NewOuter, OldAssetImportData->GetClass());
				ensure(OldAssetImportData->Rename(*NewName.ToString(), NewOuter, REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty));
			}

			if (OldAssetUserData.Num() > 0)
			{
				NewTexture->AssetUserData.Reserve(OldAssetUserData.Num() + NewTexture->AssetUserData.Num());
				for (const TStrongObjectPtr<UAssetUserData>& UserData : OldAssetUserData)
				{
					NewTexture->AssetUserData.Add(UserData.Get());

					UObject* NewOuter = NewTexture;
					FName NewName = MakeUniqueObjectName(NewOuter, UserData->GetClass());
					ensure(UserData->Rename(*NewName.ToString(), NewOuter, REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty));
				}
			}
		}
#endif	  // WITH_EDITOR

		ImportAssetResult.ImportedObject = SparseVolumeTexture;
	}

	return ImportAssetResult;
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeSparseVolumeTextureFactory::ImportAsset_Async(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSparseVolumeTextureFactory::ImportAsset_Async);

	using namespace UE::Interchange;
	using namespace UE::Interchange::Volume;
	using namespace UE::InterchangeSparseVolumeTextureFactory::Private;

	FImportAssetResult ImportAssetResult;
	ImportAssetResult.bIsFactorySkipAsset = bSkipImport;

	auto HandleFailure = [this, &Arguments](const FText& Info)
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName = Arguments.SourceData->GetFilename();
		Message->DestinationAssetName = Arguments.AssetName;
		Message->AssetType = GetFactoryClass();
		Message->Text = Info;
	};

	if (!Arguments.AssetNode)
	{
		HandleFailure(
			LOCTEXT("SparseVolumeTextureFactory_Async_AssetNodeNull", "UInterchangeSparseVolumeTextureFactory: Asset node parameter is null.")
		);
		return ImportAssetResult;
	}

	UInterchangeSparseVolumeTextureFactoryNode* FactoryNode = Cast<UInterchangeSparseVolumeTextureFactoryNode>(Arguments.AssetNode);
	if (!FactoryNode)
	{
		HandleFailure(LOCTEXT(
			"SparseVolumeTextureFactory_Async_NodeWrongClass",
			"UInterchangeSparseVolumeTextureFactory: Asset node parameter is not a child of UInterchangeSparseVolumeTextureFactoryNode."
		));
		return ImportAssetResult;
	}

	UObject* CreatedAsset = nullptr;
	FSoftObjectPath ReferenceObject;
	if (Arguments.AssetNode->GetCustomReferenceObject(ReferenceObject))
	{
		CreatedAsset = ReferenceObject.TryLoad();
	}

	// Do not override an asset we skip
	if (bSkipImport)
	{
		ImportAssetResult.ImportedObject = CreatedAsset;
		return ImportAssetResult;
	}

	USparseVolumeTexture* SparseVolumeTexture = Cast<USparseVolumeTexture>(CreatedAsset);
	if (!SparseVolumeTexture)
	{
		HandleFailure(LOCTEXT(
			"SparseVolumeTextureFactory_Async_CannotCreateAsync",
			"UInterchangeSparseVolumeTextureFactory: Could not create an USparseVolumeTexture asset."
		));
		return ImportAssetResult;
	}

	const IInterchangeVolumePayloadInterface* TranslatorInterface = Cast<IInterchangeVolumePayloadInterface>(Arguments.Translator);
	if (!TranslatorInterface)
	{
		HandleFailure(LOCTEXT(
			"SparseVolumeTextureFactory_Async_TranslatorPayloadInterface",
			"UInterchangeSparseVolumeTextureFactory: The translator does not implement the Interchange volume payload interface."
		));
		return ImportAssetResult;
	}

	// Get the source translated nodes for this volume (could have multiple if this is an animated SVT and we have multiple frames)
	TArray<const UInterchangeVolumeNode*> VolumeNodes = GetVolumeNodes(FactoryNode, Arguments.NodeContainer);
	if (VolumeNodes.Num() == 0)
	{
		HandleFailure(
			LOCTEXT("SparseVolumeTextureFactory_Async_NoTranslatedNode", "Asset node parameter class doesn't target any UInterchangeVolumeNode.")
		);
		return ImportAssetResult;
	}

	// Produce an FAssignmentInfo we'll reuse for all payload keys below
	TOptional<FAssignmentInfo> AssignmentInfo = GetAssignmentInfo(FactoryNode, VolumeNodes, Arguments.NodeContainer);
	if (!AssignmentInfo.IsSet())
	{
		HandleFailure(LOCTEXT(
			"SparseVolumeTextureFactory_Async_NoAssignmentInfo",
			"Failed to extract a valid grid assignment info from UInterchangeVolumeNode."
		));
		return ImportAssetResult;
	}

	// Get a single volume bounds we'll share across all payloads
	FIntVector3 VolumeBoundsMin;
	FIntVector3 VolumeBoundsMax;
	ComputeExpandedVolumeBounds(VolumeNodes, Arguments.NodeContainer, VolumeBoundsMin, VolumeBoundsMax);

	bool bAtLeastOneValidPayloadKey = false;

	int32 NumFrames = 0;

	TArray<FVolumePayloadKey> PayloadKeys;
	PayloadKeys.Reserve(VolumeNodes.Num());

	TArray<FVolumePayload> VolumePayloads;
	VolumePayloads.Reserve(VolumeNodes.Num());

	// Collect the payload keys from each volume node
	for (const UInterchangeVolumeNode* VolumeNode : VolumeNodes)
	{
		// Emplace even if we fail so that we keep a one-to-one match with the VolumeNodes array
		FVolumePayloadKey& Key = PayloadKeys.Emplace_GetRef();
		FVolumePayload& Payload = VolumePayloads.Emplace_GetRef();

		Payload.VolumeNodeUid = VolumeNode->GetUniqueID();

		FString FileName;
		bool bSuccess = VolumeNode->GetCustomFileName(FileName);
		if (!bSuccess || FileName.IsEmpty())
		{
			continue;
		}

		bAtLeastOneValidPayloadKey = true;
		Key.FileName = FileName;
		Key.AssignmentInfo = AssignmentInfo.GetValue();
		Key.VolumeBoundsMin = VolumeBoundsMin;

		VolumeNode->GetCustomFrameIndicesInAnimation(Payload.PayloadFrameIndices);

		if (Payload.PayloadFrameIndices.Num() == 0)
		{
			// No animated frames --> Volume node wants to become a static SVT. Let's consider it as one frame,
			// as that is how we'll process its payload later in FillSparseVolumeTextureWithPayloadData()
			NumFrames += 1;
		}
		else
		{
			NumFrames += Payload.PayloadFrameIndices.Num();
		}
	}
	if (!bAtLeastOneValidPayloadKey)
	{
		HandleFailure(
			LOCTEXT("SparseVolumeTextureFactory_InvalidPayloadKey", "None of the translated UInterchangeVolumeGridNodes has a valid payload key.")
		);
		return ImportAssetResult;
	}

	// Fetch the actual payloads in parallel (presumably they're all different files, but it should work even in the edge case
	// that the same file shows up more than once somehow)
	const int32 NumBatches = PayloadKeys.Num();
	ParallelFor(
		NumBatches,
		[&PayloadKeys, &VolumePayloads, TranslatorInterface](int32 Index)
		{
			VolumePayloads[Index].PayloadData = TranslatorInterface->GetVolumePayloadData(PayloadKeys[Index]);
		}
	);

	bool bAtLeastOneValidPayloadData = false;
	for (const FVolumePayload& VolumePayload : VolumePayloads)
	{
		if (VolumePayload.PayloadData.IsSet())
		{
			bAtLeastOneValidPayloadData = true;
			break;
		}
	}
	if (!bAtLeastOneValidPayloadData)
	{
		HandleFailure(LOCTEXT(
			"SparseVolumeTextureFactory_Async_CannotRetrievePayload",
			"UInterchangeSparseVolumeTextureFactory: The factory couldn't retrieve any valid payload from the translator."
		));
		return ImportAssetResult;
	}

	// Check if the payload is valid for the Texture
	bool bCanSetup = bAtLeastOneValidPayloadData;
	if (UStaticSparseVolumeTexture* StaticVolumeTexture = Cast<UStaticSparseVolumeTexture>(SparseVolumeTexture))
	{
		bCanSetup = bCanSetup && NumFrames == 1;
	}
	else if (UAnimatedSparseVolumeTexture* AnimatedVolumeTexture = Cast<UAnimatedSparseVolumeTexture>(SparseVolumeTexture))
	{
		bCanSetup = bCanSetup && NumFrames >= 1;
	}
	if (!bCanSetup)
	{
		HandleFailure(LOCTEXT(
			"SparseVolumeTextureFactory_Async_CannotSetup",
			"UInterchangeSparseVolumeTextureFactory: The factory cannot setup the created asset with the provided payload data."
		));
		return ImportAssetResult;
	}

#if WITH_EDITOR
	// Create SourceFile hashes while we're still in an async thread (we'll move this into AssetImportData later)
	{
		// Just hash one file though, as that's what the standard SVT importer seems to also do
		SourceFiles.Empty(1);
		FAssetImportInfo::FSourceFile& SourceFile = SourceFiles.Emplace_GetRef();
		SourceFile.RelativeFilename = Arguments.SourceData->GetFilename();
		HashSourceFiles(SourceFiles);
	}
#endif	  // WITH_EDITOR

	ProcessedPayloads = MoveTemp(VolumePayloads);
	return ImportAssetResult;
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeSparseVolumeTextureFactory::EndImportAsset_GameThread(
	const FImportAssetObjectParams& Arguments
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSparseVolumeTextureFactory::ImportAsset_Async);

	using namespace UE::InterchangeSparseVolumeTextureFactory::Private;

	FImportAssetResult ImportAssetResult;

	// If we're not meant to modify the asset just return early
	if (bSkipImport)
	{
		ImportAssetResult.bIsFactorySkipAsset = true;
		return ImportAssetResult;
	}

	// We only handle SVT factory nodes
	UInterchangeSparseVolumeTextureFactoryNode* FactoryNode = Cast<UInterchangeSparseVolumeTextureFactoryNode>(Arguments.AssetNode);
	if (!FactoryNode)
	{
		return ImportAssetResult;
	}

	// Get the asset we're currently importing
	USparseVolumeTexture* ImportedSparseVolumeTexture = nullptr;
	{
		FSoftObjectPath ReferenceObject;
		if (FactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ImportedSparseVolumeTexture = Cast<USparseVolumeTexture>(ReferenceObject.TryLoad());
		}

		if (!ImportedSparseVolumeTexture)
		{
			return ImportAssetResult;
		}
	}

	const bool bIsReimport = Arguments.ReimportObject != nullptr;

	if (HasValidPayloads())
	{
		ImportAssetResult.ImportedObject = ImportedSparseVolumeTexture;
	}
	// Abandon asset if it failed to import
	else
	{
		// Dispose of asset if it's not a reimport
		if (!bIsReimport)
		{
			ImportedSparseVolumeTexture->RemoveFromRoot();
			ImportedSparseVolumeTexture->MarkAsGarbage();
			FactoryNode->SetCustomReferenceObject(nullptr);
		}

		FactoryNode->SetEnabled(false);

		bSkipImport = true;
		ImportAssetResult.ImportedObject = nullptr;
		ImportAssetResult.bIsFactorySkipAsset = bSkipImport;
		return ImportAssetResult;
	}

	FillSparseVolumeTextureWithPayloadData(ImportedSparseVolumeTexture, ProcessedPayloads);

#if WITH_EDITOR
	if (bIsReimport)
	{
		if (UStreamableSparseVolumeTexture* OriginalTexture = Cast<UStreamableSparseVolumeTexture>(Arguments.ReimportObject))
		{
			if (UInterchangeAssetImportData* AssetImportData = Cast<UInterchangeAssetImportData>(OriginalTexture->AssetImportData))
			{
				UInterchangeFactoryBaseNode* PreviousNode = AssetImportData->GetStoredFactoryNode(AssetImportData->NodeUniqueID);

				// Create a factory node filled with the property values we can extract from our new imported asset
				UInterchangeFactoryBaseNode* CurrentNode = UInterchangeFactoryBaseNode::DuplicateWithObject(FactoryNode, ImportedSparseVolumeTexture);

				UE::Interchange::FFactoryCommon::ApplyReimportStrategyToAsset(ImportedSparseVolumeTexture, PreviousNode, CurrentNode, FactoryNode);
			}
		}
	}
	else
	{
		FactoryNode->ApplyAllCustomAttributeToObject(ImportedSparseVolumeTexture);
	}
#endif	  // WITH_EDITOR

	return ImportAssetResult;
}

void UInterchangeSparseVolumeTextureFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSparseVolumeTextureFactory::SetupObject_GameThread);

	using namespace UE::InterchangeSparseVolumeTextureFactory::Private;

	if (bSkipImport)
	{
		return;
	}

	Super::SetupObject_GameThread(Arguments);

#if WITH_EDITOR
	// Setup asset import data
	//
	// The Streamable derived class is the one that stores the AssetImportData, but all SVT classes that are actual assets
	// derive it, so it should be safe to assume we have a Streamable here
	UStreamableSparseVolumeTexture* Streamable = Cast<UStreamableSparseVolumeTexture>(Arguments.ImportedObject);
	if (Streamable && Arguments.SourceData && HasValidPayloads())
	{
		// We must call the Update of the asset source file in the main thread because UAssetImportData::Update execute some delegate we do not
		// control
		UE::Interchange::FFactoryCommon::FSetImportAssetDataParameters SetImportAssetDataParameters(
			Streamable,
			Streamable->AssetImportData,
			Arguments.SourceData,
			Arguments.NodeUniqueID,
			Arguments.NodeContainer,
			Arguments.OriginalPipelines,
			Arguments.Translator
		);
		SetImportAssetDataParameters.SourceFiles = MoveTemp(SourceFiles);

		Streamable->AssetImportData = UE::Interchange::FFactoryCommon::SetImportAssetData(SetImportAssetDataParameters);
	}
#endif	  // WITH_EDITOR

	// Temp workaround for UE-255278: It seems that existing HVComponents that are using our SVT will just stop rendering the volume
	// after we reimport the SVT here, until we trigger a refresh by e.g. reseating the material like we do here, or by opening the material editor.
	// This seems to be something related to HVComponents and not SVTs directly, as the SVTViewer component doesn't show this issue, but we have 
	// not yet narrowed down what the actual root cause is
	if (Arguments.bIsReimport)
	{
		for (TObjectIterator<UHeterogeneousVolumeComponent> It; It; ++It)
		{
			// Ideally we'd check whether this HVComponent is using this texture that was reimported, but there's no
			// public interface for that and honestly it may take even longer to check than to just refresh them all anyway
			if (UMaterialInterface* Material = It->GetMaterial(0))
			{
				It->SetMaterial(0, Material);
				It->MarkRenderStateDirty();
			}
		}
	}
}

bool UInterchangeSparseVolumeTextureFactory::GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const
{
#if WITH_EDITOR
	if (const UStreamableSparseVolumeTexture* Streamable = Cast<UStreamableSparseVolumeTexture>(Object))
	{
		return UE::Interchange::FFactoryCommon::GetSourceFilenames(Streamable->AssetImportData.Get(), OutSourceFilenames);
	}
#endif

	return false;
}

bool UInterchangeSparseVolumeTextureFactory::SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const
{
#if WITH_EDITOR
	if (const UStreamableSparseVolumeTexture* Streamable = Cast<UStreamableSparseVolumeTexture>(Object))
	{
		return UE::Interchange::FFactoryCommon::SetSourceFilename(Streamable->AssetImportData.Get(), SourceFilename, SourceIndex);
	}
#endif

	return false;
}

void UInterchangeSparseVolumeTextureFactory::BackupSourceData(const UObject* Object) const
{
#if WITH_EDITOR
	if (const UStreamableSparseVolumeTexture* Streamable = Cast<UStreamableSparseVolumeTexture>(Object))
	{
		UE::Interchange::FFactoryCommon::BackupSourceData(Streamable->AssetImportData.Get());
	}
#endif
}

void UInterchangeSparseVolumeTextureFactory::ReinstateSourceData(const UObject* Object) const
{
#if WITH_EDITOR
	if (const UStreamableSparseVolumeTexture* Streamable = Cast<UStreamableSparseVolumeTexture>(Object))
	{
		UE::Interchange::FFactoryCommon::ReinstateSourceData(Streamable->AssetImportData.Get());
	}
#endif
}

void UInterchangeSparseVolumeTextureFactory::ClearBackupSourceData(const UObject* Object) const
{
#if WITH_EDITOR
	if (const UStreamableSparseVolumeTexture* Streamable = Cast<UStreamableSparseVolumeTexture>(Object))
	{
		UE::Interchange::FFactoryCommon::ClearBackupSourceData(Streamable->AssetImportData.Get());
	}
#endif
}

bool UInterchangeSparseVolumeTextureFactory::HasValidPayloads() const
{
	for (const FVolumePayload& Payload : ProcessedPayloads)
	{
		if (Payload.PayloadData.IsSet())
		{
			return true;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
