// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeCommonAnimationPayload.h"
#include "InterchangeFactoryBase.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeAnimSequenceFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

class UAnimSequence;
class UInterchangeAnimSequenceFactoryNode;
class UInterchangeSceneNode;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeAnimSequenceFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()
public:

	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() override { return EInterchangeFactoryAssetType::Animations; }
	
	UE_API virtual void CreatePayloadTasks(const FImportAssetObjectParams& Arguments, bool bAsync, TArray<TSharedPtr<UE::Interchange::FInterchangeTaskBase>>& PayloadTasks) override;

	UE_API virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	
	UE_API virtual FImportAssetResult ImportAsset_Async(const FImportAssetObjectParams& Arguments) override;
	
	UE_API virtual FImportAssetResult EndImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;

	UE_API virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;
	UE_API virtual void BuildObject_GameThread(const FSetupObjectParams& Arguments, bool& OutPostEditchangeCalled) override;

	UE_API virtual bool GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const override;
	UE_API virtual bool SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const override;
	UE_API virtual void BackupSourceData(const UObject* Object) const override;
	UE_API virtual void ReinstateSourceData(const UObject* Object) const override;
	UE_API virtual void ClearBackupSourceData(const UObject* Object) const override;
	
	struct FBoneTrackData
	{
		TMap<const UInterchangeSceneNode*, UE::Interchange::FAnimationPayloadData> PreProcessedAnimationPayloads;
		double MergedRangeStart = 0.0;
		double MergedRangeEnd = 0.0;
	};

	struct FMorphTargetData
	{
		TMap<FString, UE::Interchange::FAnimationPayloadData> CurvesPayloads;
		TMap<FString, FString> CurveNodeNamePerPayloadKey;
	};

private:
	UE_API bool IsBoneTrackAnimationValid(const UInterchangeAnimSequenceFactoryNode* AnimSequenceFactoryNode, const FImportAssetObjectParams& Arguments);

	//The imported AnimSequence
	TObjectPtr<UAnimSequence> AnimSequence = nullptr;

	//Bone track animations payload data
	FBoneTrackData BoneTrackData;

	//Morph target curves payload data
	FMorphTargetData MorphTargetData;

	//Animation queries
	TMap< TTuple<FString, FInterchangeAnimationPayLoadKey>, UE::Interchange::FAnimationPayloadQuery> BoneAnimationPayloadQueries;
	
	//Animation queries results
	TMap< TTuple<FString, FInterchangeAnimationPayLoadKey>, UE::Interchange::FAnimationPayloadData> BoneAnimationPayloadResults;

	friend bool operator==(const TTuple<FString, FInterchangeAnimationPayLoadKey>& A, const TTuple<FString, FInterchangeAnimationPayLoadKey>& B)
	{
		return A.Key.Equals(B.Key) && A.Value == B.Value;
	}

	friend uint32 GetTypeHash(const TTuple<FString, FInterchangeAnimationPayLoadKey>& Tupple)
	{
		return HashCombine(GetTypeHash(Tupple.Key), GetTypeHash(Tupple.Value));
	}

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////
};


#undef UE_API
