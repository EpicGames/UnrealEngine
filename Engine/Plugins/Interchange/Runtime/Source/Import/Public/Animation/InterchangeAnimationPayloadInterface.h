// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Async/Future.h"
#include "CoreMinimal.h"
#include "Misc/FrameRate.h"
#include "UObject/Interface.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "InterchangeAnimationTrackSetNode.h"
#include "InterchangeCommonAnimationPayload.h"

#include "InterchangeAnimationPayloadInterface.generated.h"

UINTERFACE(MinimalAPI)
class UInterchangeAnimationPayloadInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Animation payload interface. Derive from this interface if your payload can import skeletal mesh
 */
class IInterchangeAnimationPayloadInterface
{
	GENERATED_BODY()
public:

	/**
	 * Return true if the translator want to import all bone animations in one query, false otherwise.
	 * 
	 * @Note - The fbx translator use the fbx sdk which cache the global transform but dirty the cache every time we evaluate at a different time. The goal is to evaluate all bones at the same time.
	 */
	virtual bool PreferGroupingBoneAnimationQueriesTogether() const
	{
		return false;
	}

	/**
	 * Get animation payload data for the specified payload key.
	 * It return an array of FRichCurve (Rich curve are float curve we can interpolate) or array of "Step" curve or an array of Baked Transformations, depending on Type
	 *
	 * @param PayloadQueries - A PayloadQuery contains all the necessary data for a Query to be processed (Including SceneNodeUID, PayloadKey, TimeDescription).
	 * @return - The resulting PayloadData.
	 * 
	 */
	virtual TArray<UE::Interchange::FAnimationPayloadData> GetAnimationPayloadData(const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQuery) const = 0;
};


