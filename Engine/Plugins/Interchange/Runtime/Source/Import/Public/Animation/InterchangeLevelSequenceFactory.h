// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeFactoryBase.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeLevelSequenceFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeLevelSequenceFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()
public:

	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() override { return EInterchangeFactoryAssetType::Animations; }
	UE_API virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////

private:
	const UInterchangeTranslatorBase* Translator = nullptr;

	UE_API UObject* ImportObjectSourceData(const FImportAssetObjectParams& Arguments);
};


#undef UE_API
