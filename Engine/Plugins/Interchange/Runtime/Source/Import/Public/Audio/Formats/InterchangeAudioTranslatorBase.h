// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "InterchangeTranslatorBase.h"
#include "Audio/InterchangeAudioPayloadInterface.h"

#include "InterchangeAudioTranslatorBase.generated.h"

#define UE_API INTERCHANGEIMPORT_API

class UInterchangeAudioSoundWaveNode;

UCLASS(MinimalAPI, BlueprintType, Abstract)
class UInterchangeAudioTranslatorBase
	: public UInterchangeTranslatorBase
	, public IInterchangeAudioPayloadInterface
{
	GENERATED_BODY()

public:
	/** Begin UInterchangeTranslatorBase API */
	UE_API virtual TArray<FString> GetSupportedFormats() const override;
	UE_API virtual EInterchangeTranslatorAssetType GetSupportedAssetTypes() const override;
	UE_API virtual bool Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const override;
	/** End UInterchangeTranslatorBase API */
protected:
	UE_API virtual UInterchangeAudioSoundWaveNode* Translate_Internal(UInterchangeBaseNodeContainer& BaseNodeContainer) const;

protected:
	UE_API void LogError(FText&& ErrorText)  const;
	UE_API void LogWarning(FText&& WarningText) const;
};

#undef UE_API
