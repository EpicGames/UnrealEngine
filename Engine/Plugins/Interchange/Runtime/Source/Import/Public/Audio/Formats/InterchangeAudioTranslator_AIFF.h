// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Audio/Formats/InterchangeAudioTranslatorBase.h"

#include "InterchangeAudioTranslator_AIFF.generated.h"

#define UE_API INTERCHANGEIMPORT_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeAudioTranslator_AIFF : public UInterchangeAudioTranslatorBase
{
	GENERATED_BODY()
public:
	UE_API virtual TArray<FString> GetSupportedFormats() const override;
};

#undef UE_API
