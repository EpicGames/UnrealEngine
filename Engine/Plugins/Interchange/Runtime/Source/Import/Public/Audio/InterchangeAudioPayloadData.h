// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Containers/Array.h"
#include "Misc/Optional.h"
#include "Audio.h"

#define UE_API INTERCHANGEIMPORT_API

namespace UE::Interchange
{
	struct FSoundWaveFactoryInfo
	{
		int32 ChannelCount = 0;
		int32 SizeOfSample = 0;
		int32 NumSamples = 0;
		int32 NumFrames = 0;
		int32 SamplesPerSec = 0;
		int32 SampleDataOffset = -1;
	};

	struct FInterchangeAudioPayloadData
	{
		FSoundWaveFactoryInfo FactoryInfo;
		FWaveModInfo WaveModInfo;
		TArray<uint8> Buffer;
	};
}

struct FInterchangeAudioPayloadDataUtils
{
public:
	static UE_API TOptional<UE::Interchange::FInterchangeAudioPayloadData> GetAudioPayloadFromSourceFileKey(const FString& PayloadSourceFileKey);
};

#undef UE_API