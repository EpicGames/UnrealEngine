// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Optional.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "InterchangeAudioPayloadData.h"

#include "InterchangeAudioPayloadInterface.generated.h"

#define UE_API INTERCHANGEIMPORT_API

UINTERFACE(MinimalAPI)
class UInterchangeAudioPayloadInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Audio Payload interface. Derive from it if your payload can import Audio
 */
class IInterchangeAudioPayloadInterface
{
	GENERATED_BODY()
public:
	/**
	 * Once the translation is done, the import process need a way to retrieve payload data.
	 * This payload will be use by the factories to create the asset.
	 *
	 * @param PayloadKey - The key to retrieve the a particular payload contain into the specified source data.
	 * @return a PayloadData containing the import sound wave data, this maybe converted data if the format is other than '.wav'. The TOptional will not be set if there is an error.
	 */
	virtual TOptional< UE::Interchange::FInterchangeAudioPayloadData > GetAudioPayloadData(const FString& PayloadSourceFileKey) const 
	{
		return FInterchangeAudioPayloadDataUtils::GetAudioPayloadFromSourceFileKey(PayloadSourceFileKey);
	}
};


#undef UE_API