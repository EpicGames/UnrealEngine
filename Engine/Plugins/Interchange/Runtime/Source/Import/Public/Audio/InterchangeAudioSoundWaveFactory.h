// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeFactoryBase.h"
#include "InterchangeAudioPayloadData.h"
#include "InterchangeAudioSoundWaveFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

class UAudioComponent;
class USoundWave;
class FWaveModInfo;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeAudioSoundWaveFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()

public:
	/** UInterchangeFactoryBase interface */
	UE_API virtual UClass* GetFactoryClass() const override;
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() override { return EInterchangeFactoryAssetType::Sounds; }
	UE_API virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual FImportAssetResult ImportAsset_Async(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual FImportAssetResult EndImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;

	UE_API virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;
	UE_API virtual void FinalizeObject_GameThread(const FSetupObjectParams& Arguments) override;

	// To allow reimporting
	UE_API virtual bool GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const override;
	UE_API virtual bool SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const override;
	UE_API virtual void BackupSourceData(const UObject* Object) const override;
	UE_API virtual void ReinstateSourceData(const UObject* Object) const override;
	UE_API virtual void ClearBackupSourceData(const UObject* Object) const override;

private:
	UE_API void SetUpSoundWaveFromPayload(const FImportAssetObjectParams& Arguments, USoundWave*& SoundWaveAsset, const UE::Interchange::FInterchangeAudioPayloadData& AudioPayloadData, FImportAssetResult& ImportAssetResult);
	UE_API bool SetUpMultichannelSoundWave(const FImportAssetObjectParams& Arguments, USoundWave*& SoundWaveAsset, const UE::Interchange::FInterchangeAudioPayloadData& AudioPayloadData, FImportAssetResult& ImportAssetResult);
	
	UE_API void StopComponentsUsingImportedSound(USoundWave* SoundWaveAsset);
	
	UE_API void LogError(const FImportAssetObjectParams& Arguments, FImportAssetResult& ImportAssetResult, const FText& ErrorText);

	UPROPERTY(Transient)
	TArray<TObjectPtr<UAudioComponent>> ComponentsToRestart;
};

#undef UE_API
