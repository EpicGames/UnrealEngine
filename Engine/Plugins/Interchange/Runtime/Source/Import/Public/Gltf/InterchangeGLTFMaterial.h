// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace GLTF {
	struct FMaterial;
	struct FTexture;
}
class UInterchangeShaderGraphNode;
class UInterchangeBaseNodeContainer;

INTERCHANGEIMPORT_API extern const FString InterchangeGltfMaterialAttributeIdentifier;

//OffsetScale for MaterialInstances
//Offset_X Offset_Y Scale_X Scale_Y for Materials
#define INTERCHANGE_GLTF_STRINGIFY(x) #x
#define DECLARE_INTERCHANGE_GLTF_MI_MAP(MapName) inline const FString MapName##Texture = TEXT(INTERCHANGE_GLTF_STRINGIFY(MapName ## Texture)); \
												 inline const FString MapName##Texture_OffsetScale = TEXT(INTERCHANGE_GLTF_STRINGIFY(MapName ## Texture_OffsetScale)); \
												 inline const FString MapName##Texture_Offset_X = TEXT(INTERCHANGE_GLTF_STRINGIFY(MapName ## Texture_Offset_X)); \
												 inline const FString MapName##Texture_Offset_Y = TEXT(INTERCHANGE_GLTF_STRINGIFY(MapName ## Texture_Offset_Y)); \
												 inline const FString MapName##Texture_Scale_X = TEXT(INTERCHANGE_GLTF_STRINGIFY(MapName ## Texture_Scale_X)); \
												 inline const FString MapName##Texture_Scale_Y = TEXT(INTERCHANGE_GLTF_STRINGIFY(MapName ## Texture_Scale_Y)); \
												 inline const FString MapName##Texture_Rotation = TEXT(INTERCHANGE_GLTF_STRINGIFY(MapName ## Texture_Rotation)); \
												 inline const FString MapName##Texture_TexCoord = TEXT(INTERCHANGE_GLTF_STRINGIFY(MapName ## Texture_TexCoord)); \
												 inline const FString MapName##Texture_TilingMethod = TEXT(INTERCHANGE_GLTF_STRINGIFY(MapName ## Texture_TilingMethod));

namespace UE::Interchange::GLTFMaterials
{
	//Inputs/Parameters (Materials/MaterialInstances)
	namespace Inputs
	{
		///PostFixes
		namespace PostFix
		{
			inline const FString Color_RGB = TEXT("_RGB");
			inline const FString Color_A = TEXT("_A");

			inline const FString TexCoord = TEXT("_TexCoord");

			inline const FString OffsetX = TEXT("_Offset_X");
			inline const FString OffsetY = TEXT("_Offset_Y");
			inline const FString ScaleX = TEXT("_Scale_X");
			inline const FString ScaleY = TEXT("_Scale_Y");

			inline const FString OffsetScale = TEXT("_OffsetScale");

			inline const FString Rotation = TEXT("_Rotation");

			inline const FString TilingMethod = TEXT("_TilingMethod");
		}

		//MetalRoughness specific:
		DECLARE_INTERCHANGE_GLTF_MI_MAP(BaseColor)
		inline const FString BaseColorFactor = TEXT("BaseColorFactor");
		inline const FString BaseColorFactor_RGB = BaseColorFactor + PostFix::Color_RGB; //Connection to inputs from BaseColorFactor.RGB
		inline const FString BaseColorFactor_A = BaseColorFactor + PostFix::Color_A; //Connection to inputs from BaseColorFactor.A

		DECLARE_INTERCHANGE_GLTF_MI_MAP(MetallicRoughness)
		inline const FString MetallicFactor = TEXT("MetallicFactor");
		inline const FString RoughnessFactor = TEXT("RoughnessFactor");

		DECLARE_INTERCHANGE_GLTF_MI_MAP(Specular)
		inline const FString SpecularFactor = TEXT("SpecularFactor");

		DECLARE_INTERCHANGE_GLTF_MI_MAP(SpecularColor)
		inline const FString SpecularColorFactor = TEXT("SpecularColorFactor");

		//SpecularGlossiness specific
		DECLARE_INTERCHANGE_GLTF_MI_MAP(Diffuse)
		inline const FString DiffuseFactor = TEXT("DiffuseFactor");
		inline const FString DiffuseFactor_RGB = DiffuseFactor + PostFix::Color_RGB; //Connection to inputs from BaseColorFactor.RGB
		inline const FString DiffuseFactor_A = DiffuseFactor + PostFix::Color_A; //Connection to inputs from BaseColorFactor.A

		DECLARE_INTERCHANGE_GLTF_MI_MAP(SpecularGlossiness)
		inline const FString SpecFactor = TEXT("SpecFactor");
		inline const FString GlossinessFactor = TEXT("GlossinessFactor");


		//Generic:
		DECLARE_INTERCHANGE_GLTF_MI_MAP(Normal)
		inline const FString NormalScale = TEXT("NormalScale");

		DECLARE_INTERCHANGE_GLTF_MI_MAP(Emissive)
		inline const FString EmissiveFactor = TEXT("EmissiveFactor");
		inline const FString EmissiveStrength = TEXT("EmissiveStrength");

		DECLARE_INTERCHANGE_GLTF_MI_MAP(Occlusion)
		inline const FString OcclusionStrength = TEXT("OcclusionStrength");

		inline const FString IOR = TEXT("IOR");

		inline const FString AlphaCutoff = TEXT("AlphaCutoff");
		inline const FString AlphaMode = TEXT("AlphaMode");

		//ClearCoat specific:
		DECLARE_INTERCHANGE_GLTF_MI_MAP(ClearCoat)
		inline const FString ClearCoatFactor = TEXT("ClearCoatFactor");

		DECLARE_INTERCHANGE_GLTF_MI_MAP(ClearCoatRoughness)
		inline const FString ClearCoatRoughnessFactor = TEXT("ClearCoatRoughnessFactor");

		DECLARE_INTERCHANGE_GLTF_MI_MAP(ClearCoatNormal)
		inline const FString ClearCoatNormalScale = TEXT("ClearCoatNormalScale");


		//Sheen specific:
		DECLARE_INTERCHANGE_GLTF_MI_MAP(SheenColor)
		inline const FString SheenColorFactor = TEXT("SheenColorFactor");

		DECLARE_INTERCHANGE_GLTF_MI_MAP(SheenRoughness)
		inline const FString SheenRoughnessFactor = TEXT("SheenRoughnessFactor");


		//Transmission specific:
		DECLARE_INTERCHANGE_GLTF_MI_MAP(Transmission)
		inline const FString TransmissionFactor = TEXT("TransmissionFactor");


		//Iridescence specific:
		inline const FString IridescenceIOR = TEXT("IridescenceIOR");
		
		DECLARE_INTERCHANGE_GLTF_MI_MAP(Iridescence)
		inline const FString IridescenceFactor = TEXT("IridescenceFactor");

		DECLARE_INTERCHANGE_GLTF_MI_MAP(IridescenceThickness)
		inline const FString IridescenceThicknessMinimum = TEXT("IridescenceThicknessMinimum");
		inline const FString IridescenceThicknessMaximum = TEXT("IridescenceThicknessMaximum");


		//Anisotropy Specific:
		DECLARE_INTERCHANGE_GLTF_MI_MAP(Anisotropy);
		inline const FString AnisotropyStrength = TEXT("AnisotropyStrength");
		inline const FString AnisotropyRotation = TEXT("AnisotropyRotation");

		//Volume Specific:
		DECLARE_INTERCHANGE_GLTF_MI_MAP(Thickness);
		inline const FString ThicknessFactor = TEXT("ThicknessFactor");
		inline const FString AttenuationDistance = TEXT("AttenuationDistance");
		inline const FString AttenuationColor = TEXT("AttenuationColor");

		namespace Configuration
		{
			//For StaticSwitch optimizations:
			// Note: only usable in Editor, the Default values are true so that Runtime will still use the complete graph (without optimization).
			inline const FString bHasBaseColorTexture = TEXT("bHasBaseColorTexture");
			inline const FString bHasMetallicRoughnessTexture = TEXT("bHasMetallicRoughnessTexture");
			inline const FString bHasDiffuseSpecGlossTexture = TEXT("bHasDiffuseSpecGlossTexture");
			inline const FString bHasEmissiveTexture = TEXT("bHasEmissiveTexture");
			inline const FString bHasNormalTexture = TEXT("bHasNormalTexture");
			inline const FString bHasOcclusionTexture = TEXT("bHasOcclusionTexture");
			inline const FString bHasSpecularTexture = TEXT("bHasSpecularTexture");
			inline const FString bHasSpecularColorTexture = TEXT("bHasSpecularColorTexture");
			inline const FString bHasClearCoatTexture = TEXT("bHasClearCoatTexture");
			inline const FString bHasSheenTexture = TEXT("bHasSheenTexture");
			inline const FString bHasTransmissionTexture = TEXT("bHasTransmissionTexture");
			inline const FString bHasIridescence = TEXT("bHasIridescence");
			inline const FString bHasAnisotropyTextureAndOrRotation = TEXT("bHasAnisotropyTextureAndOrRotation"); //AnistropyRotation can affect the Tangents so we check
			inline const FString bHasThicknessTexture = TEXT("bHasThicknessTexture");
		}
	}

	enum EShadingModel : uint8
	{
		DEFAULT = 0, //MetalRoughness
		UNLIT,
		CLEARCOAT,
		SHEEN,
		TRANSMISSION,
		SPECULARGLOSSINESS,
		SHADINGMODELCOUNT
	};

	enum EAlphaMode : uint8
	{
		Opaque = 0,
		Mask,
		Blend
	};

	struct FGLTFMaterialInformation
	{
		FString MaterialFunctionPath;
		FString MaterialPath;
		TArray<FString> MaterialFunctionOutputs;

		FGLTFMaterialInformation(const FString& InMaterialFunctionPath,
			const FString& InMaterialPath,
			const TArray<FString>& InMaterialFunctionOutputs)
			: MaterialFunctionPath(InMaterialFunctionPath)
			, MaterialPath(InMaterialPath)
			, MaterialFunctionOutputs(InMaterialFunctionOutputs)
		{
		}
	};

	static const TMap<EShadingModel, FGLTFMaterialInformation> ShadingModelToMaterialInformation = {
		{EShadingModel::DEFAULT,
		FGLTFMaterialInformation(
			TEXT("/InterchangeAssets/gltf/MaterialBodies/MF_Default_Body.MF_Default_Body"),
			TEXT("/InterchangeAssets/gltf/M_Default.M_Default"),
			TArray<FString>{
					TEXT("BaseColor"),
					TEXT("Metallic"),
					TEXT("Specular"),
					TEXT("Roughness"),
					TEXT("EmissiveColor"),
					TEXT("Opacity"),
					TEXT("OpacityMask"),
					TEXT("Normal"),
					TEXT("Occlusion")})
		},

		{EShadingModel::UNLIT,
		FGLTFMaterialInformation(
			TEXT("/InterchangeAssets/gltf/MaterialBodies/MF_Unlit_Body.MF_Unlit_Body"),
			TEXT("/InterchangeAssets/gltf/M_Unlit.M_Unlit"),
			TArray<FString>{
					TEXT("UnlitColor"),
					TEXT("Opacity"),
					TEXT("OpacityMask")})
		},

		{EShadingModel::CLEARCOAT,
		FGLTFMaterialInformation(
			TEXT("/InterchangeAssets/gltf/MaterialBodies/MF_ClearCoat_Body.MF_ClearCoat_Body"),
			TEXT("/InterchangeAssets/gltf/M_ClearCoat.M_ClearCoat"),
			TArray<FString>{
					TEXT("ClearCoatNormal"),
					TEXT("BaseColor"),
					TEXT("Metallic"),
					TEXT("Specular"),
					TEXT("Roughness"),
					TEXT("EmissiveColor"),
					TEXT("Opacity"),
					TEXT("OpacityMask"),
					TEXT("Normal"),
					TEXT("ClearCoat"),
					TEXT("ClearCoatRoughness"),
					TEXT("Occlusion")})
		},

		{EShadingModel::SHEEN,
		FGLTFMaterialInformation(
			TEXT("/InterchangeAssets/gltf/MaterialBodies/MF_Sheen_Body.MF_Sheen_Body"),
			TEXT("/InterchangeAssets/gltf/M_Sheen.M_Sheen"),
			TArray<FString>{
					TEXT("BaseColor"),
					TEXT("Metallic"),
					TEXT("Specular"),
					TEXT("Roughness"),
					TEXT("EmissiveColor"),
					TEXT("Opacity"),
					TEXT("OpacityMask"),
					TEXT("Normal"),
					TEXT("SheenColor"),
					TEXT("SheenRoughness"),
					TEXT("Occlusion")})
		},

		{EShadingModel::TRANSMISSION,
		FGLTFMaterialInformation(
			TEXT("/InterchangeAssets/gltf/MaterialBodies/MF_Transmission_Body.MF_Transmission_Body"),
			TEXT("/InterchangeAssets/gltf/M_Transmission.M_Transmission"),
			TArray<FString>{
					TEXT("TransmissionColor"),
					TEXT("BaseColor"),
					TEXT("Metallic"),
					TEXT("Specular"),
					TEXT("Roughness"),
					TEXT("EmissiveColor"),
					TEXT("Opacity"),
					TEXT("Normal"),
					TEXT("Occlusion")})
		},

		{EShadingModel::SPECULARGLOSSINESS,
		FGLTFMaterialInformation(
			TEXT("/InterchangeAssets/gltf/MaterialBodies/MF_SpecularGlossiness_Body.MF_SpecularGlossiness_Body"),
			TEXT("/InterchangeAssets/gltf/M_SpecularGlossiness.M_SpecularGlossiness"),
			TArray<FString>{
					TEXT("BaseColor"),
					TEXT("Metallic"),
					TEXT("Roughness"),
					TEXT("EmissiveColor"),
					TEXT("Opacity"),
					TEXT("OpacityMask"),
					TEXT("Normal"),
					TEXT("Occlusion")})
		}
	};

	inline TArray<FString> GetRequiredMaterialFunctionPaths()
	{
		TArray<FString> Result;
		for (const TPair<EShadingModel, FGLTFMaterialInformation>& Entry : ShadingModelToMaterialInformation)
		{
			Result.Add(Entry.Value.MaterialFunctionPath);
		}
		return Result;
	}

	inline TMap<FString, EShadingModel> GetMaterialFunctionPathsToShadingModels()
	{
		TMap<FString, EShadingModel> Result;
		for (const TPair<EShadingModel, FGLTFMaterialInformation>& Entry : ShadingModelToMaterialInformation)
		{
			Result.Add(Entry.Value.MaterialFunctionPath, Entry.Key);
		}
		return Result;
	}

	inline TMap<FString, EShadingModel> GetMaterialPathsToShadingModels()
	{
		TMap<FString, EShadingModel> Result;
		for (const TPair<EShadingModel, FGLTFMaterialInformation>& Entry : ShadingModelToMaterialInformation)
		{
			Result.Add(Entry.Value.MaterialPath, Entry.Key);
		}
		return Result;
	}

	void HandleGltfMaterial(UInterchangeBaseNodeContainer& NodeContainer, const GLTF::FMaterial& GltfMaterial, const TArray<GLTF::FTexture>& Textures, UInterchangeShaderGraphNode* ShaderGraphNode);

	bool AreRequiredPackagesLoaded();
};
