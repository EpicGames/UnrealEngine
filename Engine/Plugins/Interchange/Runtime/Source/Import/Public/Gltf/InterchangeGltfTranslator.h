// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GLTFAsset.h"
#include "GLTFMaterial.h"
#include "InterchangeTranslatorBase.h"
#include "Animation/InterchangeAnimationPayloadInterface.h"
#include "Mesh/InterchangeMeshPayload.h"
#include "Mesh/InterchangeMeshPayloadInterface.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Scene/InterchangeVariantSetPayloadInterface.h"
#include "Texture/InterchangeTexturePayloadInterface.h"
#include "Texture/InterchangeTextureLightProfilePayloadInterface.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeGltfTranslator.generated.h"

class UInterchangeShaderGraphNode;
class UInterchangeShaderNode;
class UInterchangeVariantSetNode;
class UInterchangeMeshNode;

/* glTF translator class support import of texture, material, static mesh, skeletal mesh, */
UCLASS(BlueprintType)
class UInterchangeGLTFTranslator : public UInterchangeTranslatorBase
	, public IInterchangeMeshPayloadInterface
	, public IInterchangeTexturePayloadInterface
	, public IInterchangeAnimationPayloadInterface
	, public IInterchangeVariantSetPayloadInterface
	, public IInterchangeTextureLightProfilePayloadInterface
{
	GENERATED_BODY()

public:
	UInterchangeGLTFTranslator();

	/** Begin UInterchangeTranslatorBase API*/
	virtual EInterchangeTranslatorType GetTranslatorType() const override;
	virtual EInterchangeTranslatorAssetType GetSupportedAssetTypes() const override;
	virtual TArray<FString> GetSupportedFormats() const override;
	virtual bool Translate( UInterchangeBaseNodeContainer& BaseNodeContainer ) const override;
	/** End UInterchangeTranslatorBase API*/

	/* IInterchangeStaticMeshPayloadInterface Begin */
	UE_DEPRECATED(5.6, "Deprecated. Use GetMeshPayloadData(const FInterchangeMeshPayLoadKey&) instead.")
	virtual TOptional<UE::Interchange::FMeshPayloadData> GetMeshPayloadData(const FInterchangeMeshPayLoadKey& PayLoadKey, const FTransform& MeshGlobalTransform) const override
	{
		using namespace UE::Interchange;
		UE::Interchange::FAttributeStorage Attributes;
		Attributes.RegisterAttribute(UE::Interchange::FAttributeKey{ MeshPayload::Attributes::MeshGlobalTransform }, MeshGlobalTransform);
		return GetMeshPayloadData(PayLoadKey, Attributes);
	}
	
	virtual TOptional<UE::Interchange::FMeshPayloadData> GetMeshPayloadData(const FInterchangeMeshPayLoadKey& PayLoadKey, const UE::Interchange::FAttributeStorage& PayloadAttributes) const;

	/* IInterchangeStaticMeshPayloadInterface End */

	/* IInterchangeTexturePayloadInterface Begin */

	virtual TOptional< UE::Interchange::FImportImage > GetTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const override;

	/* IInterchangeTexturePayloadInterface End */

	/* IInterchangeAnimationPayloadInterface Begin */
	TOptional<UE::Interchange::FAnimationPayloadData> GetAnimationPayloadData(const UE::Interchange::FAnimationPayloadQuery& PayloadQuery) const;
	virtual TArray<UE::Interchange::FAnimationPayloadData> GetAnimationPayloadData(const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries) const override;
	/* IInterchangeAnimationPayloadInterface End */

	/* IInterchangeVariantSetPayloadInterface Begin */
	virtual TOptional<UE::Interchange::FVariantSetPayloadData> GetVariantSetPayloadData(const FString& PayloadKey) const override;
	/* IInterchangeVariantSetPayloadInterface End */

	/* IInterchangeTextureLightProfilePayloadInterface Begin */
	virtual TOptional<UE::Interchange::FImportLightProfile> GetLightProfilePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const override;
	/* IInterchangeTextureLightProfilePayloadInterface End */

protected:
	using FNodeUidMap = TMap<const GLTF::FNode*, FString>;

	void HandleGltfSkeletons( UInterchangeBaseNodeContainer& NodeContainer, const FString& SceneNodeUid, const TArray<int32>& SkinnedMeshNodes, TSet<int>& UnusedMeshIndices ) const;
	void HandleGltfNode( UInterchangeBaseNodeContainer& NodeContainer, const GLTF::FNode& GltfNode, const FString& ParentNodeUid, const int32 NodeIndex, 
		bool &bHasVariants, TArray<int32>& SkinnedMeshNodes, TSet<int>& UnusedMeshIndices, const TMap<int32, FTransform>& T0Transforms, const FString& SceneNodeUid) const;
	void HandleGltfVariants(UInterchangeBaseNodeContainer& NodeContainer, const FString& FileName) const;
	UInterchangeMeshNode* HandleGltfMesh(UInterchangeBaseNodeContainer& NodeContainer, const GLTF::FMesh& GltfMesh, int MeshIndex,
		TSet<int>& UnusedMeshIndices, const FString& SkeletalName = "" /*If set it creates the mesh even if it was already created (for Skeletals)*/, const FString& SkeletalId = "") const;

	bool GetVariantSetPayloadData(UE::Interchange::FVariantSetPayloadData& PayloadData) const;

private:
	void SetTextureSRGB(UInterchangeBaseNodeContainer& NodeContainer, const GLTF::FTextureMap& TextureMap, bool bSRGB) const;

	GLTF::FAsset GltfAsset;
	mutable FNodeUidMap NodeUidMap;

	bool bRenderSettingsClearCoatEnableSecondNormal = false;
};


