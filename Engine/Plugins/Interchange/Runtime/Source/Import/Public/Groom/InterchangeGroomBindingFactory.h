// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeFactoryBase.h"

#include "InterchangeGroomBindingFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeGroomBindingFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()

public:
	// Begin UInterchangeFactoryBase interface
	UE_API virtual UClass* GetFactoryClass() const override;
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() override	{ return EInterchangeFactoryAssetType::Grooms; }
	UE_API virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	// End UInterchangeFactoryBase interface

};

#undef UE_API