// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Groom/InterchangeGroomPayloadData.h"
#include "InterchangeFactoryBase.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeGroomFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeGroomFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()

public:
	// Begin UInterchangeFactoryBase interface
	UE_API virtual UClass* GetFactoryClass() const override;
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() override	{ return EInterchangeFactoryAssetType::Grooms; }
	UE_API virtual void CreatePayloadTasks(const FImportAssetObjectParams& Arguments, bool bAsync, TArray<TSharedPtr<UE::Interchange::FInterchangeTaskBase>>& PayloadTasks) override;
	UE_API virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;
	UE_API virtual bool GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const override;
	UE_API virtual bool SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const override;
	UE_API virtual void BackupSourceData(const UObject* Object) const override;
	UE_API virtual void ReinstateSourceData(const UObject* Object) const override;
	UE_API virtual void ClearBackupSourceData(const UObject* Object) const override;
	// End UInterchangeFactoryBase interface

private:
	TOptional<UE::Interchange::FGroomPayloadData> GroomPayload;
};

#undef UE_API