// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HairDescription.h"

namespace UE::Interchange
{
	struct FGroomPayloadData
	{
		FHairDescription HairDescription;
	};
}