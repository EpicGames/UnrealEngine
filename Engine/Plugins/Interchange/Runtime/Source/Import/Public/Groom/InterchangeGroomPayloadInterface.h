// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Groom/InterchangeGroomPayloadData.h"
#include "InterchangeGroomNode.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeGroomPayloadInterface.generated.h"

UINTERFACE(MinimalAPI)
class UInterchangeGroomPayloadInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Payload interface for groom data
 */
class IInterchangeGroomPayloadInterface
{
	GENERATED_BODY()

public:
	virtual TOptional<UE::Interchange::FGroomPayloadData> GetGroomPayloadData(const FInterchangeGroomPayloadKey& PayloadKey) const
	{
		return {};
	}
};

