// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/AssetUserData.h"

#include "InterchangeAssetUserData.generated.h"

/** Asset user data that can be used with Interchange on Actors and other objects  */
UCLASS(MinimalAPI, BlueprintType, meta = (ScriptName = "InterchangeUserData", DisplayName = "Interchange User Data"))
class UInterchangeAssetUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Interchange User Data", meta = (ScriptName = "Metadata", DisplayName = "Metadata"))
	TMap<FString, FString> MetaData;
};

/** Asset user data that can be used with Interchange on Levels. This will reside in the world settings of the ULevel*/
UCLASS(MinimalAPI, BlueprintType, meta = (ScriptName = "InterchangeLevelUserData", DisplayName = "Interchange Level User Data"))
class UInterchangeLevelAssetUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Interchange User Data", meta = (ScriptName = "SceneImportPaths", DisplayName = "Scene Import Paths"))
	TArray<FString> SceneImportPaths;
};
