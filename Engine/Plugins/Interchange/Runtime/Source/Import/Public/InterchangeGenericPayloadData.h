// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeGenericPayloadData.generated.h"

/**
 * Base class for generic payload data. This is meant to be derived, in order to provide custom payload data
 * without having to specialize the translators to implement another payload interface.
 *
 * See UInterchangeGenericPayloadInterface.
 */
UCLASS(MinimalAPI)
class UInterchangeGenericPayloadData : public UObject
{
	GENERATED_BODY()
};