// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeGenericPayloadData.h"

#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeGenericPayloadInterface.generated.h"

/**
 * Payload interface implemented by translators that want to allow user code to provide different payload types to Interchange.
 *
 * The USD translator uses this mechanism: A user plugin may implement a schema handler that can provide custom
 * payload data, and also register a factory that can handle it. Without a mechanism such as this however, there would
 * be no way of getting that payload data from the schema handler to the factory, as the translator would need to be also
 * modified with a custom payload interface.
 *
 * Instead, the USD translator already implements this interface, and the schema handler can provide a payload data class
 * that derives UInterchangeGenericPayloadData, while the factory can try casting its received UInterchangeGenericPayloadData
 * to the derived type.
 */
UINTERFACE(MinimalAPI)
class UInterchangeGenericPayloadInterface : public UInterface
{
	GENERATED_BODY()
};

class IInterchangeGenericPayloadInterface
{
	GENERATED_BODY()

public:
	virtual TObjectPtr<UInterchangeGenericPayloadData> GetGenericPayloadData(const FString& PayloadKey) const
	{
		return {};
	}
};
