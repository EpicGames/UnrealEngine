// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

/**
 * The public interface to this module
 */
class IInterchangeImportModule : public IModuleInterface
{

public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IInterchangeImportModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IInterchangeImportModule >( "InterchangeImport" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "InterchangeImport" );
	}
	
	/**
	 * Checks to see if Substrate support is enabled both in the Project Settings and for Interchange
	 * 
	 * @return True if Substrate is enabled
	 */
	virtual bool IsSubstrateEnabled() const { return false; };

	/**
	 * Checks to see if Substrate Adaptive GBuffer is enabled in the Project Settings
	 *
	 * @return True if Adaptive GBuffer is enabled
	 */
	virtual bool IsSubstrateAdaptiveGBufferEnabled() const { return false; }
};

