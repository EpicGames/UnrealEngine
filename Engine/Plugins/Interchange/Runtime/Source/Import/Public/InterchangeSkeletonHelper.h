// Copyright Epic Games, Inc. All Rights Reserved. 
#pragma once

#if WITH_EDITOR

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
//#include "ReferenceSkeleton.h"

struct FReferenceSkeleton;
class UInterchangeBaseNode;
class UInterchangeBaseNodeContainer;
class UInterchangeResultsContainer;
class UInterchangeSceneNode;
class USkeleton;
struct FMeshBoneInfo;

namespace UE::Interchange::Private
{

	struct FJointInfo
	{
		FString Name;
		int32 ParentIndex;  // 0 if this is the root bone.  
		FTransform	LocalTransform; // local transform
	};
	
	struct FSkeletonHelper
	{
	public:
		static INTERCHANGEIMPORT_API bool IsValidSocket(const UInterchangeBaseNodeContainer* NodeContainer, const UInterchangeSceneNode* Node);

		static INTERCHANGEIMPORT_API bool ProcessImportMeshSkeleton(TObjectPtr<UInterchangeResultsContainer> Results
			, const USkeleton* SkeletonAsset
			, FReferenceSkeleton& RefSkeleton
			, const UInterchangeBaseNodeContainer* NodeContainer
			, const FString& RootJointNodeId
			, TArray<SkeletalMeshImportData::FBone>& RefBonesBinary
			, const bool bUseTimeZeroAsBindPose
			, bool& bOutDiffPose
			, bool bImportSockets);

		static INTERCHANGEIMPORT_API bool IsCompatibleSkeleton(const USkeleton* Skeleton
			, const FString RootJoinUid
			, const UInterchangeBaseNodeContainer* BaseNodeContainer
			, bool bConvertStaticToSkeletalActive
			, bool bCheckForIdenticalSkeleton
			, bool bImportSockets);
		static INTERCHANGEIMPORT_API void RecursiveAddSkeletonMetaDataValues(UInterchangeBaseNodeContainer* NodeContainer, UInterchangeBaseNode* DestinationNode, const FString& JointUid);

		static INTERCHANGEIMPORT_API void RecursiveBoneHasBindPose(const UInterchangeBaseNodeContainer* NodeContainer, const FString& JointNodeId, bool& bHasBoneWithoutBindPose);

		static INTERCHANGEIMPORT_API void RecursiveAddBones(const UInterchangeBaseNodeContainer* NodeContainer
			, const FString& JointNodeId
			, TArray <FJointInfo>& JointInfos
			, int32 ParentIndex
			, TArray<SkeletalMeshImportData::FBone>& RefBonesBinary
			, TSet<FName>& UsedBoneNames
			, const bool bUseTimeZeroAsBindPose
			, bool& bOutDiffPose
			, TArray<FString>& OutBoneNotBindNames
			, bool bImportSockets);

	private:
		static FName SkeletalLodGetBoneName(const TArray<FMeshBoneInfo>& SkeletalLodRawInfos, int32 BoneIndex);
		static int32 SkeletalLodFindBoneIndex(const TArray<FMeshBoneInfo>& SkeletalLodRawInfos, FName BoneName);
		static int32 SkeletalLodGetParentIndex(const TArray<FMeshBoneInfo>& SkeletalLodRawInfos, int32 BoneIndex);
		static bool DoesParentChainMatch(int32 StartBoneIndex, const FReferenceSkeleton& SkeletonRef, const TArray<FMeshBoneInfo>& SkeletalLodRawInfos);

		static void RecursiveBuildSkeletalSkeleton(const FString JoinToAddUid
			, const int32 ParentIndex
			, const UInterchangeBaseNodeContainer* BaseNodeContainer
			, TArray<FMeshBoneInfo>& SkeletalLodRawInfos
			, bool bConvertStaticToSkeletalActive
			, bool bImportSockets);
	};
} //namespace UE::Interchange::Private

#endif //WITH_EDITOR
