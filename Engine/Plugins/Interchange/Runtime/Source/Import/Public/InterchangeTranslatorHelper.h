// Copyright Epic Games, Inc. All Rights Reserved. 
#pragma once

#include "InterchangeSourceData.h"
#include "InterchangeTranslatorBase.h"
#include "Containers/UnrealString.h"

#define UE_API INTERCHANGEIMPORT_API

class UInterchangeResultsContainer;
class UInterchangeAnalyticsHandlerBase;

namespace UE::Interchange::Private
{
	struct FScopedTranslator
	{
	public:
		UE_API FScopedTranslator(const FString& PayLoadKey, UInterchangeResultsContainer* Results, TObjectPtr<UInterchangeAnalyticsHandlerBase> AnalyticsHandler);
		UE_API ~FScopedTranslator();

		template <class T>
		const T* GetPayLoadInterface() const
		{
			if(!IsValid())
			{ 
				return nullptr;
			}
			const T* PayloadInterface = Cast<T>(SourceTranslator);
			return PayloadInterface;
		}
	
	private:
		UE_API bool IsValid() const;

		TObjectPtr<UInterchangeTranslatorBase> SourceTranslator = nullptr;
		TObjectPtr<UInterchangeSourceData> PayloadSourceData = nullptr;
	};
} //namespace UE::Interchange::Private

#undef UE_API
