// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeFactoryBase.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialFunction.h"

#include "InterchangeMaterialFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

class UInterchangeBaseMaterialFactoryNode;
class UInterchangeMaterialExpressionFactoryNode;
class UInterchangeMaterialInstanceFactoryNode;
class UMaterial;
class UMaterialExpression;
class UMaterialExpressionMaterialFunctionCall;
class UMaterialInstance;

namespace UE::Interchange::Materials::HashUtils
{
	class FInterchangeMaterialInstanceOverridesAPI
	{
	public:
		/**
		 * Generates Attribute Key String from the AttributeType and Hash
		 */
		static UE_API FString MakeOverrideParameterName(UE::Interchange::EAttributeTypes AttributeType, int32 Hash, bool Prefix = true);

		/**
		 * Generates Attribute Key String from the DisplayLable
		 */
		static UE_API FString MakeOverrideParameterName(const FString& DisplayLabel);

		/**
		 * Generates Attribute Key for Expression Name
		 */
		static UE_API FString MakeExpressionNameString();

		/**
		 * Retrieves the Parameter Name from AttributeKey
		 */
		static UE_API bool GetOverrideParameterName(const UE::Interchange::FAttributeKey& AttributeKey, FString& OverrideParameterName);

		/**
		 * Checks if the Node has a MaterialExpressionName override
		 */
		static UE_API bool HasMaterialExpressionNameOverride(const UInterchangeBaseNode* BaseNode);

		/**
		 * Retrieves the Attribute Keys of the Leaf Inputs.
		 */
		static UE_API void GatherLeafInputs(const UInterchangeBaseNode* BaseNode, TArray<UE::Interchange::FAttributeKey>& OutLeafInputAttributeKeys);

	private:
		static const TCHAR* ExpressionNameAttributeKey;
		static const TCHAR* OverrideParameterPrefix;
		static const TCHAR* OverrideParameterSeparator;
		static const TCHAR* OverrideHashSeparator;
	};
}

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeMaterialFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()

public:
	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() override { return EInterchangeFactoryAssetType::Materials; }
	UE_API virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual FImportAssetResult ImportAsset_Async(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;
	UE_API virtual bool GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const override;
	UE_API virtual bool SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const override;
	UE_API virtual void BackupSourceData(const UObject* Object) const override;
	UE_API virtual void ReinstateSourceData(const UObject* Object) const override;
	UE_API virtual void ClearBackupSourceData(const UObject* Object) const override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////
private:
#if WITH_EDITOR
	UE_API void SetupMaterial(UMaterial* Material, const FImportAssetObjectParams& Arguments, const UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode);
#endif // #if WITH_EDITOR

	UE_API void SetupMaterialInstance(UMaterialInstance& MaterialInstance, const UInterchangeBaseNodeContainer& NodeContainer, const UInterchangeMaterialInstanceFactoryNode& FactoryNode, bool bResetInstance);
	UE_API void SetupReimportedMaterialInstance(UMaterialInstance& MaterialInstance, const UInterchangeBaseNodeContainer& NodeContainer, const UInterchangeMaterialInstanceFactoryNode& FactoryNode, const UInterchangeMaterialInstanceFactoryNode& PreviousFactoryNode);

	//If we import without a pure material translator, we should not override an existing material and we must skip the import. See the implementation of BeginImportAssetObject_GameThread function.
	bool bSkipImport = false;
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeMaterialFunctionFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()

public:
	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() override { return EInterchangeFactoryAssetType::Materials; }
	UE_API virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual FImportAssetResult ImportAsset_Async(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////

private:
#if WITH_EDITOR
	UE_API void SetupMaterial(class UMaterialFunction* Material, const FImportAssetObjectParams& Arguments, const class UInterchangeMaterialFunctionFactoryNode* MaterialFactoryNode);
#endif
	bool bSkipImport = false;
};

/**
 * This class intends to avoid a race condition when importing 
 * several material functions at once, for example when importing the MaterialX file standard_surface_chess_set
 * Only one instance of a material function can update it
 */
class FInterchangeImportMaterialAsyncHelper
{
	FInterchangeImportMaterialAsyncHelper() = default;

public:

	FInterchangeImportMaterialAsyncHelper(const FInterchangeImportMaterialAsyncHelper&) = delete;
	FInterchangeImportMaterialAsyncHelper& operator=(const FInterchangeImportMaterialAsyncHelper&) = delete;

	static FInterchangeImportMaterialAsyncHelper& GetInstance();

#if WITH_EDITOR
	void UpdateFromFunctionResource(UMaterialExpressionMaterialFunctionCall* MaterialFunctionCall);

	void UpdateFromFunctionResource(UMaterialFunctionInterface* MaterialFunction);
#endif

private:

	FCriticalSection UpdatedMaterialFunctionCallsLock;
	FCriticalSection UpdatedMaterialFunctionsLock;
};

#undef UE_API
