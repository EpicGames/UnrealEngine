// Copyright Epic Games, Inc. All Rights Reserved. 
#pragma once
#include "CoreMinimal.h"

class UInterchangeBaseNode;
class UInterchangeBaseNodeContainer;
class UInterchangeShaderGraphNode;

namespace UE::Interchange::Materials::Private
{
	struct FMaterialNodesHelper
	{
	public:
		static bool SetupScalarParameter(UInterchangeBaseNodeContainer& BaseNodeContainer, UInterchangeShaderGraphNode* ShaderGraphNode, const FString& InputName, const float& AttributeValue);
		static bool SetupVectorParameter(UInterchangeBaseNodeContainer& BaseNodeContainer, UInterchangeShaderGraphNode* ShaderGraphNode, const FString& InputName, const FLinearColor& AttributeValue);
	};
} 
