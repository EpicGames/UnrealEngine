// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "InterchangeTranslatorBase.h"
#include "Texture/InterchangeTexturePayloadInterface.h"
#include "Texture/InterchangeBlockedTexturePayloadInterface.h"

#include "InterchangeMaterialXTranslator.generated.h"

#define UE_API INTERCHANGEIMPORT_API

class UInterchangeTextureNode;
class UInterchangeBaseLightNode;
class UInterchangeSceneNode;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeMaterialXTranslator
	: public UInterchangeTranslatorBase
	, public IInterchangeTexturePayloadInterface
	, public IInterchangeBlockedTexturePayloadInterface
{
	GENERATED_BODY()

public:

	/** Begin UInterchangeTranslatorBase API*/

	UE_API virtual EInterchangeTranslatorType GetTranslatorType() const override;

	UE_API virtual EInterchangeTranslatorAssetType GetSupportedAssetTypes() const override;

	UE_API virtual TArray<FString> GetSupportedFormats() const override;

	/**
	 * Translate the associated source data into a node hold by the specified nodes container.
	 *
	 * @param BaseNodeContainer - The container where to add the translated Interchange nodes.
	 * @return true if the translator can translate the source data, false otherwise.
	 */
	UE_API virtual bool Translate( UInterchangeBaseNodeContainer& BaseNodeContainer ) const override;
	/** End UInterchangeTranslatorBase API*/

	using IInterchangeTexturePayloadInterface::GetTexturePayloadData;
	UE_API virtual TOptional<UE::Interchange::FImportImage> GetTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const override;
	UE_API static TOptional<UE::Interchange::FImportImage> GetTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath, UInterchangeResultsContainer* InResults, TObjectPtr<UInterchangeAnalyticsHandlerBase> InAnalyticsHandler);

	using IInterchangeBlockedTexturePayloadInterface::GetBlockedTexturePayloadData;
	UE_API virtual TOptional<UE::Interchange::FImportBlockedImage> GetBlockedTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const override;
	UE_API static TOptional<UE::Interchange::FImportBlockedImage> GetBlockedTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath, UInterchangeResultsContainer* InResults, TObjectPtr<UInterchangeAnalyticsHandlerBase> InAnalyticsHandler);

};

#undef UE_API
