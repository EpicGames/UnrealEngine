// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"

#include "MaterialExpressionSwizzle.generated.h"

UCLASS()
class UE_DEPRECATED(5.7, "UMaterialExpressionMaterialXSwizzle is now deprecated since <swizzle> node has been removed from MaterialX 1.39.3. Please use UMaterialExpressionComponent instead.")
UMaterialExpressionMaterialXSwizzle : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FExpressionInput Input;

	UPROPERTY(EditAnywhere, Category = Swizzle, Meta = (ShowAsInputPin = "Advanced"))
	FString Channels;

	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
#endif
	//~ End UMaterialExpressionMaterialX Interface
};

