// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Materials/MaterialExpressionTextureSampleParameter2D.h"
#include "MaterialExpressionTextureSampleParameterBlur.generated.h"

UENUM()
enum class EMaterialXTextureSampleBlurFilter : uint8
{
	Box,
	Gaussian
};

UENUM()
enum class EMAterialXTextureSampleBlurKernel
{
	Kernel1 UMETA(DisplayName = "1x1", ToolTip="No Blur"),
	Kernel3 UMETA(DisplayName = "3x3"),
	Kernel5 UMETA(DisplayName = "5x5"),
	Kernel7 UMETA(DisplayName = "7x7"),
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, meta = (Private))
class UMaterialExpressionMaterialXTextureSampleParameterBlur: public UMaterialExpressionTextureSampleParameter2D
{
	GENERATED_UCLASS_BODY()

	/** The size of the blur kernel, relative to 0-1 UV space. */
	UPROPERTY(EditAnywhere, Category = Filtering, Meta = (ShowAsInputPin = "Advanced"))
	EMAterialXTextureSampleBlurKernel KernelSize = EMAterialXTextureSampleBlurKernel::Kernel1;

	/** Size of the filter when we sample a texture coordinate */
	UPROPERTY(EditAnywhere, Category = Filtering, Meta = (ShowAsInputPin = "Advanced"))
	float FilterSize = 1.f;

	/** Offset of the filter when we sample a texture coordinate */
	UPROPERTY(EditAnywhere, Category = Filtering, Meta = (ShowAsInputPin = "Advanced"))
	float FilterOffset = 0.f;

	/** Filter to use when we blur a Texture: Gaussian or Box Linear filter*/
	UPROPERTY(EditAnywhere, Category = Filtering, Meta = (ShowAsInputPin = "Advanced"))
	EMaterialXTextureSampleBlurFilter Filter = EMaterialXTextureSampleBlurFilter::Gaussian;

	//~ Begin UMaterialExpressionMaterialX Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpressionMaterialX Interface

private:

#if WITH_EDITOR
	const float* GetKernel(int32& FilterWidth) const;
#endif
};