// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Import/Private/InterchangeImportLog.h"
#include "AnalyticsEventAttribute.h"
#include "InterchangeMaterialDefinitions.h"
#include "InterchangeTranslatorBase.h"
#include "Logging/StructuredLog.h"
#include "MaterialXCore/Document.h"
#include "MaterialX/InterchangeMaterialXDefinitions.h"

namespace UE::Interchange::MaterialX::Private
{
	static inline FText FormatFText(const FText& Text)
	{
		return Text;
	}

	template<typename... Args>
	FText FormatFText(const FText& Text, Args&&... args)
	{
		TArray<FFormatArgumentValue> FormatArgs = { FFormatArgumentValue(FText::FromString(Forward<Args>(args)))... };
		return FText::Format(Text, FormatArgs);
	}
}
#define MTLX_LOG(Key, Format, ...) \
	if (Translator) \
	{ \
		const FText LocText = NSLOCTEXT("MaterialX", Key, Format); \
		UInterchangeResultWarning_Generic* Message = Translator->AddMessage<UInterchangeResultWarning_Generic>(); \
		Message->Text = UE::Interchange::MaterialX::Private::FormatFText(LocText, ##__VA_ARGS__); \
	} \
	else \
	{ \
		UE_LOGFMT(LogInterchangeImport, Warning, Format, ##__VA_ARGS__); \
	}
													

class FMaterialXBase : public TSharedFromThis<FMaterialXBase>
{
protected:

	FMaterialXBase(UInterchangeBaseNodeContainer& BaseNodeContainer);

public:

	enum class EMaterialXLog{Error, Warning};

	virtual UInterchangeBaseNode* Translate(MaterialX::NodePtr Node) = 0;

	virtual ~FMaterialXBase() = default;
	
	/**
	 * Update a MaterialX Document recursively by either initializing nodes (like <tiledimage>) or updating some nodes (e.g: math nodes with neutral elements)
	 *
	 * @param Graph - The graph to update and all of its sub graphs
	 */
	static void UpdateDocumentRecursively(MaterialX::GraphElementPtr Graph);

	/** 
	 * Set the node definition of a shader, useful when a shader has multiple definition per types and when we know the type only at the top level during a surfacematerial for example and not a construction-time. 
	 * Make sure that the pointer don't cease to exist, no allocation is performed here, use the definition in MaterialXDefinitions
	 */
	void SetNodeDefinition(const char* NodeDefinition);

	/** Set the translator, only used for logging messages*/
	void SetTranslator(const UInterchangeTranslatorBase* InTranslator);

	void CollectAnalytics(TArray<FAnalyticsEventAttribute>& OutAnalyticsAttributes);

protected:

	/**
	 * @param const char * - New name of the Input to copy
	 * @param MaterialX::InputPtr - Input to copy
	 */
	using FInputToCopy = TPair<const char*, MaterialX::InputPtr>;

	/**
	 * @param const char * - Attribute
	 * @param const char * - Value
	 */
	using FAttributeValue = TPair<const char*, const char*>;

	/**
	 * @param TArray<FAttributeValueType> - Array of attribute/value
	 */
	using FAttributeValueArray = TArray<FAttributeValue>;

	/**
	 * @param TArray<FAttributeValueType> - Array of attribute/value
	 */
	using FInputToCreate = TPair<const char*, FAttributeValueArray>; // Name - Attributes

	/**
	 * Add a texcoord input to the tiledimage nodes if the input is missing, and connect it to a texcoord node in the nodegraph
	 * This function should be called before the subgraphs substitution to avoid missing UVs input on a texture
	 * See for example standard_surface_brick_procedural where the tiledimage nodes have no texcoord input
	 *
	 * @param Node - The nodegraph to retrieve the different tiledimage nodes
	 */
	static void AddTexCoordToTiledImageNodes(MaterialX::ElementPtr Graph);

	/**
	 * Convert math nodes (add, sub, div, mul) that have a neutral input, to the MaterialX <dot> node which is simply the identity or a noop
	 * This allows us to optimize the final shader by avoiding unnecessary operation on a material (for example a*1 = a)
	 * This function should be called after the subgraphs substitution in order to search for all the nodes even in the subgraphs
	 *
	 * @param NodeGraph - The node graph to retrieve the different math nodes
	 */
	static void ConvertNeutralNodesToDot(MaterialX::ElementPtr NodeGraph);

	/** Convert <geompropvalue> nodes that are vector2 with the exact input geomprop named "st" as <texcoord> nodes*/
	static void ConvertGeomPropValueNodesToTexCoord(MaterialX::GraphElementPtr Graph);

	/**
	 * Helper function that create a MaterialX node, useful to insert a node between 2 nodes especially when an operation cannot be done in one pass
	 * If the node already exists in the graph, this one is returned
	 *
	 * @param NodeGraph - The node graph to insert the new node into
	 * @param NodeName - The name of the node
	 * @param Category - The category of the node to create
	 * @param InputsToCopy - The inputs from another node/nodegraph to copy in the new node, can be empty
	 * @param InputsToCreate - The inputs to create from scratch and insert in the new node, can be empty
	 *
	 * @return The newly created node
	 */
	static MaterialX::NodePtr CreateNode(MaterialX::ElementPtr NodeGraph, const char* NodeName, const char* Category, const TArray<FInputToCopy>& InputsToCopy, const TArray<FInputToCreate>& InputsToCreate);


	/**
	 * Return the innermost color space of an element in the current scope, if it has none, it will take the one from its parents
	 *
	 * @param Element - the Element to retrieve the color space from (can be anything, an input, a node, a nodegraph, etc.)
	 *
	 * @return a color space or an empty string
	 *
	 */
	static FString GetColorSpace(MaterialX::ElementPtr Element);


	/**
	 * Retrieve the input from a surfaceshader node, or take the default input from the library,
	 * this function should only be called after testing the MaterialX libraries have been successfully imported, meaning the node definition of the surfaceshader
	 * should always be valid
	 *
	 * @param StandardSurface - the <standard_surface> node
	 * @param InputName - the input name to retrieve
	 *
	 * @return the input from the given name
	 */
	MaterialX::InputPtr GetInput(MaterialX::NodePtr Node, const char* InputName);

	/**
	 * Helper function that returns a linear color after a color space conversion, the function makes no assumption on the input, and it should have a value of Color3-4 type
	 *
	 * @param Input - The input that has a Color3-4 value in it
	 *
	 * @return The linear color after color space conversion
	 */
	FLinearColor GetLinearColor(MaterialX::InputPtr Input) const;

protected:

	UInterchangeBaseNodeContainer& NodeContainer;

	/** Used for logging from either MaterialX or USD translator*/
	const UInterchangeTranslatorBase* Translator;

	/** To avoid locking Analytics helper in the translator for each add, at the end of the translat,  every attribute will be concatenated*/
	TArray<FAnalyticsEventAttribute> AnalyticsAttributes;

	/** It's up to the derive class to initialize it with the proper node definition*/
	const char* NodeDefinition; 

	bool bIsSubstrateEnabled;

	bool bIsSubstrateAdaptiveGBufferEnabled;
};
#endif
