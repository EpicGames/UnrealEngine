// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#if WITH_EDITOR

#include "MaterialXBase.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "InterchangeSceneNode.h"
#include "InterchangeLightNode.h"

class FMaterialXLightShaderAbstract : public FMaterialXBase
{
protected:

	FMaterialXLightShaderAbstract(UInterchangeBaseNodeContainer& BaseNodeContainer);

	virtual void PreTranslate();

	virtual void PostTranslate();

	FTransform Transform;
	MaterialX::NodePtr LightShaderNode;
	UInterchangeSceneNode* SceneNode;
	UInterchangeBaseLightNode* LightNode;
};
#endif