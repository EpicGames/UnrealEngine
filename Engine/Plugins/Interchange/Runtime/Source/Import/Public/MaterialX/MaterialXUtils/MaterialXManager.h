// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_EDITOR

#include "MaterialXCore/Document.h"
#include "Misc/TVariant.h"
#include "MaterialX/InterchangeMaterialXDefinitions.h"
#include <unordered_set>

class FMaterialXBase;
class UInterchangeBaseNodeContainer;
class UInterchangeShaderNode;
class FMaterialXSurfaceShaderAbstract;
class UInterchangeTranslatorBase;

#endif // WITH_EDITOR

namespace UE::Interchange::MaterialX
{
	// Load necessary material functions, this function can only be called in the Game Thread
	INTERCHANGEIMPORT_API bool AreMaterialFunctionPackagesLoaded();
}

#if WITH_EDITOR

class FMaterialXManager
{
public:

	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<FMaterialXBase>, FOnGetMaterialXInstance, UInterchangeBaseNodeContainer&);

	static INTERCHANGEIMPORT_API FMaterialXManager& GetInstance();

	/** 
	 * Translate a MaterialX file
	 * @param Filename - Name of the MaterialX file to translate
	 * @param BaseNodeContainer - Node Container
	 * @param Translator - The Translator if this function is called from a Translaotr, used to log errors, otherwise the errors are logged in the default output
	 * 
	 * @return true if the file has been successfully translated
	 */
	INTERCHANGEIMPORT_API bool Translate(const FString& Filename, UInterchangeBaseNodeContainer& BaseNodeContainer, const UInterchangeTranslatorBase* Translator = nullptr);

	/** Translate a MaterialX Document */
	INTERCHANGEIMPORT_API bool Translate(MaterialX::DocumentPtr Document, UInterchangeBaseNodeContainer& BaseNodeContainer, const UInterchangeTranslatorBase* Translator = nullptr);

	/** Find a corresponding Material Expression Input given a (Category [NodeGroup] [Type], Input) pair*/
	INTERCHANGEIMPORT_API const FString* FindMatchingInput(const FString& CategoryKey, const FString& InputKey, const FString& NodeGroup = {}, const FString& Type = {}) const;

	/** Find a stored Material Expression Input */
	INTERCHANGEIMPORT_API const FString* FindMaterialExpressionInput(const FString& InputKey) const;

	/** Find a matching Material Expression given a MaterialX category [nodegroup] [type]*/
	INTERCHANGEIMPORT_API const FString* FindMatchingMaterialExpression(const FString& CategoryKey, const FString& NodeGroup = {}, const FString& Type= {}) const;

	/** Find a matching Material Function given a MaterialX category*/
	INTERCHANGEIMPORT_API bool FindMatchingMaterialFunction(const FString& CategoryKey, const FString*& MaterialFunctionPath, uint8& EnumType, uint8& EnumValue) const;

	INTERCHANGEIMPORT_API TSharedPtr<FMaterialXBase> GetShaderTranslator(const FString& CategoryShader, UInterchangeBaseNodeContainer& NodeContainer);

	INTERCHANGEIMPORT_API void RegisterMaterialXInstance(const FString& CategoryShader, FOnGetMaterialXInstance MaterialXInstanceDelegate);
	
	INTERCHANGEIMPORT_API bool IsSubstrateEnabled() const;

	INTERCHANGEIMPORT_API bool IsSubstrateAdaptiveGBufferEnabled() const;

	/** Return true if the node category should be filtered out during the flattening of the subgraphs*/
	INTERCHANGEIMPORT_API bool FilterNodeGraph(MaterialX::NodePtr Node) const;

	/** Remove the inputs of a node with no match in UE Material Expressions*/
	INTERCHANGEIMPORT_API void RemoveInputs(MaterialX::NodePtr Node) const;

	/** Find or Add a Texture Node UID, if not found it creates a hash of the path*/
	INTERCHANGEIMPORT_API FString FindOrAddTextureNodeUid(const FString& TexturePath);

	/**
	 * Add all the inputs from the nodedef in case the node doesn't have them all (we don't have a 1:1 match on the default values otherwise)
	 * For exemple 'min' defaults to (0, 0) whereas UE to (0, 1)
	 * We don't want to add every node, since we don't a have perfect match for each input
	 */
	INTERCHANGEIMPORT_API void AddInputsFromNodeDef(MaterialX::NodePtr Node) const;

	FMaterialXManager(const FMaterialXManager&) = delete;
	FMaterialXManager& operator=(const FMaterialXManager&) = delete;

	static INTERCHANGEIMPORT_API const TCHAR TexturePayloadSeparator;

private:

	/**
	* FString: Material Function path
	* Enums: data-driven BSDF nodes
	*/
	using FMaterialXMaterialFunction = TVariant<FString, EInterchangeMaterialXShaders, EInterchangeMaterialXBSDF, EInterchangeMaterialXEDF, EInterchangeMaterialXVDF>;

	struct FKeyExpression
	{
		template<typename CategoryString>
		FKeyExpression(CategoryString&& Category)
			: Category{ std::forward<CategoryString>(Category) }
		{}

		template<typename CategoryString, typename NodeGroupString, typename TypeString>
		FKeyExpression(CategoryString&& Category, NodeGroupString&& NodeGroup, TypeString&& Type)
			: Category{ std::forward<CategoryString>(Category) }
			, NodeGroup{ std::forward<NodeGroupString>(NodeGroup) }
			, Type{ std::forward<TypeString>(Type) }
		{}

		[[nodiscard]] inline bool operator==(const FKeyExpression& Rhs) const
		{
			return Category == Rhs.Category && NodeGroup == Rhs.NodeGroup && Type == Rhs.Type;
		}

		friend inline uint32 GetTypeHash(const FKeyExpression& Key)
		{
			return HashCombine(HashCombine(GetTypeHash(Key.Category), GetTypeHash(Key.NodeGroup)), GetTypeHash(Key.Type));
		}

		FString Category;
		FString NodeGroup; // node group is optional, some nodes in MaterialX have different material expressions in UE depending on the nodegroup
		FString Type; // Type is optional, some nodes in MaterialX have different material expressions in UE depending on the type
	};

	INTERCHANGEIMPORT_API FMaterialXManager();

	/** Helper function that returns the input as a string by also adding it in the MaterialExpressionInputs container*/
	INTERCHANGEIMPORT_API const TCHAR* ExpressionInput(const TCHAR* Input);

	/** Helper function that returns the material function package path as a string by also adding it in the MaterialFunctionsToLoad container,
	  * in order to centralize material functions that need to be loaded in the Game Thread by AreMaterialFunctionPackagesLoaded function*/
	INTERCHANGEIMPORT_API const TCHAR* MaterialFunctionPackage(const TCHAR* Input);

	/** The different inputs of material expression that we may encounter, the MaterialX Document is modified consequently regarding those*/
	TSet<FString> MaterialExpressionInputs;

	/** List of material functions packages to load in the Game Thread. */
	TSet<FString> MaterialFunctionsToLoad;

	/** Given a MaterialX node (category (optionally a nodegroup) - input), return the UE/Interchange input name.*/
	TMap<TPair<FKeyExpression, FString>, FString> MatchingInputNames;

	/** Given a MaterialX node category, optionally with a node group, return the UE material expression class name*/
	TMap<FKeyExpression, FString> MatchingMaterialExpressions;

	/** Container of a MaterialX document, to translate the different nodes based on their category*/
	TMap<FString, FOnGetMaterialXInstance> MaterialXContainerDelegates;

	/** Given a MaterialX node category, return the UE material function, used for BSDF nodes*/
	TMap<FString, FMaterialXMaterialFunction> MatchingMaterialFunctions;

	/**
	 * Mapping between texture fullpath and their UID
	 * used to allow same name textures with different path,
	 * but avoid duplicating textures across different materials in the same file (for example the chess_set)
	 * The map is reset after each translate
	 */
	TMap<FString, FString> TextureNodeUids;

	/** Categories of nodes to skip during the phase of flattening the subgraphs. Basically if a node has a nodegraph it will be processed as is instead of its nodegraph*/
	std::unordered_set<std::string> CategoriesToSkip;

	/** Categories of nodes to add the default inputs from the nodedefs if they are not present on the node */
	std::unordered_set<std::string> NodeDefsCategories;

	/** Inputs of the nodes category to remove because there's no equivalent in UE Material Expression, to avoid creating a floating Scalar/Vector parameter*/
	std::unordered_map<std::string, std::vector<std::string>> NodeInputsToRemove;

	bool bIsSubstrateEnabled;

	bool bIsSubstrateAdaptiveGBufferEnabled;

	friend bool UE::Interchange::MaterialX::AreMaterialFunctionPackagesLoaded();
};
#endif // WITH_EDITOR