// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "Engine/TextureDefines.h"
#include "InterchangeShaderGraphNode.h"
#include "InterchangeTranslatorBase.h"
#include "InterchangeTexture2DNode.h"
#include "MaterialXBase.h"
#include "MaterialXManager.h"

enum EVectorNoiseFunction : int;

class FMaterialXSurfaceShaderAbstract : public FMaterialXBase
{
public:

	/**
	 * Get the normal input of a surfaceshader, used to plug it in the displacementshader
	 * @param SurfaceShaderNode - The surfaceshader that we return the input from
	 * @return the input of the normal
	 */
	virtual MaterialX::InputPtr GetInputNormal(MaterialX::NodePtr Node, const char*& InputNormal) const;

protected:

	static FString EmptyString;

	// MaterialX states the default output name of the different nodes is 'out'.
	static FString DefaultOutput;

	friend class FMaterialXSurfaceMaterial;
	friend class FMaterialXVolumeMaterial;

	struct FConnectNode
	{
		// The MaterialX node of a given type used to create the appropriate shader node.
		MaterialX::NodePtr UpstreamNode;

		// The shader node to connect to.
		UInterchangeShaderNode* ParentShaderNode;

		// The input of the ParentShaderNode to connect to.
		FString InputChannelName;

		// The output name of the MaterialX node that we need to connect to.
		// This is the active output (e.g: the output given by an input connecting 2 nodes)
		// The default name is 'out' as stated by the standard library. (not necessarily always the case though)
		FString OutputName{ DefaultOutput };
	};
		
	DECLARE_DELEGATE_OneParam(FOnConnectNodeOutputToInput, const FConnectNode&);

	FMaterialXSurfaceShaderAbstract(UInterchangeBaseNodeContainer & BaseNodeContainer);

	/**
	 * Add an attribute to a shader node from the given MaterialX input. Only floats and linear colors are supported.
	 *
	 * @param Input - The MaterialX input to retrieve and add the value from. Must be of type float/color/vector.
	 * @param InputChannelName - The name of the shader node's input to add the attribute.
	 * @param ShaderNode - The shader node to which we want to add the attribute.
	 * @param bBoolAsScalar - Create a scalar instead of a static bool parameter when applicable.
	 *
	 * @return true if the attribute was successfully added.
	 */
	bool AddAttribute(MaterialX::InputPtr Input, const FString& InputChannelName, UInterchangeShaderNode* ShaderNode, int32 OutputIndex = 0, bool bBoolAsScalar = false);

	/**
	 * Add an attribute to a shader node from the given MaterialX input if that input has either a value or an interface name.
	 *
	 * @param Input - The MaterialX input to retrieve and add the value from. Must be of type float/color/vector.
	 * @param InputChannelName - The name of the shader node's input to add the attribute.
	 * @param ShaderNode - The shader node to which we want to add the attribute.
	 * @param bBoolAsScalar - Create a scalar instead of a static bool parameter when applicable.
	 *
	 * @return true if the attribute was successfully added.
	 */
	bool AddAttributeFromValueOrInterface(MaterialX::InputPtr Input, const FString& InputChannelName, UInterchangeShaderNode* ShaderNode, int32 OutputIndex = 0, bool bBoolAsScalar = false);

	/**
	 * Add a bool attribute to a shader node only if its value taken from the input is not equal to its default value. Return false if the attribute does not exist or if we cannot add it
	 *
	 * @param Input - The MaterialX input to retrieve and add the value from, must be of type bool
	 * @param InputChannelName - The name of the shader node's input to add the attribute
	 * @param ShaderNode - The shader node to which we want to add the attribute
	 * @param DefaultValue - the default value to test the input against
	 * @param bAsScalar - Create a scalar value instead of a static bool parameter
	 *
	 * @return true if the attribute was successfully added
	 */
	bool AddBooleanAttribute(MaterialX::InputPtr Input, const FString& InputChannelName, UInterchangeShaderNode* ShaderNode, bool bAsScalar = false);

	/**
	 * Add a float attribute to a shader node only if its value taken from the input is not equal to its default value. Return false if the attribute does not exist or if we cannot add it
	 *
	 * @param Input - The MaterialX input to retrieve and add the value from. Must be of type float.
	 * @param InputChannelName - The name of the shader node's input to add the attribute.
	 * @param ShaderNode - The shader node to which we want to add the attribute.
	 * @param DefaultValue - the default value to test the input against.
	 *
	 * @return true if the attribute was successfully added.
	 */
	bool AddFloatAttribute(MaterialX::InputPtr Input, const FString& InputChannelName, UInterchangeShaderNode* ShaderNode, float DefaultValue = std::numeric_limits<float>::max());

	/**
	 * Add an integer attribute to a shader node only if its value taken from the input is not equal to its default value. Return false if the attribute does not exist or if we cannot add it
	 *
	 * @param Input - The MaterialX input to retrieve and add the value from. Must be of type float.
	 * @param InputChannelName - The name of the shader node's input to add the attribute.
	 * @param ShaderNode - The shader node to which we want to add the attribute.
	 * @param DefaultValue - the default value to test the input against.
	 *
	 * @return true if the attribute was successfully added.
	 */
	bool AddIntegerAttribute(MaterialX::InputPtr Input, const FString& InputChannelName, UInterchangeShaderNode* ShaderNode, int DefaultValue = std::numeric_limits<int>::max());

	/**
	 * Add an FLinearColor attribute to a shader node only if its value taken from the input is not equal to its default value. Return false if the attribute does not exist or if we cannot add it.
	 *
	 * @param Input - The MaterialX input to retrieve and add the value from. Must be of type color.
	 * @param InputChannelName - The name of the shader node's input to add the attribute.
	 * @param ShaderNode - The shader node to which we want to add the attribute.
	 * @param DefaultValue - the default value to test the input against.
	 *
	 * @return true if the attribute was successfully added.
	 */
	bool AddLinearColorAttribute(MaterialX::InputPtr Input, const FString& InputChannelName, UInterchangeShaderNode* ShaderNode, const FLinearColor& DefaultValue = FLinearColor{std::numeric_limits<float>::max(), std::numeric_limits<float>::max(), std::numeric_limits<float>::max()}, int32 OutputIndex = 0);

	/**
	 * Add an FLinearColor attribute to a shader node only if its value taken from the input is not equal to its default value. Return false if the attribute does not exist or if we cannot add it.
	 *
	 * @param Input - The MaterialX input to retrieve and add the value from. Must be of type vector.
	 * @param InputChannelName - The name of the shader node's input to add the attribute.
	 * @param ShaderNode - The shader node to which we want to add the attribute.
	 * @param DefaultValue - the default value to test the input against.
	 *
	 * @return true if the attribute was successfully added.
	 */
	bool AddVectorAttribute(MaterialX::InputPtr Input, const FString& InputChannelName, UInterchangeShaderNode* ShaderNode, const FVector4f& DefaultValue = FVector4f{ std::numeric_limits<float>::max(), std::numeric_limits<float>::max(), std::numeric_limits<float>::max() }, int32 OutputIndex = 0);
	bool AddVector2Attribute(MaterialX::InputPtr Input, const FString& InputChannelName, UInterchangeShaderNode* ShaderNode, const FVector2f& DefaultValue = FVector2f{ std::numeric_limits<float>::max(), std::numeric_limits<float>::max()}, int32 OutputIndex = 0);
	/**
	 * Connect an output either from a node name or a node graph from a MaterialX input to the shader node.
	 * 
	 * @param InputName - The name of the input of the SurfaceShaderNode to retrieve
	 * @param ShaderNode - The Interchange shader node to connect the MaterialX's node or node graph to
	 * @param InputShaderName - The name of the input of the shader node to connect to
	 * @param DefaultValue - The default value of the MaterialX input
	 * @param bIsTangentSpaceInput - Set the tangent space along the path of an input
	 * @param bUseDefaultValue - Usually we want to avoid creating default values on direct inputs of shaders (StandardSurface, OpenPBR), but some shaders (UsdPreviewSurface) we need to decide that at run-time
	 */
	template<typename T>
	bool ConnectNodeOutputToInput(const char* InputName, UInterchangeShaderNode* ShaderNode, const FString& InputShaderName, T DefaultValue, bool bIsTangentSpaceInput = false)
	{
		MaterialX::InputPtr Input = GetInput(SurfaceShaderNode, InputName);

		TGuardValue<bool>InputTypeBeingProcessedGuard(bTangentSpaceInput, bIsTangentSpaceInput);

		bool bIsConnected = ConnectNodeGraphOutputToInput(Input, ShaderNode, InputShaderName);

		if(!bIsConnected)
		{
			bIsConnected = ConnectNodeNameOutputToInput(Input, ShaderNode, InputShaderName);
			if(!bIsConnected)
			{
				// only handle float, linear color and vector here, for other types, the child should handle them as it is most likely not an input but a parameter to set in Interchange
				// handle integers as scalars
				if constexpr(std::is_same_v<decltype(DefaultValue), float> || std::is_same_v<decltype(DefaultValue), int32>)
				{
					bIsConnected = AddFloatAttribute(Input, InputShaderName, ShaderNode, DefaultValue);
				}
				else if constexpr(std::is_same_v<decltype(DefaultValue), FLinearColor>)
				{
					bIsConnected = AddLinearColorAttribute(Input, InputShaderName, ShaderNode, DefaultValue);
				}
				else if constexpr(std::is_same_v<decltype(DefaultValue), FVector4f> || std::is_same_v<decltype(DefaultValue), FVector3f> || std::is_same_v<decltype(DefaultValue), FVector2f>)
				{
					bIsConnected = AddVectorAttribute(Input, InputShaderName, ShaderNode, DefaultValue);
				}
				if constexpr(std::is_same_v<decltype(DefaultValue), bool>)
				{
					bIsConnected = AddBooleanAttribute(Input, InputShaderName, ShaderNode);
				}
			}
		}

		return bIsConnected;
	};

	/**
	 * Connect an ouput in the NodeGraph to the ShaderGraph.
	 *
	 * @param InputToNodeGraph - The input from the standard surface to retrieve the output in the NodeGraph.
	 * @param ShaderNode - The Interchange shader node to connect the MaterialX's node graph to.
	 * @param ParentInputName - The name of the input of the shader node that we want the node graph to be connected to.
	 *
	 * @return true if the given input is attached to one of the outputs of a node graph.
	 */
	bool ConnectNodeGraphOutputToInput(MaterialX::InputPtr InputToNodeGraph, UInterchangeShaderNode* ShaderNode, const FString& ParentInputName);

	/**
	 * Create and connect the output of a MaterialX node that has already a matching in UE to a shader node.
	 * If not, search for a registered delegate.
	 *
	 * @param Node - The MaterialX node of a given type used to create the appropriate shader node.
	 * @param ParentShaderNode - The shader node to connect to.
	 * @param InputChannelName - The input of the ParentShaderNode to connect to.
	 *
	 * @return true if a shader node has been successfully created and is connected to the given input.
	 */
	bool ConnectMatchingNodeOutputToInput(const FConnectNode& Connect);

	/**
	 * Create and connect manually the output of a MaterialX node to a shader node.
	 *
	 * @param Edge - The MaterialX edge that has the current node, its parent and the input bridge between the two.
	 * @param ParentShaderNode - The shader node to connect to.
	 * @param InputChannelName - The input of the ParentShaderNode to connect to.
	 *
	 * @return true if a shader node has been successfully created and is connected to the given input.
	 */
	void ConnectNodeCategoryOutputToInput(const MaterialX::Edge& Edge, UInterchangeShaderNode* ParentShaderNode, const FString& InputChannelName, const FString& OutputName = TEXT("out"));

	/**
	 * Create and connect a node name directly connected from an input to a shader node.
	 *
	 * @param Node - The MaterialX node of a given type used to create the appropriate shader node.
	 * @param ParentShaderNode - The shader node to connect to.
	 * @param InputChannelName - The input of the ParentShaderNode to connect to.
	 *
	 * @return true if a shader node has been successfully created and is connected to the given input.
	 */
	bool ConnectNodeNameOutputToInput(MaterialX::InputPtr Input, UInterchangeShaderNode* ShaderNode, const FString& ParentInputName);

	/** Begin Connect MaterialX nodes*/

	/** <constant> */
	void ConnectConstantInputToOutput(const FConnectNode& Connect);

	/** <extract> */
	void ConnectExtractInputToOutput(const FConnectNode& Connect);

	/** <dot> */
	void ConnectDotInputToOutput(const FConnectNode& Connect);

	/** <transformpoint> */
	void ConnectTransformPositionInputToOutput(const FConnectNode& Connect);

	/** <transformvector> */
	void ConnectTransformVectorInputToOutput(const FConnectNode& Connect);

	/** <rotate2d> */
	void ConnectRotate2DInputToOutput(const FConnectNode& Connect);

	/** <image> */
	void ConnectImageInputToOutput(const FConnectNode& Connect);

	/** <ifgreater> */
	void ConnectIfGreaterInputToOutput(const FConnectNode& Connect);

	/** <ifgreatereq> */
	void ConnectIfGreaterEqInputToOutput(const FConnectNode& Connect);

	/** <ifequal> */
	void ConnectIfEqualInputToOutput(const FConnectNode& Connect);

	/** <outside> */
	void ConnectOutsideInputToOutput(const FConnectNode& Connect);

	/** <position> */
	void ConnectPositionInputToOutput(const FConnectNode& Connect);

	/** <normal> */
	void ConnectNormalInputToOutput(const FConnectNode& Connect);

	/** <tangent> */
	void ConnectTangentInputToOutput(const FConnectNode& Connect);

	/** <bitangent> */
	void ConnectBitangentInputToOutput(const FConnectNode& Connect);

	/** <time> */
	void ConnectTimeInputToOutput(const FConnectNode& Connect);

	/** <noise2d> */
	void ConnectNoise2DInputToOutput(const FConnectNode& Connect);

	/** <cellnoise2d> */
	void ConnectCellNoise2DInputToOutput(const FConnectNode& Connect);

	/** <worleynoise2d> */
	void ConnectWorleyNoise2DInputToOutput(const FConnectNode& Connect);

	/** <noise3d> */
	void ConnectNoise3DInputToOutput(const FConnectNode& Connect);

	/** <cellnoise3d> */
	void ConnectCellNoise3DInputToOutput(const FConnectNode& Connect);

	/** <worleynoise3d> */
	void ConnectWorleyNoise3DInputToOutput(const FConnectNode& Connect);

	/** <heighttonormal> */
	void ConnectHeightToNormalInputToOutput(const FConnectNode& Connect);

	/** <blur> */
	void ConnectBlurInputToOutput(const FConnectNode& Connect);

	/** <texcoord> */
	void ConnectTexCoordInputToOutput(const FConnectNode& Connect);

	/** <switch> */
	void ConnectSwitchInputToOutput(const FConnectNode& Connect);

	/** <normalmap> */
	void ConnectNormalMapInputToOutput(const FConnectNode& Connect);

	/** <refract> */
	void ConnectRefractInputToOutput(const FConnectNode& Connect);

	/** <viewdirection> */
	void ConnectViewDirectionInputToOutput(const FConnectNode& Connect);

	/** <and> */
	void ConnectAndInputToOutput(const FConnectNode& Connect);

	/** <or> */
	void ConnectOrInputToOutput(const FConnectNode& Connect);

	/** <xor> */
	void ConnectXorInputToOutput(const FConnectNode& Connect);

	/** <not> */
	void ConnectNotInputToOutput(const FConnectNode& Connect);

	/** <sparse_volume> is define in Engine/Binaries/ThirdParty/MaterialX/libraries/Interchange */
	void ConnectSparseVolumeInputToOutput(const FConnectNode& Connect);

	/** <luminance>*/
	void ConnectLuminanceInputToOutput(const FConnectNode& Connect);
	
	/** <hextiledimage> <hextilednormalmap>  */
	void ConnectHexTiledInputToOutput(const FConnectNode& Connect);

	/** <latlongimage> */
	void ConnectLatLongImageInputToOutput(const FConnectNode& Connect);

	/** <fractal2d/3d> */
	void ConnectFractalInputToOutput(const FConnectNode& Connect);
	/** End Connect MaterialX nodes*/

	/** Function used only by both Standard Surface and OpenPBR to have both Subsurface and Transmission*/
	[[nodiscard]]
	bool ConnectSubsurfaceTransmissionToSurfaceShader(UInterchangeShaderNode* OpenPbrStandarSurfaceShaderNode, EInterchangeMaterialXShaders SurfaceShaderType													  
													  , const FName& BaseWeight, const FName& BaseColor
													  , const FName& TransmissionWeight, const FName& TransmissionColor, const FName& TransmissionDepth, const FName& TransmissionDispersionAbbeNumber, const FName& TransmissionDispersionScale, const FName& TransmissionScatter, const FName& TransmissionScatterAnisotropy
													  , const FName& SubsurfaceWeight, const FName& SubsurfaceColor, const FName& SubsurfaceRadiusScale
													  , const FName & FrontMaterialOpaque, const FName & FrontMaterialTranslucent);

	/**
	 * Create a ComponentMask shader node.
	 *
	 * @param RGBA - The mask component. For example: 0b1011 -> Only RBA are toggled
	 * @param Element - The element that we take the whole hierarchy from to ensure the uniqueness of the name
	 * @param NodeName - the name of the shader node.
	 * @param OutputName - the name of the output of the MaterialX node. The default name is 'out' as stated by the standard library.
	 * @return The ComponentMask node.
	 */
	UInterchangeShaderNode* CreateMaskShaderNode(uint8 RGBA, MaterialX::ElementPtr Element, const FString& NodeName, const FString& OutputName = TEXT("out"));

	/**
	 * Helper function to create an InterchangeShaderNode.
	 *
	 * @param Element - The element that we take the whole hierarchy from to ensure the uniqueness of the name
	 * @param NodeName - The name of the shader node.
	 * @param ShaderType - The type of shader node we want to create.
	 * @param OutputName - The output name of the MaterialX node. The default name is 'out' as stated by the standard library.
	 *
	 * @return The shader node that was created.
	 */
	UInterchangeShaderNode* CreateShaderNode(MaterialX::ElementPtr Element, const FString& NodeName, const FString& ShaderType, const FString& OutputName = TEXT("out"));

	/**
	 * Helper function to create an InterchangeFunctionCallShaderNode.
	 *
	 * @param NodeName - The name of the shader node.
	 * @param FunctionPath - The path to the Material Function we want to create.
	 * @param OutputName - The output name of the MaterialX node. The default name is 'out' as stated by the standard library.
	 *
	 * @return The shader node that was created.
	 */
	UInterchangeFunctionCallShaderNode* CreateFunctionCallShaderNode(MaterialX::ElementPtr Element, const FString& NodeName, const FString& FunctionPath, const FString& OutputName = TEXT("out"));
	UInterchangeFunctionCallShaderNode* CreateFunctionCallShaderNode(MaterialX::ElementPtr Element, const FString& NodeName, uint8 EnumType, uint8 EnumValue, const FString& OutputName = TEXT("out"));

	/**
	 * Helper function to create an InterchangeTextureNode.
	 *
	 * @param Node - The MaterialX node. This should be of the category <image>. No test is done on it.
	 *
	 * @return The texture node that was created.
	 */
	template<typename TextureTypeNode>
	UInterchangeTextureNode* CreateTextureNode(MaterialX::NodePtr Node) const
	{
		static_assert(std::is_convertible_v<TextureTypeNode*, UInterchangeTextureNode*>, "CreateTextureNode only accepts type that derived from UInterchangeTextureNode");
		UInterchangeTexture2DNode* TextureNode = nullptr;

		//A node image should have an input file otherwise the user should check its default value
		if(Node)
		{
			if(MaterialX::InputPtr InputFile = Node->getInput("file"); InputFile && (InputFile->hasValue() || InputFile->hasInterfaceName()))
			{
				FString Filepath;
				// a <geompropvalue> converted as an <image> as no filepath by default, since the texture will be created later, we just put a placeholder, in order for the factory to not complain
				// the texture will be overridden later in the ExecutePostFactoryImport
				const bool bGeomPropImage = Node->getTypedAttribute<bool>(MaterialX::Attributes::GeomPropImage);
				if (bGeomPropImage)
				{
					Filepath = FPaths::Combine(FPaths::EnginePluginsDir(), TEXT("Interchange"), TEXT("Editor"), TEXT("Content"), TEXT("Resources"), TEXT("Interchange_PixelPowerOfTwo.png"));
					Filepath = FPaths::ConvertRelativePathToFull(Filepath);
				}
				else
				{
					Filepath = InputFile->hasValue() ? InputFile->getValueString().c_str() : InputFile->getInterfaceInput()->getValueString().c_str();
					const FString FilePrefix = GetFilePrefix(InputFile);
					Filepath = FPaths::Combine(FilePrefix, Filepath);
					
					// replace <UDIM> by a wildcard instead of just 1001, because the UDIM can only be for example one single texture like blabla.1003.png, which end up in a non VT
					if (Filepath.ReplaceInline(TEXT("<UDIM>"), TEXT("*")))
					{
						TArray<FString> FileNames;
						// we want to preserve the path given by the FilePrefix
						FString SubPath = FPaths::GetPath(Filepath);
						IFileManager::Get().FindFiles(FileNames, *FPaths::ConvertRelativePathToFull(FPaths::GetPath(Node->getActiveSourceUri().c_str()), Filepath), true, false);
						if (!FileNames.IsEmpty())
						{
							Filepath = FPaths::Combine(SubPath, FileNames[0]);
						}
						else
						{
							MTLX_LOG("MaterialXUDIM", "Couldn't find any UDIM textures in the folder {0}", *SubPath);
						}
					}
				}

				const FString Filename = FPaths::GetCleanFilename(Filepath);
				
				FString TextureNodeUID = FMaterialXManager::GetInstance().FindOrAddTextureNodeUid(Filepath);				
				
				// We need to duplicate the texture for a geomprop, at the end the texture will be the baking texture
				if (bGeomPropImage)
				{
					TextureNodeUID = FPaths::GetPath(TextureNodeUID) + TEXT("\\") + SurfaceMaterialName + TEXT("_") + Node->getName().c_str();
				}

				//Only add the TextureNode once
				TextureNode = const_cast<UInterchangeTexture2DNode*>(Cast<UInterchangeTexture2DNode>(NodeContainer.GetNode(TextureNodeUID)));
				if(TextureNode == nullptr)
				{
					TextureNode = NewObject<TextureTypeNode>(&NodeContainer);
					NodeContainer.SetupNode(TextureNode, TextureNodeUID, Filename, EInterchangeNodeContainerType::TranslatedAsset);

					if (bGeomPropImage)
					{
						TextureNode->SetDisplayLabel(Node->getName().c_str());
					}

					if(FPaths::IsRelative(Filepath))
					{
						Filepath = FPaths::ConvertRelativePathToFull(FPaths::GetPath(Node->getActiveSourceUri().c_str()), Filepath);
					}

					TextureNode->SetPayLoadKey(Filepath);

					const FString ColorSpace = GetColorSpace(InputFile);
					const bool bIsSRGB = ColorSpace == TEXT("srgb_texture");
					TextureNode->SetCustomSRGB(bIsSRGB);

					auto GetAddressMode = [Node](const char* InputName)
					{
						EInterchangeTextureWrapMode WrapMode = EInterchangeTextureWrapMode::Wrap;
						if(MaterialX::InputPtr InputAddressMode = Node->getInput(InputName))
						{
							const std::string& AddressMode = InputAddressMode->getValueString();
							if(AddressMode == "clamp")
							{
								WrapMode = EInterchangeTextureWrapMode::Clamp;
							}
							else if(AddressMode == "mirror")
							{
								WrapMode = EInterchangeTextureWrapMode::Mirror;
							}
							else
							{
								WrapMode = EInterchangeTextureWrapMode::Wrap;
							}
						}

						return WrapMode;
					};

					EInterchangeTextureWrapMode WrapModeU = GetAddressMode("uaddressmode");
					EInterchangeTextureWrapMode WrapModeV = GetAddressMode("vaddressmode");

					TextureNode->SetCustomWrapU(WrapModeU);
					TextureNode->SetCustomWrapV(WrapModeV);

					// Encode the compression in the payloadKey
					if(bTangentSpaceInput && Node->getType() == "vector3")
					{
						TextureNode->SetPayLoadKey(TextureNode->GetPayLoadKey().GetValue() + FMaterialXManager::TexturePayloadSeparator + FString::FromInt(TextureCompressionSettings::TC_Normalmap));
					}
				}
			}
		}

		return TextureNode;
	}

	/**
	 * Get the UE corresponding name of a MaterialX Node category and input for a material.
	 *
	 * @param Input - MaterialX input.
	 *
	 * @return The matched name of the Node/Input, or an empty string.
	 */
	const FString& GetMatchedInputName(MaterialX::NodePtr Node, MaterialX::InputPtr Input) const;

	/**
	 * Get the input name. Use this function instead of getName() because this returns the name that will be used by UE inputs even if a renaming has occurred.
	 *
	 * @param Input - The input to retrieve the name from.
	 *
	 * @return The input name.
	 */
	FString GetInputName(MaterialX::InputPtr Input) const;

	/**
	 * Return the innermost file prefix of an element in the current scope. If it has none, it will take the one from its parents.
	 *
	 * @param Element - the Element to retrieve the file prefix from. This can be anything: an input, a node, a nodegraph, and so on.
	 *
	 * @return a file prefix or an empty string.
	 *
	 */
	FString GetFilePrefix(MaterialX::ElementPtr Element) const;

	/**
	 * Helper function that returns a vector. The function makes no assumption on the input, and it should have a value of vectorN type.
	 *
	 * @param Input - The input that has a vectorN value in it.
	 *
	 * @return The vector.
	 */
	FLinearColor GetVector(MaterialX::InputPtr Input) const;

	/**
	 * Retrieve the Interchange parent name of a MaterialX node. Useful when a node is a combination of several nodes connected to different inputs, such as Noise3D.
	 *
	 * @param Node - The node whose parent name is retrieved.
	 * @param ConnectedInput - The connected input to the Node, may be null, may happen at the top of a graph when there's upstream/downstream connection
	 * @return The node of the parent.
	 */
	FString GetAttributeParentName(MaterialX::NodePtr Node, MaterialX::InputPtr ConnectedInput) const;

	/**
	 * Ensure that we put in the node container the unique name, 2 nodes in a same file may have the same name as long as their parent's name is different, we just traverse the whole hierarchy to have a unique name
	 * 
	 * @param Element - An Element that may be an Input a node or a nodegraph
	 * @return The unique name of the Element taking the whole hierarchy into account
	 */
	FString GetUniqueName(MaterialX::ElementPtr Element) const;

	virtual void RegisterConnectNodeOutputToInputDelegates();

	/**
	 * Set the matching inputs names of a node to correspond to the one used by UE. The matching name is stored under the attribute UE::NewName.
	 *
	 * @param Node - Look up to all inputs of Node and set the matching name attribute.
	 */
	void SetMatchingInputsNames(MaterialX::NodePtr Node) const;

	/**
	 * Add the input new name under the attribute UE::NewName.
	 *
	 * @param Input - The input to add the proper attribute.
	 * @param NewName - the new name of the input.
	 */
	void SetAttributeNewName(MaterialX::InputPtr Input, const char* NewName) const;

	/**
	 * This function should be called first by the Translate method of derived class, SurfaceShaderNode should initialized first by the derived class
	 * 
	 * @param ShaderType - the type of ShaderGraphNode to create
	 * @return The shader node created, usually a function call shader node
	 */
	UInterchangeShaderNode* Translate(EInterchangeMaterialXShaders ShaderType);

	/**
	 * This function should be called first by the Translate method of derived class, SurfaceShaderNode should initialized first by the derived class
	 *
	 * @param ShaderNode - the shader node that was created, can be either a shader node or a function call shader node
	 * @return The shader node created with all inputs connected to it
	 */
	UInterchangeShaderNode* Translate(UInterchangeShaderNode* ShaderNode);

	virtual UInterchangeBaseNode* Translate(MaterialX::NodePtr ShaderNode) override = 0;

private:
	
	UInterchangeShaderNode* ConnectGeometryInputToOutput(const FConnectNode& Connect, const FString& ShaderType, const FString& TransformShaderType, const FString& TransformInput, const FString& TransformSourceType, int32 TransformSource, const FString& TransformType, int32 TransformSDestination, bool bIsVector = true);
	UInterchangeShaderNode* ConnectNoise2DInputToOutput(const FConnectNode& Connect, const FString& ShaderType, EVectorNoiseFunction NoiseFunction, uint8 Mask);
	void ConnectLogicalInputToOutput(const FConnectNode& Connect, const FString& LogicalMaterialFunction);
protected:

	/** 
	 * @param Get<0>: node
	 * @param Get<1>: output
	 */
	using FNodeOutput = TPair<FString, FString>;

	/** Store the shader nodes only when we create the shader graph node*/
	TMap<FNodeOutput, UInterchangeShaderNode*> ShaderNodes;

	/** Matching MaterialX category and Connect function*/
	TMap<FString, FOnConnectNodeOutputToInput> MatchingConnectNodeDelegates;

	/** The surface shader node processed during the Translate, up to the derived class to initialize it*/
	MaterialX::NodePtr SurfaceShaderNode;

	/** Initialized by the material shader (e.g: surfacematerial), the derived class should only set the ShaderType */
	UInterchangeShaderGraphNode* ShaderGraphNode;

	/** Here for log purpose, it makes the logs easier to read especially when several materials are imported at once*/
	FString SurfaceMaterialName;

	/** Used for texture compression and transform to tangent space nodes coming from inputs such as coat_normal  */
	bool bTangentSpaceInput;
};
#endif