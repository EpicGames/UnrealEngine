// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeCommonAnimationPayload.h"
#include "InterchangeFactoryBase.h"
#include "Templates/PimplPtr.h"

#include "InterchangeGeometryCacheFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

class UGeometryCacheTrack;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeGeometryCacheFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()
public:

	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() override { return EInterchangeFactoryAssetType::Meshes; }
	UE_API virtual void CreatePayloadTasks(const FImportAssetObjectParams& Arguments, bool bAsync, TArray<TSharedPtr<UE::Interchange::FInterchangeTaskBase>>& PayloadTasks) override;
	UE_API virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual FImportAssetResult ImportAsset_Async(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual FImportAssetResult EndImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;
	UE_API virtual bool GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const override;
	UE_API virtual bool SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const override;
	UE_API virtual void BackupSourceData(const UObject* Object) const override;
	UE_API virtual void ReinstateSourceData(const UObject* Object) const override;
	UE_API virtual void ClearBackupSourceData(const UObject* Object) const override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////

private:
	TArray<TObjectPtr<UGeometryCacheTrack>> Tracks;
	TPimplPtr<class FGeometryCacheComponentResetAsset> ResetAssetOnReimport;

	// Animation queries results
	TMap<FString, UE::Interchange::FAnimationPayloadData> TransformAnimationPayloadResults;
};

#undef UE_API
