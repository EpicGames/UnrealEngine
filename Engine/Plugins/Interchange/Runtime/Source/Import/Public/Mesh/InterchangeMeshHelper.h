// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "MeshDescription.h"
#include "StaticMeshOperations.h"

#include "InterchangeFactoryBase.h"

class UBodySetup;
class UInterchangeBaseNodeContainer;
class UInterchangeSceneNode;
class UInterchangeMeshFactoryNode;
class UInterchangeStaticMeshFactoryNode;
class UInterchangeStaticMeshLodDataNode;
class UMaterialInterface;
class USkeletalMesh;
class UStaticMesh;
struct FKAggregateGeom;
struct FSkeletalMaterial;
struct FStaticMaterial;
struct FInterchangeMeshPayLoadKey;
enum class EInterchangeMeshCollision : uint8;

namespace UE::Interchange
{
	struct FMeshPayloadData;
	struct FMeshPayload;
	struct FNaniteAssemblyDescription;
}

namespace UE::Interchange::Private::MeshHelper
{
	/** These are largely copied from StaticMeshImportUtils.h */

	INTERCHANGEIMPORT_API bool AddConvexGeomFromVertices(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		const FMeshDescription& MeshDescription,
		FKAggregateGeom& AggGeom
	);

	INTERCHANGEIMPORT_API bool DecomposeConvexMesh(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		const FMeshDescription& MeshDescription,
		UBodySetup* BodySetup
	);

	INTERCHANGEIMPORT_API bool AddBoxGeomFromTris(
		const FMeshDescription& MeshDescription, 
		FKAggregateGeom& AggGeom,
		bool bForcePrimitiveGeneration = true
	);

	INTERCHANGEIMPORT_API bool AddSphereGeomFromVertices(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		const FMeshDescription& MeshDescription,
		FKAggregateGeom& AggGeom,
		bool bForcePrimitiveGeneration = true
	);

	INTERCHANGEIMPORT_API bool AddCapsuleGeomFromVertices(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		const FMeshDescription& MeshDescription,
		FKAggregateGeom& AggGeom
	);

	INTERCHANGEIMPORT_API bool ImportBoxCollision(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		const TMap<FInterchangeMeshPayLoadKey, FMeshPayload>& BoxCollisionPayloads,
		UStaticMesh* StaticMesh,
		bool bForcePrimitiveGeneration = true
	);

	INTERCHANGEIMPORT_API bool ImportCapsuleCollision(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		const TMap<FInterchangeMeshPayLoadKey, FMeshPayload>& CapsuleCollisionPayloads,
		UStaticMesh* StaticMesh
	);

	INTERCHANGEIMPORT_API bool ImportSphereCollision(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		const TMap<FInterchangeMeshPayLoadKey, FMeshPayload>& SphereCollisionPayloads, 
		UStaticMesh* StaticMesh,
		bool bForcePrimitiveGeneration = true
	);

	INTERCHANGEIMPORT_API bool ImportConvexCollision(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		const TMap<FInterchangeMeshPayLoadKey, FMeshPayload>& ConvexCollisionPayloads,
		UStaticMesh* StaticMesh,
		const UInterchangeStaticMeshLodDataNode* LodDataNode
	);

	INTERCHANGEIMPORT_API bool ImportSockets(
		const UInterchangeFactoryBase::FImportAssetObjectParams& Arguments,
		UStaticMesh* StaticMesh,
		const UInterchangeStaticMeshFactoryNode* FactoryNode
	);

	INTERCHANGEIMPORT_API void RemapPolygonGroups(
		const FMeshDescription& SourceMesh,
		FMeshDescription& TargetMesh,
		PolygonGroupMap& RemapPolygonGroup
	);

	/* return the result of the global transform with the geometric and pivot transform of the scene node. */
	INTERCHANGEIMPORT_API void AddSceneNodeGeometricAndPivotToGlobalTransform(FTransform& GlobalTransform, const UInterchangeSceneNode* SceneNode, const bool bBakeMeshes, const bool bBakePivotMeshes);

	/**
	* In case the input SceneNode is a JointNode we are going to calculate the SkeletonTransform of the given Node (for bBakeMeshes=false).
	* RootJointParentUidAndGlobalInverseMatrixOptional - Specifically used for SkeletalMesh's transform generations.
	* bProcessingSkeletalMesh - Changes transform generation behavior for Non Joint SceneNode transforms 
	* 							(we always bake to RootJoint or SceneRoot for SkinnedMeshes).
	*/
	INTERCHANGEIMPORT_API FTransform CalculateCompleteSceneNodeTransform(
		UInterchangeBaseNodeContainer* BaseNodeContainer, const UInterchangeSceneNode* SceneNode,
		const bool bBakeMeshes, const bool bBakePivotMeshes,
		const FTransform& GlobalOffsetTransform,
		const bool bUseTimeZeroAsBindPose = false,
		const TOptional<TPair<FString, FMatrix>>& RootJointParentUidAndGlobalInverseMatrixOptional = TOptional<TPair<FString, FMatrix>>(),
		const bool bProcessingSkeletalMesh = false,
		const bool bAlignSkeletalMeshInScene = false);

	INTERCHANGEIMPORT_API void SkeletalMeshFactorySetupAssetMaterialArray(TArray<FSkeletalMaterial>& ExistMaterials
		, TMap<FString, FString> ImportedSlotMaterialDependencies
		, const UInterchangeBaseNodeContainer* NodeContainer
		, const bool bIsReimport);

	INTERCHANGEIMPORT_API void StaticMeshFactorySetupAssetMaterialArray(TArray<FStaticMaterial>& ExistMaterials
		, TMap<FString, FString> ImportedSlotMaterialDependencies
		, const UInterchangeBaseNodeContainer* NodeContainer
		, const bool bIsReimport);

	INTERCHANGEIMPORT_API void GeometryCacheFactorySetupAssetMaterialArray(TArray<TObjectPtr<UMaterialInterface>>& ExistMaterials
		, TArray<FName>& MaterialSlotNames
		, TMap<FString, FString> ImportedSlotMaterialDependencies
		, const UInterchangeBaseNodeContainer* NodeContainer
		, const bool bIsReimport);

	INTERCHANGEIMPORT_API void CopyMorphTargetsMeshDescriptionToSkeletalMeshDescription(TArray<FString>& SkeletonMorphCurveMetadataNames
		, const TMap<FString, TOptional<UE::Interchange::FMeshPayloadData>>& LodMorphTargetMeshDescriptions
		, FMeshDescription& DestinationMeshDescription
		, const bool bMergeMorphTargetWithSameName);

#if WITH_EDITOR
	INTERCHANGEIMPORT_API void RemapSkeletalMeshVertexColorToMeshDescription(const USkeletalMesh* SkeletalMesh, const int32 LODIndex, FMeshDescription& MeshDescription);
#endif

	/* Checks name for UBX_, UCX_, MCDCX_, USP_, UCP_ prefixes.*/
	INTERCHANGEIMPORT_API EInterchangeMeshCollision GetMeshCollisionFromName(const FString& Name);

	/* Checks if Name has any of the prefixes mentioned in GetMeshCollisionFromName, without the '_', this primarily used for checking for typos. */
	INTERCHANGEIMPORT_API bool HasPotentialMeshCollisionPrefix(const FString& Name);

#if WITH_EDITOR
	/** Nanite assembly builder helper for static and skeletal meshes. */
	class FInterchangeNaniteAssemblyBuilder
	{
	public:
		~FInterchangeNaniteAssemblyBuilder();

		/**
		 * Create a builder for the given skeletal mesh. The given mesh factory node must hold the mesh part dependency information. 
		 */
		static FInterchangeNaniteAssemblyBuilder Create(USkeletalMesh* SkeletalMesh, const UInterchangeMeshFactoryNode* MeshFactoryNode, const UInterchangeBaseNodeContainer* NodeContainer);

		/**
		 * Create a builder for the given static mesh. The given mesh factory node must hold the mesh part dependency information. 
		 */
		static FInterchangeNaniteAssemblyBuilder Create(UStaticMesh* StaticMesh, const UInterchangeMeshFactoryNode* MeshFactoryNode, const UInterchangeBaseNodeContainer* NodeContainer);

		/**
		 * Add the given FNaniteAssemblyDescription data to the builder. This can be called more than
		 * once in order to merge multiple descriptions into a single skeletal/static mesh assembly.
		 */
		INTERCHANGEIMPORT_API void AddDescription(const TOptional<UE::Interchange::FNaniteAssemblyDescription>& Description);

		class ImplBase;

	private:

		TUniquePtr<ImplBase> Impl;
		explicit FInterchangeNaniteAssemblyBuilder(TUniquePtr<ImplBase> Impl);
	};
#endif
} //ns UE::Interchange::Private::MeshHelper
