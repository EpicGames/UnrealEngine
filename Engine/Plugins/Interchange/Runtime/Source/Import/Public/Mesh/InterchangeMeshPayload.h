// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshDescription.h"

struct FNaniteAssemblyBoneInfluence;

namespace UE
{
	namespace Interchange
	{
		struct FNaniteAssemblyDescription
		{
			struct FBoneInfluence
			{
				uint32 BoneIndex = INDEX_NONE;

				float BoneWeight = 1.0f;
			};

			/* Assembly Transforms */
			TArray<FTransform> Transforms;

			/* Assembly Part Indices, must be the same length as Transforms */
			TArray<int32> PartIndices;

			/* List of interchange mesh uids, or, UE asset paths per index */
			TArray<TArray<FString>> PartUids;

			/* List of bone influences. */
			// For SkeletalMesh assemblies there should one or more influences per transform. 
			// For StaticMesh assemblies this should be empty.
			TArray<FBoneInfluence> BoneInfluences;

			/* Number of bone influences per instance */
			TArray<int32> NumInfluencesPerInstance;

			/* Check that the above arrays have been authored correctly */
			bool IsValid(FString* OutReason = nullptr) const;
		};

		struct FMeshPayloadData
		{
			/* MESH */
			//Currently the skeletalmesh payload data is editor only, we have to move to something available at runtime
			FMeshDescription MeshDescription;

			/* SKELETAL */
			//This map the indice use in the meshdescription to the bone name, so we can use this information to remap properly the skinning when we merge the meshdescription
			TArray<FString> JointNames;

			/* MORPH */
			//We don't have to store GlobalTransform here anymore, since The Mesh node parent bake transform was pass to the payload request.
			//The vertex offset of the morph target in case we combine mesh node together
			int32 VertexOffset;
			//The name of the morph target
			FString MorphTargetName;

			/* NANITE ASSEMBLY */
			// Optional Nanite Assembly data that should be attached to this skeletal or static mesh.
			TOptional<FNaniteAssemblyDescription> NaniteAssemblyDescription;
		};

		struct FMeshPayload
		{
			FString MeshName;
			TOptional<UE::Interchange::FMeshPayloadData> PayloadData;
			FTransform Transform = FTransform::Identity;
		};
	}//ns Interchange
}//ns UE
