// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Async/Future.h"
#include "InterchangeMeshDefinitions.h"
#include "InterchangeSourceData.h"
#include "Mesh/InterchangeMeshPayload.h"
#include "InterchangeMeshNode.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"

#include "Types/AttributeStorage.h"

#include "InterchangeMeshPayloadInterface.generated.h"

UINTERFACE(MinimalAPI)
class UInterchangeMeshPayloadInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Static mesh payload interface. Derive from this interface if your payload can import static mesh
 */
class IInterchangeMeshPayloadInterface
{
	GENERATED_BODY()
public:

	UE_DEPRECATED(5.6, "Deprecated. Use GetMeshPayloadData(const FInterchangeMeshPayLoadKey&) instead.")
	virtual TOptional<UE::Interchange::FMeshPayloadData> GetMeshPayloadData(const FInterchangeMeshPayLoadKey& PayLoadKey, const FTransform& MeshGlobalTransform) const
	{
		using namespace UE::Interchange;
		UE::Interchange::FAttributeStorage Attributes;
		Attributes.RegisterAttribute(UE::Interchange::FAttributeKey{ MeshPayload::Attributes::MeshGlobalTransform }, MeshGlobalTransform);
		return GetMeshPayloadData(PayLoadKey, Attributes);
	}

	/**
	 * Once the translation is done, the import process need a way to retrieve payload data.
	 * This payload will be use by the factories to create the asset.
	 *
	 * @param PayloadKey - The key to retrieve a particular payload contained into the specified source data.
	 * @param PayloadAttributes - Attributes passed by pipelines to the Translators
	 * @return a PayloadData containing the the data ask with the key.
	 */
	virtual TOptional<UE::Interchange::FMeshPayloadData> GetMeshPayloadData(const FInterchangeMeshPayLoadKey& PayLoadKey, const UE::Interchange::FAttributeStorage& PayloadAttributes) const = 0;
};


