// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeTranslatorBase.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Mesh/InterchangeMeshPayload.h"
#include "Mesh/InterchangeMeshPayloadInterface.h"
#include "Texture/InterchangeTexturePayloadInterface.h"

#include "InterchangeOBJTranslator.generated.h"

#define UE_API INTERCHANGEIMPORT_API

struct FObjData;


UCLASS(MinimalAPI, BlueprintType)
class UInterchangeOBJTranslator : public UInterchangeTranslatorBase,
	                                                    public IInterchangeMeshPayloadInterface,
	                                                    public IInterchangeTexturePayloadInterface
{
	GENERATED_BODY()

public:

	UE_API UInterchangeOBJTranslator();
	UE_API virtual ~UInterchangeOBJTranslator();


	UE_API virtual TArray<FString> GetSupportedFormats() const override;
	UE_API virtual EInterchangeTranslatorAssetType GetSupportedAssetTypes() const override;
	UE_API virtual EInterchangeTranslatorType GetTranslatorType() const override;

	/**
	 * Translate the associated source data into a node hold by the specified nodes container.
	 *
	 * @param BaseNodeContainer - The unreal objects descriptions container where to put the translated source data.
	 * @return true if the translator can translate the source data, false otherwise.
	 */
	UE_API virtual bool Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const override;


	/* IInterchangeStaticMeshPayloadInterface Begin */

	/**
	 * Once the translation is done, the import process need a way to retrieve payload data.
	 * This payload will be use by the factories to create the asset.
	 *
	 * @param PayloadKey - The key to retrieve the a particular payload contain into the specified source data.
	 * @return a PayloadData containing the the data ask with the key.
	 */
	UE_DEPRECATED(5.6, "Deprecated. Use GetMeshPayloadData(const FInterchangeMeshPayLoadKey&) instead.")
	virtual TOptional<UE::Interchange::FMeshPayloadData> GetMeshPayloadData(const FInterchangeMeshPayLoadKey& PayLoadKey, const FTransform& MeshGlobalTransform) const override
	{
		using namespace UE::Interchange;
		UE::Interchange::FAttributeStorage Attributes;
		Attributes.RegisterAttribute(UE::Interchange::FAttributeKey{ MeshPayload::Attributes::MeshGlobalTransform }, MeshGlobalTransform);
		return GetMeshPayloadData(PayLoadKey, Attributes);
	}

	UE_API virtual TOptional<UE::Interchange::FMeshPayloadData> GetMeshPayloadData(const FInterchangeMeshPayLoadKey& PayLoadKey, const UE::Interchange::FAttributeStorage& PayloadAttributes) const override;

	/* IInterchangeStaticMeshPayloadInterface End */

	//////////////////////////////////////////////////////////////////////////
	/* IInterchangeTexturePayloadInterface Begin */

	/**
	 * Once the translation is done, the import process need a way to retrieve payload data.
	 * This payload will be use by the factories to create the asset.
	 *
	 * @param InSourceData - The source data containing the data to translate
	 * @param PayloadKey - The key to retrieve the a particular payload contain into the specified source data.
	 * @return a PayloadData containing the imported data. The TOptional will not be set if there is an error.
	 */
	UE_API virtual TOptional<UE::Interchange::FImportImage> GetTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const override;
	/* IInterchangeTexturePayloadInterface End */

private:
	TPimplPtr<FObjData> ObjDataPtr;
};

#undef UE_API
