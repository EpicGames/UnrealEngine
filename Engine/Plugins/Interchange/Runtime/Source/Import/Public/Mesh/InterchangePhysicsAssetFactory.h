// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeFactoryBase.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangePhysicsAssetFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangePhysicsAssetFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()
public:

	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() override { return EInterchangeFactoryAssetType::Physics; }
	UE_API virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual FImportAssetResult ImportAsset_Async(const FImportAssetObjectParams& Arguments) override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////
};


#undef UE_API
