// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rendering/SkinWeightProfile.h"
#include "ClothingAsset.h"
#include "CoreMinimal.h"
#include "InterchangeFactoryBase.h"
#include "InterchangeMeshNode.h"
#include "Mesh/InterchangeMeshPayload.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "SkeletalMeshTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeSkeletalMeshFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

class UInterchangeInstancedStaticMeshComponentNode;
class UInterchangeSceneNode;
class UInterchangeSkeletalMeshFactoryNode;
class USkeletalMesh;
class USkeleton;

// Utility class used on reimport only to: 
// - Call PostEditchange at the end of the reimport which will reallocate the render resource
// - Used to recreate all skinned mesh components for a given skinned asset via FSkinnedMeshComponentRecreateRenderStateContext.
// - Lock the skeletal mesh's properties while it is updated
class FScopedSkeletalMeshReimportUtility : public FSkinnedMeshComponentRecreateRenderStateContext
{
public:
	FScopedSkeletalMeshReimportUtility(USkeletalMesh* InSkeletalMesh);
	~FScopedSkeletalMeshReimportUtility();

private:
#if WITH_EDITOR
	TUniquePtr<FScopedSkeletalMeshPostEditChange> ScopedPostEditChange;
#endif
	FEvent* LockPropertiesEvent = nullptr;
};

namespace UE::Interchange
{
	//Get the mesh node context for each MeshUids
	struct FMeshNodeContext
	{
		const UInterchangeMeshNode* MeshNode = nullptr;
		
		FTransform SceneGlobalTransform;
		//If MeshNodes was resolved from a UInterchangeSceneNode it can have a non-identity GeometricTransform, which is needed for T0 re-skin calculations.
		FTransform GeometricTransform;
		
		/*
		* SceneNodeName is needed to identify the bone targeting given Mesh for Rigid to Rigged conversions.
		* It's also used for the GeometryParts.
		*	This is the instancing UInterchangeSceneNode's name.
		*/
		FString SceneNodeName;

		FInterchangeMeshPayLoadKey TranslatorPayloadKey;

		//Return a new key with the translator key merge with the transform
		FInterchangeMeshPayLoadKey GetTranslatorAndTransformPayloadKey() const;

		FInterchangeMeshPayLoadKey GetMorphTargetAndTransformPayloadKey(const FInterchangeMeshPayLoadKey& MorphTargetKey) const;

		//Return the translator key merge with the transform
		FString GetUniqueId() const;
	};
} //UE::Interchange

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSkeletalMeshFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()
public:
	struct FImportAssetObjectLODData
	{
		int32 LodIndex = INDEX_NONE;
		TArray<FName> ExistingOriginalPerSectionMaterialImportName;
#if WITH_EDITOR
		TArray<SkeletalMeshImportData::FMaterial> ImportedMaterials;
		TArray<SkeletalMeshImportData::FBone> RefBonesBinary;
#endif
		TArray<UE::Interchange::FMeshNodeContext> MeshNodeContexts;
		bool bUseTimeZeroAsBindPose = false;
		bool bDiffPose = false;

		//Store Morphtarget name, we want to add skeleton curve meta data ingame thread (FinalizeObject_GameThread)
		TArray<FString> SkeletonMorphCurveMetadataNames;
	};

	struct FImportAssetObjectData
	{
		bool bIsReImport = false;
		USkeleton* SkeletonReference = nullptr;
		bool bApplyGeometryOnly = false;
		TArray<FImportAssetObjectLODData> LodDatas;

		TArray<FSkinWeightProfileInfo> ExistingSkinWeightProfileInfos;
		TArray<ClothingAssetUtils::FClothingAssetMeshBinding> ExistingClothingBindings;
#if WITH_EDITOR
		TArray<FMeshDescription> ExistingAlternateMeshDescriptionPerLOD;
#endif

		bool IsValid() const;
	};
	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() override { return EInterchangeFactoryAssetType::Meshes; }
	UE_API virtual void CreatePayloadTasks(const FImportAssetObjectParams& Arguments, bool bAsync, TArray<TSharedPtr<UE::Interchange::FInterchangeTaskBase>>& PayloadTasks) override;
	UE_API virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual FImportAssetResult ImportAsset_Async(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual FImportAssetResult EndImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual void Cancel() override;
	UE_API virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;
	UE_API virtual void BuildObject_GameThread(const FSetupObjectParams& Arguments, bool& OutPostEditchangeCalled) override;
	UE_API virtual void FinalizeObject_GameThread(const FSetupObjectParams& Arguments) override;
	UE_API virtual bool GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const override;
	UE_API virtual bool SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const override;
	UE_API virtual bool SetReimportSourceIndex(const UObject* Object, int32 SourceIndex) const override;
	UE_API virtual void BackupSourceData(const UObject* Object) const override;
	UE_API virtual void ReinstateSourceData(const UObject* Object) const override;
	UE_API virtual void ClearBackupSourceData(const UObject* Object) const override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////

	struct FLodPayloads
	{
		TMap<FInterchangeMeshPayLoadKey, TOptional<UE::Interchange::FMeshPayloadData>> MeshPayloadPerKey;
		TMap<FInterchangeMeshPayLoadKey, TOptional<UE::Interchange::FMeshPayloadData>> MorphPayloadPerKey;
	};

	struct FNaniteAssemblyData
	{
		UE::Interchange::FNaniteAssemblyDescription Description;
		TArray<FString> JointNames;
	};

private:
	TUniquePtr<FScopedSkeletalMeshReimportUtility> ScopedReimportUtility;

	TMap<int32, FLodPayloads> PayloadsPerLodIndex;

	FImportAssetObjectData ImportAssetObjectData;

	TArray<FNaniteAssemblyData> NaniteAssemblyData;
};


#undef UE_API
