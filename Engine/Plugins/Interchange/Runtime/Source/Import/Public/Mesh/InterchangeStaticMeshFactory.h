// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeFactoryBase.h"
#include "InterchangeMeshDefinitions.h"
#include "InterchangeMeshNode.h"
#include "Mesh/InterchangeMeshPayload.h"
#include "MeshDescription.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeStaticMeshFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API


class UBodySetup;
class UStaticMesh;
class UInterchangeStaticMeshFactoryNode;
class UInterchangeStaticMeshLodDataNode;
struct FKAggregateGeom;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeStaticMeshFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()
public:

	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() override { return EInterchangeFactoryAssetType::Meshes; }
	UE_API virtual void CreatePayloadTasks(const FImportAssetObjectParams& Arguments, bool bAsync, TArray<TSharedPtr<UE::Interchange::FInterchangeTaskBase>>& PayloadTasks) override;
	UE_API virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual FImportAssetResult ImportAsset_Async(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual FImportAssetResult EndImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;
	UE_API virtual void BuildObject_GameThread(const FSetupObjectParams& Arguments, bool& OutPostEditchangeCalled) override;
	UE_API virtual bool GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const override;
	UE_API virtual bool SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const override;
	UE_API virtual void BackupSourceData(const UObject* Object) const override;
	UE_API virtual void ReinstateSourceData(const UObject* Object) const override;
	UE_API virtual void ClearBackupSourceData(const UObject* Object) const override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////

private: 
	struct FLodPayloads
	{
		TMap<FInterchangeMeshPayLoadKey, UE::Interchange::FMeshPayload> MeshPayloadPerKey;
		TMap<FInterchangeMeshPayLoadKey, UE::Interchange::FMeshPayload> CollisionBoxPayloadPerKey;
		TMap<FInterchangeMeshPayLoadKey, UE::Interchange::FMeshPayload> CollisionCapsulePayloadPerKey;
		TMap<FInterchangeMeshPayLoadKey, UE::Interchange::FMeshPayload> CollisionSpherePayloadPerKey;
		TMap<FInterchangeMeshPayLoadKey, UE::Interchange::FMeshPayload> CollisionConvexPayloadPerKey;
	};

	TMap<int32, FLodPayloads> PayloadsPerLodIndex;

	UE_API void CommitMeshDescriptions(UStaticMesh& StaticMesh);
	UE_API void BuildFromMeshDescriptions(UStaticMesh& StaticMesh);

#if WITH_EDITORONLY_DATA
	UE_API void SetupSourceModelsSettings(UStaticMesh& StaticMesh, const TArray<FMeshDescription>& LodMeshDescriptions, bool bAutoComputeLODScreenSizes, const TArray<float>& LodScreenSizes, int32 PreviousLodCount, int32 FinalLodCount, bool bIsAReimport);
#endif
	struct FImportAssetObjectData
	{
		TArray<FMeshDescription> LodMeshDescriptions;
		EInterchangeMeshCollision Collision = EInterchangeMeshCollision::None;
		FKAggregateGeom AggregateGeom;
		bool bIsAppGame = false;
		bool bImportedCustomCollision = false;
		bool bImportCollision = false;
	};
	FImportAssetObjectData ImportAssetObjectData;
};

#undef UE_API
