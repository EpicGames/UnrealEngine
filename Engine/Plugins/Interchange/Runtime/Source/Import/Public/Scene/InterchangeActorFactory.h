// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeFactoryBase.h"
#include "InterchangeSourceData.h"
//#include "InterchangeResult.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeActorFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

class AActor;
class UInterchangeActorFactoryNode;
class USceneComponent;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeActorFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()
public:

	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;

protected:
	UE_API virtual void ExecuteResetObjectProperties(const UInterchangeBaseNodeContainer* BaseNodeContainer, UInterchangeFactoryBaseNode* FactoryNode, UObject* ImportedObject) override;

	UE_API virtual UObject* ImportSceneObject_GameThread(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams) override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////

	UE_API void ProcessTags(UInterchangeActorFactoryNode* FactoryNode, AActor* SpawnedActor);
	UE_API void ProcessLayerNames(UInterchangeActorFactoryNode* FactoryNode, AActor* SpawnedActor);
#if WITH_EDITORONLY_DATA
	UE_API void AddUniqueLayersToWorld(UWorld* World, const TSet<FString>& LayerNames);
#endif

protected:
	/**
	 * Method called in UInterchangeActorFactory::ImportSceneObject_GameThread to allow
	 * child class to complete the creation of the actor.
	 * This method is expected to return the UOBject to apply the factory node's custom attributes to.
	 */
	UE_API virtual UObject* ProcessActor(AActor& SpawnedActor, const UInterchangeActorFactoryNode& FactoryNode, const UInterchangeBaseNodeContainer& NodeContainer, const FImportSceneObjectsParams& Params);


	/**
	 * Method used to apply all the custom attributes to the Actor or its sub components.
	 * Base class method will simply use ActorHelper to apply all the custom attributes to the object.
	 */
	UE_API virtual void ApplyAllCustomAttributesToObject(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams, AActor& SpawnedActor, UObject* ObjectToUpdate);

	template <typename T>
	void LogMessage(const FImportSceneObjectsParams& Params, const FText& Message, const FString& ActorLabel)
	{
		T* Item = /*Super::*/AddMessage<T>();
		FillMessage(Params, Message, ActorLabel, *Item);
	}

	template <typename T>
	void FillMessage(const FImportSceneObjectsParams& Params, const FText& Message, const FString& ActorLabel, T& Result)
	{
		Result.SourceAssetName = Params.SourceData ? Params.SourceData->GetFilename() : TEXT("Unknown file");
		Result.DestinationAssetName = Params.ObjectName;
		Result.AssetType = GetFactoryClass();
		Result.Text = Message;

		if (!ActorLabel.IsEmpty())
		{
			Result.AssetFriendlyName = ActorLabel;
		}
	}
};


#undef UE_API
