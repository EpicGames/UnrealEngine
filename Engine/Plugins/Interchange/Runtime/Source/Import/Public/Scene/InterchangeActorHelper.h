// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "InterchangeFactoryBase.h"

#define UE_API INTERCHANGEIMPORT_API

class AActor;

class UInterchangeActorFactoryNode;
class UInterchangeAssetUserData;
class UInterchangeBaseNodeContainer;
class UInterchangeFactoryBaseNode;
class UInterchangeMeshActorFactoryNode;
class UInterchangeSceneImportAsset;

class UMeshComponent;
class UWorld;

namespace UE::Interchange::ActorHelper
{
	/**
	 * Returns the parent actor of the actor described by the FactoryNode using the node hierarchy.
	 */
	UE_API AActor* GetSpawnedParentActor(const UInterchangeBaseNodeContainer* NodeContainer, const UInterchangeActorFactoryNode* FactoryNode);

	/**
	 * Spawns and return a AAActor using the given SceneObjectsParam.
	 * The actor's parent will be determine by the node hierarchy and its type by the factory node's ObjectClass.
	 */
	UE_API AActor* SpawnFactoryActor(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams, FText& OutReason);
	UE_API AActor* SpawnFactoryActor(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams);

	/**
	 * Returns the factory node of the asset instanced by ActorFactoryNode.
	 */
	UE_API const UInterchangeFactoryBaseNode* FindAssetInstanceFactoryNode(const UInterchangeBaseNodeContainer* NodeContainer, const UInterchangeFactoryBaseNode* ActorFactoryNode);

	/**
	 * Applies material slot dependencies stored in ActorFactoryNode to MeshComponent.
	 * In case of a reimport, PreviousActorFactoryNode is used to determine which slots should be updated
	 * according to the requested import strategy associated with ActorFactoryNode, i.e. do nothing, apply imported slots,
	 * apply imported slots while preserving changes made after the last import or reimport.
	 */
	UE_API void ApplySlotMaterialDependencies(const UInterchangeBaseNodeContainer& NodeContainer, const UInterchangeMeshActorFactoryNode& ActorFactoryNode, UMeshComponent& MeshComponent, const UInterchangeMeshActorFactoryNode* PreviousActorFactoryNode = nullptr);

	/**
	 * Applies custom attributes from to ObjectToUpdate based on content of CreateSceneObjectsParams
	 * Reimport policies will be applied if applicable
	 */
	UE_API void ApplyAllCustomAttributes(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams, UObject& ObjectToUpdate);

	/**
	 * Adds metadata to Actor via UInterchangeAssetUserData
	 */
	UE_API void AddInterchangeAssetUserDataToActor(AActor* Actor, const UInterchangeSceneImportAsset* SceneImportAsset, const UInterchangeFactoryBaseNode* FactoryNode);

	/**
	 * Adds metadata to Actor via UInterchangeAssetUserData
	 */
	UE_API void AddInterchangeLevelAssetUserDataToWorld(UWorld* World, const UInterchangeSceneImportAsset* SceneImportAsset);
}

#undef UE_API