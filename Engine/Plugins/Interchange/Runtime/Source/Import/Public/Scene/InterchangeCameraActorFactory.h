// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Scene/InterchangeActorFactory.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeCameraActorFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeCineCameraActorFactory : public UInterchangeActorFactory
{
	GENERATED_BODY()
public:

	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////

	//////////////////////////////////////////////////////////////////////////
	// Interchange actor factory interface begin

	UE_API virtual UObject* ProcessActor(class AActor& SpawnedActor, const UInterchangeActorFactoryNode& FactoryNode, const UInterchangeBaseNodeContainer& NodeContainer, const FImportSceneObjectsParams& Params) override;

	UE_API virtual void ApplyAllCustomAttributesToObject(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams, AActor& SpawnedActor, UObject* ObjectToUpdate) override;

	// Interchange actor factory interface end
	//////////////////////////////////////////////////////////////////////////
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeCameraActorFactory : public UInterchangeActorFactory
{
	GENERATED_BODY()
public:

	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////

	//////////////////////////////////////////////////////////////////////////
	// Interchange actor factory interface begin

	UE_API virtual UObject* ProcessActor(class AActor& SpawnedActor, const UInterchangeActorFactoryNode& FactoryNode, const UInterchangeBaseNodeContainer& NodeContainer, const FImportSceneObjectsParams& Params) override;

	UE_API virtual void ApplyAllCustomAttributesToObject(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams, AActor& SpawnedActor, UObject* ObjectToUpdate) override;
	// Interchange actor factory interface end
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
