// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Scene/InterchangeActorFactory.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeGeometryCacheActorFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

class AActor;
class AGeometryCacheMeshActor;
class UGeometryCacheComponent;
class UInterchangeActorFactoryNode;
class UInterchangeMeshActorFactoryNode;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeGeometryCacheActorFactory : public UInterchangeActorFactory
{
	GENERATED_BODY()
public:

	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;

	UE_API virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;

	virtual UObject* ImportSceneObject_GameThread(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams) override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////

protected:
	//////////////////////////////////////////////////////////////////////////
	// Interchange actor factory interface begin

	UE_API virtual UObject* ProcessActor(class AActor& SpawnedActor, const UInterchangeActorFactoryNode& FactoryNode, const UInterchangeBaseNodeContainer& NodeContainer, const FImportSceneObjectsParams& Params) override;


	UE_API virtual void ExecuteResetObjectProperties(const UInterchangeBaseNodeContainer* BaseNodeContainer, UInterchangeFactoryBaseNode* FactoryNode, UObject* ImportedObject) override;
	// Interchange actor factory interface end
	//////////////////////////////////////////////////////////////////////////

private:

	// Helper function to assign a static mesh to its component while checking for other settings like navigation bounds and material dependencies.
	void ApplyGeometryCacheToComponent(const UInterchangeFactoryBaseNode* MeshNode, UGeometryCacheComponent* GeometryCacheComponent, const UInterchangeBaseNodeContainer& NodeContainer, const UInterchangeMeshActorFactoryNode* MeshFactoryNode);

	// Cache the previous factory node passed in the FImportSceneObjectsParams struct
	// PreviousFactoryNode will be used to respect requested reimport's strategy when updating material slots
	const UInterchangeFactoryBaseNode* PreviousFactoryNode = nullptr;
};


#undef UE_API
