// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Scene/InterchangeActorFactory.h"

#include "InterchangeHeterogeneousVolumeActorFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeHeterogeneousVolumeActorFactory : public UInterchangeActorFactory
{
	GENERATED_BODY()

public:
	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////

protected:
	//////////////////////////////////////////////////////////////////////////
	// Interchange actor factory interface begin

	UE_API virtual UObject* ProcessActor(
		class AActor& SpawnedActor,
		const UInterchangeActorFactoryNode& FactoryNode,
		const UInterchangeBaseNodeContainer& NodeContainer,
		const FImportSceneObjectsParams& Params
	) override;

	// Interchange actor factory interface end
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
