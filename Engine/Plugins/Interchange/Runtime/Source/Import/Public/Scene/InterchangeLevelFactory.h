// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeFactoryBase.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeLevelFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

UCLASS(MinimalAPI, BlueprintType, Experimental)
class UInterchangeLevelFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()
public:

	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() override { return EInterchangeFactoryAssetType::Levels; }
	UE_API virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;
	// For a level reimport, returns the level created on the previous import if found
	UE_API virtual UObject* GetObjectToReimport(UObject* ReimportObject, const UInterchangeFactoryBaseNode& FactoryNode, const FString& PackageName, const FString& AssetName, const FString& SubPathString) override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////
};


#undef UE_API
