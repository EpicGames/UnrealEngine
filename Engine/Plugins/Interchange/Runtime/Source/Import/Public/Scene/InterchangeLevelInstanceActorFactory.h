// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeFactoryBase.h"
#include "Scene/InterchangeActorFactory.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeLevelInstanceActorFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

class AActor;
class UInterchangeActorFactoryNode;
class UInterchangeBaseNodeContainer;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeLevelInstanceActorFactory : public UInterchangeActorFactory
{
	GENERATED_BODY()
public:

	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;
	UE_API virtual UObject* ImportSceneObject_GameThread(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams) override;
	// For a level reimport, returns the LevelInstance actor created on the previous import if found
	UE_API virtual UObject* GetObjectToReimport(UObject* ReimportObject, const UInterchangeFactoryBaseNode& FactoryNode, const FString& PackageName, const FString& AssetName, const FString& SubPathString) override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////

};


#undef UE_API
