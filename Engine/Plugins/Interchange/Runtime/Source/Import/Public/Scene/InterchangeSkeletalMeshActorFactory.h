// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeActorFactory.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeSkeletalMeshActorFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

class AActor;
class ASkeletalMeshActor;
class UInterchangeActorFactoryNode;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSkeletalMeshActorFactory : public UInterchangeActorFactory
{
	GENERATED_BODY()
public:

	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;

	virtual UObject* ImportSceneObject_GameThread(const UInterchangeFactoryBase::FImportSceneObjectsParams& CreateSceneObjectsParams) override
	{
		// Cache the previous factory node on reimport
		PreviousFactoryNode = CreateSceneObjectsParams.bIsReimport ? CreateSceneObjectsParams.ReimportFactoryNode : nullptr;
		return Super::ImportSceneObject_GameThread(CreateSceneObjectsParams);
	}

	UE_API virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////

protected:
	//////////////////////////////////////////////////////////////////////////
	// Interchange actor factory interface begin

	UE_API virtual UObject* ProcessActor(class AActor& SpawnedActor, const UInterchangeActorFactoryNode& FactoryNode, const UInterchangeBaseNodeContainer& NodeContainer, const FImportSceneObjectsParams& Params) override;

	virtual void ExecuteResetObjectProperties(const UInterchangeBaseNodeContainer* BaseNodeContainer, UInterchangeFactoryBaseNode* FactoryNode, UObject* ImportedObject) override;
	// Interchange actor factory interface end
	//////////////////////////////////////////////////////////////////////////
	
	// Cache the previous factory node passed in the FImportSceneObjectsParams struct
	// PreviousFactoryNode will be used to respect requested reimport's strategy when updating material slots
	const UInterchangeFactoryBaseNode* PreviousFactoryNode = nullptr;
};


#undef UE_API
