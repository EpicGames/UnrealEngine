// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Scene/InterchangeActorFactory.h"

#include "InterchangeSkyLightActorFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSkyLightActorFactory : public UInterchangeActorFactory
{
	GENERATED_BODY()

public:
	UE_API virtual UClass* GetFactoryClass() const override;
	UE_API virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;

protected:
	UE_API virtual UObject* ProcessActor(
		class AActor& SpawnedActor,
		const UInterchangeActorFactoryNode& FactoryNode,
		const UInterchangeBaseNodeContainer& NodeContainer,
		const FImportSceneObjectsParams& Params
	) override;
};

#undef UE_API
