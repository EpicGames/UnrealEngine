// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Future.h"
#include "InterchangeSourceData.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"

#include "InterchangeVariantSetPayloadInterface.generated.h"

namespace UE::Interchange
{
	enum class EVariantPropertyCaptureCategory : uint16
	{
		Undefined         = 0x0000,
		Generic           = 0x0001,
		RelativeLocation  = 0x0002,
		RelativeRotation  = 0x0004,
		RelativeScale3D   = 0x0008,
		Visibility        = 0x0010,
		Material          = 0x0020,
		Color             = 0x0040,
		Option            = 0x0080
	};
	ENUM_CLASS_FLAGS(EVariantPropertyCaptureCategory)

	struct FVariantPropertyCaptureData
	{
		FString PropertyPath;
		EVariantPropertyCaptureCategory Category = EVariantPropertyCaptureCategory::Undefined;
		TOptional<TArray<uint8>> Buffer;
		TOptional<FString> ObjectUid;
	};

	struct FVariantBinding
	{
		FString TargetUid;
		TArray<FVariantPropertyCaptureData> Captures;
	};

	struct FVariant
	{
		FString DisplayText;
		TArray<FVariantBinding> Bindings;
	};

	struct FVariantSetPayloadData
	{
		TArray<FVariant> Variants;
	};
}

UINTERFACE(MinimalAPI)
class UInterchangeVariantSetPayloadInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * VariantSet payload interface. Derive from this interface if your payload can import Variant's data
 */
class IInterchangeVariantSetPayloadInterface
{
	GENERATED_BODY()
public:

	/**
	 * Once the translation is done, the import process need a way to retrieve payload data.
	 * This payload will be use by the factories to create the asset.
	 *
	 * @param PayloadKey - The key to retrieve the a particular payload contain into the specified source data.
	 * @return a PayloadData containing the the data ask with the key.
	 */
	virtual TOptional<UE::Interchange::FVariantSetPayloadData> GetVariantSetPayloadData(const FString& PayloadKey) const = 0;
};


