// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeFactoryBase.h"

#include "InterchangeSpecularProfileFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSpecularProfileFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()

public:

	UE_API virtual UClass* GetFactoryClass() const override;
	UE_API virtual EInterchangeFactoryAssetType GetFactoryAssetType() override;
	UE_API virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
};

#undef UE_API
