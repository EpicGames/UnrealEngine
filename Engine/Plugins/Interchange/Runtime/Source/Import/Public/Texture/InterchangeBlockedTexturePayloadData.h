// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Texture/InterchangeTexturePayloadData.h"

#include "CoreMinimal.h"
#include "Engine/Texture.h"
#include "Memory/SharedBuffer.h"

#define UE_API INTERCHANGEIMPORT_API

namespace UE
{
	namespace Interchange
	{
		// Also known as a UDIMs texture
		struct FImportBlockedImage
		{
			FUniqueBuffer RawData;
			TArray<FTextureSourceBlock> BlocksData;

			ETextureSourceFormat Format = TSF_Invalid;
			TextureCompressionSettings CompressionSettings = TC_Default;
			bool bSRGB = true;
			TOptional<TextureMipGenSettings> MipGenSettings;
			TOptional<FTextureCreatorApplicationMetadata> TextureCreatorApplicationMetadata;

			UE_API int64 ComputeBufferSize() const;
			UE_API bool IsValid() const;


			UE_API bool InitDataSharedAmongBlocks(const FImportImage& Image);

			/**
			 * Everything except the RawData and BlocksData must be initialized before calling this function
			 */
			UE_API bool InitBlockFromImage(int32 BlockX, int32 BlockY, const FImportImage& Image);

			/**
			 * Everything except the RawData must be initialized before calling this function
			 */
			UE_API bool MigrateDataFromImagesToRawData(TArray<FImportImage>& Images);
		};
	}
}

#undef UE_API
