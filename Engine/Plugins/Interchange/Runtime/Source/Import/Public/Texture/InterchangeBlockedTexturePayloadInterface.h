// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Texture/InterchangeBlockedTexturePayloadData.h"

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeBlockedTexturePayloadInterface.generated.h"

class UInterchangeSourceData;
class FString;

UINTERFACE(MinimalAPI)
class UInterchangeBlockedTexturePayloadInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Blocked Texture payload interface (Also know as UDIM(s)). Derive from it if your payload can import blocked/UDIMs texture
 */
class IInterchangeBlockedTexturePayloadInterface
{
	GENERATED_BODY()
public:

	UE_DEPRECATED(5.3, "Deprecated. Use GetBlockedTexturePayloadData(const FString&, TOptional<FString>&) instead.")
	virtual TOptional<UE::Interchange::FImportBlockedImage> GetBlockedTexturePayloadData(const UInterchangeSourceData* PayloadSourceData, const FString& PayloadKey)
	{
		TOptional<FString> AlternateTexturePath;
		return GetBlockedTexturePayloadData(PayloadKey, AlternateTexturePath);
	}

	/**
	 * Once the translation is done, the import process need a way to retrieve payload data.
	 * This payload will be use by the factories to create the asset.
	 *
	 * @param PayloadKey - The key to retrieve the a particular payload contain into the specified source data.
	 * @param AlternateTexturePath - When applicable, set to the path of the file actually loaded to create the FImportImage.
	 * @return a PayloadData containing the import blocked image data. The TOptional will not be set if there is an error.
	 */
	virtual TOptional<UE::Interchange::FImportBlockedImage> GetBlockedTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const {return {};}
};

