// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeTranslatorBase.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Texture/InterchangeSlicedTexturePayloadData.h"
#include "Texture/InterchangeSlicedTexturePayloadInterface.h"
#include "Texture/InterchangeTexturePayloadData.h"
#include "Texture/InterchangeTexturePayloadInterface.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeDDSTranslator.generated.h"

#define UE_API INTERCHANGEIMPORT_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeDDSTranslator : public UInterchangeTranslatorBase, public IInterchangeTexturePayloadInterface, public IInterchangeSlicedTexturePayloadInterface
{
	GENERATED_BODY()
public:

	UE_API virtual TArray<FString> GetSupportedFormats() const override;
	virtual EInterchangeTranslatorAssetType GetSupportedAssetTypes() const override { return EInterchangeTranslatorAssetType::Textures; }
	
	/*
	 * return true if the translator can translate the specified source data.
	 */
	UE_API virtual bool CanImportSourceData(const UInterchangeSourceData* InSourceData) const override;

	/**
	 * Translate the associated source data into a node hold by the specified nodes container.
	 *
	 * @param BaseNodeContainer - The unreal objects descriptions container where to put the translated source data.
	 * @return true if the translator can translate the source data, false otherwise.
	 */
	UE_API virtual bool Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const override;

	/* IInterchangeTexturePayloadInterface Begin */

	/**
	 * Once the translation is done, the import process need a way to retrieve payload data.
	 * This payload will be use by the factories to create the asset.
	 *
	 * @param PayloadKey - Unused. The translator uses its SourceData property to extract the payload.
	 * @param AlternateTexturePath - Unused. The translator uses its SourceData property to extract the payload.
	 * @return a PayloadData containing the import image data. The TOptional will not be set if there is an error.
	 */
	UE_API virtual TOptional<UE::Interchange::FImportImage> GetTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const override;

	/* IInterchangeTexturePayloadInterface End */


	/* IInterchangeSlicedTexturePayloadInterface Begin */

	/**
	 * Once the translation is done, the import process need a way to retrieve payload data.
	 * This payload will be use by the factories to create the asset.
	 *
	 * @param PayloadKey - Unused. The translator uses its SourceData property to extract the payload.
	 * @param AlternateTexturePath - Unused. The translator uses its SourceData property to extract the payload.
	 * @return a PayloadData containing the import image data. The TOptional will not be set if there is an error.
	 */
	UE_API virtual TOptional<UE::Interchange::FImportSlicedImage> GetSlicedTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const override;

	/* IInterchangeSlicedTexturePayloadInterface End */
};


#undef UE_API
