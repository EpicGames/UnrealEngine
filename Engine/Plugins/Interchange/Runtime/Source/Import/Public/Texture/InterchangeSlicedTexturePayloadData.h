// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Texture/InterchangeTexturePayloadData.h"

#define UE_API INTERCHANGEIMPORT_API

namespace UE
{
	namespace Interchange
	{
		struct FImportSlicedImage : public FImportImage
		{
			int32 NumSlice = 0;
			bool bIsVolume = false;

			UE_API void Init(int32 InSizeX, int32 InSizeY, int32 InNumSlice, int32 InNumMips, ETextureSourceFormat InFormat, bool InSRGB);
			UE_API void InitVolume(int32 InSizeX, int32 InSizeY, int32 InNumSlice, int32 InNumMips, ETextureSourceFormat InFormat, bool InSRGB);

			UE_API const uint8* GetMipData(int32 InMipIndex, int32 InSliceIndex = INDEX_NONE) const;
			UE_API uint8* GetMipData(int32 InMipIndex, int32 InSliceIndex = INDEX_NONE);

			UE_API virtual int64 GetMipSize(int32 InMipIndex) const override;
			UE_API virtual int64 ComputeBufferSize() const override;

			UE_API virtual bool IsValid() const override;
		};
	}
}

#undef UE_API
