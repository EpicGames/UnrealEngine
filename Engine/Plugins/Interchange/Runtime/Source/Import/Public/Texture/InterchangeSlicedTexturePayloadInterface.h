// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "InterchangeSourceData.h"
#include "Texture/InterchangeSlicedTexturePayloadData.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeSlicedTexturePayloadInterface.generated.h"

#define UE_API INTERCHANGEIMPORT_API

UINTERFACE(MinimalAPI)
class UInterchangeSlicedTexturePayloadInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Texture payload interface. Derive from it if your payload can import texture
 */
class IInterchangeSlicedTexturePayloadInterface
{
	GENERATED_BODY()
public:

	UE_DEPRECATED(5.3, "Deprecated. Use GetSlicedTexturePayloadData(const FString&, TOptional<FString>&) instead.")
	virtual TOptional<UE::Interchange::FImportSlicedImage> GetSlicedTexturePayloadData(const UInterchangeSourceData* SourceData, const FString& PayloadKey) const
	{
		TOptional<FString> AlternateTexturePath;
		return GetSlicedTexturePayloadData(PayloadKey, AlternateTexturePath);
	}
	
	/**
	 * Once the translation is done, the import process need a way to retrieve payload data.
	 * This payload will be use by the factories to create the asset.
	 *
	 * @param PayloadKey - The key to retrieve the a particular payload contain into the specified source data.
	 * @param AlternateTexturePath - When applicable, set to the path of the file actually loaded to create the FImportImage.
	 * @return a PayloadData containing the import image data. The TOptional will not be set if there is an error.
	 */
	virtual TOptional<UE::Interchange::FImportSlicedImage> GetSlicedTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const PURE_VIRTUAL(IInterchangeSlicedTexturePayloadInterface::GetSlicedTexturePayloadData, return{};);
};

#undef UE_API
