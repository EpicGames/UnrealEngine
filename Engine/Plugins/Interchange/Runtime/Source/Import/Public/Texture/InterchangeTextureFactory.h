// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorFramework/AssetImportData.h"
#include "InterchangeFactoryBase.h"
#include "Misc/TVariant.h"
#include "Serialization/EditorBulkData.h"
#include "Texture/InterchangeBlockedTexturePayloadData.h"
#include "Texture/InterchangeSlicedTexturePayloadData.h"
#include "Texture/InterchangeTextureLightProfilePayloadData.h"
#include "Texture/InterchangeTextureLightProfilePayloadData.h"
#include "Texture/InterchangeTexturePayloadData.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeTextureFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

class UInterchangeTextureFactoryNode;

namespace UE::Interchange::Private::InterchangeTextureFactory
{
	using FTexturePayloadVariant = TVariant<FEmptyVariantState
		, TOptional<FImportImage>
		, TOptional<FImportBlockedImage>
		, TOptional<FImportSlicedImage>
		, TOptional<FImportLightProfile>>;

	struct FProcessedPayload
	{
		FProcessedPayload() = default;
		FProcessedPayload(FProcessedPayload&&) = default;
		FProcessedPayload& operator=(FProcessedPayload&&) = default;

		FProcessedPayload(const FProcessedPayload&) = delete;
		FProcessedPayload& operator=(const FProcessedPayload&) = delete;

		FProcessedPayload& operator=(UE::Interchange::Private::InterchangeTextureFactory::FTexturePayloadVariant&& InPayloadVariant);

		bool IsValid() const;

		UE::Interchange::Private::InterchangeTextureFactory::FTexturePayloadVariant SettingsFromPayload;
		UE::Serialization::FEditorBulkData::FSharedBufferWithID PayloadAndId;
	};
}

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeTextureFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()
public:

	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	UE_API virtual UClass* GetFactoryClass() const override;
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() override { return EInterchangeFactoryAssetType::Textures; }
	UE_API virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual FImportAssetResult ImportAsset_Async(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual FImportAssetResult EndImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;
	UE_API virtual bool GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const override;
	UE_API virtual bool SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const override;
	UE_API virtual void BackupSourceData(const UObject* Object) const override;
	UE_API virtual void ReinstateSourceData(const UObject* Object) const override;
	UE_API virtual void ClearBackupSourceData(const UObject* Object) const override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////

private:
	/**
	 * Check for any invalid resolutions and remove those from the payload
	 */
	UE_API void CheckForInvalidResolutions(UE::Interchange::Private::InterchangeTextureFactory::FTexturePayloadVariant& InPayloadVariant, const UInterchangeSourceData* SourceData, const UInterchangeTextureFactoryNode* TextureFactoryNode);
	
	UE::Interchange::Private::InterchangeTextureFactory::FProcessedPayload ProcessedPayload;
	TOptional<FString> AlternateTexturePath;

#if WITH_EDITORONLY_DATA
	//  The data for the source files will be stored here during the import
	TArray<FAssetImportInfo::FSourceFile> SourceFiles;
#endif // WITH_EDITORONLY_DATA

	//If we import without a pure texture translator, we should not override an existing texture and we must skip the import. See the implementation of BeginImportAssetObject_GameThread function.
	bool bSkipImport = false;

	// This is set to true if the factory itself called NewObject and created the texture asset referenced by the factory node
	bool bCreatedAsset = false;
};


#undef UE_API
