// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Texture/InterchangeTexturePayloadData.h"

namespace UE::Interchange
{
	struct FImportLightProfile : public FImportImage
	{
		float Brightness = 0.0f;

		float TextureMultiplier = 0.0f;

		TArray64<uint8> SourceDataBuffer;
	};
}
