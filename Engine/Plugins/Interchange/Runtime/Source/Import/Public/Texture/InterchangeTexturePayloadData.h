// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Engine/Texture.h"
#include "Memory/SharedBuffer.h"

#define UE_API INTERCHANGEIMPORT_API

namespace UE
{
	namespace Interchange
	{
		struct FTextureCreatorApplicationMetadata
		{
			FString ApplicationVendor;
			FString ApplicationName;
			FString ApplicationVersion;
		};

		struct FImportImage
		{
			virtual ~FImportImage() = default;
			FImportImage() = default;
			FImportImage(FImportImage&&) = default;
			FImportImage& operator=(FImportImage&&) = default;

			FImportImage(const FImportImage&) = delete;
			FImportImage& operator=(const FImportImage&) = delete;

			FUniqueBuffer RawData;

			/** Which compression format (if any) that is applied to RawData */
			ETextureSourceCompressionFormat RawDataCompressionFormat = TSCF_None;

			ETextureSourceFormat Format = TSF_Invalid;
			TextureCompressionSettings CompressionSettings = TC_Default;
			int32 NumMips = 0;
			int32 SizeX = 0;
			int32 SizeY = 0;
			bool bSRGB = true;
			TOptional<TextureMipGenSettings> MipGenSettings;
			TOptional<FTextureCreatorApplicationMetadata> TextureCreatorApplicationMetadata;

			UE_API void Init2DWithParams(int32 InSizeX, int32 InSizeY, ETextureSourceFormat InFormat, bool bInSRGB, bool bShouldAllocateRawData = true);
			UE_API void Init2DWithParams(int32 InSizeX, int32 InSizeY, int32 InNumMips, ETextureSourceFormat InFormat, bool bInSRGB, bool bShouldAllocateRawData = true);

			UE_API virtual int64 GetMipSize(int32 InMipIndex) const;
			UE_API virtual int64 ComputeBufferSize() const;

			UE_API TArrayView64<uint8> GetArrayViewOfRawData();
			UE_API virtual bool IsValid() const;
		};

	}//ns Interchange
}//ns UE


#undef UE_API
