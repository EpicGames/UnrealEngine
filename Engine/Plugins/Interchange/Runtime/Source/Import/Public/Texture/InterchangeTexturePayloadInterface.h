// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "InterchangeSourceData.h"
#include "Texture/InterchangeTexturePayloadData.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeTexturePayloadInterface.generated.h"

#define UE_API INTERCHANGEIMPORT_API

UINTERFACE(MinimalAPI)
class UInterchangeTexturePayloadInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Texture payload interface. Derive from it if your payload can import texture
 */
class IInterchangeTexturePayloadInterface
{
	GENERATED_BODY()
public:

	UE_DEPRECATED(5.3, "Deprecated. Use GetTexturePayloadData(const FString&, TOptional<FString>&) instead.")
	virtual TOptional<UE::Interchange::FImportImage> GetTexturePayloadData(const UInterchangeSourceData* PayloadSourceData, const FString& PayloadKey) const
	{
		TOptional<FString> AlternateTexturePath;
		return GetTexturePayloadData(PayloadKey, AlternateTexturePath);
	}

	/**
	 * Once the translation is done, the import process need a way to retrieve payload data.
	 * This payload will be use by the factories to create the asset.
	 *
	 * @param PayloadKey - The key to retrieve the a particular payload contain into the specified source data.
	 * @param AlternateTexturePath - When applicable, set to the path of the file actually loaded to create the FImportImage.
	 * @return a PayloadData containing the import image data. The TOptional will not be set if there is an error.
	 */
	virtual TOptional<UE::Interchange::FImportImage> GetTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const PURE_VIRTUAL(IInterchangeTexturePayloadInterface::GetTexturePayloadData, return{};);

	/**
	 * Does this translator support the request for a compressed source data?
	 */
	virtual bool SupportCompressedTexturePayloadData() const { return false; }

	UE_DEPRECATED(5.3, "Deprecated. Use GetCompressedTexturePayloadData(const FString&, TOptional<FString>&) instead.")
	virtual TOptional<UE::Interchange::FImportImage> GetCompressedTexturePayloadData(const UInterchangeSourceData* PayloadSourceData, const FString& PayloadKey)
	{
		TOptional<FString> AlternateTexturePath;
		return GetCompressedTexturePayloadData(FString(), AlternateTexturePath);
	}

	/**
	 * Once the translation is done, the import process need a way to retrieve payload data.
	 * This payload will be use by the factories to create the asset.
	 *
	 * @param PayloadKey - The key to retrieve the a particular payload contain into the specified source data.
	 * @param AlternateTexturePath - When applicable, set to the path of the file actually loaded to create the FImportImage.
	 * @return a PayloadData containing the import image data with image data being the original file format generally. The TOptional will not be set if there is an error.
	 */
	virtual TOptional<UE::Interchange::FImportImage> GetCompressedTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const { return {}; }
};


#undef UE_API
