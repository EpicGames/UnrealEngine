// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "InterchangeTranslatorBase.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Texture/InterchangeTexturePayloadData.h"
#include "Texture/InterchangeTexturePayloadInterface.h"

#include "InterchangeUEJPEGTranslator.generated.h"

#define UE_API INTERCHANGEIMPORT_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeUEJPEGTranslator : public UInterchangeTranslatorBase, public IInterchangeTexturePayloadInterface
{
	GENERATED_BODY()
public:

	UE_API virtual TArray<FString> GetSupportedFormats() const override;
	virtual EInterchangeTranslatorAssetType GetSupportedAssetTypes() const override { return EInterchangeTranslatorAssetType::Textures; }

	/**
	 * Translate the associated source data into a node hold by the specified nodes container.
	 *
	 * @param BaseNodeContainer - The unreal objects descriptions container where to put the translated source data.
	 * @return true if the translator can translate the source data, false otherwise.
	 */
	UE_API virtual bool Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const override;

	/* IInterchangeTexturePayloadInterface Begin */

	/**
	 * Once the translation is done, the import process need a way to retrieve payload data.
	 * This payload will be use by the factories to create the asset.
	 *
	 * @param PayloadKey - Unused. The translator uses its SourceData property to extract the payload.
	 * @param AlternateTexturePath - Unused. The translator uses its SourceData property to extract the payload.
	 * @return a PayloadData containing the import image data. The TOptional will not be set if there is an error.
	 */
	UE_API virtual TOptional<UE::Interchange::FImportImage> GetTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const override;

	UE_API virtual bool SupportCompressedTexturePayloadData() const override;

	/**
	 * Once the translation is done, the import process need a way to retrieve payload data.
	 * This payload will be use by the factories to create the asset.
	 *
	 * @param PayloadKey - Unused. The translator uses its SourceData property to extract the payload.
	 * @param AlternateTexturePath - Unused. The translator uses its SourceData property to extract the payload.
	 * @return a PayloadData containing the import image data. The TOptional will not be set if there is an error.
	 */
	UE_API virtual TOptional<UE::Interchange::FImportImage> GetCompressedTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const override;
	
	/* IInterchangeTexturePayloadInterface End */

private:

	UE_API TOptional<UE::Interchange::FImportImage> GetTexturePayloadImplementation(TArray64<uint8>&& SourceDataBuffer, bool bShouldImportRaw) const;
};


#undef UE_API
