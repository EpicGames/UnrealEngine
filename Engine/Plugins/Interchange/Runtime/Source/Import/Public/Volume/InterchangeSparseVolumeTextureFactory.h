// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeFactoryBase.h"
#include "Volume/InterchangeVolumePayloadData.h"

#include "CoreMinimal.h"
#include "EditorFramework/AssetImportData.h"
#include "Serialization/EditorBulkData.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeSparseVolumeTextureFactory.generated.h"

#define UE_API INTERCHANGEIMPORT_API

class UInterchangeSparseVolumeTextureFactoryNode;
class USparseVolumeTexture;
namespace UE::Interchange
{
	struct FVolumePayloadKey;
}

struct FVolumePayload
{
	FString VolumeNodeUid;
	TArray<int32> PayloadFrameIndices;
	TOptional<UE::Interchange::FVolumePayloadData> PayloadData;
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSparseVolumeTextureFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()

public:
	// Begin UInterchangeFactoryBase interface
	UE_API virtual UClass* GetFactoryClass() const override;
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() override
	{
		return EInterchangeFactoryAssetType::Textures;
	}
	UE_API virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual FImportAssetResult ImportAsset_Async(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual FImportAssetResult EndImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	UE_API virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;
	UE_API virtual bool GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const override;
	UE_API virtual bool SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const override;
	UE_API virtual void BackupSourceData(const UObject* Object) const override;
	UE_API virtual void ReinstateSourceData(const UObject* Object) const override;
	UE_API virtual void ClearBackupSourceData(const UObject* Object) const override;
	// End UInterchangeFactoryBase interface

private:
	UE_API bool HasValidPayloads() const;

private:
#if WITH_EDITORONLY_DATA
	// The data for the source files will be collecetd here during the import during an async task,
	// and later be added to AssetImportData
	TArray<FAssetImportInfo::FSourceFile> SourceFiles;
#endif

	// Where we retain the processed payloads between the different factory interface function calls
	TArray<FVolumePayload> ProcessedPayloads;

	// We turn this to true when we run into an existing asset that we shouldn't overwrite/modify (e.g. when reimporting
	// some other asset type, like reimporting a material and finding an existing SVT that it wants to use)
	bool bSkipImport = false;
};

#undef UE_API
