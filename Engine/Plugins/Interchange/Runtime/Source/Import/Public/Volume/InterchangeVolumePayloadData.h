// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SparseVolumeTexture/SparseVolumeTextureData.h"

namespace UE::Interchange
{
	// Contains everything we need to use as source data for a SparseVolumeTexture asset
	struct FVolumePayloadData
	{
		virtual ~FVolumePayloadData() = default;
		FVolumePayloadData() = default;
		FVolumePayloadData(FVolumePayloadData&&) = default;
		FVolumePayloadData& operator=(FVolumePayloadData&&) = default;

		FVolumePayloadData(const FVolumePayloadData&) = delete;
		FVolumePayloadData& operator=(const FVolumePayloadData&) = delete;

	public:
		UE::SVT::FTextureData TextureData;
		FTransform Transform;
	};
}	 // namespace UE::Interchange
