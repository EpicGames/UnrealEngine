// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Volume/InterchangeVolumePayloadData.h"
#include "Volume/InterchangeVolumePayloadKey.h"

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeVolumePayloadInterface.generated.h"

UINTERFACE(MinimalAPI)
class UInterchangeVolumePayloadInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Payload interface for volume data (e.g. OpenVDB)
 */
class IInterchangeVolumePayloadInterface
{
	GENERATED_BODY()

public:
	virtual TOptional<UE::Interchange::FVolumePayloadData> GetVolumePayloadData(const UE::Interchange::FVolumePayloadKey& PayloadKey) const
	{
		return {};
	}
};

