// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Volume/InterchangeVolumeDefinitions.h"

namespace UE::Interchange
{
	// Contains everything we need to query a factory for some FVolumePayloadData
	struct FVolumePayloadKey
	{
		FString FileName;
		Volume::FAssignmentInfo AssignmentInfo;
		FIntVector3 VolumeBoundsMin;
	};
}
