// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeTranslatorBase.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeVolumeTranslatorSettings.generated.h"

UCLASS(BlueprintType, editinlinenew, MinimalAPI)
class UInterchangeVolumeTranslatorSettings : public UInterchangeTranslatorSettings
{
	GENERATED_BODY()

public:
	/**
	 * Whether we should search the same folder of the translated file for other .vdb files whose file names differ from it
	 * only by a numbered suffix (e.g. "Tornado_0001.vdb, Tornado_0002.vdb, etc.), and translate those too.
	 *
	 * This is useful if you intend on importing an animated SparseVolumeTexture.
	 */
	UPROPERTY(EditAnywhere, Category = "OpenVDB Translator")
	bool bTranslateAdjacentNumberedFiles = true;

	/**
	 * Identifier set on the generated volume nodes (via SetCustomAnimationID) if they are part of an animation, to identify said animation.
	 * Volume nodes that share the same AnimationID belong to the same volume animation.
	 * This can be left empty to have an ID automatically generated, or can be set to something to force it to that value.
	 */
	UPROPERTY()
	FString AnimationID;
};
