// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fbx/InterchangeFbxMessages.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeFbxMessages)


#define LOCTEXT_NAMESPACE "InterchangeFbxMessages"

FText UInterchangeResultMeshWarning_Generic::GetText() const
{
	// Any occurrences of {MeshName} in the supplied text will be replaced by the actual mesh name
	FFormatNamedArguments Args
	{
		{ TEXT("MeshName"), FText::FromString(MeshName) }
	};

	return FText::Format(Text, Args);
}


FText UInterchangeResultMeshError_Generic::GetText() const
{
	// Any occurrences of {MeshName} in the supplied text will be replaced by the actual mesh name
	FFormatNamedArguments Args
	{
		{ TEXT("MeshName"), FText::FromString(MeshName) }
	};

	return FText::Format(Text, Args);
}


FText UInterchangeResultMeshWarning_TooManyUVs::GetText() const
{
	FFormatNamedArguments Args
	{
		{ TEXT("MeshName"), FText::FromString(MeshName) },
		{ TEXT("ExcessUVs"), ExcessUVs}
	};

	return FText::Format(LOCTEXT("TooManyUVs", "Reached the maximum number of UV Channels for mesh '{MeshName}' - discarding {ExcessUVs} {ExcessUVs}|plural(one=channel,other=channels)."), Args);
}

FText UInterchangeResultTextureDisplay_TextureFileDoNotExist::GetText() const
{
	FFormatNamedArguments Args
	{
		{ TEXT("TextureName"), FText::FromString(TextureName) },
		{ TEXT("MaterialName"), FText::FromString(MaterialName) }
	};

	if (MaterialName.IsEmpty())
	{
		return FText::Format(LOCTEXT("TextureFileCannotBeOpened", "Cannot open texture file '{TextureName}'."), Args);
	}
	
	return FText::Format(LOCTEXT("CannotOpenTextureFileOnImport", "Cannot open texture file '{TextureName}' when importing material '{MaterialName}'."), Args);
}


#undef LOCTEXT_NAMESPACE

