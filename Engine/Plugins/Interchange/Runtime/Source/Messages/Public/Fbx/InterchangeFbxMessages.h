// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeResult.h"

#include "InterchangeFbxMessages.generated.h"

#define UE_API INTERCHANGEMESSAGES_API


/**
 * Base class for FBX parser warnings
 */
UCLASS(MinimalAPI)
class UInterchangeResultMeshWarning : public UInterchangeResultWarning
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FString MeshName;
};

/**
 * Base class for FBX parser warnings
 */
UCLASS(MinimalAPI)
class UInterchangeResultTextureDisplay : public UInterchangeResultDisplay_Generic
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FString TextureName;
};

/**
 * Base class for FBX parser warnings
 */
UCLASS(MinimalAPI)
class UInterchangeResultTextureWarning : public UInterchangeResultWarning
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FString TextureName;
};

/**
 * Base class for FBX parser errors
 */
UCLASS(MinimalAPI)
class UInterchangeResultMeshError : public UInterchangeResultError
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FString MeshName;
};


/**
 * A generic class for FBX parser warnings, with no additional metadata, and where the text is specified by the user
 */
UCLASS(MinimalAPI)
class UInterchangeResultMeshWarning_Generic : public UInterchangeResultMeshWarning
{
	GENERATED_BODY()

public:
	UE_API virtual FText GetText() const override;

	UPROPERTY()
	FText Text;
};


/**
 * A generic class for FBX parser errors, with no additional metadata, and where the text is specified by the user
 */
UCLASS(MinimalAPI)
class UInterchangeResultMeshError_Generic : public UInterchangeResultMeshError
{
	GENERATED_BODY()

public:
	UE_API virtual FText GetText() const override;

	UPROPERTY()
	FText Text;
};


/**
 * 
 */
UCLASS(MinimalAPI)
class UInterchangeResultMeshWarning_TooManyUVs : public UInterchangeResultMeshWarning
{
	GENERATED_BODY()

public:
	UE_API virtual FText GetText() const override;

	UPROPERTY()
	int32 ExcessUVs;
};

/**
 * A generic class for FBX parser warnings, with no additional metadata, and where the text is specified by the user
 */
UCLASS(MinimalAPI)
class UInterchangeResultTextureDisplay_TextureFileDoNotExist : public UInterchangeResultTextureDisplay
{
	GENERATED_BODY()

public:
	UE_API virtual FText GetText() const override;

	UPROPERTY()
	FString MaterialName;
};

#undef UE_API
