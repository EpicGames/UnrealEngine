// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class InterchangeNodes : ModuleRules
	{
		public InterchangeNodes(ReadOnlyTargetRules Target) : base(Target)
		{
			// This module does not depend on the Engine module. This is intentional.
			// The InterchangeWorker program does not compile against the Engine module
			// but depends on this module.
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"InterchangeCore",
				}
			);
		}
	}
}
