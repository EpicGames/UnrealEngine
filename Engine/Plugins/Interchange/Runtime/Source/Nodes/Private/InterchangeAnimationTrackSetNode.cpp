// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeAnimationTrackSetNode.h"

// UInterchangeAnimationTrackSetNode

//Interchange namespace

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeAnimationTrackSetNode)
namespace UE
{
	namespace Interchange
	{
		const FAttributeKey& FAnimationStaticData::AnimationPayLoadUidKey()
		{
			static FAttributeKey AttributeKey(TEXT("__AnimationPayloadUidKey__"));
			return AttributeKey;
		}

		const FAttributeKey& FAnimationStaticData::AnimationPayLoadTypeKey()
		{
			static FAttributeKey AttributeKey(TEXT("__AnimationPayloadTypeKey__"));
			return AttributeKey;
		}

		const FAttributeKey& FAnimationStaticData::MorphTargetAnimationPayLoadUidKey()
		{
			static FAttributeKey AttributeKey(TEXT("__MorphTargetAnimationPayloadUidKey__"));
			return AttributeKey;
		}

		const FAttributeKey& FAnimationStaticData::MorphTargetAnimationPayLoadTypeKey()
		{
			static FAttributeKey AttributeKey(TEXT("__MorphTargetAnimationPayloadTypeKey__"));
			return AttributeKey;
		}

		namespace Animation
		{
			FFrameRate ConvertSampleRateToFrameRate(double SampleRate)
			{
				double IntegralPart = FMath::FloorToDouble(SampleRate);
				double FractionalPart = SampleRate - IntegralPart;
				const int32 Tolerance = 1000000;
				double Divisor = static_cast<double>(FMath::GreatestCommonDivisor(FMath::RoundToInt(FractionalPart * Tolerance), Tolerance));
				int32 Denominator = static_cast<int32>(Tolerance / Divisor);
				int32 Numerator = static_cast<int32>(IntegralPart * Denominator + FMath::RoundToDouble(FractionalPart * Tolerance) / Divisor);
				check(Denominator != 0);
				return FFrameRate(Numerator, Denominator);
			}
		}
	}
}

UInterchangeAnimationTrackSetNode::UInterchangeAnimationTrackSetNode()
{
	CustomAnimationTrackUids.Initialize(Attributes, TEXT("AnimationTrackUids"));
}

int32 UInterchangeAnimationTrackSetNode::GetCustomAnimationTrackUidCount() const
{
	return CustomAnimationTrackUids.GetCount();
}

void UInterchangeAnimationTrackSetNode::GetCustomAnimationTrackUids(TArray<FString>& OutAnimationTrackUids) const
{
	CustomAnimationTrackUids.GetItems(OutAnimationTrackUids);
}

void UInterchangeAnimationTrackSetNode::GetCustomAnimationTrackUid(const int32 Index, FString& OutAnimationTrackUid) const
{
	CustomAnimationTrackUids.GetItem(Index, OutAnimationTrackUid);
}

bool UInterchangeAnimationTrackSetNode::AddCustomAnimationTrackUid(const FString& AnimationTrackUid)
{
	return CustomAnimationTrackUids.AddItem(AnimationTrackUid);
}

bool UInterchangeAnimationTrackSetNode::RemoveCustomAnimationTrackUid(const FString& AnimationTrackUid)
{
	return CustomAnimationTrackUids.RemoveItem(AnimationTrackUid);
}

bool UInterchangeAnimationTrackSetNode::GetCustomFrameRate(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(FrameRate, float);
}

bool UInterchangeAnimationTrackSetNode::SetCustomFrameRate(const float& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(FrameRate, float);
}

// UInterchangeAnimationTrackBaseNode

bool UInterchangeAnimationTrackBaseNode::GetCustomCompletionMode(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(CompletionMode, int32);
}

bool UInterchangeAnimationTrackBaseNode::SetCustomCompletionMode(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(CompletionMode, int32);
}

// UInterchangeAnimationTrackNode

bool UInterchangeAnimationTrackNode::GetCustomActorDependencyUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ActorDependency, FString);
}

bool UInterchangeAnimationTrackNode::SetCustomActorDependencyUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ActorDependency, FString);
}

bool UInterchangeAnimationTrackNode::GetCustomAnimationPayloadKey(FInterchangeAnimationPayLoadKey& AnimationPayLoadKey) const
{
	FString UniqueId;
	EInterchangeAnimationPayLoadType PayLoadType;

	//PayLoadKey
	{
		if (!Attributes->ContainAttribute(UE::Interchange::FAnimationStaticData::AnimationPayLoadUidKey()))
		{
			return false;
		}
		UE::Interchange::FAttributeStorage::TAttributeHandle<FString> AttributeHandle = Attributes->GetAttributeHandle<FString>(UE::Interchange::FAnimationStaticData::AnimationPayLoadUidKey());
		if (!AttributeHandle.IsValid())
		{
			return false;
		}
		UE::Interchange::EAttributeStorageResult Result = AttributeHandle.Get(UniqueId);
		if (!IsAttributeStorageResultSuccess(Result))
		{
			LogAttributeStorageErrors(Result, TEXT("UInterchangeMeshNode.GetPayLoadKey"), UE::Interchange::FAnimationStaticData::AnimationPayLoadUidKey());
			return false;
		}
	}

	//PayLoadType
	{
		if (!Attributes->ContainAttribute(UE::Interchange::FAnimationStaticData::AnimationPayLoadTypeKey()))
		{
			return false;
		}
		UE::Interchange::FAttributeStorage::TAttributeHandle<EInterchangeAnimationPayLoadType> AttributeHandle = Attributes->GetAttributeHandle<EInterchangeAnimationPayLoadType>(UE::Interchange::FAnimationStaticData::AnimationPayLoadTypeKey());
		if (!AttributeHandle.IsValid())
		{
			return false;
		}

		UE::Interchange::EAttributeStorageResult Result = AttributeHandle.Get(PayLoadType);
		if (!IsAttributeStorageResultSuccess(Result))
		{
			LogAttributeStorageErrors(Result, TEXT("UInterchangeMeshNode.GetPayLoadTypeKey"), UE::Interchange::FAnimationStaticData::AnimationPayLoadTypeKey());
			return false;
		}
	}
	
	AnimationPayLoadKey.UniqueId = UniqueId;
	AnimationPayLoadKey.Type = PayLoadType;

	return true;
}

bool UInterchangeAnimationTrackNode::SetCustomAnimationPayloadKey(const FString& InUniqueId, const EInterchangeAnimationPayLoadType& InType)
{
	UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FAnimationStaticData::AnimationPayLoadUidKey(), InUniqueId);
	if (!IsAttributeStorageResultSuccess(Result))
	{
		LogAttributeStorageErrors(Result, TEXT("UInterchangeAnimationTrackNode.SetPayLoadKey"), UE::Interchange::FAnimationStaticData::AnimationPayLoadUidKey());
		return false;
	}
	else
	{
		Result = Attributes->RegisterAttribute(UE::Interchange::FAnimationStaticData::AnimationPayLoadTypeKey(), InType);
		if (!IsAttributeStorageResultSuccess(Result))
		{
			LogAttributeStorageErrors(Result, TEXT("UInterchangeAnimationTrackNode.SetPayLoadTypeKey"), UE::Interchange::FAnimationStaticData::AnimationPayLoadTypeKey());
			return false;
		}
	}

	return true;
}

bool UInterchangeAnimationTrackNode::GetCustomFrameCount(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(FrameCount, int32);
}

bool UInterchangeAnimationTrackNode::SetCustomFrameCount(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(FrameCount, int32);
}

bool UInterchangeAnimationTrackNode::GetCustomPropertyTrack(FName& AttributeValue) const
{
	return false;
}

bool UInterchangeAnimationTrackNode::SetCustomPropertyTrack(const FName& AttributeValue)
{
	return false;
}

bool UInterchangeAnimationTrackNode::SetCustomPropertyTrack(EInterchangePropertyTracks AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(PropertyTrack, EInterchangePropertyTracks);
}

bool UInterchangeAnimationTrackNode::GetCustomPropertyTrack(EInterchangePropertyTracks& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(PropertyTrack, EInterchangePropertyTracks);
}

// UInterchangeAnimationTrackSetInstanceNode

bool UInterchangeAnimationTrackSetInstanceNode::SetCustomTimeScale(const float& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(TimeScale, float);
}

bool UInterchangeAnimationTrackSetInstanceNode::GetCustomTimeScale(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(TimeScale, float);
}

bool UInterchangeAnimationTrackSetInstanceNode::SetCustomDuration(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Duration, int32);
}

bool UInterchangeAnimationTrackSetInstanceNode::GetCustomDuration(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Duration, int32);
}

bool UInterchangeAnimationTrackSetInstanceNode::SetCustomStartFrame(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(StartFrame, int32);
}

bool UInterchangeAnimationTrackSetInstanceNode::GetCustomStartFrame(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(StartFrame, int32);
}

bool UInterchangeAnimationTrackSetInstanceNode::SetCustomTrackSetDependencyUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(TrackSetDependencyUid, FString);
}

bool UInterchangeAnimationTrackSetInstanceNode::GetCustomTrackSetDependencyUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(TrackSetDependencyUid, FString);
}

// UInterchangeTransformAnimationTrackNode

bool UInterchangeTransformAnimationTrackNode::SetCustomUsedChannels(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(UsedChannels, int32);
}

bool UInterchangeTransformAnimationTrackNode::GetCustomUsedChannels(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(UsedChannels, int32);
}

// UInterchangeSkeletalAnimationTrackNode

UInterchangeSkeletalAnimationTrackNode::UInterchangeSkeletalAnimationTrackNode()
{
	SceneNodeAnimationPayloadKeyUidMap.Initialize(Attributes.ToSharedRef(), TEXT("__SceneNodeAnimationPayloadKeyUidMap__"));
	SceneNodeAnimationPayloadKeyTypeMap.Initialize(Attributes.ToSharedRef(), TEXT("__SceneNodeAnimationPayloadKeyTypeMap__"));

	MorphTargetPayloadKeyUidMap.Initialize(Attributes.ToSharedRef(), TEXT("__MorphTargetPayloadKeyUidMap__"));
	MorphTargetPayloadKeyTypeMap.Initialize(Attributes.ToSharedRef(), TEXT("__MorphTargetPayloadKeyTypeMap__"));
}

bool UInterchangeSkeletalAnimationTrackNode::GetCustomSkeletonNodeUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SkeletonNodeUid, FString);
}

bool UInterchangeSkeletalAnimationTrackNode::SetCustomSkeletonNodeUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SkeletonNodeUid, FString);
}

bool UInterchangeSkeletalAnimationTrackNode::GetCustomAnimationSampleRate(double& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AnimationSampleRate, double);
}

bool UInterchangeSkeletalAnimationTrackNode::SetCustomAnimationSampleRate(const double& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AnimationSampleRate, double);
}

bool UInterchangeSkeletalAnimationTrackNode::GetCustomAnimationStartTime(double& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AnimationStartTime, double);
}

bool UInterchangeSkeletalAnimationTrackNode::SetCustomAnimationStartTime(const double& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AnimationStartTime, double);
}

bool UInterchangeSkeletalAnimationTrackNode::GetCustomAnimationStopTime(double& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AnimationStopTime, double);
}

bool UInterchangeSkeletalAnimationTrackNode::SetCustomAnimationStopTime(const double& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AnimationStopTime, double);
}

bool UInterchangeSkeletalAnimationTrackNode::SetCustomSourceTimelineAnimationStartTime(const double& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SourceTimelineAnimationStartTime, double);
}

bool UInterchangeSkeletalAnimationTrackNode::GetCustomSourceTimelineAnimationStartTime(double& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceTimelineAnimationStartTime, double);
}

bool UInterchangeSkeletalAnimationTrackNode::SetCustomSourceTimelineAnimationStopTime(const double& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SourceTimelineAnimationStopTime, double);
}

bool UInterchangeSkeletalAnimationTrackNode::GetCustomSourceTimelineAnimationStopTime(double& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceTimelineAnimationStopTime, double);
}

void UInterchangeSkeletalAnimationTrackNode::GetSceneNodeAnimationPayloadKeys(TMap<FString, FString>& OutSceneNodeAnimationPayloadKeyUids, TMap<FString, uint8>& OutSceneNodeAnimationPayloadKeyTypes) const
{
	OutSceneNodeAnimationPayloadKeyUids = SceneNodeAnimationPayloadKeyUidMap.ToMap();
	OutSceneNodeAnimationPayloadKeyTypes = SceneNodeAnimationPayloadKeyTypeMap.ToMap();
}

bool UInterchangeSkeletalAnimationTrackNode::SetAnimationPayloadKeyForSceneNodeUid(const FString& SceneNodeUid, const FString& InUniqueId, const EInterchangeAnimationPayLoadType& InType)
{
	bool bSuccess = SceneNodeAnimationPayloadKeyUidMap.SetKeyValue(SceneNodeUid, InUniqueId);
	bSuccess &= SceneNodeAnimationPayloadKeyTypeMap.SetKeyValue(SceneNodeUid, (uint8)InType);
	return bSuccess;
}

void UInterchangeSkeletalAnimationTrackNode::GetMorphTargetNodeAnimationPayloadKeys(TMap<FString, FString>& OutMorphTargetNodeAnimationPayloadKeyUids, TMap<FString, uint8>& OutMorphTargetNodeAnimationPayloadKeyTypes) const
{
	OutMorphTargetNodeAnimationPayloadKeyUids = MorphTargetPayloadKeyUidMap.ToMap();
	OutMorphTargetNodeAnimationPayloadKeyTypes = MorphTargetPayloadKeyTypeMap.ToMap();
}

bool UInterchangeSkeletalAnimationTrackNode::SetAnimationPayloadKeyForMorphTargetNodeUid(const FString& MorphTargetNodeUid, const FString& InUniqueId, const EInterchangeAnimationPayLoadType& InType)
{
	bool bSuccess = MorphTargetPayloadKeyUidMap.SetKeyValue(MorphTargetNodeUid, InUniqueId);
	bSuccess &= MorphTargetPayloadKeyTypeMap.SetKeyValue(MorphTargetNodeUid, (uint8)InType);
	return bSuccess;
}

bool UInterchangeSkeletalAnimationTrackNode::IsNodeAnimatedWithBakedCurve(const FString& SceneNodeUid) const
{
	uint8 OutValue = 0;
	if (SceneNodeAnimationPayloadKeyTypeMap.GetValue(SceneNodeUid, OutValue))
	{
		return (OutValue == static_cast<uint8>(EInterchangeAnimationPayLoadType::BAKED));
	}
	return false;
}
