// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeAudioSoundWaveNode.h"

#include "Misc/Paths.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeAudioSoundWaveNode)

namespace UE::Interchange
{
	const FAttributeKey& FSoundWaveNodeStaticData::PayloadSourceFileKey()
	{
		static FAttributeKey AttributeKey(TEXT("__PayloadSourceFile__"));
		return AttributeKey;
	}
}

UInterchangeAudioSoundWaveNode* UInterchangeAudioSoundWaveNode::Create(UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeName)
{
	UInterchangeAudioSoundWaveNode* SoundWaveNode = NewObject<UInterchangeAudioSoundWaveNode>();
	check(SoundWaveNode);

	const FString SoundWaveNodeUid = MakeNodeUid(NodeName);
	NodeContainer.SetupNode(SoundWaveNode, SoundWaveNodeUid, NodeName, EInterchangeNodeContainerType::TranslatedAsset);
	
	return SoundWaveNode;
}


const TOptional<FString> UInterchangeAudioSoundWaveNode::GetPayloadKey() const
{
	using namespace UE::Interchange;
	FString OutAttributeValue;
	if (InterchangePrivateNodeBase::GetCustomAttribute(*Attributes, FSoundWaveNodeStaticData::PayloadSourceFileKey(), TEXT("UInterchangeAudioSoundWaveNode.GetPayloadKey"), OutAttributeValue))
	{
		return TOptional<FString>(MoveTemp(OutAttributeValue));
	}

	return TOptional<FString>();
}

void UInterchangeAudioSoundWaveNode::SetPayloadKey(const FString& PayloadKey)
{
	using namespace UE::Interchange;
	InterchangePrivateNodeBase::SetCustomAttribute(*Attributes, FSoundWaveNodeStaticData::PayloadSourceFileKey(), TEXT("UInterchangeAudioSoundWaveNode.SetPayloadKey"), PayloadKey);
}