// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeGroomBindingNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGroomBindingNode)

FString UInterchangeGroomBindingNode::GetTypeName() const
{
	const FString TypeName = TEXT("GroomBindingNode");
	return TypeName;
}

bool UInterchangeGroomBindingNode::GetGroomDependencyUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GroomUid, FString);
}

bool UInterchangeGroomBindingNode::SetGroomDependencyUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(GroomUid, FString);
}

bool UInterchangeGroomBindingNode::GetTargetMeshDependencyUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(TargetMeshUid, FString);
}

bool UInterchangeGroomBindingNode::SetTargetMeshDependencyUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(TargetMeshUid, FString);
}
