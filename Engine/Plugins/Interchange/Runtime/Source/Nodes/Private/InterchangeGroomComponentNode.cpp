// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeGroomComponentNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGroomComponentNode)

UInterchangeGroomComponentNode::UInterchangeGroomComponentNode()
{
}

FString UInterchangeGroomComponentNode::GetTypeName() const
{
	return TEXT("UInterchangeGroomComponentNode");
}

bool UInterchangeGroomComponentNode::GetGroomDependencyUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GroomUid, FString);
}

bool UInterchangeGroomComponentNode::SetGroomDependencyUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(GroomUid, FString);
}

bool UInterchangeGroomComponentNode::GetGroomBindingDependencyUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GroomBindingUid, FString);
}

bool UInterchangeGroomComponentNode::SetGroomBindingDependencyUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(GroomBindingUid, FString);
}
