// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeGroomNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGroomNode)

namespace UE::Interchange
{
	const FAttributeKey& FGroomNodeStaticData::PayloadKey()
	{
		static FAttributeKey AttributeKey(TEXT("__PayloadKey__"));
		return AttributeKey;
	}

	const FAttributeKey& FGroomNodeStaticData::PayloadTypeKey()
	{
		static FAttributeKey AttributeKey(TEXT("__PayloadTypeKey__"));
		return AttributeKey;
	}
}

FString UInterchangeGroomNode::GetTypeName() const
{
	const FString TypeName = TEXT("GroomNode");
	return TypeName;
}

const TOptional<FInterchangeGroomPayloadKey> UInterchangeGroomNode::GetPayloadKey() const
{
	FString PayloadKey;
	EInterchangeGroomPayLoadType PayloadType;

	//PayLoadKey
	{
		if (!Attributes->ContainAttribute(UE::Interchange::FGroomNodeStaticData::PayloadKey()))
		{
			return {};
		}
		UE::Interchange::FAttributeStorage::TAttributeHandle<FString> AttributeHandle = Attributes->GetAttributeHandle<FString>(UE::Interchange::FGroomNodeStaticData::PayloadKey());
		if (!AttributeHandle.IsValid())
		{
			return {};
		}
		UE::Interchange::EAttributeStorageResult Result = AttributeHandle.Get(PayloadKey);
		if (!IsAttributeStorageResultSuccess(Result))
		{
			LogAttributeStorageErrors(Result, TEXT("UInterchangeGroomNode.GetPayloadKey"), UE::Interchange::FGroomNodeStaticData::PayloadKey());
			return {};
		}
	}

	//PayLoadType
	{
		if (!Attributes->ContainAttribute(UE::Interchange::FGroomNodeStaticData::PayloadTypeKey()))
		{
			return {};
		}
		UE::Interchange::FAttributeStorage::TAttributeHandle<EInterchangeGroomPayLoadType> AttributeHandle = Attributes->GetAttributeHandle<EInterchangeGroomPayLoadType>(UE::Interchange::FGroomNodeStaticData::PayloadTypeKey());
		if (!AttributeHandle.IsValid())
		{
			return {};
		}

		UE::Interchange::EAttributeStorageResult Result = AttributeHandle.Get(PayloadType);
		if (!IsAttributeStorageResultSuccess(Result))
		{
			LogAttributeStorageErrors(Result, TEXT("UInterchangeGroomNode.GetPayloadTypeKey"), UE::Interchange::FGroomNodeStaticData::PayloadTypeKey());
			return {};
		}
	}

	return TOptional<FInterchangeGroomPayloadKey>(FInterchangeGroomPayloadKey(PayloadKey, PayloadType));
}

void UInterchangeGroomNode::SetPayloadKey(const FString& PayloadKey, EInterchangeGroomPayLoadType PayLoadType)
{
	UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FGroomNodeStaticData::PayloadKey(), PayloadKey);
	if (!IsAttributeStorageResultSuccess(Result))
	{
		LogAttributeStorageErrors(Result, TEXT("UInterchangeGroomNode.SetPayloadKey"), UE::Interchange::FGroomNodeStaticData::PayloadKey());
		return;
	}
	Result = Attributes->RegisterAttribute(UE::Interchange::FGroomNodeStaticData::PayloadTypeKey(), PayLoadType);
	if (!IsAttributeStorageResultSuccess(Result))
	{
		LogAttributeStorageErrors(Result, TEXT("UInterchangeGroomNode.SetPayloadTypeKey"), UE::Interchange::FGroomNodeStaticData::PayloadTypeKey());
	}
}

bool UInterchangeGroomNode::GetCustomStartFrame(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(StartFrame, int32);
}

bool UInterchangeGroomNode::SetCustomStartFrame(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(StartFrame, int32);
}

bool UInterchangeGroomNode::GetCustomEndFrame(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(EndFrame, int32);
}

bool UInterchangeGroomNode::SetCustomEndFrame(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(EndFrame, int32);
}

bool UInterchangeGroomNode::GetCustomNumFrames(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(NumFrames, int32);
}

bool UInterchangeGroomNode::SetCustomNumFrames(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(NumFrames, int32);
}

bool UInterchangeGroomNode::GetCustomFrameRate(double& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(FrameRate, double);
}

bool UInterchangeGroomNode::SetCustomFrameRate(const double& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(FrameRate, double);
}

bool UInterchangeGroomNode::GetCustomGroomCacheAttributes(EInterchangeGroomCacheAttributes& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GroomCacheAttributes, EInterchangeGroomCacheAttributes);
}

bool UInterchangeGroomNode::SetCustomGroomCacheAttributes(const EInterchangeGroomCacheAttributes& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(GroomCacheAttributes, EInterchangeGroomCacheAttributes);
}
