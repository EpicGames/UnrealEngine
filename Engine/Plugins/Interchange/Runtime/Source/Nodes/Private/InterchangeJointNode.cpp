// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeJointNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeJointNode)

UInterchangeJointNode::UInterchangeJointNode()
{
	MeshToGlobalBindPoseReferences.Initialize(Attributes.ToSharedRef(), UE::Interchange::FSceneNodeStaticData::GetMeshToGlobalBindPoseReferencesString());
}

#if WITH_EDITOR

FString UInterchangeJointNode::GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	const FString NodeAttributeKeyString = NodeAttributeKey.ToString();

	if (NodeAttributeKey == Macro_CustomBindPoseLocalTransformKey || NodeAttributeKey == Macro_CustomTimeZeroLocalTransformKey ||
		NodeAttributeKey == Macro_CustomHasInvalidBindPoseKey || NodeAttributeKey == Macro_CustomGlobalMatrixForT0RebindingKey)
	{
		return FString(TEXT("Joint"));
	}

	return Super::GetAttributeCategory(NodeAttributeKey);
}

#endif //WITH_EDITOR

FName UInterchangeJointNode::GetIconName() const
{
	static const FString SpecializedType = TEXT("SceneGraphIcon.Joint");
	return FName(*SpecializedType);
}

bool UInterchangeJointNode::GetBindPoseLocalTransform(FTransform& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(BindPoseLocalTransform, FTransform);
}

bool UInterchangeJointNode::SetBindPoseLocalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& AttributeValue)
{
	// Modifying a Local BindPose only affects the BindPose hierarchy.
	// No need to invalidate GlobalTransform or TimeZeroGlobalTransform
	if (GlobalTransformCache.Contains(Macro_CustomBindPoseLocalTransformKey))
	{
		// Only clear the BindPose globals
		ResetGlobalTransformCacheInternal(BaseNodeContainer, Macro_CustomBindPoseLocalTransformKey);
	}
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(BindPoseLocalTransform, FTransform);
}

bool UInterchangeJointNode::GetBindPoseGlobalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& GlobalOffsetTransform, FTransform& AttributeValue) const
{
	return GetGlobalTransformWithOffsetInternal(Macro_CustomBindPoseLocalTransformKey, BaseNodeContainer, GlobalOffsetTransform, AttributeValue);
}

bool UInterchangeJointNode::GetTimeZeroLocalTransform(FTransform& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(TimeZeroLocalTransform, FTransform);
}

bool UInterchangeJointNode::SetTimeZeroLocalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& AttributeValue)
{
	// Modifying a Local TimeZero only affects the TimeZero hierarchy.
	// No need to invalidate GlobalTransform or BindPoseGlobalTransform
	if (GlobalTransformCache.Contains(Macro_CustomTimeZeroLocalTransformKey))
	{
		// Only clear the TimeZero globals
		ResetGlobalTransformCacheInternal(BaseNodeContainer, Macro_CustomTimeZeroLocalTransformKey);
	}
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(TimeZeroLocalTransform, FTransform);
}

bool UInterchangeJointNode::GetTimeZeroGlobalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& GlobalOffsetTransform, FTransform& AttributeValue) const
{
	return GetGlobalTransformWithOffsetInternal(Macro_CustomTimeZeroLocalTransformKey, BaseNodeContainer, GlobalOffsetTransform, AttributeValue);
}

bool UInterchangeJointNode::GetGlobalBindPoseReferenceFromMeshUID(const FString& MeshUID, FMatrix& GlobalBindPoseReference) const
{
	return MeshToGlobalBindPoseReferences.GetValue(MeshUID, GlobalBindPoseReference);
}

void UInterchangeJointNode::SetMeshUIDToGlobalBindPoseReferenceMap(const TMap<FString, FMatrix>& InMeshToGlobalBindPoseReferences)
{
	for (const TPair<FString, FMatrix>& Entry : InMeshToGlobalBindPoseReferences)
	{
		MeshToGlobalBindPoseReferences.SetKeyValue(Entry.Key, Entry.Value);
	}
}

bool UInterchangeJointNode::SetHasInvalidBindPose(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(HasInvalidBindPose, bool);
}

bool UInterchangeJointNode::GetHasInvalidBindPose(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(HasInvalidBindPose, bool);
}

bool UInterchangeJointNode::SetGlobalMatrixForT0Rebinding(const FMatrix& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(GlobalMatrixForT0Rebinding, FMatrix);
}

bool UInterchangeJointNode::GetGlobalMatrixForT0Rebinding(FMatrix& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GlobalMatrixForT0Rebinding, FMatrix);
}