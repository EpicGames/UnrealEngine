// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeLightNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeLightNode)

FStringView UInterchangeBaseLightNode::StaticAssetTypeName()
{
	return TEXT("Light");
}

FString UInterchangeBaseLightNode::GetTypeName() const
{
	return TEXT("BaseLightNode");
}

bool UInterchangeBaseLightNode::GetCustomLightColor(FLinearColor& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(LightColor, FLinearColor)
}

bool UInterchangeBaseLightNode::SetCustomLightColor(const FLinearColor& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(LightColor, FLinearColor)
}

bool UInterchangeBaseLightNode::GetCustomIntensity(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(LightIntensity, float)
}

bool UInterchangeBaseLightNode::SetCustomIntensity(float AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(LightIntensity, float)
}

bool UInterchangeBaseLightNode::GetCustomTemperature(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Temperature, float)
}

bool UInterchangeBaseLightNode::SetCustomTemperature(float AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Temperature, float)
}

bool UInterchangeBaseLightNode::GetCustomUseTemperature(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(UseTemperature, bool)
}

bool UInterchangeBaseLightNode::SetCustomUseTemperature(bool AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(UseTemperature, bool)
}

FString UInterchangeLightNode::GetTypeName() const
{
	return TEXT("LightNode");
}

bool UInterchangeLightNode::GetCustomIntensityUnits(EInterchangeLightUnits& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(IntensityUnits, EInterchangeLightUnits)
}

bool UInterchangeLightNode::SetCustomIntensityUnits(const EInterchangeLightUnits & AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(IntensityUnits, EInterchangeLightUnits )
}

bool UInterchangeLightNode::GetCustomAttenuationRadius(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttenuationRadius, float)
}

bool UInterchangeLightNode::SetCustomAttenuationRadius(float AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AttenuationRadius, float)
}

bool UInterchangeLightNode::GetCustomIESTexture(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(IESTexture, FString)
}

bool UInterchangeLightNode::SetCustomIESTexture(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(IESTexture, FString)
}

bool UInterchangeLightNode::GetCustomUseIESBrightness(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(UseIESBrightness, bool);
}

bool UInterchangeLightNode::SetCustomUseIESBrightness(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(UseIESBrightness, bool);
}

bool UInterchangeLightNode::GetCustomIESBrightnessScale(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(IESBrightnessScale, float);
}

bool UInterchangeLightNode::SetCustomIESBrightnessScale(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(IESBrightnessScale, float);
}

bool UInterchangeLightNode::GetCustomRotation(FRotator& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Rotation, FRotator);
}

bool UInterchangeLightNode::SetCustomRotation(const FRotator& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Rotation, FRotator);
}


FString UInterchangePointLightNode::GetTypeName() const
{
	return TEXT("PointLightNode");
}

bool UInterchangePointLightNode::GetCustomUseInverseSquaredFalloff(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(UseInverseSquaredFalloff, bool)
}

bool UInterchangePointLightNode::SetCustomUseInverseSquaredFalloff(bool AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(UseInverseSquaredFalloff, bool)
}

bool UInterchangePointLightNode::GetCustomLightFalloffExponent(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(LightFalloffExponent, float)
}

bool UInterchangePointLightNode::SetCustomLightFalloffExponent(float AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(LightFalloffExponent, float)
}

bool UInterchangePointLightNode::GetCustomSourceRadius(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceRadius, float)
}

bool UInterchangePointLightNode::SetCustomSourceRadius(float AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SourceRadius, float)
}

FString UInterchangeSpotLightNode::GetTypeName() const
{
	return TEXT("SpotLightNode");
}

bool UInterchangeSpotLightNode::GetCustomInnerConeAngle(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(InnerConeAngle, float)
}

bool UInterchangeSpotLightNode::SetCustomInnerConeAngle(float AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(InnerConeAngle, float)
}

bool UInterchangeSpotLightNode::GetCustomOuterConeAngle(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(OuterConeAngle, float)
}

bool UInterchangeSpotLightNode::SetCustomOuterConeAngle(float AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(OuterConeAngle, float)
}

FString UInterchangeRectLightNode::GetTypeName() const
{
	return TEXT("RectLightNode");
}

bool UInterchangeRectLightNode::GetCustomSourceWidth(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceWidth, float)
}

bool UInterchangeRectLightNode::SetCustomSourceWidth(float AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SourceWidth, float)
}

bool UInterchangeRectLightNode::GetCustomSourceHeight(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceHeight, float)
}

bool UInterchangeRectLightNode::SetCustomSourceHeight(float AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SourceHeight, float)
}

FString UInterchangeDirectionalLightNode::GetTypeName() const
{
	return TEXT("DirectionalLightNode");
}

bool UInterchangeDirectionalLightNode::GetCustomLightSourceAngle(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(LightSourceAngle, float)
}

bool UInterchangeDirectionalLightNode::SetCustomLightSourceAngle(float AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(LightSourceAngle, float)
}

FString UInterchangeSkyLightNode::GetTypeName() const
{
	return TEXT("SkyLightNode");
}

bool UInterchangeSkyLightNode::GetCustomCubemapDependency(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(CubemapDependency, FString)
}

bool UInterchangeSkyLightNode::SetCustomCubemapDependency(FString AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(CubemapDependency, FString)
}

bool UInterchangeSkyLightNode::GetCustomSourceType(EInterchangeSkyLightSourceType& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceType, EInterchangeSkyLightSourceType)
}

bool UInterchangeSkyLightNode::SetCustomSourceType(EInterchangeSkyLightSourceType AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SourceType, EInterchangeSkyLightSourceType)
}
