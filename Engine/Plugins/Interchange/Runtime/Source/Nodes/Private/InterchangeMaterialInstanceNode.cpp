// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeMaterialInstanceNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "InterchangeShaderGraphNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeMaterialInstanceNode)

FString UInterchangeMaterialInstanceNode::GetTypeName() const
{
    return TEXT("MaterialInstanceNode");
}

FName UInterchangeMaterialInstanceNode::GetIconName() const
{
	return FName(TEXT("ClassIcon.MaterialInstanceConstant"));
}

FString UInterchangeMaterialInstanceNode::MakeNodeUid(const FStringView NodeName, const FStringView ParentNodeUid)
{
	FString MaterialInstanceNodeUid = FString(UInterchangeBaseNode::HierarchySeparator) + TEXT("MaterialInstances") + FString(UInterchangeBaseNode::HierarchySeparator);

	if(ParentNodeUid.IsEmpty())
	{
		MaterialInstanceNodeUid += NodeName;
	}
	else
	{
		MaterialInstanceNodeUid += ParentNodeUid + FString(UInterchangeBaseNode::HierarchySeparator) + NodeName;
	}

	return MaterialInstanceNodeUid;
}

UInterchangeMaterialInstanceNode* UInterchangeMaterialInstanceNode::Create(UInterchangeBaseNodeContainer* NodeContainer, const FStringView NodeName, const FStringView ParentNodeUid)
{
	check(NodeContainer);

	const FString MaterialInstanceNodeUid = MakeNodeUid(NodeName, ParentNodeUid);

	UInterchangeMaterialInstanceNode* MaterialInstanceNode = NewObject<UInterchangeMaterialInstanceNode>(NodeContainer);
	NodeContainer->SetupNode(MaterialInstanceNode, MaterialInstanceNodeUid, FString(NodeName), EInterchangeNodeContainerType::TranslatedAsset, FString(ParentNodeUid));

	return MaterialInstanceNode;
}

bool UInterchangeMaterialInstanceNode::SetCustomParent(const FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Parent, FString);
}

bool UInterchangeMaterialInstanceNode::GetCustomParent(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Parent, FString);
}

bool UInterchangeMaterialInstanceNode::AddScalarParameterValue(const FString& ParameterName, float AttributeValue)
{
	return AddFloatAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(ParameterName), AttributeValue);
}

bool UInterchangeMaterialInstanceNode::GetScalarParameterValue(const FString& ParameterName, float& AttributeValue) const
{
	return GetFloatAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(ParameterName), AttributeValue);
}

bool UInterchangeMaterialInstanceNode::AddVectorParameterValue(const FString& ParameterName, const FLinearColor& AttributeValue)
{
	return AddLinearColorAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(ParameterName), AttributeValue);
}

bool UInterchangeMaterialInstanceNode::GetVectorParameterValue(const FString& ParameterName, FLinearColor& AttributeValue) const
{
	return GetLinearColorAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(ParameterName), AttributeValue);
}

bool UInterchangeMaterialInstanceNode::AddTextureParameterValue(const FString& ParameterName, const FString& AttributeValue)
{
	return AddStringAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(ParameterName), AttributeValue);
}

bool UInterchangeMaterialInstanceNode::GetTextureParameterValue(const FString& ParameterName, FString& AttributeValue) const
{
	return GetStringAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(ParameterName), AttributeValue);
}

bool UInterchangeMaterialInstanceNode::AddStaticSwitchParameterValue(const FString& ParameterName, bool AttributeValue)
{
	return AddBooleanAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(ParameterName), AttributeValue);
}

bool UInterchangeMaterialInstanceNode::GetStaticSwitchParameterValue(const FString& ParameterName, bool& AttributeValue) const
{
	return GetBooleanAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(ParameterName), AttributeValue);
}

bool UInterchangeMaterialInstanceNode::GetCustomBlendMode(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(BlendMode, int32);
}

bool UInterchangeMaterialInstanceNode::SetCustomBlendMode(int32 AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(BlendMode, int32);
}

bool UInterchangeMaterialInstanceNode::GetCustomTwoSided(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(TwoSided, bool);
}

bool UInterchangeMaterialInstanceNode::SetCustomTwoSided(bool AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(TwoSided, bool);
}

bool UInterchangeMaterialInstanceNode::GetCustomEnableTessellation(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(EnableTessellation, bool);
}

bool UInterchangeMaterialInstanceNode::SetCustomEnableTessellation(bool AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(EnableTessellation, bool);
}