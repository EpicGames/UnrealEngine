// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeMaterialReferenceNode.h"

#include "InterchangeShaderGraphNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeMaterialReferenceNode)

FString UInterchangeMaterialReferenceNode::GetTypeName() const
{
	return TEXT("MaterialReferenceNode");
}

bool UInterchangeMaterialReferenceNode::GetCustomContentPath(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ContentPath, FString);
}

bool UInterchangeMaterialReferenceNode::SetCustomContentPath(const FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ContentPath, FString);
}

