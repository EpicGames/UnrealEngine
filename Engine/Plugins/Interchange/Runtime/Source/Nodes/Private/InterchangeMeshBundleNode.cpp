// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeMeshBundleNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeMeshBundleNode)

UInterchangeMeshBundleNode::UInterchangeMeshBundleNode()
{
	MeshNodeToTransforms.Initialize(Attributes.ToSharedRef(), TEXT("__MeshNodeToTransforms__Key"));
}

void UInterchangeMeshBundleNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading() && bIsInitialized)
	{
		MeshNodeToTransforms.RebuildCache();
	}
}

FString UInterchangeMeshBundleNode::GetTypeName() const
{
	const static FString TypeName = TEXT("MeshBundleNode");
	return TypeName;
}

FName UInterchangeMeshBundleNode::GetIconName() const
{
	static const FString MeshIconName = TEXT("MeshIcon.Static");
	return FName(*MeshIconName);
}

bool UInterchangeMeshBundleNode::AddMeshNodeTransform(const FString& MeshNodeUid, const FTransform& Transform)
{
	TArray<FTransform> Transforms;
	MeshNodeToTransforms.GetValue(MeshNodeUid, Transforms);

	Transforms.Add(Transform);
	return MeshNodeToTransforms.SetKeyValue(MeshNodeUid, Transforms);
}

bool UInterchangeMeshBundleNode::AddMeshNodeTransforms(const FString& MeshNodeUid, const TArray<FTransform>& InTransforms)
{
	TArray<FTransform> Transforms;
	MeshNodeToTransforms.GetValue(MeshNodeUid, Transforms);

	Transforms.Append(InTransforms);
	return MeshNodeToTransforms.SetKeyValue(MeshNodeUid, Transforms);
}

bool UInterchangeMeshBundleNode::GetMeshNodeUids(TArray<FString>& OutMeshNodeUid) const
{
	return MeshNodeToTransforms.ToMap().GetKeys(OutMeshNodeUid) > 0;
}

bool UInterchangeMeshBundleNode::GetMeshNodeTransforms(const FString& MeshNodeUid, TArray<FTransform>& OutNodeTransforms) const
{
	return MeshNodeToTransforms.GetValue(MeshNodeUid, OutNodeTransforms);
}

bool UInterchangeMeshBundleNode::RemoveMeshNodeUid(const FString& MeshNodeUid)
{
	if (MeshNodeToTransforms.RemoveKey(MeshNodeUid))
	{
		return true;
	}

	return false;
}

void UInterchangeMeshBundleNode::ResetMeshNodeUids()
{
	MeshNodeToTransforms.Empty();
}

bool UInterchangeMeshBundleNode::GetCustomSkeletonDependencyUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SkeletonDependencyUid, FString);
}

bool UInterchangeMeshBundleNode::SetCustomSkeletonDependencyUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SkeletonDependencyUid, FString);
}

bool UInterchangeMeshBundleNode::IsSkinnedMeshBundle() const
{
	FString SkeletonUid;
	return GetCustomSkeletonDependencyUid(SkeletonUid) && !SkeletonUid.IsEmpty();
}

void UInterchangeMeshBundleNode::GetAllMeshNodesAndTransforms(TMap<FString, TArray<FTransform>>& OutMeshNodeToTransforms) const
{
	OutMeshNodeToTransforms = MeshNodeToTransforms.ToMap();
}

void UInterchangeMeshBundleNode::SetAllMeshNodesAndTransforms(const TMap<FString, TArray<FTransform>>& InMeshNodeToTransforms)
{
	MeshNodeToTransforms = InMeshNodeToTransforms;
}