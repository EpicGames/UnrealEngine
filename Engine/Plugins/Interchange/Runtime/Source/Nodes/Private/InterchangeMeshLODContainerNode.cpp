// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeMeshLODContainerNode.h"

#include "InterchangeMeshNode.h"
#include "Logging/LogMacros.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

DEFINE_LOG_CATEGORY(LogInterchangeMeshLODContainerNode);

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeMeshLODContainerNode)

const FString MeshLODContainerPrefix = TEXT("\\LODContainer\\");

UInterchangeMeshLODContainerNode::UInterchangeMeshLODContainerNode()
{
	LODMeshDataPerLODIndices.Initialize(Attributes.ToSharedRef(), TEXT("__LODMeshDataPerLODIndices_Key"));
}

FString UInterchangeMeshLODContainerNode::GetTypeName() const
{
	const FString TypeName = TEXT("MeshLODContainerNode");
	return TypeName;
}

FName UInterchangeMeshLODContainerNode::GetIconName() const
{
	const FString IconName = TEXT("SceneGraphIcon.LodGroup");
	return FName(*IconName);
}

bool UInterchangeMeshLODContainerNode::AddMeshLODNodeUid(const FString& MeshLODNodeUid)
{
	return AddMeshForLODIndex(0, MeshLODNodeUid, FTransform::Identity);
}

void UInterchangeMeshLODContainerNode::GetMeshLODNodeUids(TArray<FString>& OutMeshLODNodeUids) const
{
	TSet<FString> MeshLODNodeUids;
	GetAllReferencedMeshUids(MeshLODNodeUids);
	OutMeshLODNodeUids = MeshLODNodeUids.Array();
}

bool UInterchangeMeshLODContainerNode::RemoveMeshLODNodeUid(const FString& MeshUid)
{
	TMap<int32, TArray<FInterchangeLODMeshData>> LODMeshesPerLODIndices = LODMeshDataPerLODIndices.ToMap();
	LODMeshDataPerLODIndices.Empty();

	bool bSuccess = true;

	for (const TPair<int32, TArray<FInterchangeLODMeshData>>& LODMeshDataPerLODIndex : LODMeshesPerLODIndices)
	{
		TArray<FInterchangeLODMeshData> LODMeshArray;

		for (const FInterchangeLODMeshData& LODMeshData : LODMeshDataPerLODIndex.Value)
		{
			if (LODMeshData.MeshUid != MeshUid)
			{
				LODMeshArray.Add(LODMeshData);
			}
		}
		
		bSuccess = LODMeshDataPerLODIndices.SetKeyValue(LODMeshDataPerLODIndex.Key, LODMeshArray) && bSuccess;
	}

	return bSuccess;
}

bool UInterchangeMeshLODContainerNode::ResetMeshLODNodeUids()
{
	LODMeshDataPerLODIndices.Empty();

	return true;
}

void UInterchangeMeshLODContainerNode::GetLODMeshDataPerLODIndices(TMap<int32, TArray<FInterchangeLODMeshData>>& OutLODMeshesPerLODIndices) const
{
	OutLODMeshesPerLODIndices = LODMeshDataPerLODIndices.ToMap();
}

void UInterchangeMeshLODContainerNode::GetAllReferencedMeshUids(TSet<FString>& OutMeshUids) const
{
	TMap<int32, TArray<FInterchangeLODMeshData>> LODMeshesPerLODIndices = LODMeshDataPerLODIndices.ToMap();
	for (const TPair<int32, TArray<FInterchangeLODMeshData>>& LODMeshes : LODMeshesPerLODIndices)
	{
		for (const FInterchangeLODMeshData& LODLODMeshData : LODMeshes.Value)
		{
			OutMeshUids.Add(LODLODMeshData.MeshUid);
		}
	}
}

bool UInterchangeMeshLODContainerNode::AddMeshForLODIndex(const int32 LODIndex, const FString& MeshUid, const FTransform& Transform)
{
	TArray<FInterchangeLODMeshData> LODMeshArray;
	LODMeshDataPerLODIndices.GetValue(LODIndex, LODMeshArray);

	LODMeshArray.Add(FInterchangeLODMeshData(MeshUid, Transform));

	return LODMeshDataPerLODIndices.SetKeyValue(LODIndex, LODMeshArray);
}

void UInterchangeMeshLODContainerNode::ValidateMorphTargets(UInterchangeBaseNodeContainer* NodeContainer, const TMap<int32, TArray<FInterchangeLODMeshData>>& LODMeshesPerLODIndices) const
{
	if (!NodeContainer)
	{
		return;
	}
	int32 MaxLODIndex = 0;
	for (const TPair<int32, TArray<FInterchangeLODMeshData>>& LODMeshesPerLODIndex : LODMeshesPerLODIndices)
	{
		MaxLODIndex = FMath::Max(MaxLODIndex, LODMeshesPerLODIndex.Key);
	}

	FString DisplayLabel = GetDisplayLabel();

	TSet<FString> AllowedLODMorphTargetNames;
	for (int32 LODIndex = 0; LODIndex <= MaxLODIndex; LODIndex++)
	{
		if (!LODMeshesPerLODIndices.Contains(LODIndex))
		{
			continue;
		}

		TSet<FString> LODMorphTargetNames;
		for (const FInterchangeLODMeshData& LODMeshTransformPair : LODMeshesPerLODIndices[LODIndex])
		{
			const UInterchangeMeshNode* MeshNode = Cast<const UInterchangeMeshNode>(NodeContainer->GetNode(LODMeshTransformPair.MeshUid));
			if (!MeshNode)
			{
				continue;
			}

			TArray<FString> LODMorphTargetUids;
			MeshNode->GetMorphTargetDependencies(LODMorphTargetUids);

			LODMorphTargetNames.Reserve(LODMorphTargetNames.Num() + LODMorphTargetUids.Num());

			for (const FString& MorphTargetUid : LODMorphTargetUids)
			{
				if (const UInterchangeMeshNode* MorphTargetNode = Cast<const UInterchangeMeshNode>(NodeContainer->GetNode(MorphTargetUid)))
				{
					LODMorphTargetNames.Add(MorphTargetNode->GetDisplayLabel());
				}
			}
		}

		if ((LODIndex != 0) && LODMorphTargetNames.Num() && !AllowedLODMorphTargetNames.Includes(LODMorphTargetNames))
		{
			//
			UE_LOGF(LogInterchangeMeshLODContainerNode, Display, "Invalid Morph Target configuration for Mesh [%ls] at LOD [%i]: The set of Morph Targets used by lower LODs should include all Morph Targets used by higher LODs.", *DisplayLabel, LODIndex);
		}

		Swap(AllowedLODMorphTargetNames, LODMorphTargetNames);
	}
}

bool UInterchangeMeshLODContainerNode::CheckForLODPattern(const FString& Candidate, int32& LODIndex, FString& Name, FString& ErrorString)
{
	LODIndex = 0;
	Name = TEXT("");
	ErrorString = TEXT("");

	const FString LODPrefix = TEXT("LOD");
	if (!Candidate.Contains(LODPrefix))
	{
		return false;
	}
	if (Candidate.StartsWith(LODPrefix))
	{
		if (int32 UnderScoreIndex = Candidate.Find(TEXT("_")); UnderScoreIndex != INDEX_NONE)
		{
			Name = Candidate.RightChop(UnderScoreIndex + 1);
			FString LODIndexString = Candidate.LeftChop(Candidate.Len() - UnderScoreIndex); //remove Name.

			LODIndexString = LODIndexString.RightChop(3); //remove LOD prefix.

			if (LODIndexString.IsNumeric())
			{
				LODIndex = FPlatformString::Atoi(*LODIndexString);
				return true;
			}
		}
	}

	ErrorString = FString::Printf(TEXT("Possible LOD pattern missmatch. Expecting 'LODxx_Name', but found '%s'."), *Candidate);

	return false;
}

FString UInterchangeMeshLODContainerNode::MakeMeshLODContainerUid(const FString& OriginalUid)
{
	return MeshLODContainerPrefix + OriginalUid;
}