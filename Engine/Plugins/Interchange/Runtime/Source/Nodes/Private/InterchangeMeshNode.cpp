// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeMeshNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeMeshNode)

//Interchange namespace
namespace UE
{
	namespace Interchange
	{
		const FAttributeKey& FMeshNodeStaticData::PayLoadKey()
		{
			static FAttributeKey AttributeKey(TEXT("__PayloadKey__"));
			return AttributeKey;
		}

		const FAttributeKey& FMeshNodeStaticData::PayLoadTypeKey()
		{
			static FAttributeKey AttributeKey(TEXT("__PayloadTypeKey__"));
			return AttributeKey;
		}

		const FAttributeKey& FMeshNodeStaticData::IsSkinnedMeshKey()
		{
			static FAttributeKey AttributeKey(TEXT("__IsSkinnedMeshKey__"));
			return AttributeKey;
		}

		const FAttributeKey& FMeshNodeStaticData::IsMorphTargetKey()
		{
			static FAttributeKey AttributeKey(TEXT("__IsMorphTargetKey__"));
			return AttributeKey;
		}

		const FAttributeKey& FMeshNodeStaticData::MorphTargetNameKey()
		{
			static FAttributeKey AttributeKey(TEXT("__MorphTargetNameKey__"));
			return AttributeKey;
		}

		const FAttributeKey& FMeshNodeStaticData::GetSkeletonDependenciesKey()
		{
			static FAttributeKey Dependencies_BaseKey(TEXT("__MeshSkeletonDependencies__"));
			return Dependencies_BaseKey;
		}

		const FAttributeKey& FMeshNodeStaticData::GetMorphTargetDependenciesKey()
		{
			static FAttributeKey Dependencies_BaseKey(TEXT("__MeshMorphTargetDependencies__"));
			return Dependencies_BaseKey;
		}

		const FAttributeKey& FMeshNodeStaticData::GetSceneInstancesUidsKey()
		{
			static FAttributeKey SceneInstanceUids_BaseKey(TEXT("__MeshSceneInstancesUids__"));
			return SceneInstanceUids_BaseKey;
		}

		const FAttributeKey& FMeshNodeStaticData::GetSlotMaterialDependenciesKey()
		{
			static FAttributeKey Dependencies_BaseKey(TEXT("__SlotMaterialDependencies__"));
			return Dependencies_BaseKey;
		}

		const FAttributeKey& FMeshNodeStaticData::GetAssemblyPartDependenciesKey()
		{
			static FAttributeKey Dependencies_BaseKey(TEXT("__AssemblyPartDependencies__"));
			return Dependencies_BaseKey;
		}

	}//ns Interchange
}//ns UE

UInterchangeMeshNode::UInterchangeMeshNode()
{
	SkeletonDependencies.Initialize(Attributes, UE::Interchange::FMeshNodeStaticData::GetSkeletonDependenciesKey().ToString());
	MorphTargetDependencies.Initialize(Attributes, UE::Interchange::FMeshNodeStaticData::GetMorphTargetDependenciesKey().ToString());
	SceneInstancesUids.Initialize(Attributes, UE::Interchange::FMeshNodeStaticData::GetSceneInstancesUidsKey().ToString());
	SlotMaterialDependencies.Initialize(Attributes.ToSharedRef(), UE::Interchange::FMeshNodeStaticData::GetSlotMaterialDependenciesKey().ToString());
	AssemblyPartDependencies.Initialize(Attributes, UE::Interchange::FMeshNodeStaticData::GetAssemblyPartDependenciesKey().ToString());
}

#if WITH_EDITOR

FString UInterchangeMeshNode::GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	FString KeyDisplayName = NodeAttributeKey.ToString();
	const FString NodeAttributeKeyString = KeyDisplayName;
	if (NodeAttributeKey == UE::Interchange::FMeshNodeStaticData::PayLoadKey())
	{
		return KeyDisplayName = TEXT("Payload Source Key");
	}
	else if (NodeAttributeKey == UE::Interchange::FMeshNodeStaticData::IsSkinnedMeshKey())
	{
		return KeyDisplayName = TEXT("Is a Skinned Mesh");
	}
	else if (NodeAttributeKey == UE::Interchange::FMeshNodeStaticData::IsMorphTargetKey())
	{
		return KeyDisplayName = TEXT("Is a Morph Target");
	}
	else if (NodeAttributeKey == UE::Interchange::FMeshNodeStaticData::MorphTargetNameKey())
	{
		return KeyDisplayName = TEXT("Morph Target Name");
	}
	else if (NodeAttributeKey == UE::Interchange::FMeshNodeStaticData::GetSkeletonDependenciesKey())
	{
		return KeyDisplayName = TEXT("Skeleton Dependencies count");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FMeshNodeStaticData::GetSkeletonDependenciesKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Skeleton Dependencies Index "), NodeAttributeKeyString);
	}
	else if (NodeAttributeKey == UE::Interchange::FMeshNodeStaticData::GetMorphTargetDependenciesKey())
	{
		return KeyDisplayName = TEXT("Morph Target Dependencies Count");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FMeshNodeStaticData::GetMorphTargetDependenciesKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Morph Target Dependencies Index "), NodeAttributeKeyString);
	}
	else if (NodeAttributeKey == UE::Interchange::FMeshNodeStaticData::GetSceneInstancesUidsKey())
	{
		return KeyDisplayName = TEXT("Scene mesh instances count");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FMeshNodeStaticData::GetSceneInstancesUidsKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Scene mesh instances Index "), NodeAttributeKeyString);
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FMeshNodeStaticData::GetSlotMaterialDependenciesKey().ToString()))
	{
		return KeyDisplayName = TEXT("Slot material dependencies");
	}
	return Super::GetKeyDisplayName(NodeAttributeKey);
}

FString UInterchangeMeshNode::GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	const FString NodeAttributeKeyString = NodeAttributeKey.ToString();;
	if (NodeAttributeKeyString.StartsWith(UE::Interchange::FMeshNodeStaticData::GetSkeletonDependenciesKey().ToString()))
	{
		return FString(TEXT("SkeletonDependencies"));
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FMeshNodeStaticData::GetMorphTargetDependenciesKey().ToString()))
	{
		return FString(TEXT("MorphTargetDependencies"));
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FMeshNodeStaticData::GetSceneInstancesUidsKey().ToString()))
	{
		return FString(TEXT("SceneInstances"));
	}
	else if (NodeAttributeKey == Macro_CustomVertexCountKey
				|| NodeAttributeKey == Macro_CustomPolygonCountKey
				|| NodeAttributeKey == Macro_CustomBoundingBoxKey
				|| NodeAttributeKey == Macro_CustomHasVertexNormalKey
				|| NodeAttributeKey == Macro_CustomHasVertexBinormalKey
				|| NodeAttributeKey == Macro_CustomHasVertexTangentKey
				|| NodeAttributeKey == Macro_CustomHasSmoothGroupKey
				|| NodeAttributeKey == Macro_CustomHasVertexColorKey
				|| NodeAttributeKey == Macro_CustomUVCountKey)
	{
		return FString(TEXT("MeshInfo"));
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FMeshNodeStaticData::GetSlotMaterialDependenciesKey().ToString()))
	{
		return FString(TEXT("SlotMaterialDependencies"));
	}
	return Super::GetAttributeCategory(NodeAttributeKey);
}

#endif //WITH_EDITOR

FString UInterchangeMeshNode::GetTypeName() const
{
	const FString TypeName = TEXT("MeshNode");
	return TypeName;
}

FName UInterchangeMeshNode::GetIconName() const
{
	FString MeshIconName = TEXT("MeshIcon.");
	if (IsSkinnedMesh())
	{
		MeshIconName += TEXT("Skinned");
	}
	else
	{
		MeshIconName += TEXT("Static");
	}
		
	return FName(*MeshIconName);
}

bool UInterchangeMeshNode::IsSkinnedMesh() const
{
	if (!Attributes->ContainAttribute(UE::Interchange::FMeshNodeStaticData::IsSkinnedMeshKey()))
	{
		return false;
	}

	UE::Interchange::FAttributeStorage::TAttributeHandle<bool> Handle = Attributes->GetAttributeHandle<bool>(UE::Interchange::FMeshNodeStaticData::IsSkinnedMeshKey());
	if (Handle.IsValid())
	{
		bool bValue = false;
		Handle.Get(bValue);
		return bValue;
	}
	return false;
}

bool UInterchangeMeshNode::SetSkinnedMesh(const bool bIsSkinnedMesh)
{
	UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FMeshNodeStaticData::IsSkinnedMeshKey(), bIsSkinnedMesh);
	if (IsAttributeStorageResultSuccess(Result))
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<bool> Handle = Attributes->GetAttributeHandle<bool>(UE::Interchange::FMeshNodeStaticData::IsSkinnedMeshKey());
		return Handle.IsValid();
	}
	return false;
}

bool UInterchangeMeshNode::IsMorphTarget() const
{
	if (!Attributes->ContainAttribute(UE::Interchange::FMeshNodeStaticData::IsMorphTargetKey()))
	{
		return false;
	}

	UE::Interchange::FAttributeStorage::TAttributeHandle<bool> Handle = Attributes->GetAttributeHandle<bool>(UE::Interchange::FMeshNodeStaticData::IsMorphTargetKey());
	if (Handle.IsValid())
	{
		bool bValue = false;
		Handle.Get(bValue);
		return bValue;
	}
	return false;
}

bool UInterchangeMeshNode::SetMorphTarget(const bool bIsMorphTarget)
{
	UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FMeshNodeStaticData::IsMorphTargetKey(), bIsMorphTarget);
	if (IsAttributeStorageResultSuccess(Result))
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<bool> Handle = Attributes->GetAttributeHandle<bool>(UE::Interchange::FMeshNodeStaticData::IsMorphTargetKey());
		return Handle.IsValid();
	}
	return false;
}

bool UInterchangeMeshNode::GetMorphTargetName(FString& OutMorphTargetName) const
{
	OutMorphTargetName.Empty();
	if (!Attributes->ContainAttribute(UE::Interchange::FMeshNodeStaticData::MorphTargetNameKey()))
	{
		return false;
	}

	UE::Interchange::FAttributeStorage::TAttributeHandle<FString> Handle = Attributes->GetAttributeHandle<FString>(UE::Interchange::FMeshNodeStaticData::MorphTargetNameKey());
	if (Handle.IsValid())
	{
		Handle.Get(OutMorphTargetName);
		return true;
	}
	return false;
}

bool UInterchangeMeshNode::SetMorphTargetName(const FString& MorphTargetName)
{
	UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FMeshNodeStaticData::MorphTargetNameKey(), MorphTargetName);
	if (IsAttributeStorageResultSuccess(Result))
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<FString> Handle = Attributes->GetAttributeHandle<FString>(UE::Interchange::FMeshNodeStaticData::MorphTargetNameKey());
		return Handle.IsValid();
	}
	return false;
}

const TOptional<FInterchangeMeshPayLoadKey> UInterchangeMeshNode::GetPayLoadKey() const
{
	FString PayloadKey;
	EInterchangeMeshPayLoadType PayLoadType;

	//PayLoadKey
	{
		if (!Attributes->ContainAttribute(UE::Interchange::FMeshNodeStaticData::PayLoadKey()))
		{
			return TOptional<FInterchangeMeshPayLoadKey>();
		}
		UE::Interchange::FAttributeStorage::TAttributeHandle<FString> AttributeHandle = Attributes->GetAttributeHandle<FString>(UE::Interchange::FMeshNodeStaticData::PayLoadKey());
		if (!AttributeHandle.IsValid())
		{
			return TOptional<FInterchangeMeshPayLoadKey>();
		}
		UE::Interchange::EAttributeStorageResult Result = AttributeHandle.Get(PayloadKey);
		if (!IsAttributeStorageResultSuccess(Result))
		{
			LogAttributeStorageErrors(Result, TEXT("UInterchangeMeshNode.GetPayLoadKey"), UE::Interchange::FMeshNodeStaticData::PayLoadKey());
			return TOptional<FInterchangeMeshPayLoadKey>();
		}
	}

	//PayLoadType
	{
		if (!Attributes->ContainAttribute(UE::Interchange::FMeshNodeStaticData::PayLoadTypeKey()))
		{
			return TOptional<FInterchangeMeshPayLoadKey>();
		}
		UE::Interchange::FAttributeStorage::TAttributeHandle<EInterchangeMeshPayLoadType> AttributeHandle = Attributes->GetAttributeHandle<EInterchangeMeshPayLoadType>(UE::Interchange::FMeshNodeStaticData::PayLoadTypeKey());
		if (!AttributeHandle.IsValid())
		{
			return TOptional<FInterchangeMeshPayLoadKey>();
		}

		UE::Interchange::EAttributeStorageResult Result = AttributeHandle.Get(PayLoadType);
		if (!IsAttributeStorageResultSuccess(Result))
		{
			LogAttributeStorageErrors(Result, TEXT("UInterchangeMeshNode.GetPayLoadTypeKey"), UE::Interchange::FMeshNodeStaticData::PayLoadTypeKey());
			return TOptional<FInterchangeMeshPayLoadKey>();
		}
	}
	
	return TOptional<FInterchangeMeshPayLoadKey>(FInterchangeMeshPayLoadKey(PayloadKey, PayLoadType));
}

void UInterchangeMeshNode::SetPayLoadKey(const FString& PayLoadKey, const EInterchangeMeshPayLoadType& PayLoadType)
{
	UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FMeshNodeStaticData::PayLoadKey(), PayLoadKey);
	if (!IsAttributeStorageResultSuccess(Result))
	{
		LogAttributeStorageErrors(Result, TEXT("UInterchangeMeshNode.SetPayLoadKey"), UE::Interchange::FMeshNodeStaticData::PayLoadKey());
	}
	else
	{
		Result = Attributes->RegisterAttribute(UE::Interchange::FMeshNodeStaticData::PayLoadTypeKey(), PayLoadType);
		if (!IsAttributeStorageResultSuccess(Result))
		{
			LogAttributeStorageErrors(Result, TEXT("UInterchangeMeshNode.SetPayLoadTypeKey"), UE::Interchange::FMeshNodeStaticData::PayLoadTypeKey());
		}
	}
}

bool UInterchangeMeshNode::GetCustomVertexCount(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(VertexCount, int32);
}

bool UInterchangeMeshNode::SetCustomVertexCount(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(VertexCount, int32);
}

bool UInterchangeMeshNode::GetCustomPolygonCount(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(PolygonCount, int32);
}

bool UInterchangeMeshNode::SetCustomPolygonCount(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(PolygonCount, int32);
}

bool UInterchangeMeshNode::GetCustomBoundingBox(FBox& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(BoundingBox, FBox);
}

bool UInterchangeMeshNode::SetCustomBoundingBox(const FBox& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(BoundingBox, FBox);
}

bool UInterchangeMeshNode::GetCustomHasVertexNormal(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(HasVertexNormal, bool);
}

bool UInterchangeMeshNode::SetCustomHasVertexNormal(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(HasVertexNormal, bool);
}

bool UInterchangeMeshNode::GetCustomHasVertexBinormal(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(HasVertexBinormal, bool);
}

bool UInterchangeMeshNode::SetCustomHasVertexBinormal(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(HasVertexBinormal, bool);
}

bool UInterchangeMeshNode::GetCustomHasVertexTangent(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(HasVertexTangent, bool);
}

bool UInterchangeMeshNode::SetCustomHasVertexTangent(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(HasVertexTangent, bool);
}

bool UInterchangeMeshNode::GetCustomHasSmoothGroup(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(HasSmoothGroup, bool);
}

bool UInterchangeMeshNode::SetCustomHasSmoothGroup(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(HasSmoothGroup, bool);
}

bool UInterchangeMeshNode::GetCustomHasVertexColor(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(HasVertexColor, bool);
}

bool UInterchangeMeshNode::SetCustomHasVertexColor(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(HasVertexColor, bool);
}

bool UInterchangeMeshNode::GetCustomUVCount(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(UVCount, int32);
}

bool UInterchangeMeshNode::SetCustomUVCount(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(UVCount, int32);
}

int32 UInterchangeMeshNode::GetSkeletonDependeciesCount() const
{
	return SkeletonDependencies.GetCount();
}

void UInterchangeMeshNode::GetSkeletonDependencies(TArray<FString>& OutDependencies) const
{
	SkeletonDependencies.GetItems(OutDependencies);
}

void UInterchangeMeshNode::GetSkeletonDependency(const int32 Index, FString& OutDependency) const
{
	SkeletonDependencies.GetItem(Index, OutDependency);
}

bool UInterchangeMeshNode::SetSkeletonDependencyUid(const FString& DependencyUid)
{ 
	return SkeletonDependencies.AddItem(DependencyUid);
}

bool UInterchangeMeshNode::RemoveSkeletonDependencyUid(const FString& DependencyUid)
{
	return SkeletonDependencies.RemoveItem(DependencyUid);
}

int32 UInterchangeMeshNode::GetMorphTargetDependeciesCount() const
{
	return MorphTargetDependencies.GetCount();
}

void UInterchangeMeshNode::GetMorphTargetDependencies(TArray<FString>& OutDependencies) const
{
	MorphTargetDependencies.GetItems(OutDependencies);
}

void UInterchangeMeshNode::GetMorphTargetDependency(const int32 Index, FString& OutDependency) const
{
	MorphTargetDependencies.GetItem(Index, OutDependency);
}

bool UInterchangeMeshNode::SetMorphTargetDependencyUid(const FString& DependencyUid)
{
	return MorphTargetDependencies.AddItem(DependencyUid);
}

bool UInterchangeMeshNode::RemoveMorphTargetDependencyUid(const FString& DependencyUid)
{
	return MorphTargetDependencies.RemoveItem(DependencyUid);
}

int32 UInterchangeMeshNode::GetSceneInstanceUidsCount() const
{
	return SceneInstancesUids.GetCount();
}

void UInterchangeMeshNode::GetSceneInstanceUids(TArray<FString>& OutDependencies) const
{
	SceneInstancesUids.GetItems(OutDependencies);
}

void UInterchangeMeshNode::GetSceneInstanceUid(const int32 Index, FString& OutDependency) const
{
	SceneInstancesUids.GetItem(Index, OutDependency);
}

bool UInterchangeMeshNode::SetSceneInstanceUid(const FString& DependencyUid)
{
	return SceneInstancesUids.AddItem(DependencyUid);
}

bool UInterchangeMeshNode::RemoveSceneInstanceUid(const FString& DependencyUid)
{
	return SceneInstancesUids.RemoveItem(DependencyUid);
}

void UInterchangeMeshNode::GetSlotMaterialDependencies(TMap<FString, FString>& OutMaterialDependencies) const
{
	OutMaterialDependencies = SlotMaterialDependencies.ToMap();
}

bool UInterchangeMeshNode::GetSlotMaterialDependencyUid(const FString& SlotName, FString& OutMaterialDependency) const
{
	return SlotMaterialDependencies.GetValue(SlotName, OutMaterialDependency);
}

bool UInterchangeMeshNode::SetSlotMaterialDependencyUid(const FString& SlotName, const FString& MaterialDependencyUid)
{
	return SlotMaterialDependencies.SetKeyValue(SlotName, MaterialDependencyUid);
}

bool UInterchangeMeshNode::RemoveSlotMaterialDependencyUid(const FString& SlotName)
{
	return SlotMaterialDependencies.RemoveKey(SlotName);
}

int32 UInterchangeMeshNode::GetAssemblyPartDependenciesCount() const
{
	return AssemblyPartDependencies.GetCount();
}

void UInterchangeMeshNode::GetAssemblyPartDependencies(TArray<FString>& OutDependencies) const
{
	AssemblyPartDependencies.GetItems(OutDependencies);
}

void UInterchangeMeshNode::GetAssemblyPartDependency(const int32 Index, FString& OutDependency) const
{
	AssemblyPartDependencies.GetItem(Index, OutDependency);
}

bool UInterchangeMeshNode::AddAssemblyPartDependencyUid(const FString& DependencyUid)
{
	return AssemblyPartDependencies.AddItem(DependencyUid);
}

bool UInterchangeMeshNode::RemoveAssemblyPartDependencyUid(const FString& DependencyUid)
{
	return AssemblyPartDependencies.RemoveItem(DependencyUid);
}

bool UInterchangeMeshNode::GetCustomCollisionType(EInterchangeMeshCollision& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(CollisionType, EInterchangeMeshCollision);
}

bool UInterchangeMeshNode::SetCustomCollisionType(EInterchangeMeshCollision AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(CollisionType, EInterchangeMeshCollision);
}

UInterchangeGeometryCacheNode::UInterchangeGeometryCacheNode()
{
}

bool UInterchangeGeometryCacheNode::GetCustomStartFrame(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(StartFrame, int32);
}

bool UInterchangeGeometryCacheNode::SetCustomStartFrame(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(StartFrame, int32);
}

bool UInterchangeGeometryCacheNode::GetCustomEndFrame(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(EndFrame, int32);
}

bool UInterchangeGeometryCacheNode::SetCustomEndFrame(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(EndFrame, int32);
}

bool UInterchangeGeometryCacheNode::GetCustomFrameRate(double& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(FrameRate, double);
}

bool UInterchangeGeometryCacheNode::SetCustomFrameRate(const double& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(FrameRate, double);
}

bool UInterchangeGeometryCacheNode::GetCustomHasConstantTopology(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(HasConstantTopology, bool);
}

bool UInterchangeGeometryCacheNode::SetCustomHasConstantTopology(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(HasConstantTopology, bool);
}
