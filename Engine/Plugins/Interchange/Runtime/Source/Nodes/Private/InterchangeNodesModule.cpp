// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeNodesModule.h"

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
//#include "InterchangeManager.h"

DEFINE_LOG_CATEGORY(LogInterchangeNodes);

class FInterchangeNodesModule : public IInterchangeNodesModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FInterchangeNodesModule, InterchangeNodes)



void FInterchangeNodesModule::StartupModule()
{
	//Register anything needed to the interchange manager
	//UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
	//InterchangeManager.Register...
}


void FInterchangeNodesModule::ShutdownModule()
{

}



