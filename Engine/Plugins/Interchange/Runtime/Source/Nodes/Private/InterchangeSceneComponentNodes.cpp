// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeSceneComponentNodes.h"
#include "InterchangeSceneNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSceneComponentNodes)

UInterchangeSceneComponentNode::UInterchangeSceneComponentNode()
{
	ComponentUids.Initialize(Attributes, TEXT("__ComponentUids__Key"));
}

FString UInterchangeSceneComponentNode::GetTypeName() const
{
	static const FString TypeName = TEXT("SceneComponentNode");
	return TypeName;
}

bool UInterchangeSceneComponentNode::AddComponentUid(const FString& ComponentUid)
{
	return ComponentUids.AddItem(ComponentUid);
}

void UInterchangeSceneComponentNode::GetComponentUids(TArray<FString>& OutComponentUids) const
{
	ComponentUids.GetItems(OutComponentUids);
}

bool UInterchangeSceneComponentNode::GetCustomLocalTransform(FTransform& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(LocalTransform, FTransform);
}

bool UInterchangeSceneComponentNode::SetCustomLocalTransform(const FTransform& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(LocalTransform, FTransform);
}

bool UInterchangeSceneComponentNode::GetCustomComponentVisibility(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ComponentVisibility, bool);
}

bool UInterchangeSceneComponentNode::SetCustomComponentVisibility(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ComponentVisibility, bool);
}

const UInterchangeSceneNode* UInterchangeSceneComponentNode::GetParentSceneNodeAndTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer
	, FTransform& SceneNodeTransform
	, bool bForceRecache) const
{
	if (bForceRecache)
	{
		CacheSceneNodeTransform.Reset();
		CacheParentSceneNode.Reset();
	}
	if (!CacheSceneNodeTransform.IsSet())
	{
		FTransform LocalTransform;
		if (GetCustomLocalTransform(LocalTransform))
		{
			//Compute the Global
			FString ParentUid = GetParentUid();
			if (!ParentUid.IsEmpty())
			{
				const UInterchangeBaseNode* ParentBaseNode = BaseNodeContainer->GetNode(ParentUid);
				FTransform SceneNodeGlobalParent;
				if (const UInterchangeSceneComponentNode* ParentSceneComponentNode = Cast<UInterchangeSceneComponentNode>(ParentBaseNode))
				{
					const UInterchangeSceneNode* ParentSceneNode = ParentSceneComponentNode->GetParentSceneNodeAndTransform(BaseNodeContainer, SceneNodeGlobalParent, bForceRecache);
					CacheParentSceneNode = ParentSceneNode;
				}
				if (const UInterchangeSceneNode* ParentSceneNode = Cast<UInterchangeSceneNode>(ParentBaseNode))
				{
					CacheParentSceneNode = ParentSceneNode;
					CacheSceneNodeTransform = LocalTransform;
				}
				CacheSceneNodeTransform = LocalTransform * SceneNodeGlobalParent;
			}
			else
			{
				//Scene Node without parent will need the global offset to be apply
				CacheSceneNodeTransform = LocalTransform;

				return nullptr;
			}
		}
		else
		{
			CacheSceneNodeTransform = FTransform::Identity;
		}
	}
	//The cache is always valid here
	if (ensure(CacheSceneNodeTransform.IsSet()))
	{
		SceneNodeTransform = CacheSceneNodeTransform.GetValue();
	}
	else
	{
		SceneNodeTransform = FTransform::Identity;
	}

	if (CacheParentSceneNode.IsSet())
	{
		return CacheParentSceneNode.GetValue();
	}
	else
	{
		return nullptr;
	}
}




FString UInterchangeInstancedStaticMeshComponentNode::GetTypeName() const
{
	const FString TypeName = TEXT("InstancedStaticMeshComponentNode");
	return TypeName;
}

UInterchangeInstancedStaticMeshComponentNode::UInterchangeInstancedStaticMeshComponentNode()
{
	InstanceTransforms.Initialize(Attributes, TEXT("__InstanceTransforms__"));
}
void UInterchangeInstancedStaticMeshComponentNode::AddInstanceTransform(const FTransform& InstanceTransform)
{
	InstanceTransforms.AddItem(InstanceTransform);
}
void UInterchangeInstancedStaticMeshComponentNode::AddInstanceTransforms(const TArray<FTransform>& InInstanceTransforms)
{
	for (const FTransform& InstanceTransform : InInstanceTransforms)
	{
		InstanceTransforms.AddItem(InstanceTransform);
	}
}
void UInterchangeInstancedStaticMeshComponentNode::GetInstanceTransforms(TArray<FTransform>& OutInstanceTransforms) const
{
	InstanceTransforms.GetItems(OutInstanceTransforms);
}

bool UInterchangeInstancedStaticMeshComponentNode::GetCustomInstancedAssetUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AssetInstanceUid, FString);
}

bool UInterchangeInstancedStaticMeshComponentNode::SetCustomInstancedAssetUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AssetInstanceUid, FString);
}




