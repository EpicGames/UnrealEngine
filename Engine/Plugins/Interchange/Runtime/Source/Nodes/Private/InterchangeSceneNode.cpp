// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeSceneNode.h"
#include "InterchangeNodeLog.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSceneNode)

DEFINE_LOG_CATEGORY(LogInterchangeNode);

//Interchange namespace
namespace UE
{
	namespace Interchange
	{

		const FAttributeKey& FSceneNodeStaticData::GetNodeSpecializeTypeBaseKey()
		{
			static const FAttributeKey SceneNodeSpecializeType_BaseKey(TEXT("SceneNodeSpecializeType"));
			return SceneNodeSpecializeType_BaseKey;
		}

		const FAttributeKey& FSceneNodeStaticData::GetMaterialDependencyUidsBaseKey()
		{
			static const FAttributeKey MaterialDependencyUids_BaseKey(TEXT("__MaterialDependencyUidsBaseKey__"));
			return MaterialDependencyUids_BaseKey;
		}

		const FString& FSceneNodeStaticData::GetJointSpecializeTypeString()
		{
			static const FString JointSpecializeTypeString(TEXT("Joint"));
			return JointSpecializeTypeString;
		}

		const FString& FSceneNodeStaticData::GetLodGroupSpecializeTypeString()
		{
			static const FString LodGroupSpecializeTypeString(TEXT("LodGroup"));
			return LodGroupSpecializeTypeString;
		}

		const FString& FSceneNodeStaticData::GetSlotMaterialDependenciesString()
		{
			static const FString SlotMaterialDependenciesString(TEXT("__SlotMaterialDependencies__"));
			return SlotMaterialDependenciesString;
		}

		const FString& FSceneNodeStaticData::GetMeshToGlobalBindPoseReferencesString()
		{
			static const FString MeshToGlobalBindPoseReferncesString(TEXT("__MeshToGlobalBindPoseReferences__"));
			return MeshToGlobalBindPoseReferncesString;
		}

		const FString& FSceneNodeStaticData::GetMorphTargetCurveWeightsKey()
		{
			static const FString MorphTargetCurvesKey(TEXT("__MorphTargetCurveWeights__Key"));
			return MorphTargetCurvesKey;
		}

		const FString& FSceneNodeStaticData::GetLayerNamesKey()
		{
			static const FString LayerNamesKey(TEXT("__LayerNames__Key"));
			return LayerNamesKey;
		}

		const FString& FSceneNodeStaticData::GetTagsKey()
		{
			static const FString TagsKey(TEXT("__Tags__Key"));
			return TagsKey;
		}

		const FString& FSceneNodeStaticData::GetCurveAnimationTypesKey()
		{
			static const FString CurveAnimationTypesKey(TEXT("__CurveAnimationTypes__Key"));
			return CurveAnimationTypesKey;
		}

		const FString& FSceneNodeStaticData::GetComponentUidsKey()
		{
			static const FString ComponentsUidsKey(TEXT("__ComponentUids__Key"));
			return ComponentsUidsKey;
		}
	}//ns Interchange
}//ns UE

UInterchangeSceneNode::UInterchangeSceneNode()
{
	NodeSpecializeTypes.Initialize(Attributes, UE::Interchange::FSceneNodeStaticData::GetNodeSpecializeTypeBaseKey().ToString());
	SlotMaterialDependencies.Initialize(Attributes.ToSharedRef(), UE::Interchange::FSceneNodeStaticData::GetSlotMaterialDependenciesString());
	MorphTargetCurveWeights.Initialize(Attributes.ToSharedRef(), UE::Interchange::FSceneNodeStaticData::GetMorphTargetCurveWeightsKey());
	LayerNames.Initialize(Attributes, UE::Interchange::FSceneNodeStaticData::GetLayerNamesKey());
	Tags.Initialize(Attributes, UE::Interchange::FSceneNodeStaticData::GetTagsKey());
	CurveAnimationTypes.Initialize(Attributes.ToSharedRef(), UE::Interchange::FSceneNodeStaticData::GetCurveAnimationTypesKey());
	ComponentUids.Initialize(Attributes.ToSharedRef(), UE::Interchange::FSceneNodeStaticData::GetComponentUidsKey());

	// The handling of the global and local transforms of a scene node should be revisited.
	// Currently if a translator does not set a local transform on a scene node some logic will fail.
	// See UE-351819 aas an example.
	// No logic should fail if the local transform was not explicitly set because it should implicitly be the identity. 
	InterchangePrivateNodeBase::SetCustomAttribute<FTransform>(*Attributes, Macro_CustomLocalTransformKey, TEXT("SceneNode.SetLocalTransform"), FTransform::Identity);
}

/**
* Return the node type name of the class. This is used when reporting errors.
*/
FString UInterchangeSceneNode::GetTypeName() const
{
	static const FString TypeName = TEXT("SceneNode");
	return TypeName;
}

#if WITH_EDITOR

FString UInterchangeSceneNode::GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	FString KeyDisplayName = NodeAttributeKey.ToString();
	const FString NodeAttributeKeyString = KeyDisplayName;
	if (NodeAttributeKey == UE::Interchange::FSceneNodeStaticData::GetNodeSpecializeTypeBaseKey())
	{
		KeyDisplayName = TEXT("Specialized type count");
		return KeyDisplayName;
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FSceneNodeStaticData::GetNodeSpecializeTypeBaseKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Specialized type index "), NodeAttributeKeyString);
	}
	else if (NodeAttributeKey == UE::Interchange::FSceneNodeStaticData::GetMaterialDependencyUidsBaseKey())
	{
		KeyDisplayName = TEXT("Material dependencies count");
		return KeyDisplayName;
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FSceneNodeStaticData::GetMaterialDependencyUidsBaseKey().ToString()))
	{
		return KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Material dependency index "), NodeAttributeKeyString);
	}
	return Super::GetKeyDisplayName(NodeAttributeKey);
}

FString UInterchangeSceneNode::GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	if (NodeAttributeKey == Macro_CustomLocalTransformKey || NodeAttributeKey == Macro_CustomAssetInstanceUidKey)
	{
		return FString(TEXT("Scene"));
	}

	const FString NodeAttributeKeyString = NodeAttributeKey.ToString();

	if (NodeAttributeKeyString.StartsWith(UE::Interchange::FSceneNodeStaticData::GetNodeSpecializeTypeBaseKey().ToString()))
	{
		return FString(TEXT("SpecializeType"));
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FSceneNodeStaticData::GetMaterialDependencyUidsBaseKey().ToString()))
	{
		return FString(TEXT("MaterialDependencies"));
	}
	
	return Super::GetAttributeCategory(NodeAttributeKey);
}

#endif //WITH_EDITOR

FName UInterchangeSceneNode::GetIconName() const
{
	FString SpecializedType;
	GetSpecializedType(0, SpecializedType);
	if (SpecializedType.IsEmpty())
	{
		return NAME_None;
	}
	SpecializedType = TEXT("SceneGraphIcon.") + SpecializedType;
	return FName(*SpecializedType);
}

bool UInterchangeSceneNode::IsSpecializedTypeContains(const FString& SpecializedType) const
{
	TArray<FString> SpecializedTypes;
	GetSpecializedTypes(SpecializedTypes);
	for (const FString& SpecializedTypeRef : SpecializedTypes)
	{
		if (SpecializedTypeRef.Equals(SpecializedType))
		{
			return true;
		}
	}
	return false;
}

int32 UInterchangeSceneNode::GetSpecializedTypeCount() const
{
	return NodeSpecializeTypes.GetCount();
}

void UInterchangeSceneNode::GetSpecializedType(const int32 Index, FString& OutSpecializedType) const
{
	NodeSpecializeTypes.GetItem(Index, OutSpecializedType);
}

void UInterchangeSceneNode::GetSpecializedTypes(TArray<FString>& OutSpecializedTypes) const
{
	NodeSpecializeTypes.GetItems(OutSpecializedTypes);
}

bool UInterchangeSceneNode::AddSpecializedType(const FString& SpecializedType)
{
	return NodeSpecializeTypes.AddItem(SpecializedType);
}

bool UInterchangeSceneNode::RemoveSpecializedType(const FString& SpecializedType)
{
	return NodeSpecializeTypes.RemoveItem(SpecializedType);
}

bool UInterchangeSceneNode::GetCustomLocalTransform(FTransform& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(LocalTransform, FTransform);
}

bool UInterchangeSceneNode::SetCustomLocalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& AttributeValue)
{
	// Modifying the LocalTransform can affect all cached global hierarchies.
    // Both BindPoseGlobalTransform and TimeZeroGlobalTransform chains may fall back to LocalTransform when no BindPose or TimeZero attribute is set.
	if (!GlobalTransformCache.IsEmpty())
	{
		// Clear ALL global transforms
		ResetGlobalTransformCachesOfNodeAndAllChildren(BaseNodeContainer, this);
	}
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(LocalTransform, FTransform);
}

bool UInterchangeSceneNode::GetCustomGlobalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& GlobalOffsetTransform, FTransform& AttributeValue) const
{
	return GetGlobalTransformWithOffsetInternal(Macro_CustomLocalTransformKey, BaseNodeContainer, GlobalOffsetTransform, AttributeValue);
}

bool UInterchangeSceneNode::GetCustomGeometricTransform(FTransform& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GeometricTransform, FTransform);
}

bool UInterchangeSceneNode::SetCustomGeometricTransform(const FTransform& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(GeometricTransform, FTransform);
}

bool UInterchangeSceneNode::GetCustomPivotNodeTransform(FTransform& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(PivotNodeTransform, FTransform);
}

bool UInterchangeSceneNode::SetCustomPivotNodeTransform(const FTransform& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(PivotNodeTransform, FTransform);
}

bool UInterchangeSceneNode::GetCustomComponentVisibility(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ComponentVisibility, bool);
}

bool UInterchangeSceneNode::SetCustomComponentVisibility(bool AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ComponentVisibility, bool);
}

bool UInterchangeSceneNode::GetCustomActorVisibility(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ActorVisibility, bool);
}

bool UInterchangeSceneNode::SetCustomActorVisibility(bool AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ActorVisibility, bool);
}

bool UInterchangeSceneNode::GetCustomAssetInstanceUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AssetInstanceUid, FString);
}

bool UInterchangeSceneNode::SetCustomAssetInstanceUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AssetInstanceUid, FString);
}

void UInterchangeSceneNode::ResetAllGlobalTransformCaches(const UInterchangeBaseNodeContainer* BaseNodeContainer)
{
	BaseNodeContainer->IterateNodesOfType<UInterchangeSceneNode>([](const FString& NodeUid, UInterchangeSceneNode* SceneNode)
		{
			SceneNode->GlobalTransformCache.Reset();
		});
}

void UInterchangeSceneNode::ResetGlobalTransformCachesOfNodeAndAllChildren(const UInterchangeBaseNodeContainer* BaseNodeContainer, const UInterchangeBaseNode* ParentNode)
{
	check(ParentNode);
	if (const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(ParentNode))
	{
		SceneNode->GlobalTransformCache.Reset();
	}
	TArray<FString> ChildrenUids = BaseNodeContainer->GetNodeChildrenUids(ParentNode->GetUniqueID());
	for (const FString& ChildUid : ChildrenUids)
	{
		if (const UInterchangeBaseNode* ChildNode = BaseNodeContainer->GetNode(ChildUid))
		{
			ResetGlobalTransformCachesOfNodeAndAllChildren(BaseNodeContainer, ChildNode);
		}
	}
}

void UInterchangeSceneNode::ResetGlobalTransformCacheInternal(const UInterchangeBaseNodeContainer* BaseNodeContainer
	, const UE::Interchange::FAttributeKey LocalTransformKey) const
{
	GlobalTransformCache.Remove(LocalTransformKey);

	TArray<FString> ChildrenUids = BaseNodeContainer->GetNodeChildrenUids(GetUniqueID());
	for (const FString& ChildUid : ChildrenUids)
	{
		if (const UInterchangeSceneNode* ChildNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(ChildUid)))
		{
			ChildNode->ResetGlobalTransformCacheInternal(BaseNodeContainer, LocalTransformKey);
		}
	}
}

bool UInterchangeSceneNode::GetGlobalTransformWithOffsetInternal(const UE::Interchange::FAttributeKey LocalTransformKey
	, const UInterchangeBaseNodeContainer* BaseNodeContainer
	, const FTransform& GlobalOffsetTransform
	, FTransform& OutAttributeValue) const
{
	if (GetGlobalTransformInternal(LocalTransformKey, BaseNodeContainer, OutAttributeValue))
	{
		OutAttributeValue *= GlobalOffsetTransform;
		return true;
	}

	return false;
}

/* This function will first check for the desired attribute (either Local, TimeZero or BindPose).
   If the attribute does not exist for this node, fallback to Local, but keep using desired attribute for parents.
   If this node does not have a Local transform either, return false and stop the global computation. */
bool UInterchangeSceneNode::GetGlobalTransformInternal(const UE::Interchange::FAttributeKey LocalTransformKey
	, const UInterchangeBaseNodeContainer* BaseNodeContainer
	, FTransform& OutAttributeValue) const
{
	if (GlobalTransformCache.Contains(LocalTransformKey))
	{
		OutAttributeValue = GlobalTransformCache.FindRef(LocalTransformKey);
	}
	else
	{
		UE::Interchange::FAttributeKey TransformKey = LocalTransformKey;
		if (!Attributes->ContainAttribute(TransformKey))
		{
			//Fallback to LocalTransform:
			if (TransformKey != Macro_CustomLocalTransformKey && Attributes->ContainAttribute(Macro_CustomLocalTransformKey))
			{
				TransformKey = Macro_CustomLocalTransformKey;
			}
			else
			{
				return false;
			}
		}

		OutAttributeValue = FTransform::Identity;

		UE::Interchange::FAttributeStorage::TAttributeHandle<FTransform> AttributeHandle;
		if (Attributes->ContainAttribute(TransformKey))
		{
			AttributeHandle = GetAttributeHandle<FTransform>(TransformKey);
		}

		if ((AttributeHandle.IsValid() && IsAttributeStorageResultSuccess(AttributeHandle.Get(OutAttributeValue))))
		{
			//Compute the Global
			FString ParentUid = GetParentUid();
			if (!ParentUid.IsEmpty())
			{
				FTransform GlobalParent = FTransform::Identity;
				if (const UInterchangeSceneNode* ParentSceneNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(ParentUid)))
				{
					ParentSceneNode->GetGlobalTransformInternal(LocalTransformKey, BaseNodeContainer, GlobalParent);
				}
				OutAttributeValue *= GlobalParent;
			}
		}

		GlobalTransformCache.Add(LocalTransformKey, OutAttributeValue);
	}

	return true;
}

void UInterchangeSceneNode::GetSlotMaterialDependencies(TMap<FString, FString>& OutMaterialDependencies) const
{
	OutMaterialDependencies = SlotMaterialDependencies.ToMap();
}

bool UInterchangeSceneNode::GetSlotMaterialDependencyUid(const FString& SlotName, FString& OutMaterialDependency) const
{
	return SlotMaterialDependencies.GetValue(SlotName, OutMaterialDependency);
}

bool UInterchangeSceneNode::SetSlotMaterialDependencyUid(const FString& SlotName, const FString& MaterialDependencyUid)
{
	return SlotMaterialDependencies.SetKeyValue(SlotName, MaterialDependencyUid);
}

bool UInterchangeSceneNode::RemoveSlotMaterialDependencyUid(const FString& SlotName)
{
	return SlotMaterialDependencies.RemoveKey(SlotName);
}


bool UInterchangeSceneNode::SetMorphTargetCurveWeight(const FString& MorphTargetName, const float& Weight)
{
	return MorphTargetCurveWeights.SetKeyValue(MorphTargetName, Weight);
}

void UInterchangeSceneNode::GetMorphTargetCurveWeights(TMap<FString, float>& OutMorphTargetCurveWeights) const
{
	OutMorphTargetCurveWeights = MorphTargetCurveWeights.ToMap();
}

bool UInterchangeSceneNode::SetCustomAnimationAssetUidToPlay(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AnimationAssetUidToPlay, FString);
}
bool UInterchangeSceneNode::GetCustomAnimationAssetUidToPlay(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AnimationAssetUidToPlay, FString);
}

void UInterchangeSceneNode::GetLayerNames(TArray<FString>& OutLayerNames) const
{
	LayerNames.GetItems(OutLayerNames);
}

bool UInterchangeSceneNode::AddLayerName(const FString& LayerName)
{
	return LayerNames.AddItem(LayerName);
}

bool UInterchangeSceneNode::RemoveLayerName(const FString& LayerName)
{
	return LayerNames.RemoveItem(LayerName);
}

void UInterchangeSceneNode::GetTags(TArray<FString>& OutTags) const
{
	Tags.GetItems(OutTags);
}

bool UInterchangeSceneNode::AddTag(const FString& Tag)
{
	return Tags.AddItem(Tag);
}

bool UInterchangeSceneNode::RemoveTag(const FString& Tag)
{
	return Tags.RemoveItem(Tag);
}

bool UInterchangeSceneNode::SetAnimationCurveTypeForCurveName(const FString& CurveName, const EInterchangeAnimationPayLoadType& AnimationCurveType)
{
	return CurveAnimationTypes.SetKeyValue(CurveName, AnimationCurveType);
}

bool UInterchangeSceneNode::GetAnimationCurveTypeForCurveName(const FString& CurveName, EInterchangeAnimationPayLoadType& OutCurveAnimationType) const
{
	return CurveAnimationTypes.GetValue(CurveName, OutCurveAnimationType);
}

bool UInterchangeSceneNode::AddComponentUid(const FString& ComponentUid)
{
	return ComponentUids.AddItem(ComponentUid);
}

void UInterchangeSceneNode::GetComponentUids(TArray<FString>& OutComponentUids) const
{
	ComponentUids.GetItems(OutComponentUids);
}

bool UInterchangeSceneNode::SetCustomIsSceneRoot(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(IsSceneRoot, bool);
}

bool UInterchangeSceneNode::GetCustomIsSceneRoot(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(IsSceneRoot, bool);
}

bool UInterchangeSceneNode::GetCustomDoNotInstantiateInLevel(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DoNotInstantiateInLevel, bool);
}

/** Sets whether the scene node was part of a Named LOD Group, indicating if it should show up in the scene or not. */
bool UInterchangeSceneNode::SetCustomDoNotInstantiateInLevel(bool AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DoNotInstantiateInLevel, bool);
}

// Deprecated
bool UInterchangeSceneNode::SetCustomLocalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& AttributeValue, bool bResetCache)
{
	return SetCustomLocalTransform(BaseNodeContainer, AttributeValue);
}

// Deprecated
bool UInterchangeSceneNode::GetCustomGlobalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& GlobalOffsetTransform, FTransform& AttributeValue, bool BForceRecache) const
{
	return GetCustomGlobalTransform(BaseNodeContainer, GlobalOffsetTransform, AttributeValue);
}

// Deprecated
bool UInterchangeSceneNode::GetCustomBindPoseLocalTransform(FTransform& AttributeValue) const
{
	UE_LOGF(LogTemp, Error, "GetCustomBindPoseLocalTransform() is deprecated on UInterchangeSceneNode. Use UInterchangeJointNode::GetBindPoseLocalTransform instead.");

	AttributeValue = FTransform::Identity;
	return false;
}

// Deprecated
bool UInterchangeSceneNode::SetCustomBindPoseLocalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& AttributeValue, bool bResetCache)
{
	UE_LOGF(LogTemp, Error, "SetCustomBindPoseLocalTransform() is deprecated on UInterchangeSceneNode. Use UInterchangeJointNode::SetBindPoseLocalTransform instead.");
	return false;
}

// Deprecated
bool UInterchangeSceneNode::GetCustomBindPoseGlobalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& GlobalOffsetTransform, FTransform& AttributeValue, bool bForceRecache) const
{
	UE_LOGF(LogTemp, Error, "GetCustomBindPoseGlobalTransform() is deprecated on UInterchangeSceneNode. Use UInterchangeJointNode::GetBindPoseGlobalTransform instead.");
	AttributeValue = FTransform::Identity;
	return false;
}

// Deprecated
bool UInterchangeSceneNode::GetCustomTimeZeroLocalTransform(FTransform& AttributeValue) const
{
	UE_LOGF(LogTemp, Error, "GetCustomTimeZeroLocalTransform() is deprecated on UInterchangeSceneNode. Use UInterchangeJointNode::GetTimeZeroLocalTransform instead.");
	AttributeValue = FTransform::Identity;
	return false;
}

// Deprecated
bool UInterchangeSceneNode::SetCustomTimeZeroLocalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& AttributeValue, bool bResetCache)
{
	UE_LOGF(LogTemp, Error, "SetCustomTimeZeroLocalTransform() is deprecated on UInterchangeSceneNode. Use UInterchangeJointNode::SetTimeZeroLocalTransform instead.");
	return false;
}

// Deprecated
bool UInterchangeSceneNode::GetCustomTimeZeroGlobalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& GlobalOffsetTransform, FTransform& AttributeValue, bool bForceRecache) const
{
	UE_LOGF(LogTemp, Error, "GetCustomTimeZeroGlobalTransform() is deprecated on UInterchangeSceneNode. Use UInterchangeJointNode::GetTimeZeroGlobalTransform instead.");
	AttributeValue = FTransform::Identity;
	return false;
}

// Deprecated
bool UInterchangeSceneNode::GetGlobalBindPoseReferenceForMeshUID(const FString& MeshUID, FMatrix& GlobalBindPoseReference) const
{
	UE_LOGF(LogInterchangeNode, Error, "GetGlobalBindPoseReferenceForMeshUID() is deprecated on UInterchangeSceneNode. Use UInterchangeJointNode instead.");
	GlobalBindPoseReference = FMatrix::Identity;
	return false;
}

// Deprecated
void UInterchangeSceneNode::SetGlobalBindPoseReferenceForMeshUIDs(const TMap<FString, FMatrix>& GlobalBindPoseReferenceForMeshUIDs)
{
	UE_LOGF(LogInterchangeNode, Error, "SetGlobalBindPoseReferenceForMeshUIDs() is deprecated on UInterchangeSceneNode. Use UInterchangeJointNode instead.");
}

// Deprecated
bool UInterchangeSceneNode::SetCustomHasBindPose(const bool& AttributeValue)
{
	UE_LOGF(LogInterchangeNode, Error, "SetCustomHasBindPose() is deprecated on UInterchangeSceneNode. Use UInterchangeJointNode::SetHasBindPose instead.");
	return false;
}

// Deprecated
bool UInterchangeSceneNode::GetCustomHasBindPose(bool& AttributeValue) const
{
	UE_LOGF(LogInterchangeNode, Error, "GetCustomHasBindPose() is deprecated on UInterchangeSceneNode. Use UInterchangeJointNode::GetHasBindPose instead.");
	AttributeValue = false;
	return false;
}

// Deprecated
bool UInterchangeSceneNode::SetCustomGlobalMatrixForT0Rebinding(const FMatrix& AttributeValue)
{
	UE_LOGF(LogInterchangeNode, Error, "SetCustomGlobalMatrixForT0Rebinding() is deprecated on UInterchangeSceneNode. Use UInterchangeJointNode::SetGlobalMatrixForT0Rebinding instead.");
	return false;
}

// Deprecated
bool UInterchangeSceneNode::GetCustomGlobalMatrixForT0Rebinding(FMatrix& AttributeValue) const
{
	UE_LOGF(LogInterchangeNode, Error, "GetCustomGlobalMatrixForT0Rebinding() is deprecated on UInterchangeSceneNode. Use UInterchangeJointNode::GetGlobalMatrixForT0Rebinding instead.");
	AttributeValue = FMatrix::Identity;
	return false;
}