// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeShaderGraphNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeShaderGraphNode)

const TCHAR* UInterchangeShaderPortsAPI::InputPrefix = TEXT("Inputs");
const TCHAR* UInterchangeShaderPortsAPI::InputSeparator = TEXT(":");
const TCHAR* UInterchangeShaderPortsAPI::OutputByIndex = TEXT("__ByIndex__");
const TCHAR* UInterchangeShaderPortsAPI::ParameterSuffix = TEXT("Parameter");

FString UInterchangeShaderPortsAPI::MakeInputConnectionKey(const FString& InputName)
{
	TStringBuilder<128> StringBuilder;
	StringBuilder.Append(InputPrefix);
	StringBuilder.Append(InputSeparator);
	StringBuilder.Append(InputName);
	StringBuilder.Append(InputSeparator);
	StringBuilder.Append(TEXT("Connect"));

	return StringBuilder.ToString();
}

FString UInterchangeShaderPortsAPI::MakeInputValueKey(const FString& InputName)
{
	TStringBuilder<128> StringBuilder;
	StringBuilder.Append(InputPrefix);
	StringBuilder.Append(InputSeparator);
	StringBuilder.Append(InputName);
	StringBuilder.Append(InputSeparator);
	StringBuilder.Append(TEXT("Value"));
	
	return StringBuilder.ToString();
}

FString UInterchangeShaderPortsAPI::MakeInputParameterKey(const FString& InputName)
{
	TStringBuilder<128> StringBuilder;
	StringBuilder.Append(InputPrefix);
	StringBuilder.Append(InputSeparator);
	StringBuilder.Append(InputName);
	StringBuilder.Append(InputSeparator);
	StringBuilder.Append(ParameterSuffix);

	return StringBuilder.ToString();
}

FString UInterchangeShaderPortsAPI::MakeInputName(const FString& InputKey)
{
	FString InputName;
	FString Discard;

	InputKey.Split(InputSeparator, &Discard, &InputName, ESearchCase::IgnoreCase, ESearchDir::FromStart);
	InputName.Split(InputSeparator, &InputName, &Discard, ESearchCase::IgnoreCase, ESearchDir::FromStart);

	return InputName;
}

bool UInterchangeShaderPortsAPI::IsAnInput(const FString& AttributeKey)
{
	TStringBuilder<128> PrefixStringBuilder;
	PrefixStringBuilder.Append(InputPrefix);
	PrefixStringBuilder.Append(InputSeparator);

	return AttributeKey.StartsWith(PrefixStringBuilder.ToString());
}

bool UInterchangeShaderPortsAPI::IsAParameter(const FString& AttributeKey)
{
	TStringBuilder<128> StringBuilder;
	StringBuilder.Append(InputSeparator);
	StringBuilder.Append(ParameterSuffix);

	return AttributeKey.EndsWith(StringBuilder.ToString());
}

bool UInterchangeShaderPortsAPI::HasInput(const UInterchangeBaseNode* InterchangeNode, const FName& InInputName)
{
	TArray<FString> InputNames;
	GatherInputs(InterchangeNode, InputNames);

	for (const FString& InputName : InputNames)
	{
		if (InputName.Equals(InInputName.ToString(), ESearchCase::IgnoreCase))
		{
			return true;
		}
	}

	return false;
}

bool UInterchangeShaderPortsAPI::HasParameter(const UInterchangeBaseNode* InterchangeNode, const FName& InInputName)
{
	FString ParameterKey = MakeInputParameterKey(InInputName.ToString());
	return InterchangeNode->HasAttribute(UE::Interchange::FAttributeKey(ParameterKey));
}

void UInterchangeShaderPortsAPI::GatherInputs(const UInterchangeBaseNode* InterchangeNode, TArray<FString>& OutInputNames)
{
	TArray< UE::Interchange::FAttributeKey > AttributeKeys;
	InterchangeNode->GetAttributeKeys(AttributeKeys);

	for (const UE::Interchange::FAttributeKey& AttributeKey : AttributeKeys)
	{
		if (IsAnInput(AttributeKey.ToString()))
		{
			OutInputNames.Add(MakeInputName(AttributeKey.ToString()));
		}
	}
}

bool UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(UInterchangeBaseNode* InterchangeNode, const FString& InputName, const FString& ExpressionUid)
{
	return InterchangeNode->AddStringAttribute(MakeInputConnectionKey(InputName), ExpressionUid);
}

bool UInterchangeShaderPortsAPI::DisconnectInput(UInterchangeBaseNode* InterchangeNode, const FString& InputName)
{
	return InterchangeNode->RemoveAttribute(MakeInputConnectionKey(InputName));
}

bool UInterchangeShaderPortsAPI::DisconnectInputFromOutputNode(UInterchangeBaseNode* InterchangeNode, const FString& InputName, FString& OutExpressionUid, FString& OutputName)
{
	if (GetInputConnection(InterchangeNode, InputName, OutExpressionUid, OutputName))
	{
		return DisconnectInput(InterchangeNode, InputName);
	}
	return false;
}

bool UInterchangeShaderPortsAPI::ConnectOuputToInputByName(UInterchangeBaseNode* InterchangeNode, const FString& InputName, const FString& ExpressionUid, const FString& OutputName)
{
	if (OutputName.IsEmpty())
	{
		return ConnectDefaultOuputToInput(InterchangeNode, InputName, ExpressionUid);
	}
	else
	{
		return InterchangeNode->AddStringAttribute(MakeInputConnectionKey(InputName), ExpressionUid + InputSeparator + OutputName);
	}
}

bool UInterchangeShaderPortsAPI::ConnectOuputToInputByIndex(UInterchangeBaseNode* InterchangeNode, const FString& InputName, const FString& ExpressionUid, int32 OutputIndex)
{
	TStringBuilder<128> StringBuilder;
	StringBuilder.Append(ExpressionUid);
	StringBuilder.Append(InputSeparator);
	StringBuilder.Append(OutputByIndex);
	StringBuilder.Append(InputSeparator);
	StringBuilder.Append(FString::FromInt(OutputIndex));

	return InterchangeNode->AddStringAttribute(MakeInputConnectionKey(InputName), StringBuilder.ToString());
}

UE::Interchange::EAttributeTypes UInterchangeShaderPortsAPI::GetInputType(const UInterchangeBaseNode* InterchangeNode, const FString& InputName, bool bIsAParameter /*= false*/)
{
	if (bIsAParameter)
	{
		return InterchangeNode->GetAttributeType(UE::Interchange::FAttributeKey(MakeInputParameterKey(InputName)));
	}
	else
	{
		return InterchangeNode->GetAttributeType(UE::Interchange::FAttributeKey(MakeInputValueKey(InputName)));
	}
}

bool UInterchangeShaderPortsAPI::GetInputConnection(const UInterchangeBaseNode* InterchangeNode, const FString& InputName, FString& OutExpressionUid, FString& OutputName)
{
	if (InterchangeNode->GetStringAttribute(MakeInputConnectionKey(InputName), OutExpressionUid))
	{
		OutExpressionUid.Split(InputSeparator, &OutExpressionUid, &OutputName);
		return true;
	}

	return false;
}

int32 UInterchangeShaderPortsAPI::GetOutputIndexFromName(const FString& OutputName)
{
	if (OutputName.StartsWith(OutputByIndex))
	{
		FString OutputIndex;
		FString Discard;

		OutputName.Split(InputSeparator, &Discard, &OutputIndex, ESearchCase::IgnoreCase, ESearchDir::FromStart);

		return FCString::Atoi(*OutputIndex);
	}

	return INDEX_NONE;
}

FString UInterchangeShaderNode::MakeNodeUid(const FStringView NodeName, const FStringView ParentNodeUid)
{
	FString ShaderNodeUid = FString(UInterchangeBaseNode::HierarchySeparator) + TEXT("Shaders") + FString(UInterchangeBaseNode::HierarchySeparator);

	if (ParentNodeUid.IsEmpty())
	{
		ShaderNodeUid += NodeName;
	}
	else
	{
		ShaderNodeUid += ParentNodeUid + FString(UInterchangeBaseNode::HierarchySeparator) + NodeName;
	}

	return ShaderNodeUid;
}

UInterchangeShaderNode* UInterchangeShaderNode::Create(UInterchangeBaseNodeContainer* NodeContainer, const FStringView NodeName, const FStringView ParentNodeUid)
{
	check(NodeContainer);

	const FString ShaderNodeUid = MakeNodeUid(NodeName, ParentNodeUid);

	UInterchangeShaderNode* ShaderNode = NewObject< UInterchangeShaderNode >(NodeContainer);
	NodeContainer->SetupNode(ShaderNode, ShaderNodeUid, FString(NodeName), EInterchangeNodeContainerType::TranslatedAsset, FString(ParentNodeUid));

	return ShaderNode;
}

FString UInterchangeShaderNode::GetTypeName() const
{
	const FString TypeName = TEXT("ShaderNode");
	return TypeName;
}

bool UInterchangeShaderNode::AddFloatInput(const FString& InputName, const float& AttributeValue, bool bIsAParameter /*= false*/)
{
	bool bResult;
	if (bIsAParameter)
	{
		bResult = AddFloatAttribute(UInterchangeShaderPortsAPI::MakeInputParameterKey(InputName), AttributeValue);
	}
	else
	{
		bResult = AddFloatAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(InputName), AttributeValue);
	}

	return bResult;
}

bool UInterchangeShaderNode::AddLinearColorInput(const FString& InputName, const FLinearColor& AttributeValue, bool bIsAParameter /*= false*/)
{
	bool bResult;
	if (bIsAParameter)
	{
		bResult = AddLinearColorAttribute(UInterchangeShaderPortsAPI::MakeInputParameterKey(InputName), AttributeValue);
	}
	else
	{
		bResult = AddLinearColorAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(InputName), AttributeValue);
	}

	return bResult;
}

bool UInterchangeShaderNode::AddStringInput(const FString& InputName, const FString& AttributeValue, bool bIsAParameter)
{
	bool bResult;
	if (bIsAParameter)
	{
		bResult = AddStringAttribute(UInterchangeShaderPortsAPI::MakeInputParameterKey(InputName), AttributeValue);
	}
	else
	{
		bResult = AddStringAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(InputName), AttributeValue);
	}

	return bResult;
}

bool UInterchangeShaderNode::GetCustomShaderType(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ShaderType, FString);
}

bool UInterchangeShaderNode::SetCustomShaderType(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ShaderType, FString);
}

FString UInterchangeShaderGraphNode::MakeNodeUid(const FStringView NodeName)
{
	return FString(UInterchangeBaseNode::HierarchySeparator) + TEXT("Shaders") + FString(UInterchangeBaseNode::HierarchySeparator) + NodeName;
}

UInterchangeShaderGraphNode* UInterchangeShaderGraphNode::Create(UInterchangeBaseNodeContainer* NodeContainer, const FStringView NodeName)
{
	check(NodeContainer);

	const FString ShaderGraphNodeUid = MakeNodeUid(NodeName);

	UInterchangeShaderGraphNode* ShaderGraphNode = NewObject< UInterchangeShaderGraphNode >(NodeContainer);
	NodeContainer->SetupNode(ShaderGraphNode, ShaderGraphNodeUid, FString(NodeName), EInterchangeNodeContainerType::TranslatedAsset);

	return ShaderGraphNode;
}

FString UInterchangeShaderGraphNode::GetTypeName() const
{
	const FString TypeName = TEXT("ShaderGraphNode");
	return TypeName;
}

bool UInterchangeShaderGraphNode::GetCustomTwoSided(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(TwoSided, bool);
}

bool UInterchangeShaderGraphNode::SetCustomTwoSided(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(TwoSided, bool);
}

bool UInterchangeShaderGraphNode::GetCustomTwoSidedTransmission(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(TwoSidedTransmission, bool);
}

bool UInterchangeShaderGraphNode::SetCustomTwoSidedTransmission(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(TwoSidedTransmission, bool);
}

bool UInterchangeShaderGraphNode::GetCustomOpacityMaskClipValue(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(OpacityMaskClipValue, float);
}

bool UInterchangeShaderGraphNode::SetCustomOpacityMaskClipValue(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(OpacityMaskClipValue, float);
}

bool UInterchangeShaderGraphNode::GetCustomIsAShaderFunction(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(IsAShaderFunction, bool);
}

bool UInterchangeShaderGraphNode::SetCustomIsAShaderFunction(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(IsAShaderFunction, bool);
}

bool UInterchangeShaderGraphNode::GetCustomScreenSpaceReflections(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ScreenSpaceReflections, bool);
}
bool UInterchangeShaderGraphNode::SetCustomScreenSpaceReflections(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ScreenSpaceReflections, bool);
}

bool UInterchangeShaderGraphNode::GetCustomBlendMode(int& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(BlendMode, int);
}

bool UInterchangeShaderGraphNode::SetCustomBlendMode(int AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(BlendMode, int);
}

bool UInterchangeShaderGraphNode::GetCustomDisplacementCenterMode(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DisplacementCenter, float);
}

bool UInterchangeShaderGraphNode::SetCustomDisplacementCenterMode(float AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DisplacementCenter, float);
}

bool UInterchangeShaderGraphNode::GetCustomDisplacementMagnitude(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DisplacementMagnitude, float);
}

bool UInterchangeShaderGraphNode::SetCustomDisplacementMagnitude(float AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DisplacementMagnitude, float);
}

FString UInterchangeFunctionCallShaderNode::GetTypeName() const
{
	const FString TypeName = TEXT("FunctionCallShaderNode");
	return TypeName;
}

bool UInterchangeFunctionCallShaderNode::GetCustomMaterialFunction(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(MaterialFunction, FString);
}

bool UInterchangeFunctionCallShaderNode::SetCustomMaterialFunction(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(MaterialFunction, FString);
}