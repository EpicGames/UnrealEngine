// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeSpecularProfileNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSpecularProfileNode)

FString UInterchangeSpecularProfileNode::MakeNodeUid(const FStringView& NodeName)
{
	return FString(UInterchangeBaseNode::HierarchySeparator) + TEXT("SpecularProfile") + FString(UInterchangeBaseNode::HierarchySeparator) + NodeName;
}

UInterchangeSpecularProfileNode* UInterchangeSpecularProfileNode::Create(UInterchangeBaseNodeContainer* NodeContainer, const FStringView& NodeName)
{
	check(NodeContainer);

	const FString SpecularProfileNodeUid = MakeNodeUid(NodeName);

	UInterchangeSpecularProfileNode* SpecularProfileNode = NewObject<UInterchangeSpecularProfileNode>(NodeContainer);
	NodeContainer->SetupNode(SpecularProfileNode, SpecularProfileNodeUid, FString{ NodeName }, EInterchangeNodeContainerType::TranslatedAsset);

	return SpecularProfileNode;
}

FString UInterchangeSpecularProfileNode::GetTypeName() const
{
	return TEXT("SpecularProfileNode");
}

bool UInterchangeSpecularProfileNode::SetCustomFormat(uint8 AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Format, uint8);
}

bool UInterchangeSpecularProfileNode::GetCustomFormat(uint8& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Format, uint8);
}

bool UInterchangeSpecularProfileNode::GetCustomTexture(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Texture, FString)
}

bool UInterchangeSpecularProfileNode::SetCustomTexture(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Texture, FString)
}
