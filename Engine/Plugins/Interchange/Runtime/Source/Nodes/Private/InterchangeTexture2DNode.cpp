// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeTexture2DNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeTexture2DNode)

FString UInterchangeTexture2DNode::MakeNodeUid(const FStringView NodeName)
{
	return UInterchangeTextureNode::MakeNodeUid(NodeName);
}

UInterchangeTexture2DNode* UInterchangeTexture2DNode::Create(UInterchangeBaseNodeContainer* NodeContainer, const FString& TextureNodeUid, const FString& TextureNodeName)
{
	UInterchangeTexture2DNode* TextureNode = NewObject< UInterchangeTexture2DNode >(NodeContainer);
	NodeContainer->SetupNode(TextureNode, TextureNodeUid, TextureNodeName, EInterchangeNodeContainerType::TranslatedAsset);

	return TextureNode;
}
