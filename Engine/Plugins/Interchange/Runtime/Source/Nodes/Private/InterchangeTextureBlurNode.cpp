// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeTextureBlurNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeTextureBlurNode)

FString UInterchangeTextureBlurNode::MakeNodeUid(const FStringView NodeName)
{
	return UInterchangeTextureNode::MakeNodeUid(NodeName);
}

UInterchangeTextureBlurNode* UInterchangeTextureBlurNode::Create(UInterchangeBaseNodeContainer* NodeContainer, const FStringView TextureNodeName)
{
	UInterchangeTextureBlurNode* TextureNode = NewObject<UInterchangeTextureBlurNode>(NodeContainer);
	const FString TextureNodeUid = UInterchangeTextureNode::MakeNodeUid(TextureNodeName);

	NodeContainer->SetupNode(TextureNode, TextureNodeUid, FString(TextureNodeName), EInterchangeNodeContainerType::TranslatedAsset);

	return TextureNode;
}
