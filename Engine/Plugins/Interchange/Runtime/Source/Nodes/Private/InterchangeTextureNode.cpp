// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeTextureNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeTextureNode)

bool UInterchangeTextureNode::ExtractNodeUidAndNameFromFilePath(const FStringView TextureFilePath, FString& OutNodeUid, FString& OutNodeName)
{
	if (TextureFilePath.IsEmpty())
	{
		return false;
	}

	FString NormalizeFilePath = FString(TextureFilePath);
	FPaths::NormalizeFilename(NormalizeFilePath);

	OutNodeName = FPaths::GetBaseFilename(NormalizeFilePath);
	const FString HashString = FMD5::HashAnsiString(*NormalizeFilePath).Left(6);
	OutNodeUid = MakeNodeUid(OutNodeName + "_" + HashString);

	return true;
}
