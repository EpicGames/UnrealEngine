// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeVariantSetNode.h"

// UInterchangeSceneVariantSetsNode

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeVariantSetNode)

UInterchangeSceneVariantSetsNode::UInterchangeSceneVariantSetsNode()
{
	CustomVariantSetUids.Initialize(Attributes, TEXT("VariantSetUids"));
}

int32 UInterchangeSceneVariantSetsNode::GetCustomVariantSetUidCount() const
{
	return CustomVariantSetUids.GetCount();
}

void UInterchangeSceneVariantSetsNode::GetCustomVariantSetUids(TArray<FString>& OutVariantSetUids) const
{
	CustomVariantSetUids.GetItems(OutVariantSetUids);
}

void UInterchangeSceneVariantSetsNode::GetCustomVariantSetUid(const int32 Index, FString& OutVariantSetUid) const
{
	CustomVariantSetUids.GetItem(Index, OutVariantSetUid);
}

bool UInterchangeSceneVariantSetsNode::AddCustomVariantSetUid(const FString& VariantSetUid)
{
	return CustomVariantSetUids.AddItem(VariantSetUid);
}

bool UInterchangeSceneVariantSetsNode::RemoveCustomVariantSetUid(const FString& VariantSetUid)
{
	return CustomVariantSetUids.RemoveItem(VariantSetUid);
}

// UInterchangeVariantSetNode

UInterchangeVariantSetNode::UInterchangeVariantSetNode()
{
	CustomDependencyUids.Initialize(Attributes, TEXT("DependencyUids"));
}

bool UInterchangeVariantSetNode::GetCustomDisplayText(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(DisplayText, FString);
}

bool UInterchangeVariantSetNode::SetCustomDisplayText(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DisplayText, FString);
}

bool UInterchangeVariantSetNode::GetCustomVariantsPayloadKey(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(VariantsPayload, FString);
}

bool UInterchangeVariantSetNode::SetCustomVariantsPayloadKey(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(VariantsPayload, FString);
}

int32 UInterchangeVariantSetNode::GetCustomDependencyUidCount() const
{
	return CustomDependencyUids.GetCount();
}

void UInterchangeVariantSetNode::GetCustomDependencyUids(TArray<FString>& OutDependencyUids) const
{
	CustomDependencyUids.GetItems(OutDependencyUids);
}

void UInterchangeVariantSetNode::GetCustomDependencyUid(const int32 Index, FString& OutDependencyUid) const
{
	CustomDependencyUids.GetItem(Index, OutDependencyUid);
}

bool UInterchangeVariantSetNode::AddCustomDependencyUid(const FString& DependencyUid)
{
	return CustomDependencyUids.AddItem(DependencyUid);
}

bool UInterchangeVariantSetNode::RemoveCustomDependencyUid(const FString& DependencyUid)
{
	return CustomDependencyUids.RemoveItem(DependencyUid);
}
