// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeVolumeNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeVolumeNode)

namespace UE::Interchange
{
	const FAttributeKey& FInterchangeVolumeNodeStaticData::GetCustomGridDependeciesBaseKey()
	{
		static FAttributeKey AttributeKey(TEXT("__GridDependencies__"));
		return AttributeKey;
	}

	const FAttributeKey& FInterchangeVolumeNodeStaticData::GetCustomFrameIndicesInAnimationBaseKey()
	{
		static FAttributeKey AttributeKey(TEXT("__FrameIndexInAnimation__"));
		return AttributeKey;
	}
}

UInterchangeVolumeNode::UInterchangeVolumeNode()
{
	GridDependencies.Initialize(Attributes, UE::Interchange::FInterchangeVolumeNodeStaticData::GetCustomGridDependeciesBaseKey().ToString());

	IndexInVolumeAnimation.Initialize(
		Attributes,
		UE::Interchange::FInterchangeVolumeNodeStaticData::GetCustomFrameIndicesInAnimationBaseKey().ToString()
	);
}

FString UInterchangeVolumeNode::GetTypeName() const
{
	return TEXT("Volume");
}

bool UInterchangeVolumeNode::GetCustomFileName(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(FileName, FString);
}

bool UInterchangeVolumeNode::SetCustomFileName(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(FileName, FString);
}

int32 UInterchangeVolumeNode::GetCustomGridDependeciesCount() const
{
	return GridDependencies.GetCount();
}

void UInterchangeVolumeNode::GetCustomGridDependecies(TArray<FString>& OutDependencies) const
{
	GridDependencies.GetItems(OutDependencies);
}

void UInterchangeVolumeNode::GetCustomGridDependency(const int32 Index, FString& OutDependency) const
{
	GridDependencies.GetItem(Index, OutDependency);
}

bool UInterchangeVolumeNode::AddCustomGridDependency(const FString& DependencyUid)
{
	return GridDependencies.AddItem(DependencyUid);
}

bool UInterchangeVolumeNode::RemoveCustomGridDependency(const FString& DependencyUid)
{
	return GridDependencies.RemoveItem(DependencyUid);
}

bool UInterchangeVolumeNode::GetCustomAnimationID(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AnimationID, FString);
}

bool UInterchangeVolumeNode::SetCustomAnimationID(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AnimationID, FString);
}

void UInterchangeVolumeNode::GetCustomFrameIndicesInAnimation(TArray<int32>& OutVolumeIndices) const
{
	IndexInVolumeAnimation.GetItems(OutVolumeIndices);
}

void UInterchangeVolumeNode::GetCustomFrameIndexInAnimation(int32 IndexIndex, int32& OutIndex) const
{
	IndexInVolumeAnimation.GetItem(IndexIndex, OutIndex);
}

bool UInterchangeVolumeNode::AddCustomFrameIndexInAnimation(int32 Index)
{
	return IndexInVolumeAnimation.AddItem(Index);
}

bool UInterchangeVolumeNode::RemoveCustomFrameIndexInAnimation(int32 FrameIndex)
{
	return IndexInVolumeAnimation.RemoveItem(FrameIndex);
}

FString UInterchangeVolumeGridNode::GetTypeName() const
{
	return TEXT("VolumeGrid");
}

bool UInterchangeVolumeGridNode::GetCustomElementType(EVolumeGridElementType& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ElementType, EVolumeGridElementType);
}

bool UInterchangeVolumeGridNode::SetCustomElementType(const EVolumeGridElementType& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ElementType, EVolumeGridElementType);
}

bool UInterchangeVolumeGridNode::GetCustomNumComponents(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(NumComponents, int32);
}

bool UInterchangeVolumeGridNode::SetCustomNumComponents(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(NumComponents, int32);
}

bool UInterchangeVolumeGridNode::GetCustomGridTransform(FTransform& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GridTransform, FTransform);
}

bool UInterchangeVolumeGridNode::SetCustomGridTransform(const FTransform& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(GridTransform, FTransform);
}

bool UInterchangeVolumeGridNode::GetCustomGridActiveAABBMin(FIntVector& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GridActiveAABBMin, FIntVector);
}

bool UInterchangeVolumeGridNode::SetCustomGridActiveAABBMin(const FIntVector& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(GridActiveAABBMin, FIntVector);
}

bool UInterchangeVolumeGridNode::GetCustomGridActiveAABBMax(FIntVector& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GridActiveAABBMin, FIntVector);
}

bool UInterchangeVolumeGridNode::SetCustomGridActiveAABBMax(const FIntVector& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(GridActiveAABBMax, FIntVector);
}

bool UInterchangeVolumeGridNode::GetCustomGridActiveDimensions(FIntVector& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(GridActiveDim, FIntVector);
}

bool UInterchangeVolumeGridNode::SetCustomGridActiveDimensions(const FIntVector& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(GridActiveDim, FIntVector);
}
