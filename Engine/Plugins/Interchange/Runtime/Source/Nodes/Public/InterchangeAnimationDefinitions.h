// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "Types/AttributeStorage.h"

#include "InterchangeAnimationDefinitions.generated.h"

UENUM()
enum class EInterchangePropertyTracks : int32
{
	/** Common Properties*/
	AffectDistanceFieldLighting,
	AffectDynamicIndirectLighting,
	AffectIndirectLightingWhileHidden,
	AutoActivate,
	BodyInstanceAngularDamping,
	BodyInstancebEnableGravity,
	BodyInstancebNotifyRigidBodyCollision,
	BodyInstancebSimulatePhysics,
	BodyInstancebUpdateKinematicFromSimulation,
	BodyInstancebUseCCD,
	BodyInstanceLinearDamping,
	BodyInstanceMassScale,
	BoundsScale,
	CastContactShadow,
	CastHiddenShadow,
	CastInsetShadow,
	CastShadow,
	CustomDepthStencilValue,
	CustomDepthStencilWriteMask,
	DefaultUpVector,
	DrawDebug,
	EmissiveLightSource,
	ExcludeFromLightAttachmentGroup,
	HiddenInGame, // This visibility property is set on the Component
	HiddenInSceneCapture,
	Holdout,
	LightAttachmentsAsGroup,
	Mobility,
	OnlyOwnerSee,
	OwnerNoSee,
	ReceivesDecals,
	RenderCustomDepth,
	RenderInDepthPass,
	RenderInMainPass,
	SingleSampleShadowFromStationaryLights,
	TranslucencySortDistanceOffset,
	VisibleInRayTracing,
	VisibleInReflections,
	VisibleInSceneCaptureOnly,
	ActorHiddenInGame, // This visibility property is set on the Actor

	/** Light Properties */
	LightAffectGlobalIllumination,
	LightAffectReflection,
	LightAffectTranslucentLighting,
	LightAtmosphereSunDiskColorScale,
	LightAtmosphereSunLight,
	LightAtmosphereSunLightIndex,
	LightAttenuationRadius,
	LightBarnDoorAngle,
	LightBarnDoorLength,
	LightBloomMaxBrightness,
	LightBloomScale,
	LightBloomThreshold,
	LightBloomTint,
	LightCascadeDistributionExponent,
	LightCascadeTransitionFraction,
	LightCastDeepShadow,
	LightCastShadows,
	LightCastVolumetricShadow,
	LightCloudAmbientOcclusionStrength,
	LightCloudScatteredLuminanceScale,
	LightCloudShadowOnAtmosphereStrength,
	LightCloudShadowOnSurfaceStrength,
	LightCloudShadowStrength,
	LightColor,
	LightDynamicShadowCascades,
	LightDynamicShadowDistanceMovableLight,
	LightDynamicShadowDistanceStationaryLight,
	LightEnableLightShaftBloom,
	LightEnableLightShaftOcclusion,
	LightFalloffExponent,
	LightForceCachedShadowsForMovablePrimitives,
	LightForwardShadingPriority,
	LightFunctionFadeDistance,
	LightFunctionScale,
	LightIESBrightnessScale,
	LightIndirectLightingIntensity,
	LightInnerConeAngle,
	LightIntensity,
	LightIntensityUnits,
	LightInverseExposureBlend,
	LightLowerHemisphereColor,
	LightmassSettingsLightSourceAngle,
	LightMinOcclusion,
	LightModulatedShadowColor,
	LightOcclusionDepthRange,
	LightOcclusionExponent,
	LightOcclusionMaskDarkness,
	LightOcclusionTint,
	LightOuterConeAngle,
	LightSamplesPerPixel,
	LightShadowAmount,
	LightShadowBias,
	LightShadowCascadeBiasDistribution,
	LightShadowDistanceFadeoutFraction,
	LightShadowSlopeBias,
	LightShadowSourceAngleFactor,
	LightShaftOverrideDirection,
	LightSoftSourceRadius,
	LightSourceAngle,
	LightSourceCubemapAngle,
	LightSourceHeight,
	LightSourceLength,
	LightSourceRadius,
	LightSourceSoftAngle,
	LightSourceWidth,
	LightSpecularScale,
	LightDiffuseScale,
	LightTemperature,
	LightTransmission,
	LightUseIESBrightness,
	LightUseInverseSquaredFalloff,
	LightUseTemperature,
	LightVolumetricScatteringIntensity,

	/** Camera Properties*/
	CameraAspectRatio,
	CameraAspectRatioAxisConstraint,
	CameraAutoCalculateOrthoPlanes,
	CameraAutoPlaneShift,
	CameraConstrainAspectRatio,
	CameraCurrentAperture,
	CameraCurrentFocalLength,
	CameraCustomNearClippingPlane,
	CameraFieldOfView,
	CameraFilmbackSensorAspectRatio,
	CameraFilmbackSensorHeight,
	CameraFilmbackSensorHorizontalOffset,
	CameraFilmbackSensorVerticalOffset,
	CameraFilmbackSensorWidth,
	CameraFocusSettingsFocusOffset,
	CameraFocusSettingsManualFocusDistance,
	CameraFocusSettingsTrackingFocusSettingsRelativeOffset,
	CameraOrthoFarClipPlane,
	CameraOrthoNearClipPlane,
	CameraOrthoWidth,
	CameraOverrideAspectRatioAxisConstraint,
	CameraPostProcessBlendWeight,
	CameraPostProcessSettingsAmbientCubemapIntensity,
	CameraPostProcessSettingsAmbientCubemapTint,
	CameraPostProcessSettingsAmbientOcclusionBias,
	CameraPostProcessSettingsAmbientOcclusionFadeDistance,
	CameraPostProcessSettingsAmbientOcclusionFadeRadius,
	CameraPostProcessSettingsAmbientOcclusionIntensity,
	CameraPostProcessSettingsAmbientOcclusionMipBlend,
	CameraPostProcessSettingsAmbientOcclusionMipScale,
	CameraPostProcessSettingsAmbientOcclusionMipThreshold,
	CameraPostProcessSettingsAmbientOcclusionPower,
	CameraPostProcessSettingsAmbientOcclusionQuality,
	CameraPostProcessSettingsAmbientOcclusionRadius,
	CameraPostProcessSettingsAmbientOcclusionStaticFraction,
	CameraPostProcessSettingsAmbientOcclusionTemporalBlendWeight,
	CameraPostProcessSettingsAutoExposureBias,
	CameraPostProcessSettingsAutoExposureHighPercent,
	CameraPostProcessSettingsAutoExposureLowPercent,
	CameraPostProcessSettingsAutoExposureMaxBrightness,
	CameraPostProcessSettingsAutoExposureMinBrightness,
	CameraPostProcessSettingsAutoExposureSpeedDown,
	CameraPostProcessSettingsAutoExposureSpeedUp,
	CameraPostProcessSettingsBloom1Size,
	CameraPostProcessSettingsBloom1Tint,
	CameraPostProcessSettingsBloom2Size,
	CameraPostProcessSettingsBloom2Tint,
	CameraPostProcessSettingsBloom3Size,
	CameraPostProcessSettingsBloom3Tint,
	CameraPostProcessSettingsBloom4Size,
	CameraPostProcessSettingsBloom4Tint,
	CameraPostProcessSettingsBloom5Size,
	CameraPostProcessSettingsBloom5Tint,
	CameraPostProcessSettingsBloom6Size,
	CameraPostProcessSettingsBloom6Tint,
	CameraPostProcessSettingsBloomConvolutionBufferScale,
	CameraPostProcessSettingsBloomConvolutionCenterUV,
	CameraPostProcessSettingsBloomConvolutionPreFilterMax,
	CameraPostProcessSettingsBloomConvolutionPreFilterMin,
	CameraPostProcessSettingsBloomConvolutionPreFilterMult,
	CameraPostProcessSettingsBloomConvolutionScatterDispersion,
	CameraPostProcessSettingsBloomConvolutionSize,
	CameraPostProcessSettingsBloomDirtMaskIntensity,
	CameraPostProcessSettingsBloomDirtMaskTint,
	CameraPostProcessSettingsBloomIntensity,
	CameraPostProcessSettingsBloomSizeScale,
	CameraPostProcessSettingsBloomThreshold,
	CameraPostProcessSettingsBlueCorrection,
	CameraPostProcessSettingsCameraISO,
	CameraPostProcessSettingsCameraShutterSpeed,
	CameraPostProcessSettingsChromaticAberrationStartOffset,
	CameraPostProcessSettingsColorContrast,
	CameraPostProcessSettingsColorContrastHighlights,
	CameraPostProcessSettingsColorContrastMidtones,
	CameraPostProcessSettingsColorContrastShadows,
	CameraPostProcessSettingsColorCorrectionHighlightsMax,
	CameraPostProcessSettingsColorCorrectionHighlightsMin,
	CameraPostProcessSettingsColorCorrectionShadowsMax,
	CameraPostProcessSettingsColorGain,
	CameraPostProcessSettingsColorGainHighlights,
	CameraPostProcessSettingsColorGainMidtones,
	CameraPostProcessSettingsColorGainShadows,
	CameraPostProcessSettingsColorGamma,
	CameraPostProcessSettingsColorGammaHighlights,
	CameraPostProcessSettingsColorGammaMidtones,
	CameraPostProcessSettingsColorGammaShadows,
	CameraPostProcessSettingsColorGradingIntensity,
	CameraPostProcessSettingsColorOffset,
	CameraPostProcessSettingsColorOffsetHighlights,
	CameraPostProcessSettingsColorOffsetMidtones,
	CameraPostProcessSettingsColorOffsetShadows,
	CameraPostProcessSettingsColorSaturation,
	CameraPostProcessSettingsColorSaturationHighlights,
	CameraPostProcessSettingsColorSaturationMidtones,
	CameraPostProcessSettingsColorSaturationShadows,
	CameraPostProcessSettingsDepthOfFieldBladeCount,
	CameraPostProcessSettingsDepthOfFieldDepthBlurAmount,
	CameraPostProcessSettingsDepthOfFieldDepthBlurRadius,
	CameraPostProcessSettingsDepthOfFieldFarBlurSize,
	CameraPostProcessSettingsDepthOfFieldFarTransitionRegion,
	CameraPostProcessSettingsDepthOfFieldFocalDistance,
	CameraPostProcessSettingsDepthOfFieldFocalRegion,
	CameraPostProcessSettingsDepthOfFieldFstop,
	CameraPostProcessSettingsDepthOfFieldMinFstop,
	CameraPostProcessSettingsDepthOfFieldNearBlurSize,
	CameraPostProcessSettingsDepthOfFieldNearTransitionRegion,
	CameraPostProcessSettingsDepthOfFieldOcclusion,
	CameraPostProcessSettingsDepthOfFieldScale,
	CameraPostProcessSettingsDepthOfFieldSkyFocusDistance,
	CameraPostProcessSettingsDepthOfFieldUseHairDepth,
	CameraPostProcessSettingsDepthOfFieldVignetteSize,
	CameraPostProcessSettingsDynamicGlobalIlluminationMethod,
	CameraPostProcessSettingsExpandGamut,
	CameraPostProcessSettingsFilmBlackClip,
	CameraPostProcessSettingsFilmGrainHighlightsMax,
	CameraPostProcessSettingsFilmGrainHighlightsMin,
	CameraPostProcessSettingsFilmGrainIntensity,
	CameraPostProcessSettingsFilmGrainIntensityHighlights,
	CameraPostProcessSettingsFilmGrainIntensityMidtones,
	CameraPostProcessSettingsFilmGrainIntensityShadows,
	CameraPostProcessSettingsFilmGrainShadowsMax,
	CameraPostProcessSettingsFilmGrainScale,
	CameraPostProcessSettingsFilmShoulder,
	CameraPostProcessSettingsFilmSlope,
	CameraPostProcessSettingsFilmToe,
	CameraPostProcessSettingsFilmWhiteClip,
	CameraPostProcessSettingsHistogramLogMax,
	CameraPostProcessSettingsHistogramLogMin,
	CameraPostProcessSettingsIndirectLightingColor,
	CameraPostProcessSettingsIndirectLightingIntensity,
	CameraPostProcessSettingsLensFlareBokehSize,
	CameraPostProcessSettingsLensFlareIntensity,
	CameraPostProcessSettingsLensFlareThreshold,
	CameraPostProcessSettingsLensFlareTint,
	CameraPostProcessSettingsLocalExposureBlurredLuminanceBlend,
	CameraPostProcessSettingsLocalExposureBlurredLuminanceKernelSizePercent,
	CameraPostProcessSettingsLocalExposureDetailStrength,
	CameraPostProcessSettingsLocalExposureHighlightContrastScale,
	CameraPostProcessSettingsLocalExposureHighlightThreshold,
	CameraPostProcessSettingsLocalExposureMiddleGreyBias,
	CameraPostProcessSettingsLocalExposureShadowContrastScale,
	CameraPostProcessSettingsLocalExposureShadowThreshold,
	CameraPostProcessSettingsLumenDiffuseColorBoost,
	CameraPostProcessSettingsLumenAmbientOcclusionIntensity,
	CameraPostProcessSettingsLumenFinalGatherLightingUpdateSpeed,
	CameraPostProcessSettingsLumenFinalGatherQuality,
	CameraPostProcessSettingsLumenFinalGatherScreenTraces,
	CameraPostProcessSettingsLumenFrontLayerTranslucencyReflections,
	CameraPostProcessSettingsLumenFullSkylightLeakingDistance,
	CameraPostProcessSettingsLumenMaxReflectionBounces,
	CameraPostProcessSettingsLumenMaxRefractionBounces,
	CameraPostProcessSettingsLumenMaxRoughnessToTraceReflections,
	CameraPostProcessSettingsLumenMaxTraceDistance,
	CameraPostProcessSettingsLumenRayLightingMode,
	CameraPostProcessSettingsLumenReflectionQuality,
	CameraPostProcessSettingsLumenReflectionsScreenTraces,
	CameraPostProcessSettingsLumenSceneDetail,
	CameraPostProcessSettingsLumenSceneLightingQuality,
	CameraPostProcessSettingsLumenSceneLightingUpdateSpeed,
	CameraPostProcessSettingsLumenSceneViewDistance,
	CameraPostProcessSettingsLumenSkylightLeaking,
	CameraPostProcessSettingsLumenSkylightLeakingTint,
	CameraPostProcessSettingsMotionBlurAmount,
	CameraPostProcessSettingsMotionBlurMax,
	CameraPostProcessSettingsMotionBlurPerObjectSize,
	CameraPostProcessSettingsPathTracingMaxBounces,
	CameraPostProcessSettingsPathTracingMaxPathIntensity,
	CameraPostProcessSettingsRayTracingAO,
	CameraPostProcessSettingsRayTracingAOIntensity,
	CameraPostProcessSettingsRayTracingAORadius,
	CameraPostProcessSettingsRayTracingAOSamplesPerPixel,
	CameraPostProcessSettingsRayTracingTranslucencyMaxRoughness,
	CameraPostProcessSettingsRayTracingTranslucencyRefraction,
	CameraPostProcessSettingsRayTracingTranslucencyRefractionRays,
	CameraPostProcessSettingsRayTracingTranslucencySamplesPerPixel,
	CameraPostProcessSettingsRayTracingTranslucencyShadows,
	CameraPostProcessSettingsReflectionMethod,
	CameraPostProcessSettingsSceneColorTint,
	CameraPostProcessSettingsSceneFringeIntensity,
	CameraPostProcessSettingsScreenSpaceReflectionIntensity,
	CameraPostProcessSettingsScreenSpaceReflectionMaxRoughness,
	CameraPostProcessSettingsScreenSpaceReflectionQuality,
	CameraPostProcessSettingsSharpen,
	CameraPostProcessSettingsTemperatureType,
	CameraPostProcessSettingsToneCurveAmount,
	CameraPostProcessSettingsTranslucencyType,
	CameraPostProcessSettingsVignetteIntensity,
	CameraPostProcessSettingsWhiteTemp,
	CameraPostProcessSettingsWhiteTint,
	CameraProjectionMode,
	CameraUpdateOrthoPlanes,
	CameraUseCameraHeightAsViewTarget,
	CameraUseFieldOfViewForLOD,

	/** Common Mesh Properties */
	MeshOverlayMaterialMaxDrawDistance,

	/** Skinned Mesh Properties */
	SkinnedMeshCapsuleIndirectShadowMinVisibility,
	SkinnedMeshCastCapsuleDirectShadow,
	SkinnedMeshCastCapsuleIndirectShadow,
	SkinnedMeshRenderStatic,
	SkinnedMeshVisibilityBasedAnimTickOption,

	/** Skeletal Mesh Properties */
	SkeletalMesh,
	SkeletalMeshAllowClothActors,
	SkeletalMeshAnimationMode,
	SkeletalMeshClothBlendWeight,
	SkeletalMeshClothMaxDistanceScale,

	/** Static Mesh Properties */
	StaticMeshDistanceFieldSelfShadowBias,
	StaticMeshEvaluateWorldPositionOffset,
	StaticMeshEvaluateWorldPositionOffsetInRayTracing,
	StaticMeshForcedLodModel,
	StaticMeshReverseCulling,
	StaticMeshWorldPositionOffsetDisableDistance,
	StaticMesh,

	/** Heterogeneous Volume Component Properties */
	HeterogeneousVolumeFrame,

	None = -1 UMETA(hidden),
	Visibility UE_DEPRECATED(5.6, "Renamed to 'ActorHiddenInGame'") = ActorHiddenInGame UMETA(hidden)
};

