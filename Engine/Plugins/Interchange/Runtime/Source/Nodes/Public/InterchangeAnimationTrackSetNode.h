// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeBaseNode.h"
#include "InterchangeAnimationDefinitions.h"

#include "InterchangeAnimationTrackSetNode.generated.h"

#define UE_API INTERCHANGENODES_API

UENUM(BlueprintType)
enum class EInterchangeAnimationPayLoadType : uint8
{
	NONE = 0,
	CURVE,
	MORPHTARGETCURVE, // Handles/generates the same properties as the CURVE variation, but the way it is acquired might be different (depending on the Translator).
	STEPCURVE,
	BAKED,
	MORPHTARGETCURVEWEIGHTINSTANCE,	//Handled within UInterchangeAnimSequenceFactory, contrary to the others which are handled in the Translators
									// Purpose of this type is to generate a 1 frame long Animation with the Instantiated Morph Target Curve Weights.
									//		This is needed for the special use cases where the imported 3d file format has a concept of MorphTargetWeight usages on StaticMeshes.
									//		UE5 does not have support for this particular concept, and the workaround is to create a 1 frame long animation with the desired MorphTargetWeight settings.
									//		This is also needed for the use cases where the import is a Level import and the 3d file format is instantiating a StaticMesh with a particular MorphTargetWeight settings.
									//			This is done through the AnimSequnce being used on the SkeletalMeshActor's instance.
									//		Related Ticket: UE-186102
	GEOMETRY_CACHE_TRANSFORM // Array of time-sampled local transforms used to build a geometry cache
};


USTRUCT(BlueprintType)
struct FInterchangeAnimationPayLoadKey
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | Node")
	FString UniqueId = "";

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | Node")
	EInterchangeAnimationPayLoadType Type = EInterchangeAnimationPayLoadType::NONE;

	FInterchangeAnimationPayLoadKey() {}

	FInterchangeAnimationPayLoadKey(const FString& InUniqueId, const EInterchangeAnimationPayLoadType& InType)
		: UniqueId(InUniqueId)
		, Type(InType)
	{
	}

	bool operator==(const FInterchangeAnimationPayLoadKey& Other) const
	{
		return UniqueId.Equals(Other.UniqueId) && Type == Other.Type;
	}

	friend uint32 GetTypeHash(const FInterchangeAnimationPayLoadKey& InterchangeAnimationPayLoadKey)
	{
		return GetTypeHash(InterchangeAnimationPayLoadKey.UniqueId + FString::FromInt(static_cast<int32>(InterchangeAnimationPayLoadKey.Type)));
	}
};

//Interchange namespace
namespace UE
{
	namespace Interchange
	{

		struct  FAnimationStaticData : public FBaseNodeStaticData
		{
			static UE_API const FAttributeKey& AnimationPayLoadUidKey();
			static UE_API const FAttributeKey& AnimationPayLoadTypeKey();
			static UE_API const FAttributeKey& MorphTargetAnimationPayLoadUidKey();
			static UE_API const FAttributeKey& MorphTargetAnimationPayLoadTypeKey();
		};

		namespace Animation
		{
			UE_API FFrameRate ConvertSampleRateToFrameRate(double SampleRate);
		}
	}
}

/**
 * Enumeration specifying which properties of a camera, light or scene node can be animated besides transform.
 */
UENUM(BlueprintType)
enum class EInterchangeAnimatedProperty : uint8
{
	None UMETA(DisplayName = "No property.", ToolTip = "The associated animation track will be ignored."),
	Visibility UMETA(DisplayName = "Visibility property.", ToolTip = "The associated animation track is applied to the visibility property of the actor."),
	MAX,
};
/**
 * Enumeration specifying how to handle the state of the animated property at the end of an animation track.
 */
enum class EInterchangeAimationCompletionMode : uint8
{
	/** Keep the animated property at the state set at the end of the animation track. */
	KeepState,
	/** Restore the animated property to its state before the start of the animation track. */
	RestoreState,
	/** Restore the animated property to the state set in the project for that property. */
	ProjectDefault,
};

/**
 * Class to represent a set of animation track nodes that share the same frame rate.
 */
UCLASS(BlueprintType, MinimalAPI)
class UInterchangeAnimationTrackSetNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

	UE_API UInterchangeAnimationTrackSetNode();

public:
	static FStringView StaticAssetTypeName()
	{
		return TEXT("AnimationTrackSet");
	}

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("AnimationTrackSetNode");
		return TypeName;
	}

	/**
	 * Retrieve the number of track dependencies for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrackSet")
	UE_API int32 GetCustomAnimationTrackUidCount() const;

	/**
	 * Retrieve the track dependency for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrackSet")
	UE_API void GetCustomAnimationTrackUids(TArray<FString>& OutAnimationTrackUids) const;

	/**
	 * Retrieve one track dependency for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrackSet")
	UE_API void GetCustomAnimationTrackUid(const int32 Index, FString& OutAnimationTrackUid) const;

	/**
	 * Add one track dependency to this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrackSet")
	UE_API bool AddCustomAnimationTrackUid(const FString& AnimationTrackUid);

	/**
	 * Remove one track dependency from this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrackSet")
	UE_API bool RemoveCustomAnimationTrackUid(const FString& AnimationTrackUid);

	/**
	 * Set the frame rate for the animations in the level sequence.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrackSet")
	UE_API bool SetCustomFrameRate(const float& AttributeValue);

	/**
	 * Get the frame rate for the animations in the level sequence.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrackSet")
	UE_API bool GetCustomFrameRate(float& AttributeValue) const;

private:
	const UE::Interchange::FAttributeKey Macro_CustomFrameRateKey = UE::Interchange::FAttributeKey(TEXT("FrameRate"));

	UE::Interchange::TArrayAttributeHelper<FString> CustomAnimationTrackUids;
};

/**
 * Abstract class providing the minimal services required for an animation track node.
 */
UCLASS(Abstract, MinimalAPI)
class UInterchangeAnimationTrackBaseNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:
	static FStringView StaticAssetTypeName()
	{
		return TEXT("AnimationTrackBaseNode");
	}

	/**
	* Return the node type name of the class. This is used when reporting errors.
	*/
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("AnimationTrackBaseNode");
		return TypeName;
	}

	/**
	 * Set how the actor's animated property should behave once its animation completes.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrack")
	UE_API bool SetCustomCompletionMode(const int32& AttributeValue);

	/**
	 * Get how the actor's animated property behaves once this animation is complete.
	 * The output value will be clamped to the range of values defined in EInterchangeAimationCompletionMode.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrack")
	UE_API bool GetCustomCompletionMode(int32& AttributeValue) const;

private:
	const UE::Interchange::FAttributeKey Macro_CustomCompletionModeKey = UE::Interchange::FAttributeKey(TEXT("CompletionMode"));
};

/**
 * Class to represent an animation that instances another animation track set node.
 */
UCLASS(BlueprintType, MinimalAPI)
class UInterchangeAnimationTrackSetInstanceNode : public UInterchangeAnimationTrackBaseNode
{
	GENERATED_BODY()

public:
	static FStringView StaticAssetTypeName()
	{
		return TEXT("AnimationTrackSetInstanceNode");
	}

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("AnimationTrackSetInstanceNode");
		return TypeName;
	}

	/**
	 * Set the time scale used for the level sequence instance.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrackSetInstance")
	UE_API bool SetCustomTimeScale(const float& AttributeValue);

	/**
	 * Get the time scale used for the level sequence instance.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrackSetInstance")
	UE_API bool GetCustomTimeScale(float& AttributeValue) const;

	/**
	 * Set the level sequence instance duration in number of frames.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrackSetInstance")
	UE_API bool SetCustomDuration(const int32& AttributeValue);

	/**
	 * Get the level sequence instance duration in number of frames.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrackSetInstance")
	UE_API bool GetCustomDuration(int32& AttributeValue) const;

	/**
	 * Set the frame where the level sequence instance starts.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrackSetInstance")
	UE_API bool SetCustomStartFrame(const int32& AttributeValue);

	/**
	 * Get the frame where the level sequence instance starts.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrackSetInstance")
	UE_API bool GetCustomStartFrame(int32& AttributeValue) const;

	/**
	 * Set the unique id of the level sequence this instance references.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrackSetInstance")
	UE_API bool SetCustomTrackSetDependencyUid(const FString& AttributeValue);

	/**
	 * Get the unique id of the level sequence this instance references.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrackSetInstance")
	UE_API bool GetCustomTrackSetDependencyUid(FString& AttributeValue) const;

private:
	const UE::Interchange::FAttributeKey Macro_CustomStartFrameKey = UE::Interchange::FAttributeKey(TEXT("StartFrame"));
	const UE::Interchange::FAttributeKey Macro_CustomDurationKey = UE::Interchange::FAttributeKey(TEXT("Duration"));
	const UE::Interchange::FAttributeKey Macro_CustomTimeScaleKey = UE::Interchange::FAttributeKey(TEXT("TimeScale"));
	const UE::Interchange::FAttributeKey Macro_CustomTrackSetDependencyUidKey = UE::Interchange::FAttributeKey(TEXT("SequenceDependencyUid"));
};

/**
 * Class to represent an animation on the property of a camera, light, or scene node
 * The list of supported properties is enumerated in EInterchangeAnimatedProperty.
 */
UCLASS(BlueprintType, MinimalAPI)
class UInterchangeAnimationTrackNode : public UInterchangeAnimationTrackBaseNode
{
	GENERATED_BODY()

public:
	static FStringView StaticAssetTypeName()
	{
		return TEXT("AnimationTrack");
	}

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("AnimationTrackNode");
		return TypeName;
	}

	/**
	 * Set the actor dependency to this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrack")
	UE_API bool SetCustomActorDependencyUid(const FString& DependencyUid);

	/**
	 * Get the actor dependency to this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrack")
	UE_API bool GetCustomActorDependencyUid(FString& DependencyUid) const;

	UE_DEPRECATED(5.5, "SetCustomPropertyTrack using an FName has been deprecated, please use the function with the enum instead.")
	UE_API bool SetCustomPropertyTrack(const FName& PropertyTrack);

	UE_DEPRECATED(5.5, "SetCustomPropertyTrack using an FName has been deprecated, please use the function with the enum instead.")
	UE_API bool GetCustomPropertyTrack(FName& PropertyTrack) const;

	/**
	 * Set the property animated by this track.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrack")
	UE_API bool SetCustomPropertyTrack(EInterchangePropertyTracks PropertyTrack);

	/**
	 * Get the property animated by this track.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrack")
	UE_API bool GetCustomPropertyTrack(EInterchangePropertyTracks& PropertyTrack) const;

	/**
	 * Set the payload key needed to retrieve the animation for this track.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrack")
	UE_API bool SetCustomAnimationPayloadKey(const FString& InUniqueId, const EInterchangeAnimationPayLoadType& InType);

	/**
	 * Get the payload key needed to retrieve the animation for this track.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrack")
	UE_API bool GetCustomAnimationPayloadKey(FInterchangeAnimationPayLoadKey& AnimationPayLoadKey) const;

	/**
	 * Set the number of frames for the animation of this track.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrack")
	UE_API bool SetCustomFrameCount(const int32& AttributeValue);

	/**
	 * Get the number of frames for the animation of this track.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | AnimationTrack")
	UE_API bool GetCustomFrameCount(int32& AttributeValue) const;

private:
	const UE::Interchange::FAttributeKey Macro_CustomActorDependencyKey = UE::Interchange::FAttributeKey(TEXT("ActorDependency"));
	const UE::Interchange::FAttributeKey Macro_CustomAnimationPayloadUidKey = UE::Interchange::FAttributeKey(TEXT("AnimationPayloadUid"));
	const UE::Interchange::FAttributeKey Macro_CustomAnimationPayloadTypeKey = UE::Interchange::FAttributeKey(TEXT("AnimationPayloadType"));
	const UE::Interchange::FAttributeKey Macro_CustomFrameCountKey = UE::Interchange::FAttributeKey(TEXT("FrameCount"));
	const UE::Interchange::FAttributeKey Macro_CustomTargetedPropertyKey = UE::Interchange::FAttributeKey(TEXT("TargetedProperty"));
	const UE::Interchange::FAttributeKey Macro_CustomPropertyTrackKey = UE::Interchange::FAttributeKey(TEXT("PropertyTrack"));
};

/**
 * Class to represent an animation on the transform of a camera, light, or scene node.
 */
UCLASS(BlueprintType, MinimalAPI)
class UInterchangeTransformAnimationTrackNode : public UInterchangeAnimationTrackNode
{
	GENERATED_BODY()

public:
	static FStringView StaticAssetTypeName()
	{
		return TEXT("TransformAnimationTrack");
	}

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("TransformAnimationTrackNode");
		return TypeName;
	}

	/**
	 * Set which channels of this animation should be used. This is a bitwise mask.
	 * Bits are interpreted as follow:
	 *    None          = 0x000,
	 *    TranslationX  = 0x001,
	 *    TranslationY  = 0x002,
	 *    TranslationZ  = 0x004,
	 *    Translation   = TranslationX | TranslationY | TranslationZ,
	 *    RotationX     = 0x008,
	 *    RotationY     = 0x010,
	 *    RotationZ     = 0x020,
	 *    Rotation      = RotationX | RotationY | RotationZ,
	 *    ScaleX        = 0x040,
	 *    ScaleY        = 0x080,
	 *    ScaleZ        = 0x100,
	 *    Scale         = ScaleX | ScaleY | ScaleZ,
	 *    AllTransform  = Translation | Rotation | Scale,
	 *    Weight        = 0x200,
	 *    All           = Translation | Rotation | Scale | Weight,
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | TransformAnimationTrack")
	UE_API bool SetCustomUsedChannels(const int32& AttributeValue);

	/**
	 * Get which channels of this animation should be used. This is a bitmask.
	 * See SetCustomUsedChannels for description of bitmask
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | TransformAnimationTrack")
	UE_API bool GetCustomUsedChannels(int32& AttributeValue) const;

private:
	const UE::Interchange::FAttributeKey Macro_CustomUsedChannelsKey = UE::Interchange::FAttributeKey(TEXT("UsedChannels"));
};

/*
* Class to hold onto the relationships between a set of animation tracks and the bones, morph targets of a skeleton.
*/
UCLASS(BlueprintType, MinimalAPI)
class UInterchangeSkeletalAnimationTrackNode : public UInterchangeAnimationTrackBaseNode
{
	GENERATED_BODY()

public:
	UE_API UInterchangeSkeletalAnimationTrackNode();

	/**
	 * Override Serialize() to restore SlotMaterialDependencies on load.
	 */
	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);

		if (Ar.IsLoading() && bIsInitialized)
		{
			SceneNodeAnimationPayloadKeyUidMap.RebuildCache();
			SceneNodeAnimationPayloadKeyTypeMap.RebuildCache();

			MorphTargetPayloadKeyUidMap.RebuildCache();
			MorphTargetPayloadKeyTypeMap.RebuildCache();
		}
	}

	static FStringView StaticAssetTypeName()
	{
		return TEXT("SkeletalAnimationTrack");
	}

	/**
	* Return the node type name of the class. This is used when reporting errors.
	*/
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("SkeletalAnimationTrack");
		return TypeName;
	}


public:
	/** Get the unique ID of the skeleton factory node. Return false if the attribute is not set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API bool GetCustomSkeletonNodeUid(FString& AttributeValue) const;

	/** Set the unique ID of the skeleton factory node. Return false if the attribute could not be set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API bool SetCustomSkeletonNodeUid(const FString& AttributeValue);

	/**
	 * Set the animation sample rate. Return false if the attribute could not be set.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API bool SetCustomAnimationSampleRate(const double& SampleRate);

	/**
	 * Get the animation sample rate. Return false if the attribute is not set.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API bool GetCustomAnimationSampleRate(double& SampleRate) const;

	/**
	 * Set the animation start time. Return false if the attribute could not be set.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API bool SetCustomAnimationStartTime(const double& StartTime);

	/**
	 * Get the animation start time. Return false if the attribute is not set.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API bool GetCustomAnimationStartTime(double& StartTime) const;
	
	/**
	 * Set the animation stop time. Return false if the attribute could not be set.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API bool SetCustomAnimationStopTime(const double& StopTime);

	/**
	 * Get the animation stop time. Return false if the attribute is not set.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API bool GetCustomAnimationStopTime(double& StopTime) const;

	/**
	 * Set the source timeline animation start time. Return false if the attribute could not be set.
	 * Note: This represent the DCC setup of the time for the animation.
	 * Optional: if not set CustomAnimationStartTime will be used
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API bool SetCustomSourceTimelineAnimationStartTime(const double& StartTime);

	/**
	 * Get the source timeline animation start time. Return false if the attribute is not set.
	 * Note: This represent the DCC setup of the time for the animation.
	 * Optional: if not set CustomAnimationStartTime will be used
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API bool GetCustomSourceTimelineAnimationStartTime(double& StartTime) const;
	
	/**
	 * Set the source timeline animation stop time. Return false if the attribute could not be set.
	 * Note: This represent the DCC setup of the time for the animation.
	 * Optional: if not set CustomAnimationStopTime will be used
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API bool SetCustomSourceTimelineAnimationStopTime(const double& StopTime);

	/**
	 * Get the source timeline animation stop time. Return false if the attribute is not set.
	 * Note: This represent the DCC setup of the time for the animation.
	 * Optional: if not set CustomAnimationStopTime will be used
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API bool GetCustomSourceTimelineAnimationStopTime(double& StopTime) const;


	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API void GetSceneNodeAnimationPayloadKeys(TMap<FString, FString>& OutSceneNodeAnimationPayloadKeyUids, TMap<FString, uint8>& OutSceneNodeAnimationPayloadKeyTypes) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API bool SetAnimationPayloadKeyForSceneNodeUid(const FString& SceneNodeUid, const FString& InUniqueId, const EInterchangeAnimationPayLoadType& InType);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API void GetMorphTargetNodeAnimationPayloadKeys(TMap<FString, FString>& OutMorphTargetNodeAnimationPayloadKeyUids, TMap<FString, uint8>& OutMorphTargetNodeAnimationPayloadKeyTypes) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API bool SetAnimationPayloadKeyForMorphTargetNodeUid(const FString& MorphTargetNodeUid, const FString& InUniqueId, const EInterchangeAnimationPayLoadType& InType);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalAnimationTrack")
	UE_API bool IsNodeAnimatedWithBakedCurve(const FString& SceneNodeUid) const;

private:
	const UE::Interchange::FAttributeKey Macro_CustomSkeletonNodeUidKey = UE::Interchange::FAttributeKey(TEXT("SkeletonNodeUid"));
	const UE::Interchange::FAttributeKey Macro_CustomAnimationSampleRateKey = UE::Interchange::FAttributeKey(TEXT("AnimationSampleRate"));
	const UE::Interchange::FAttributeKey Macro_CustomAnimationStartTimeKey = UE::Interchange::FAttributeKey(TEXT("AnimationStartTime"));
	const UE::Interchange::FAttributeKey Macro_CustomAnimationStopTimeKey = UE::Interchange::FAttributeKey(TEXT("AnimationStopTime"));
	const UE::Interchange::FAttributeKey Macro_CustomSourceTimelineAnimationStartTimeKey = UE::Interchange::FAttributeKey(TEXT("SourceTimelineAnimationStartTime"));
	const UE::Interchange::FAttributeKey Macro_CustomSourceTimelineAnimationStopTimeKey = UE::Interchange::FAttributeKey(TEXT("SourceTimelineAnimationStopTime"));
	
	UE::Interchange::TMapAttributeHelper<FString, FString>	SceneNodeAnimationPayloadKeyUidMap;
	UE::Interchange::TMapAttributeHelper<FString, uint8>	SceneNodeAnimationPayloadKeyTypeMap;

	UE::Interchange::TMapAttributeHelper<FString, FString>	MorphTargetPayloadKeyUidMap;
	UE::Interchange::TMapAttributeHelper<FString, uint8>	MorphTargetPayloadKeyTypeMap;
};

#undef UE_API
