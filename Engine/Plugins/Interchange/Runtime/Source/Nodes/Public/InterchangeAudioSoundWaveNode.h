// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeAudioSoundWaveNode.generated.h"

#define UE_API INTERCHANGENODES_API

class UInterchangeBaseNodeContainer;

namespace UE::Interchange
{
	struct FSoundWaveNodeStaticData : FBaseNodeStaticData
	{
		static UE_API const FAttributeKey& PayloadSourceFileKey();
	};
}

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeAudioSoundWaveNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:
	static UE_API UInterchangeAudioSoundWaveNode* Create(UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeName);

	virtual FString GetTypeName() const override
	{
		return TEXT("SoundWaveNode");
	}

	static FString MakeNodeUid(const FStringView NodeName)
	{
		return TEXT("SoundWaves") + FString(HierarchySeparator) + NodeName;
	}

	UE_API const TOptional<FString> GetPayloadKey() const;

	UE_API void SetPayloadKey(const FString& PayloadKey);
	
};

#undef UE_API
