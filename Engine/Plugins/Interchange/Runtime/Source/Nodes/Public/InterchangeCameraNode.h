// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeCameraNode.generated.h"

// Mirrors EAutoExposureMethod from Scene.h
UENUM(BlueprintType)
enum class EInterchangeAutoExposureMethod : uint8
{
	/** requires compute shader to construct 64 bin histogram */
	Histogram  UMETA(DisplayName = "Auto Exposure Histogram"),

	/** faster method that computes single value by downsampling */
	Basic      UMETA(DisplayName = "Auto Exposure Basic"),

	/** Uses camera settings. */
	Manual   UMETA(DisplayName = "Manual"),

	MAX,
};

UCLASS(BlueprintType, MinimalAPI)
class UInterchangeBaseCameraNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomNearClipPlane(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(NearClipPlane, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomNearClipPlane(const float& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(NearClipPlane, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomAutoExposureMethod(EInterchangeAutoExposureMethod& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AutoExposureMethod, EInterchangeAutoExposureMethod);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomAutoExposureMethod(const EInterchangeAutoExposureMethod& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AutoExposureMethod, EInterchangeAutoExposureMethod);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomExposureCompensation(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(ExposureCompensation, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomExposureCompensation(const float& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ExposureCompensation, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomExposureFstop(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(ExposureFstop, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomExposureFstop(const float& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ExposureFstop, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomExposureISO(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(ExposureISO, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomExposureISO(const float& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ExposureISO, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomExposureShutterSpeed(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(ExposureShutterSpeed, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomExposureShutterSpeed(const float& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ExposureShutterSpeed, float);
	}

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(NearClipPlane);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AutoExposureMethod);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ExposureCompensation);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ExposureFstop);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ExposureISO);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ExposureShutterSpeed);
};

UCLASS(BlueprintType, MinimalAPI)
class UInterchangePhysicalCameraNode : public UInterchangeBaseCameraNode
{
	GENERATED_BODY()

public:
	static FStringView StaticAssetTypeName()
	{
		return TEXT("PhysicalCamera");
	}

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("PhysicalCameraNode");
		return TypeName;
	}

public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomFocalLength(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(FocalLength, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomFocalLength(const float& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(FocalLength, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomFocusDistance(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(FocusDistance, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomFocusDistance(const float& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(FocusDistance, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomSensorWidth(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(SensorWidth, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomSensorWidth(const float& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SensorWidth, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomSensorHeight(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(SensorHeight, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomSensorHeight(const float& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SensorHeight, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomSensorHorizontalOffset(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(SensorHorizontalOffset, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomSensorHorizontalOffset(const float& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SensorHorizontalOffset, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomSensorVerticalOffset(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(SensorVerticalOffset, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomSensorVerticalOffset(const float& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SensorVerticalOffset, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomEnableDepthOfField(bool& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(EnableDepthOfField, bool);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomEnableDepthOfField(const bool& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(EnableDepthOfField, bool);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomDepthOfFieldFstop(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(DepthOfFieldFstop, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomDepthOfFieldFstop(const float& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DepthOfFieldFstop, float);
	}

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(FocalLength);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(FocusDistance);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SensorWidth);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SensorHeight);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SensorHorizontalOffset);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SensorVerticalOffset);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(EnableDepthOfField);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(DepthOfFieldFstop);
};

UENUM(BlueprintType)
enum class EInterchangeCameraProjectionType : uint8
{
	Perspective,
	Orthographic
};

// Primarily used for Ortho Camera
UCLASS(BlueprintType, MinimalAPI)
class UInterchangeStandardCameraNode : public UInterchangeBaseCameraNode
{
	GENERATED_BODY()

public:
	static FStringView StaticAssetTypeName()
	{
		return TEXT("StandardCamera");
	}

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("StandardCameraNode");
		return TypeName;
	}

public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomProjectionMode(EInterchangeCameraProjectionType& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(ProjectionMode, EInterchangeCameraProjectionType);
	}
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomProjectionMode(const EInterchangeCameraProjectionType& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ProjectionMode, EInterchangeCameraProjectionType);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomWidth(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(Width, float);
	}
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomWidth(const float& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Width, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomFarClipPlane(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(FarClipPlane, float);
	}
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomFarClipPlane(const float& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(FarClipPlane, float);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomAspectRatio(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(AspectRatio, float);
	}
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomAspectRatio(const float& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AspectRatio, float);
	}

	//Field of View in Degrees.
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool GetCustomFieldOfView(float& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(FieldOfView, float);
	}
	//Field of View in Degrees.
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Camera")
	bool SetCustomFieldOfView(const float& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(FieldOfView, float);
	}

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ProjectionMode);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Width);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(FarClipPlane);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AspectRatio);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(FieldOfView);
};