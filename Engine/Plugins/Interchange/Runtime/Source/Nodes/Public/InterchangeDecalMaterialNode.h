// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeShaderGraphNode.h"

#include "InterchangeDecalMaterialNode.generated.h"

class IDatasmithBaseMaterialElement;
class UInterchangeBaseNodeContainer;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeDecalMaterialNode : public UInterchangeShaderNode
{
	GENERATED_BODY()

public:

	bool GetCustomDiffuseTexturePath(FString& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(DiffuseTexturePath, FString);
	}

	bool SetCustomDiffuseTexturePath(FString AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DiffuseTexturePath, FString);
	}

	bool GetCustomNormalTexturePath(FString& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(NormalTexturePath, FString);
	}

	bool SetCustomNormalTexturePath(FString AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(NormalTexturePath, FString);
	}

private:
	const UE::Interchange::FAttributeKey Macro_CustomDiffuseTexturePathKey = UE::Interchange::FAttributeKey(TEXT("DiffuseTexturePath"));
	const UE::Interchange::FAttributeKey Macro_CustomNormalTexturePathKey= UE::Interchange::FAttributeKey(TEXT("NormalTexturePath"));
};
