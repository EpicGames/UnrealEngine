// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeDecalNode.generated.h"

UCLASS(BlueprintType, MinimalAPI)
class UInterchangeDecalNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
	bool GetCustomSortOrder(int32& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(SortOrder, int32);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
	bool SetCustomSortOrder(const int32& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SortOrder, int32);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
	bool GetCustomDecalSize(FVector& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(DecalSize, FVector);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
	bool SetCustomDecalSize(const FVector& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DecalSize, FVector);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
	bool GetCustomDecalMaterialPathName(FString& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(DecalMaterialPathName, FString);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Decal")
	bool SetCustomDecalMaterialPathName(const FString& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(DecalMaterialPathName, FString);
	}

private:
	const UE::Interchange::FAttributeKey Macro_CustomSortOrderKey = UE::Interchange::FAttributeKey(TEXT("SortOrder"));
	const UE::Interchange::FAttributeKey Macro_CustomDecalSizeKey = UE::Interchange::FAttributeKey(TEXT("DecalSize"));
	const UE::Interchange::FAttributeKey Macro_CustomDecalMaterialPathNameKey = UE::Interchange::FAttributeKey(TEXT("DecalMaterialPathName"));
};
