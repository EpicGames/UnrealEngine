// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeGroomBindingNode.generated.h"

#define UE_API INTERCHANGENODES_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeGroomBindingNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:
	UE_API virtual FString GetTypeName() const override;

	/** Get the groom asset to build the binding from */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Binding")
	UE_API bool GetGroomDependencyUid(FString& AttributeValue) const;

	/** Set the groom asset to build the binding from */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Binding")
	UE_API bool SetGroomDependencyUid(const FString& AttributeValue);

	/** Get the target mesh to build the binding from */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Binding")
	UE_API bool GetTargetMeshDependencyUid(FString& AttributeValue) const;

	/** Set the target mesh to build the binding from */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Binding")
	UE_API bool SetTargetMeshDependencyUid(const FString& AttributeValue);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(GroomUid);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(TargetMeshUid);
};

#undef UE_API