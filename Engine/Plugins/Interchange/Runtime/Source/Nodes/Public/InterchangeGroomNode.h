// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeGroomNode.generated.h"

#define UE_API INTERCHANGENODES_API

namespace UE::Interchange
{
	struct FGroomNodeStaticData : public FBaseNodeStaticData
	{
		static UE_API const FAttributeKey& PayloadKey();
		static UE_API const FAttributeKey& PayloadTypeKey();
	};
}

UENUM(BlueprintType)
enum class EInterchangeGroomPayLoadType : uint8
{
	STATIC		UMETA(DisplayName = "Groom Asset"),
	ANIMATED	UMETA(DisplayName = "Groom Cache")
};

UENUM(BlueprintType)
enum class EInterchangeGroomCacheAttributes : uint8
{
	None = 0,
	Position = 1,
	Width = 1 << 1,
	Color = 1 << 2,
	
	// For display names
	PositionWidth = (Position | Width) UMETA(DisplayName = "Position & Width"),
	PositionColor = (Position | Color) UMETA(DisplayName = "Position & Color"),
	WidthColor = (Width | Color) UMETA(DisplayName = "Width & Color"),
	PositionWidthColor = (Position | Width | Color) UMETA(DisplayName = "Position, Width, Color")
};

USTRUCT(BlueprintType)
struct FInterchangeGroomPayloadKey
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | Groom")
	FString UniqueId = "";

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | Groom")
	EInterchangeGroomPayLoadType Type = EInterchangeGroomPayLoadType::STATIC;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | Groom Cache")
	int32 FrameNumber = 0;

	FInterchangeGroomPayloadKey() {}

	FInterchangeGroomPayloadKey(const FString& InUniqueId, const EInterchangeGroomPayLoadType& InType)
		: UniqueId(InUniqueId)
		, Type(InType)
	{
	}

	FInterchangeGroomPayloadKey(const FString& InUniqueId, int32 InFrameNumber)
		: UniqueId(InUniqueId)
		, Type(EInterchangeGroomPayLoadType::ANIMATED)
		, FrameNumber(InFrameNumber)
	{
	}

	bool operator==(const FInterchangeGroomPayloadKey& Other) const
	{
		return UniqueId.Equals(Other.UniqueId) && Type == Other.Type && FrameNumber == Other.FrameNumber;
	}

	friend uint32 GetTypeHash(const FInterchangeGroomPayloadKey& InterchangeGroomPayLoadKey)
	{
		return GetTypeHash(InterchangeGroomPayLoadKey.UniqueId + FString::FromInt(static_cast<int32>(InterchangeGroomPayLoadKey.Type)) + FString::FromInt(InterchangeGroomPayLoadKey.FrameNumber));
	}
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeGroomNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:
	UE_API virtual FString GetTypeName() const override;

	UE_API const TOptional<FInterchangeGroomPayloadKey> GetPayloadKey() const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom")
	UE_API void SetPayloadKey(const FString& PayloadKey, EInterchangeGroomPayLoadType PayLoadType = EInterchangeGroomPayLoadType::STATIC);

	/** Get the start frame index of the animation */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool GetCustomStartFrame(int32& AttributeValue) const;

	/** Set the start frame index of the animation */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool SetCustomStartFrame(const int32& AttributeValue);

	/** Get the end frame index of the animation */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool GetCustomEndFrame(int32& AttributeValue) const;

	/** Set the end frame index of the animation */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool SetCustomEndFrame(const int32& AttributeValue);

	/** Get the number of frames in the animation */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool GetCustomNumFrames(int32& AttributeValue) const;

	/** Set the number of frames in the animation */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool SetCustomNumFrames(const int32& AttributeValue);

	/** Get the animation's frame rate */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool GetCustomFrameRate(double& AttributeValue) const;

	/** Set the animation's frame rate */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool SetCustomFrameRate(const double& AttributeValue);

	/** Get the groom attributes that are animated */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool GetCustomGroomCacheAttributes(EInterchangeGroomCacheAttributes& AttributeValue) const;

	/** Set the groom attributes that are animated */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Groom Cache")
	UE_API bool SetCustomGroomCacheAttributes(const EInterchangeGroomCacheAttributes& AttributeValue);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(NumFrames);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(StartFrame);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(EndFrame);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(FrameRate);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(GroomCacheAttributes);
};

#undef UE_API