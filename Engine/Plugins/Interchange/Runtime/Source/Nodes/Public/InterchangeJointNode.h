// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeSceneNode.h"

#include "InterchangeJointNode.generated.h"

#define UE_API INTERCHANGENODES_API

class UInterchangeBaseNodeContainer;

UCLASS(BlueprintType, MinimalAPI)
class UInterchangeJointNode : public UInterchangeSceneNode
{
	GENERATED_BODY()

public:
	/*
	* UInterchangeJointNode
	*
	* Bind pose transform is the transform of the joint when the binding with the mesh was done.
	*
	* Time-zero transform is the transform of the node at time zero.
	* Pipelines often have the option to evaluate the joint at time zero to create the bind pose.
	* Time-zero bind pose is also used if the translator did not find any bind pose, or if we import
	* an unskinned mesh as a skeletal mesh (rigid mesh).
	*/

	UE_API UInterchangeJointNode();

	UE_API virtual FName GetIconName() const override;

#if WITH_EDITOR
	UE_API virtual FString GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;
#endif //WITH_EDITOR

	/** Get the local transform of the bind pose scene node. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint")
	UE_API bool GetBindPoseLocalTransform(FTransform& AttributeValue) const;

	/** Set the local transform of the bind pose scene node. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint")
	UE_API bool SetBindPoseLocalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& AttributeValue);

	/** Get the global transform of the bind pose scene node. This value is computed from the local transforms of all parent bind poses. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint")
	UE_API bool GetBindPoseGlobalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& GlobalOffsetTransform, FTransform& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint")
	UE_API void SetMeshUIDToGlobalBindPoseReferenceMap(const TMap<FString, FMatrix>& InMeshToGlobalBindPoseReferences);

	/** Get the Global Bind Pose Reference for given MeshUID. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint")
	UE_API bool GetGlobalBindPoseReferenceFromMeshUID(const FString& MeshUID, FMatrix& GlobalBindPoseReference) const;
		
	/** Sets if Joint has invalid Bind Pose. Automatic T0 usage will be configured in case if the Skeleton contains at least 1 Joint with invalid BindPose. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint")
	UE_API bool SetHasInvalidBindPose(const bool& bHasInvalidBindPose);

	/** Gets if the joint has invalid BindPose (if the setter was used, otherwise returns with false and T0 evaluation presumes bHasInvalidBindPose == false). */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint")
	UE_API bool GetHasInvalidBindPose(bool & bHasInvalidBindPose) const;

	/** Get the local transform of the time-zero scene node. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint")
	UE_API bool GetTimeZeroLocalTransform(FTransform& AttributeValue) const;

	/** Set the local transform of the time-zero scene node. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint")
	UE_API bool SetTimeZeroLocalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& AttributeValue);

	/** Get the global transform of the time-zero scene node. This value is computed from the local transforms of all parent time-zero scene nodes. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint")
	UE_API bool GetTimeZeroGlobalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& GlobalOffsetTransform, FTransform& AttributeValue) const;

	/** Sets the Global Transformation Matrix used for T0 rebinding.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint")
	UE_API bool SetGlobalMatrixForT0Rebinding(const FMatrix& AttributeValue);

	/** Gets the Global Transformation Matrix used for T0 rebinding.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint")
	UE_API bool GetGlobalMatrixForT0Rebinding(FMatrix& AttributeValue) const;

private:
	//Scene node Local bind pose transforms
	IMPLEMENT_NODE_ATTRIBUTE_KEY(BindPoseLocalTransform);

	//Tracks if Joint node Has invalid Bind Pose.
	IMPLEMENT_NODE_ATTRIBUTE_KEY(HasInvalidBindPose);

	//Scene node local transforms at time zero. This attribute is important for rigid mesh import or if the translator did not fill the bind pose.
	IMPLEMENT_NODE_ATTRIBUTE_KEY(TimeZeroLocalTransform);

	//Global Transform Matrix specifically used for T0 re-binding.
	IMPLEMENT_NODE_ATTRIBUTE_KEY(GlobalMatrixForT0Rebinding);

	//BindPose References per Mesh for a JointNode.
	UE::Interchange::TMapAttributeHelper<FString, FMatrix> MeshToGlobalBindPoseReferences;
};

#undef UE_API
