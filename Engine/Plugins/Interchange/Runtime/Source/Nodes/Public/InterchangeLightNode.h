// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeLightNode.generated.h"

#define UE_API INTERCHANGENODES_API

// This enum is used as a placeholder for ELightUnits. Because InterchangeWorker is not compiled against Engine, the LightFactoryNode is not affected.
UENUM()
enum class EInterchangeLightUnits : uint8
{
	Unitless,
	Candelas,
	Lumens,
	EV,
	Nits,
};

// Mirrors ESkyLightSourceType
UENUM()
enum class EInterchangeSkyLightSourceType : uint8
{
	CapturedScene,
	SpecifiedCubemap
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeBaseLightNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:
	static UE_API FStringView StaticAssetTypeName();

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	UE_API virtual FString GetTypeName() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | BaseLight")
	UE_API bool GetCustomLightColor(FLinearColor& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | BaseLight")
	UE_API bool SetCustomLightColor(const FLinearColor& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | BaseLight")
	UE_API bool GetCustomIntensity(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | BaseLight")
	UE_API bool SetCustomIntensity(float AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | BaseLight")
	UE_API bool GetCustomTemperature(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | BaseLight")
	UE_API bool SetCustomTemperature(float AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | BaseLight")
	UE_API bool GetCustomUseTemperature(bool & AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | BaseLight")
	UE_API bool SetCustomUseTemperature(bool AttributeValue);

private:

	IMPLEMENT_NODE_ATTRIBUTE_KEY(LightColor)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(LightIntensity)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Temperature)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(UseTemperature)
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeLightNode : public UInterchangeBaseLightNode
{
	GENERATED_BODY()

public:

	UE_API virtual FString GetTypeName() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Light")
	UE_API bool GetCustomIntensityUnits(EInterchangeLightUnits& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Light")
	UE_API bool SetCustomIntensityUnits(const EInterchangeLightUnits & AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Light")
	UE_API bool GetCustomAttenuationRadius(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Light")
	UE_API bool SetCustomAttenuationRadius(float AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Light")
	UE_API bool GetCustomIESTexture(FString& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Light")
	UE_API bool SetCustomIESTexture(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Light")
	UE_API bool GetCustomUseIESBrightness(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Light")
	UE_API bool SetCustomUseIESBrightness(const bool& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Light")
	UE_API bool GetCustomIESBrightnessScale(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Light")
	UE_API bool SetCustomIESBrightnessScale(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Light")
	UE_API bool GetCustomRotation(FRotator& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Light")
	UE_API bool SetCustomRotation(const FRotator& AttributeValue, bool bAddApplyDelegate = true);


private:

	IMPLEMENT_NODE_ATTRIBUTE_KEY(IntensityUnits)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AttenuationRadius)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(IESTexture)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(UseIESBrightness)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(IESBrightnessScale)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Rotation)
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangePointLightNode : public UInterchangeLightNode
{
	GENERATED_BODY()

public:

	UE_API virtual FString GetTypeName() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | PointLight")
	UE_API bool GetCustomUseInverseSquaredFalloff(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | PointLight")
	UE_API bool SetCustomUseInverseSquaredFalloff(bool AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | PointLight")
	UE_API bool GetCustomLightFalloffExponent(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | PointLight")
	UE_API bool SetCustomLightFalloffExponent(float AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | PointLight")
	UE_API bool GetCustomSourceRadius(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | PointLight")
	UE_API bool SetCustomSourceRadius(float AttributeValue);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(UseInverseSquaredFalloff)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(LightFalloffExponent)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SourceRadius)
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSpotLightNode : public UInterchangePointLightNode
{
	GENERATED_BODY()

public:

	UE_API virtual FString GetTypeName() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SpotLight")
	UE_API bool GetCustomInnerConeAngle(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SpotLight")
	UE_API bool SetCustomInnerConeAngle(float AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SpotLight")
	UE_API bool GetCustomOuterConeAngle(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SpotLight")
	UE_API bool SetCustomOuterConeAngle(float AttributeValue);

private:

	IMPLEMENT_NODE_ATTRIBUTE_KEY(InnerConeAngle)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(OuterConeAngle)
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeRectLightNode : public UInterchangeLightNode
{
	GENERATED_BODY()

public:

	UE_API virtual FString GetTypeName() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | RectLight")
	UE_API bool GetCustomSourceWidth(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | RectLight")
	UE_API bool SetCustomSourceWidth(float AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | RectLight")
	UE_API bool GetCustomSourceHeight(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | RectLight")
	UE_API bool SetCustomSourceHeight(float AttributeValue);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SourceWidth)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SourceHeight)
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeDirectionalLightNode : public UInterchangeBaseLightNode
{
	GENERATED_BODY()

public:

	UE_API virtual FString GetTypeName() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | DirectionalLight")
	UE_API bool GetCustomLightSourceAngle(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | DirectionalLight")
	UE_API bool SetCustomLightSourceAngle(float AttributeValue);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(LightSourceAngle)
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSkyLightNode : public UInterchangeBaseLightNode
{
	GENERATED_BODY()

public:

	UE_API virtual FString GetTypeName() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkyLight")
	UE_API bool GetCustomCubemapDependency(FString& TextureCubeNodeUid) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkyLight")
	UE_API bool SetCustomCubemapDependency(FString TextureCubeNodeUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkyLight")
	UE_API bool GetCustomSourceType(EInterchangeSkyLightSourceType& SourceType) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkyLight")
	UE_API bool SetCustomSourceType(EInterchangeSkyLightSourceType SourceType);

private:

	IMPLEMENT_NODE_ATTRIBUTE_KEY(CubemapDependency)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SourceType)
};

#undef UE_API
