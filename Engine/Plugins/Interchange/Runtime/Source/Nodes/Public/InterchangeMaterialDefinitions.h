// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"

namespace UE
{
namespace Interchange
{
	namespace Materials
	{
		namespace Standard
		{
			namespace Nodes
			{
				namespace Add
				{
					const FName Name = TEXT("Add");

					namespace Inputs
					{
						const FName A = TEXT("A");
						const FName B = TEXT("B");
					}
				}

				namespace FlattenNormal
				{
					const FName Name = TEXT("FlattenNormal");

					namespace Inputs
					{
						const FName Normal = TEXT("Normal");
						const FName Flatness = TEXT("Flatness");
					}
				}

				namespace Lerp
				{
					const FName Name = TEXT("Lerp");

					namespace Inputs
					{
						const FName A = TEXT("A"); // Type: linear color
						const FName B = TEXT("B"); // Type: linear color
						const FName Factor = TEXT("Factor"); // Type: float
					}
				}

				namespace Luminance
				{
					const FName Name = TEXT("Luminance");

					namespace Inputs
					{
						const FName Input = TEXT("Input");
					}

					namespace Attributes
					{
						const FName LuminanceFactors = TEXT("LuminanceFactors");
					}
				}

				namespace Mask
				{
					const FName Name = TEXT("ComponentMask");

					namespace Attributes
					{
						const FName R = TEXT("R");
						const FName G = TEXT("G");
						const FName B = TEXT("B");
						const FName A = TEXT("A");
					}

					namespace Inputs
					{
						const FName Input = TEXT("Input");
					}
				}

				namespace Multiply
				{
					const FName Name = TEXT("Multiply");

					namespace Inputs
					{
						const FName A = TEXT("A");
						const FName B = TEXT("B");
					}
				}

				namespace MakeFloat3
				{
					const FName Name = TEXT("MakeFloat3");
					namespace Inputs
					{
						const FName X = TEXT("X");
						const FName Y = TEXT("Y");
						const FName Z = TEXT("Z");
					}
				}

				namespace Noise
				{
					const FName Name = TEXT("Noise");

					namespace Inputs
					{
						const FName Position = TEXT("Position");
						const FName FilterWidth = TEXT("FilterWidth");
					}

					namespace Attributes
					{
						const FName Scale = TEXT("Scale");
						const FName Quality = TEXT("Quality");
						const FName Function = TEXT("NoiseFunction");
						const FName Turbulence = TEXT("bTurbulence");
						const FName Levels = TEXT("Levels");
						const FName OutputMin = TEXT("OutputMin");
						const FName OutputMax = TEXT("OutputMax");
						const FName LevelScale = TEXT("LevelScale");
						const FName Tiling = TEXT("bTiling");
						const FName RepeatSize = TEXT("RepeatSize");
					}
				}

				namespace NormalFromHeightMap
				{
					const FName Name = TEXT("NormalFromHeightMap");

					namespace Inputs
					{
						const FName HeightMap = TEXT("Height Map");                // Type: FString (unique id of a texture node)
						const FName Intensity = TEXT("Normal Map Intensity");      // Type: float
						const FName Offset = TEXT("Height Map UV Offset");         // Type: float
						const FName Coordinates = TEXT("Coordinates");             // Type: vec2
						const FName Channel = TEXT("Height Map Channel Selector"); // Type: vec4
					}
				}

				namespace OneMinus
				{
					const FName Name = TEXT("OneMinus");

					namespace Inputs
					{
						const FName Input = TEXT("Input");
					}
				}

				namespace Rotator
				{
					const FName Name = TEXT("Rotator");

					namespace Attributes
					{
						const FName CenterX = TEXT("CenterX");
						const FName CenterY = TEXT("CenterY");
						const FName Speed = TEXT("Speed");
					}

					namespace Inputs
					{
						const FName Coordinate = TEXT("Coordinate");
						const FName Time = TEXT("Time");
					}
				}

				namespace RotateAboutAxis
				{
					const FName Name = TEXT("RotateAboutAxis");

					namespace Attributes
					{
						const FName Period = TEXT("Period");
					}

					namespace Inputs
					{
						const FName NormalizedRotationAxis = TEXT("NormalizedRotationAxis");
						const FName RotationAngle = TEXT("RotationAngle");
						const FName PivotPoint = TEXT("PivotPoint");
						const FName Position = TEXT("Position");
					}
				}

				namespace Switch
				{
					const FName Name = TEXT("Switch");

					namespace Inputs
					{
						const FName Value = TEXT("SwitchValue");
						const FName Default = TEXT("SwitchDefault");
						const FName InputName = TEXT("SwitchInputName"); // The caller should concatenate this input with an index to get the InputName in the array of the MaterialExpressionSwitch
					}

					namespace Attributes
					{
						const FName InputCount = TEXT("SwitchInputCount"); // Used to know the number of inputs in the Switch
					}
				}

				namespace TextureCoordinate
				{
					const FName Name = TEXT("TextureCoordinate");

					namespace Inputs
					{
						const FName Index = TEXT("Index"); // Type: int
						const FName UTiling = TEXT("UTiling"); // Type: float
						const FName VTiling = TEXT("VTiling"); // Type: float
						const FName Offset = TEXT("Offset"); // Type: vec2
						const FName Scale = TEXT("Scale"); // Type: vec2
						const FName Rotate = TEXT("Rotate"); // Type: float, Range: 0-1
						const FName RotationCenter = TEXT("RotationCenter"); // Type: vec2
					}
				}

				namespace TextureObject
				{
					const FName Name = TEXT("TextureObject");

					namespace Inputs
					{
						const FName Texture = TEXT("TextureUid"); // Type: FString (unique id of a texture node)
					}
				}

				namespace TextureProperty
				{
					const FName Name = TEXT("TextureProperty");

					namespace Inputs
					{
						const FName Texture = TEXT("TextureUid"); // Type: FString (unique id of a texture node)
					}

					namespace Attributes
					{
						const FName Property = TEXT("Property"); // Type: enum
					}
				}

				namespace TextureSample
				{
					const FName Name = TEXT("TextureSample");

					namespace Inputs
					{
						const FName Coordinates = TEXT("Coordinates"); // Type: vec2
						const FName Texture = TEXT("TextureUid"); // Type: FString (unique id of a texture node)
					}

					namespace Outputs
					{
						const FName RGB = TEXT("RGB"); // Type: linear color
						const FName R = TEXT("R"); // Type: float
						const FName G = TEXT("G"); // Type: float
						const FName B = TEXT("B"); // Type: float
						const FName A = TEXT("A"); // Type: float
						const FName RGBA = TEXT("RGBA"); // Type: linear color
					}
				}

				namespace TextureSampleBlur
				{
					using namespace TextureSample;					

					namespace Attributes
					{
						const FName KernelSize = TEXT("KernelSize"); // Type: float
						const FName FilterSize = TEXT("FilterSize"); // Type: float
						const FName FilterOffset= TEXT("FilterOffset"); // Type: float
						const FName Filter = TEXT("Filter"); // Type: int
					}
				}

				namespace Time
				{
					const FName Name = TEXT("Time");

					namespace Attributes
					{
						const FName IgnorePause= TEXT("bIgnorePause");
						const FName OverridePeriod = TEXT("bOverride_Period");
						const FName Period = TEXT("Period");
					}
				}

				namespace TransformPosition
				{
					const FName Name = TEXT("TransformPosition");

					namespace Attributes
					{
						const FName TransformSourceType = TEXT("TransformSourceType");
						const FName TransformType = TEXT("TransformType");
					}

					namespace Inputs
					{
						const FName Input = TEXT("Input");
					}
				}

				namespace TransformVector
				{
					const FName Name = TEXT("Transform");

					namespace Attributes
					{
						const FName TransformSourceType = TEXT("TransformSourceType");
						const FName TransformType = TEXT("TransformType");
					}

					namespace Inputs
					{
						const FName Input = TEXT("Input");
					}
				}

				namespace Trigonometry
				{
					namespace Inputs
					{
						const FName Input = TEXT("Input");
					}

					namespace Attributes
					{
						const FName Period = TEXT("Period");
					}
				}

				namespace Cosine
				{
					const FName Name = TEXT("Cosine");
					using namespace Trigonometry;
				}

				namespace Sine
				{
					const FName Name = TEXT("Sine");
					using namespace Trigonometry;
				}

				namespace Tangent
				{
					const FName Name = TEXT("Tangent");
					using namespace Trigonometry;
				}

				namespace VectorNoise
				{

					const FName Name = TEXT("VectorNoise");

					namespace Attributes
					{
						const FName Function = TEXT("NoiseFunction");
						const FName Quality = TEXT("Quality");
						const FName Tiling = TEXT("bTiling");
						const FName TileSize = TEXT("TileSize");
					}

					namespace Inputs
					{
						const FName Position = TEXT("Position");
					}
				}

				namespace VertexColor
				{
					const FName Name = TEXT("VertexColor");
				}

				namespace ScalarParameter
				{
					const FName Name = TEXT("ScalarParameter");
					namespace Attributes
					{
						const FName DefaultValue = TEXT("DefaultValue");
					}
				}

				namespace VectorParameter
				{
					const FName Name = TEXT("VectorParameter");
					namespace Attributes
					{
						const FName DefaultValue = TEXT("DefaultValue");
					}
				}

				namespace StaticBoolParameter
				{
					const FName Name = TEXT("StaticBoolParameter");
					namespace Attributes
					{
						const FName DefaultValue = TEXT("DefaultValue");
					}
				}

				namespace SlabBSDF
				{
					const FName Name = TEXT("SubstrateSlabBSDF");

					namespace Attributes
					{
						const FName SpecularProfile = TEXT("SpecularProfile");
					}

					namespace Inputs
					{
						const FName DiffuseAlbedo = TEXT("DiffuseAlbedo");
						const FName F0 = TEXT("F0");
						const FName F90 = TEXT("F90");
						const FName Roughness = TEXT("Roughness");
						const FName Anisotropy = TEXT("Anisotropy");
						const FName Normal = TEXT("Normal");
						const FName Tangent = TEXT("Tangent");
						const FName SSSMFP = TEXT("SSSMFP");
						const FName SSSMFPScale = TEXT("SSSMFPScale");
						const FName SSSPhaseAnisotropy = TEXT("SSSPhaseAnisotropy");
						const FName EmissiveColor = TEXT("EmissiveColor");
						const FName SecondRoughness = TEXT("SecondRoughness");
						const FName SecondRoughnessWeight = TEXT("SecondRoughnessWeight");
						const FName FuzzRoughness = TEXT("FuzzRoughness");
						const FName FuzzAmount = TEXT("FuzzAmount");
						const FName FuzzColor = TEXT("FuzzColor");
						const FName GlintValue = TEXT("GlintValue");
						const FName GlintUV = TEXT("GlintUV");
					}
				}
			}
		}

		namespace Common
		{
			namespace Parameters
			{
				const FName EmissiveColor = TEXT("EmissiveColor"); // Type: linear color
				const FName Normal = TEXT("Normal"); // Type: vector3f
				const FName Tangent = TEXT("Tangent"); // Type: vector3f
				const FName Opacity = TEXT("Opacity"); // Type: float
				const FName OpacityMask = TEXT("OpacityMask"); // Type: float
				const FName Occlusion = TEXT("Occlusion"); // Type: float
				const FName IndexOfRefraction = TEXT("IOR"); // Type: float
				const FName BxDF = TEXT("BxDF"); // input/output of BSDF or BRDF or BXDF or BTDF data
				const FName Refraction = TEXT("Refraction"); // input/output of BSDF or BRDF or BXDF or BTDF data
				const FName Anisotropy = TEXT("Anisotropy"); // Type: float
				const FName Displacement = TEXT("Displacement"); // Type: float
			}
		}

		namespace Lambert
		{
			namespace Parameters
			{
				using namespace Common::Parameters;

				const FName DiffuseColor = TEXT("DiffuseColor"); // Type: linear color
			}
		}

		namespace OpenPBRSurface
		{
			const FName Name = TEXT("open_pbr_surface");

			namespace Parameters
			{
				const FName BaseWeight = TEXT("base_weight");
				const FName BaseColor = TEXT("base_color");
				const FName BaseRoughness = TEXT("base_roughness");
				const FName BaseMetalness = TEXT("base_metalness");
				const FName SpecularWeight = TEXT("specular_weight");
				const FName SpecularColor = TEXT("specular_color");
				const FName SpecularRoughness = TEXT("specular_roughness");
				const FName SpecularIOR = TEXT("specular_ior");
				const FName SpecularIORLevel = TEXT("specular_ior_level");
				const FName SpecularAnisotropy = TEXT("specular_anisotropy");
				const FName SpecularRotation = TEXT("specular_rotation");
				const FName TransmissionWeight = TEXT("transmission_weight");
				const FName TransmissionColor = TEXT("transmission_color");
				const FName TransmissionDepth = TEXT("transmission_depth");
				const FName TransmissionScatter = TEXT("transmission_scatter");
				const FName TransmissionScatterAnisotropy = TEXT("transmission_scatter_anisotropy");
				const FName TransmissionDispersionScale = TEXT("transmission_dispersion_scale");
				const FName TransmissionDispersionAbbeNumber = TEXT("transmission_dispersion_abbe_number");
				const FName SubsurfaceWeight = TEXT("subsurface_weight");
				const FName SubsurfaceColor = TEXT("subsurface_color");
				const FName SubsurfaceRadius = TEXT("subsurface_radius");
				const FName SubsurfaceRadiusScale = TEXT("subsurface_radius_scale");
				const FName SubsurfaceAnisotropy = TEXT("subsurface_anisotropy");
				const FName FuzzWeight = TEXT("fuzz_weight");
				const FName FuzzColor = TEXT("fuzz_color");
				const FName FuzzRoughness = TEXT("fuzz_roughness");
				const FName CoatWeight = TEXT("coat_weight");
				const FName CoatColor = TEXT("coat_color");
				const FName CoatRoughness = TEXT("coat_roughness");
				const FName CoatAnisotropy = TEXT("coat_anisotropy");
				const FName CoatRotation = TEXT("coat_rotation");
				const FName CoatIOR = TEXT("coat_ior");
				const FName CoatIORLevel = TEXT("coat_ior_level");
				const FName ThinFilmThickness = TEXT("thin_film_thickness");
				const FName ThinFilmIOR = TEXT("thin_film_ior");
				const FName EmissionLuminance = TEXT("emission_luminance");
				const FName EmissionColor = TEXT("emission_color");
				const FName GeometryOpacity = TEXT("geometry_opacity");
				const FName GeometryThinWalled = TEXT("geometry_thin_walled");
				const FName GeometryNormal = TEXT("geometry_normal");
				const FName GeometryCoatNormal = TEXT("geometry_coat_normal");
				const FName GeometryTangent = TEXT("geometry_tangent");
			}

			namespace SubstrateMaterial
			{
				namespace Outputs
				{
					const FName FrontMaterial = TEXT("OpenPBR_FrontMaterial");
					const FName OpacityMask = TEXT("OpacityMask");
				}
			}
		}

		namespace Phong
		{
			namespace Parameters
			{
				using namespace Lambert::Parameters;

				const FName SpecularColor = TEXT("SpecularColor"); // Type: linear color
				const FName Shininess = TEXT("Shininess"); // Type: float, this is the specular exponent, expected range: 2-100
				const FName AmbientColor = TEXT("AmbientColor"); // Type: linear color
			}
		}

		/** PBR Specular/Glossiness model */
		namespace PBRSG
		{
			namespace Parameters
			{
				using namespace Common::Parameters;

				const FName DiffuseColor = TEXT("DiffuseColor"); // Type: vector3
				const FName SpecularColor = TEXT("SpecularColor"); // Type: vector3
				const FName Glossiness = TEXT("Glossiness"); // Type: float
			}
		}

		/** PBR Metallic/Roughness model */
		namespace PBRMR
		{
			namespace Parameters
			{
				using namespace Common::Parameters;

				const FName BaseColor = TEXT("BaseColor"); // Type: vector3
				const FName Metallic = TEXT("Metallic"); // Type: float
				const FName Specular = TEXT("Specular"); // Type: float
				const FName Roughness = TEXT("Roughness"); // Type: float
			}
		}

		namespace ClearCoat
		{
			namespace Parameters
			{
				const FName ClearCoat = TEXT("ClearCoat"); // Type: float
				const FName ClearCoatRoughness = TEXT("ClearCoatRoughness"); // Type: float
				const FName ClearCoatNormal = TEXT("ClearCoatNormal"); // Type: vector3
			}
		}

		namespace ThinTranslucent
		{
			namespace Parameters
			{
				const FName TransmissionColor = TEXT("TransmissionColor"); // Type: vector3
				const FName SurfaceCoverage = TEXT("SurfaceCoverage"); // Type: float
			}
		}

		namespace Sheen
		{
			namespace Parameters
			{
				const FName SheenColor = TEXT("SheenColor"); // Type: vector3
				const FName SheenRoughness = TEXT("SheenRoughness"); // Type: float
			}
		}

		namespace Subsurface
		{
			namespace Parameters
			{
				const FName SubsurfaceColor = TEXT("SubsurfaceColor"); // Type: linear color
			}
		}

		namespace StandardSurface
		{
			const FName Name = TEXT("standard_surface");

			namespace Parameters
			{
				const FName Base = TEXT("base");
				const FName BaseColor = TEXT("base_color");
				const FName DiffuseRoughness = TEXT("diffuse_roughness");
				const FName Metalness = TEXT("metalness");
				const FName Specular = TEXT("specular");
				const FName SpecularColor = TEXT("specular_color");
				const FName SpecularRoughness = TEXT("specular_roughness");
				const FName SpecularIOR = TEXT("specular_IOR");
				const FName SpecularAnisotropy = TEXT("specular_anisotropy");
				const FName SpecularRotation = TEXT("specular_rotation");
				const FName Transmission = TEXT("transmission");
				const FName TransmissionColor = TEXT("transmission_color");
				const FName TransmissionDepth = TEXT("transmission_depth");
				const FName TransmissionScatter = TEXT("transmission_scatter");
				const FName TransmissionScatterAnisotropy = TEXT("transmission_scatter_anisotropy");
				const FName TransmissionDispersion = TEXT("transmission_dispersion");
				const FName TransmissionExtraRoughness = TEXT("transmission_extra_roughness");
				const FName Subsurface = TEXT("subsurface");
				const FName SubsurfaceColor = TEXT("subsurface_color");
				const FName SubsurfaceRadius = TEXT("subsurface_radius");
				const FName SubsurfaceScale = TEXT("subsurface_scale");
				const FName SubsurfaceAnisotropy = TEXT("subsurface_anisotropy");
				const FName Sheen = TEXT("sheen");
				const FName SheenColor = TEXT("sheen_color");
				const FName SheenRoughness = TEXT("sheen_roughness");
				const FName Coat = TEXT("coat");
				const FName CoatColor = TEXT("coat_color");
				const FName CoatRoughness = TEXT("coat_roughness");
				const FName CoatAnisotropy = TEXT("coat_anisotropy");
				const FName CoatRotation = TEXT("coat_rotation");
				const FName CoatIOR = TEXT("coat_IOR");
				const FName CoatNormal = TEXT("coat_normal");
				const FName CoatAffectColor = TEXT("coat_affect_color");
				const FName CoatAffectRoughness = TEXT("coat_affect_roughness");
				const FName ThinFilmThickness = TEXT("thin_film_thickness");
				const FName ThinFilmIOR = TEXT("thin_film_IOR");
				const FName Emission = TEXT("emission");
				const FName EmissionColor = TEXT("emission_color");
				const FName Opacity = TEXT("opacity");
				const FName ThinWalled = TEXT("thin_walled");
				const FName Normal = TEXT("normal");
				const FName Tangent = TEXT("tangent");
			}

			// These outputs are only used for Substrate
			namespace SubstrateMaterial
			{
				namespace Outputs
				{
					const FName Opaque = TEXT("Substrate StandardSurface Opaque");
					const FName Translucent = TEXT("Substrate StandardSurface Translucent");
					const FName Opacity = TEXT("Geometry Opacity");
					const FName Refraction = TEXT("Refraction (IOR)");
				}
			}
		}

		namespace Surface
		{
			const FName Name = TEXT("surface");

			namespace Parameters
			{
				const FName BSDF = TEXT("bsdf");
				const FName EDF = TEXT("edf");
				const FName Opacity = TEXT("opacity");
			}

			namespace Outputs
			{
				const FName Surface = TEXT("Surface");
			}

			namespace Substrate
			{
				namespace Outputs
				{
					using namespace Surface::Outputs;
					const FName Opacity = TEXT("Opacity");
				}
			}
		}

		namespace SurfaceUnlit
		{
			const FName Name = TEXT("surface_unlit");

			namespace Parameters
			{
				const FName Emission = TEXT("emission");
				const FName EmissionColor = TEXT("emission_color");
				const FName Transmission = TEXT("transmission");
				const FName TransmissionColor = TEXT("transmission_color");
				const FName Opacity = TEXT("opacity");
			}

			namespace Outputs
			{
				const FName OpacityMask = TEXT("OpacityMask");
			}

			namespace Substrate
			{
				namespace Outputs
				{
					using namespace SurfaceUnlit::Outputs;
					const FName SurfaceUnlit = TEXT("Surface Unlit");
				}
			}
		}

		namespace UsdPreviewSurface
		{
			const FName Name = TEXT("UsdPreviewSurface");

			namespace Parameters
			{
				const FName DiffuseColor = TEXT("diffuseColor");
				const FName EmissiveColor = TEXT("emissiveColor");
				const FName SpecularColor = TEXT("specularColor");
				const FName Metallic = TEXT("metallic");
				const FName Roughness = TEXT("roughness");
				const FName Clearcoat = TEXT("clearcoat");
				const FName ClearcoatRoughness = TEXT("clearcoatRoughness");
				const FName Opacity = TEXT("opacity");
				const FName OpacityThreshold = TEXT("opacityThreshold");
				const FName IOR = TEXT("ior");
				const FName Normal = TEXT("normal");
				const FName Displacement = TEXT("displacement");
				const FName Occlusion = TEXT("occlusion");
			}

			// These outputs are only used for Substrate
			namespace SubstrateMaterial
			{
				namespace Outputs
				{
					const FName FrontMaterial = TEXT("Substrate UsdPreviewSurface");
					const FName Displacement = TEXT("Displacement");
					const FName Occlusion = TEXT("Occlusion");
				}
			}
		}

		namespace Unlit
		{
			namespace Parameters
			{
				const FName UnlitColor = TEXT("UnlitColor"); // Type: linear color
			}
		}

		namespace SubstrateMaterial
		{
			const FName SpecularProfile = TEXT("SpecularProfile"); // Used by MaterialInstances otherwise look at SubstrateSlabBSDF

			namespace Parameters
			{
				const FName FrontMaterial = TEXT("Front Material");
				const FName OpacityMask = TEXT("Opacity Mask");
				const FName Displacement = TEXT("Displacement");
				const FName Occlusion = TEXT("Occlusion");
			}			
		}

		namespace Factory
		{
			namespace Expression
			{
				const FName Path = TEXT("MaterialExpressionPath"); // Attribute used by the factory to store the path of a UMaterialExpression
			}
		}
	}
}
}
