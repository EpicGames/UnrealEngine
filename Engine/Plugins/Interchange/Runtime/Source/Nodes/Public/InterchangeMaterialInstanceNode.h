// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeMaterialInstanceNode.generated.h"

#define UE_API INTERCHANGENODES_API

class UInterchangeBaseNodeContainer;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeMaterialInstanceNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:

	/**
     * Return the node type name of the class. This is used when reporting errors.
     */
	UE_API virtual FString GetTypeName() const override;

	UE_API virtual FName GetIconName() const override;

	/**
	 * Build and return a UID name for a material instance node.
	 */
	static UE_API FString MakeNodeUid(const FStringView NodeName, const FStringView ParentNodeUid);

	/**
     * Creates a new UInterchangeMaterialInstanceNode and adds it to NodeContainer as a translated node.
     */
	static UE_API UInterchangeMaterialInstanceNode* Create(UInterchangeBaseNodeContainer* NodeContainer, const FStringView NodeName, const FStringView ParentNodeUid);

	/** Path to an existing Material or the UID of a ShaderGraphNode*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material Instance")
	UE_API bool SetCustomParent(const FString& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material Instance")
	UE_API bool GetCustomParent(FString& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material Instance")
	UE_API bool AddScalarParameterValue(const FString & ParameterName, float AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material Instance")
	UE_API bool GetScalarParameterValue(const FString& ParameterName, float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material Instance")
	UE_API bool AddVectorParameterValue(const FString& ParameterName, const FLinearColor& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material Instance")
	UE_API bool GetVectorParameterValue(const FString& ParameterName, FLinearColor& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material Instance")
	UE_API bool AddTextureParameterValue(const FString& ParameterName, const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material Instance")
	UE_API bool GetTextureParameterValue(const FString& ParameterName, FString& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material Instance")
	UE_API bool AddStaticSwitchParameterValue(const FString& ParameterName, bool AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material Instance")
	UE_API bool GetStaticSwitchParameterValue(const FString& ParameterName, bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomBlendMode(int32& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomBlendMode(int32 AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomTwoSided(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomTwoSided(bool AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomEnableTessellation(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomEnableTessellation(bool AttributeValue);

private:

	IMPLEMENT_NODE_ATTRIBUTE_KEY(Parent);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(BlendMode);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(TwoSided);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(EnableTessellation);
};

#undef UE_API
