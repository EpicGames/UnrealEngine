// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeMaterialReferenceNode.generated.h"

#define UE_API INTERCHANGENODES_API

/**
 * Describes a reference to an existing (as in, not imported) material.
 *
 * The idea is that mesh / actor nodes can reference one of these nodes as a slot dependency, and
 * Interchange will assign that existing material to the corresponding slot during import
 */
UCLASS(MinimalAPI, BlueprintType)
class UInterchangeMaterialReferenceNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:
	UE_API virtual FString GetTypeName() const override;

public:
	/** Gets the content path of the target material (e.g. "/Game/MyFolder/Red.Red") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material Reference")
	UE_API bool GetCustomContentPath(FString& AttributeValue) const;

	/** Sets the content path of the target material (e.g. "/Game/MyFolder/Red.Red") */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material Reference")
	UE_API bool SetCustomContentPath(const FString& AttributeValue) const;

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ContentPath);
};

#undef UE_API
