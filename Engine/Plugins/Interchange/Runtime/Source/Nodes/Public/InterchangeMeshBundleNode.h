// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeMeshBundleNode.generated.h"

#define UE_API INTERCHANGENODES_API

/**
 * This node represents a collection of UInterchangeMeshNode UIDs, each associated with N transforms.
 *
 * This is used in the context of collapsing, where the transforms represent collapsed instances of the corresponding
 * mesh nodes. The whole MeshBundle may be converted into a single static mesh or a single skeletal mesh. In both cases,
 * the transforms are relative to the "collapsing root".
 *
 * When the bundle has a valid SkeletonDependencyUid, it represents a group of skinned meshes that should be combined into
 * a single skeletal mesh asset.
 */
UCLASS(MinimalAPI, BlueprintType)
class UInterchangeMeshBundleNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:

	UE_API UInterchangeMeshBundleNode();

	UE_API virtual void Serialize(FArchive& Ar) override;

	UE_API virtual FString GetTypeName() const override;
	UE_API virtual FName GetIconName() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | MeshBundle")
	UE_API bool AddMeshNodeTransform(const FString& MeshNodeUid, const FTransform& Transform);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | MeshBundle")
	UE_API bool AddMeshNodeTransforms(const FString& MeshNodeUid, const TArray<FTransform>& Transforms);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | MeshBundle")
	UE_API bool GetMeshNodeUids(TArray<FString>& OutMeshNodeUids) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | MeshBundle")
	UE_API bool GetMeshNodeTransforms(const FString& MeshNodeUid, TArray<FTransform>& OutNodeTransforms) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | MeshBundle")
	UE_API bool RemoveMeshNodeUid(const FString& MeshLODNodeUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | MeshBundle")
	UE_API void ResetMeshNodeUids();

	/** Get the skeleton dependency UID for this bundle. Returns false if not set. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MeshBundle")
	UE_API bool GetCustomSkeletonDependencyUid(FString& AttributeValue) const;

	/** Set the skeleton dependency UID. When set, this bundle represents a group of skinned meshes. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | MeshBundle")
	UE_API bool SetCustomSkeletonDependencyUid(const FString& AttributeValue);

	/** Returns true if this bundle has a valid skeleton dependency, meaning it represents skinned meshes. */
	UE_API bool IsSkinnedMeshBundle() const;

public:
	// Separate C++ only functions as the signature is not valid for a UFUNCTION
	UE_API void GetAllMeshNodesAndTransforms(TMap<FString, TArray<FTransform>>& OutMeshNodeToTransforms) const;
	UE_API void SetAllMeshNodesAndTransforms(const TMap<FString, TArray<FTransform>>& InMeshNodeToTransforms);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SkeletonDependencyUid);

	UE::Interchange::TMapAttributeHelper<FString, TArray<FTransform>> MeshNodeToTransforms;
};

#undef UE_API
