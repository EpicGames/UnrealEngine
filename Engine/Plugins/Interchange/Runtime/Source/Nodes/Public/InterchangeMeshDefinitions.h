// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "Types/AttributeStorage.h"

#include "InterchangeMeshDefinitions.generated.h"

UENUM(BlueprintType)
enum class EInterchangeMeshCollision : uint8
{
	/** Generates a new box collision mesh encompassing the static mesh*/
	Box,

	/** Generates a new sphere collision mesh encompassing the static mesh*/
	Sphere,

	/** Generates a new capsule collision mesh encompassing the static mesh*/
	Capsule,

	/** Generates a new axis-aligned box collision mesh with the 4 X-axis aligned edges beveled (10 total sides) encompassing the static mesh*/
	Convex10DOP_X UMETA(DisplayName="10DOP-X Simplified"),

	/** Generates a new axis-aligned box collision mesh with the 4 Y-axis aligned edges beveled (10 total sides) encompassing the static mesh*/
	Convex10DOP_Y UMETA(DisplayName = "10DOP-Y Simplified"),

	/** Generates a new axis-aligned box collision mesh with the 4 Z-axis aligned edges beveled (10 total sides) encompassing the static mesh*/
	Convex10DOP_Z UMETA(DisplayName = "10DOP-Z Simplified"),

	/** Generates a new axis-aligned box collision mesh with all edges beveled (18 total sides) encompassing the static mesh*/
	Convex18DOP UMETA(DisplayName = "18DOP Simplified"),
	
	/** Generates a new axis-aligned box collision mesh with all edges and corners beveled (26 total sides) encompassing the static mesh*/
	Convex26DOP UMETA(DisplayName = "26DOP Simplified"),

	/** Generates no collisions, but continue to import custom collisions if the file has ones*/
	None = 0xFF
};

UENUM(BlueprintType)
enum class EInterchangeMotionVectorsHandling : uint8
{
	/** No motion vectors will be present in the geometry cache. */
	NoMotionVectors,
	/** Imports the velocities and converts them to motion vectors. This will increase file size as the motion vectors will be stored on disc. */
	ImportVelocitiesAsMotionVectors,
	/** Force calculation of motion vectors during import. This will increase file size as the motion vectors will be stored on disc. */
	CalculateMotionVectorsDuringImport
};

namespace UE
{
	namespace Interchange
	{
		namespace MeshPayload
		{
			namespace Attributes
			{
				/** Global transform of the scene node that corresponds to the root joint of the skeleton. Already includes GlobalOffsetTransform */
				const FString RootJointGlobalTransform = TEXT("__Payload__RootJointGlobalTransform");

				/** Global transform for the mesh scene node. Already includes GlobalOffsetTransform */
				const FString MeshGlobalTransform = TEXT("__Payload__MeshGlobalTransform");

				/** Additional transform to be added last during the mesh baking (so as if an additional parent transform of the top level actor) */
				const FString GlobalOffsetTransform = TEXT("__Payload__GlobalOffsetTransform");

				/** Whether to perform mesh baking or not */
				const FString BakeMeshes = TEXT("__Payload__BakeMeshes");
			}
		}
	}
}