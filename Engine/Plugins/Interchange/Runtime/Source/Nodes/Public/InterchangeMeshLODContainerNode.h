// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeMeshLODContainerNode.generated.h"

#define UE_API INTERCHANGENODES_API

DECLARE_LOG_CATEGORY_EXTERN(LogInterchangeMeshLODContainerNode, Log, All)

class UInterchangeBaseNodeContainer;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeMeshLODContainerNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:

	inline static const FString MeshLODPrefix = TEXT("\\MeshLOD\\");

	UE_API UInterchangeMeshLODContainerNode();

	UE_API virtual FString GetTypeName() const override;

	UE_API virtual FName GetIconName() const override;

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);

		if (Ar.IsLoading() && bIsInitialized)
		{
			LODMeshDataPerLODIndices.RebuildCache();
		}
	}

	/*Deprecated functions*/
	UE_DEPRECATED(5.8, "Deprecated. Use AddMeshForLODIndex instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | LODContainer")
	UE_API bool AddMeshLODNodeUid(const FString& MeshLODNodeUid);

	UE_DEPRECATED(5.8, "Deprecated. Use GetLODMeshDataPerLODIndices or GetAllReferencedMeshUids instead.")
	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | LODContainer")
	UE_API void GetMeshLODNodeUids(TArray<FString>& OutMeshLODNodeUid) const;
	/*End if deorecated functions*/

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | LODContainer")
	UE_API bool RemoveMeshLODNodeUid(const FString& MeshLODNodeUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | LODContainer")
	UE_API bool ResetMeshLODNodeUids();

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | LODContainer")
	UE_API bool AddMeshForLODIndex(const int32 LODIndex, const FString& MeshUid, const FTransform& Transform);

	/**
	* Gets all LOD Meshes (MeshUID+Transform) per LOD Indices.
	*	We only use the MeshUid and Transform attributes from FInterchangeLODMeshData.
	*/
	UE_API void GetLODMeshDataPerLODIndices(TMap<int32, TArray<FInterchangeLODMeshData>>& OutLODMeshesPerLODIndices) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | LODContainer")
	UE_API void GetAllReferencedMeshUids(TSet<FString>& OutMeshUids) const;

	/**
	* Validates that all MorphTargets used by LOD Index X are present and used by LOD Index X-1.
	* In case the validation failed we report warning.
	*/
	UE_API void ValidateMorphTargets(UInterchangeBaseNodeContainer* NodeContainer, const TMap<int32, TArray<FInterchangeLODMeshData>>& LODMeshesPerLODIndices) const;


	/**
	* Supported LOD Pattern: "LODx_Name", where xx is a number and the Name is the LOD Container/Group's name.
	*/
	UE_API static bool CheckForLODPattern(const FString& Candidate, int32& LODIndex, FString& LODContainerName, FString& ErrorString);

	UE_API static FString MakeMeshLODContainerUid(const FString& OriginalUid);
private:
	/*
	* Attribute to store FInterchangeLODMeshData per LODIndex.
	* #interchange_LODRefactor_Note: we only use MeshUid and Transform from FInterchangeLODMeshData at this stage.
	*/
	UE::Interchange::TMapAttributeHelper<int32, TArray<FInterchangeLODMeshData>> LODMeshDataPerLODIndices;
};

#undef UE_API
