// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeMeshDefinitions.h"
#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeMeshNode.generated.h"

#define UE_API INTERCHANGENODES_API

//Interchange namespace
namespace UE
{
	namespace Interchange
	{

		struct FMeshNodeStaticData : public FBaseNodeStaticData
		{
			static UE_API const FAttributeKey& PayLoadKey();
			static UE_API const FAttributeKey& PayLoadTypeKey();
			static UE_API const FAttributeKey& IsSkinnedMeshKey();
			static UE_API const FAttributeKey& IsMorphTargetKey();
			static UE_API const FAttributeKey& MorphTargetNameKey();
			static UE_API const FAttributeKey& GetSkeletonDependenciesKey();
			static UE_API const FAttributeKey& GetMorphTargetDependenciesKey();
			static UE_API const FAttributeKey& GetSceneInstancesUidsKey();
			static UE_API const FAttributeKey& GetSlotMaterialDependenciesKey();
			static UE_API const FAttributeKey& GetAssemblyPartDependenciesKey();
		};

	}//ns Interchange
}//ns UE

UENUM(BlueprintType)
enum class EInterchangeMeshPayLoadType : uint8
{
	NONE = 0,
	STATIC,
	SKELETAL,
	MORPHTARGET,
	ANIMATED
};


USTRUCT(BlueprintType)
struct FInterchangeMeshPayLoadKey
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | Mesh")
	FString UniqueId = "";

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | Mesh")
	EInterchangeMeshPayLoadType Type = EInterchangeMeshPayLoadType::NONE;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | GeometryCache")
	int32 FrameNumber = 0;

	FInterchangeMeshPayLoadKey() {}

	FInterchangeMeshPayLoadKey(const FString& InUniqueId, const EInterchangeMeshPayLoadType& InType)
		: UniqueId(InUniqueId)
		, Type(InType)
	{
	}

	FInterchangeMeshPayLoadKey(const FString& InUniqueId, int32 InFrameNumber)
		: UniqueId(InUniqueId)
		, Type(EInterchangeMeshPayLoadType::ANIMATED)
		, FrameNumber(InFrameNumber)
	{
	}

	bool operator==(const FInterchangeMeshPayLoadKey& Other) const
	{
		return UniqueId.Equals(Other.UniqueId) && Type == Other.Type && FrameNumber == Other.FrameNumber;
	}

	//Return the translator key merge with the transform
	static FString GetTransformString(const FTransform& Transform)
	{
		const FQuat R(Transform.GetRotation());
		const FVector TT(Transform.GetTranslation());
		const FVector S(Transform.GetScale3D());
		return FString::Printf(TEXT("%.5f,%.5f,%.5f|%.5f,%.5f,%.5f,%.5f|%.5f,%.5f,%.5f"), TT.X, TT.Y, TT.Z, R.X, R.Y, R.Z, R.W, S.X, S.Y, S.Z);
	}

	friend uint32 GetTypeHash(const FInterchangeMeshPayLoadKey& InterchangeMeshPayLoadKey)
	{
		return GetTypeHash(InterchangeMeshPayLoadKey.UniqueId + FString::FromInt(static_cast<int32>(InterchangeMeshPayLoadKey.Type)) + FString::FromInt(InterchangeMeshPayLoadKey.FrameNumber));
	}
};


UCLASS(BlueprintType, MinimalAPI)
class UInterchangeMeshNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:
	UE_API UInterchangeMeshNode();

#if WITH_EDITOR
	UE_API virtual FString GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;

	UE_API virtual FString GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;
#endif //WITH_EDITOR

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	UE_API virtual FString GetTypeName() const override;

	/**
	 * Icon names are created by adding "InterchangeIcon_" in front of the specialized type. If there is no special type, the function will return NAME_None, which will use the default icon.
	 */
	UE_API virtual FName GetIconName() const override;

	/**
	 * Override Serialize() to restore SlotMaterialDependencies on load.
	 */
	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);

		if (Ar.IsLoading() && bIsInitialized)
		{
			SlotMaterialDependencies.RebuildCache();
		}
	}

	/**
	 * Return true if this node represents a skinned mesh.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool IsSkinnedMesh() const;

	/**
	 * Set the IsSkinnedMesh attribute to determine whether this node represents a skinned mesh.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetSkinnedMesh(const bool bIsSkinnedMesh);

	/**
	 * Return true if this node represents a morph target.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool IsMorphTarget() const;

	/**
	 * Set the IsMorphTarget attribute to determine whether this node represents a morph target.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetMorphTarget(const bool bIsMorphTarget);

	/**
	 * Get the morph target name.
	 * Return true if we successfully retrieved the MorphTargetName attribute.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetMorphTargetName(FString& OutMorphTargetName) const;

	/**
	 * Set the MorphTargetName attribute to determine the name of the morph target.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetMorphTargetName(const FString& MorphTargetName);

	UE_API virtual const TOptional<FInterchangeMeshPayLoadKey> GetPayLoadKey() const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API virtual void SetPayLoadKey(const FString& PayLoadKey, const EInterchangeMeshPayLoadType& PayLoadType);

	/** Query the vertex count of this mesh. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetCustomVertexCount(int32& AttributeValue) const;

	/** Set the vertex count of this mesh. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetCustomVertexCount(const int32& AttributeValue);

	/** Query the polygon count of this mesh. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetCustomPolygonCount(int32& AttributeValue) const;

	/** Set the polygon count of this mesh. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetCustomPolygonCount(const int32& AttributeValue);

	/** Query the bounding box of this mesh. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetCustomBoundingBox(FBox& AttributeValue) const;

	/** Set the bounding box of this mesh. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetCustomBoundingBox(const FBox& AttributeValue);

	/** Query whether this mesh has vertex normals. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetCustomHasVertexNormal(bool& AttributeValue) const;

	/** Set the vertex normal attribute of this mesh. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetCustomHasVertexNormal(const bool& AttributeValue);

	/** Query whether this mesh has vertex bi-normals. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetCustomHasVertexBinormal(bool& AttributeValue) const;

	/** Set the vertex bi-normal attribute of this mesh. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetCustomHasVertexBinormal(const bool& AttributeValue);

	/** Query whether this mesh has vertex tangents. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetCustomHasVertexTangent(bool& AttributeValue) const;

	/** Set the vertex tangent attribute of this mesh. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetCustomHasVertexTangent(const bool& AttributeValue);

	/** Query whether this mesh has smoothing groups. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetCustomHasSmoothGroup(bool& AttributeValue) const;

	/** Set the smoothing group attribute of this mesh. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetCustomHasSmoothGroup(const bool& AttributeValue);

	/** Query whether this mesh has vertex colors. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetCustomHasVertexColor(bool& AttributeValue) const;

	/** Set the vertex color attribute of this mesh. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetCustomHasVertexColor(const bool& AttributeValue);

	/** Query the UV count of this mesh. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetCustomUVCount(int32& AttributeValue) const;

	/** Set the UV count attribute of this mesh. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetCustomUVCount(const int32& AttributeValue);

	/**
	 * Retrieve the number of skeleton dependencies for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API int32 GetSkeletonDependeciesCount() const;

	/**
	 * Retrieve the skeleton dependency for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API void GetSkeletonDependencies(TArray<FString>& OutDependencies) const;

	/**
	 * Retrieve the specified skeleton dependency for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API void GetSkeletonDependency(const int32 Index, FString& OutDependency) const;

	/**
	 * Add the specified skeleton dependency to this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetSkeletonDependencyUid(const FString& DependencyUid);

	/**
	 * Remove the specified skeleton dependency from this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool RemoveSkeletonDependencyUid(const FString& DependencyUid);

	/**
	 * Retrieve the number of morph target dependencies for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API int32 GetMorphTargetDependeciesCount() const;

	/**
	 * Retrieve all morph target dependencies for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API void GetMorphTargetDependencies(TArray<FString>& OutDependencies) const;

	/**
	 * Retrieve the specified morph target dependency for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API void GetMorphTargetDependency(const int32 Index, FString& OutDependency) const;

	/**
	 * Add the specified morph target dependency to this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetMorphTargetDependencyUid(const FString& DependencyUid);

	/**
	 * Remove the specified morph target dependency from this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool RemoveMorphTargetDependencyUid(const FString& DependencyUid);

	/**
	 * Retrieve the number of scene nodes instancing this mesh.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API int32 GetSceneInstanceUidsCount() const;

	/**
	 * Retrieve the asset instances this scene node refers to.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API void GetSceneInstanceUids(TArray<FString>& OutDependencies) const;

	/**
	 * Retrieve the asset instance this scene node refers to.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API void GetSceneInstanceUid(const int32 Index, FString& OutDependency) const;

	/**
	 * Add the specified asset instance this scene node refers to.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetSceneInstanceUid(const FString& DependencyUid);

	/**
	 * Remove the specified asset instance this scene node refers to.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool RemoveSceneInstanceUid(const FString& DependencyUid);

	/**
	 * Retrieve the correspondence table between slot names and assigned materials for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API void GetSlotMaterialDependencies(TMap<FString, FString>& OutMaterialDependencies) const;

	/**
	 * Retrieve the specified Material dependency for a given slot of this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool GetSlotMaterialDependencyUid(const FString& SlotName, FString& OutMaterialDependency) const;

	/**
	 * Add the specified Material dependency to a specific slot name of this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool SetSlotMaterialDependencyUid(const FString& SlotName, const FString& MaterialDependencyUid);

	/**
	 * Remove the Material dependency associated with the given slot name from this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | StaticMesh")
	UE_API bool RemoveSlotMaterialDependencyUid(const FString& SlotName);

	/**
    * Retrieve the number of Nanite assembly part dependencies for this object.
    */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API int32 GetAssemblyPartDependenciesCount() const;

	/**
	 * Retrieve the Nanite assembly part dependencies for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API void GetAssemblyPartDependencies(TArray<FString>& OutDependencies) const;

	/**
	 * Retrieve the specified Nanite assembly part dependency for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API void GetAssemblyPartDependency(const int32 Index, FString& OutDependency) const;

	/**
	 * Add the specified Nanite assembly part dependency to this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool AddAssemblyPartDependencyUid(const FString& DependencyUid);

	/**
	 * Remove the specified Nanite assembly part dependency from this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool RemoveAssemblyPartDependencyUid(const FString& DependencyUid);

	/**
	 * Get the type of collision shapes we should generate from this mesh.
	 * Note: This is a separate mechanism from the FBX-style collision shape name prefixes. For now, these
	 * collision shapes will only be used for the static mesh generated from this very same Mesh node
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool GetCustomCollisionType(EInterchangeMeshCollision& AttributeValue) const;

	/** Set the type of collision shapes we should generate from this mesh */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Mesh")
	UE_API bool SetCustomCollisionType(EInterchangeMeshCollision AttributeValue);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(VertexCount);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(PolygonCount);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(BoundingBox);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(HasVertexNormal);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(HasVertexBinormal);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(HasVertexTangent);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(HasSmoothGroup);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(HasVertexColor);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(UVCount);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(CollisionType);

	UE::Interchange::TArrayAttributeHelper<FString> SkeletonDependencies;
	UE::Interchange::TArrayAttributeHelper<FString> MaterialDependencies;
	UE::Interchange::TArrayAttributeHelper<FString> MorphTargetDependencies;
	UE::Interchange::TArrayAttributeHelper<FString> AssemblyPartDependencies;
	UE::Interchange::TArrayAttributeHelper<FString> SceneInstancesUids;

	UE::Interchange::TMapAttributeHelper<FString, FString> SlotMaterialDependencies;
};

UCLASS(BlueprintType, MinimalAPI)
class  UInterchangeGeometryCacheNode : public UInterchangeMeshNode
{
	GENERATED_BODY()

public:
	UE_API UInterchangeGeometryCacheNode();

	/** Query the start frame of the animated mesh. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool GetCustomStartFrame(int32& AttributeValue) const;

	/** Set the start frame of the animated mesh. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool SetCustomStartFrame(const int32& AttributeValue);

	/** Query the end frame of the animated mesh. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool GetCustomEndFrame(int32& AttributeValue) const;

	/** Set the end frame of the animated mesh. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool SetCustomEndFrame(const int32& AttributeValue);

	/** Query the frame rate of the animated mesh. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool GetCustomFrameRate(double& AttributeValue) const;

	/** Set the frame rate of the animated mesh. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool SetCustomFrameRate(const double& AttributeValue);

	/** Query whether this animated mesh has constant topoplogy. Return false if the attribute was not set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool GetCustomHasConstantTopology(bool& AttributeValue) const;

	/** Set the constant topology attribute of this animated mesh. Return false if the attribute could not be set.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | GeometryCache")
	UE_API bool SetCustomHasConstantTopology(const bool& AttributeValue);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(StartFrame);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(EndFrame);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(FrameRate);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(HasConstantTopology);
};

#undef UE_API