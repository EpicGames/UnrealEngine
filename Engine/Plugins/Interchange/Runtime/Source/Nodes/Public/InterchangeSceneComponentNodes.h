// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeSceneComponentNodes.generated.h"

#define UE_API INTERCHANGENODES_API

class UInterchangeSceneNode;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSceneComponentNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:

	UE_API UInterchangeSceneComponentNode();

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	UE_API virtual FString GetTypeName() const override;


	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component")
	UE_API bool AddComponentUid(const FString& ComponentUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component")
	UE_API void GetComponentUids(TArray<FString>& OutComponentUids) const;


	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Component")
	UE_API bool GetCustomLocalTransform(FTransform& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Component")
	UE_API bool SetCustomLocalTransform(const FTransform& AttributeValue);


	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Component")
	UE_API bool GetCustomComponentVisibility(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Component")
	UE_API bool SetCustomComponentVisibility(const bool& AttributeValue);


	/** Get's the SceneNode that the SceneComponentNode belongs to, also calculates the GlobalTransform within the SceneNode space.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | Instanced")
	UE_API const UInterchangeSceneNode* GetParentSceneNodeAndTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, FTransform& SceneNodeTransform, bool bForceRecache = false) const;

private:
	//Component's Local Transform.
	const UE::Interchange::FAttributeKey Macro_CustomLocalTransformKey = UE::Interchange::FAttributeKey(TEXT("LocalTransform"));

	//Component's Visibility
	const UE::Interchange::FAttributeKey Macro_CustomComponentVisibilityKey = UE::Interchange::FAttributeKey(TEXT("ComponentVisibility"));

	//Children Component Uids.
	UE::Interchange::TArrayAttributeHelper<FString> ComponentUids;

	//mutable caches for global transforms
	mutable TOptional<FTransform> CacheSceneNodeTransform;
	mutable TOptional<const UInterchangeSceneNode*> CacheParentSceneNode;
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeInstancedStaticMeshComponentNode : public UInterchangeSceneComponentNode
{
	GENERATED_BODY()

public:
	UE_API UInterchangeInstancedStaticMeshComponentNode();

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	UE_API virtual FString GetTypeName() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | Instanced")
	UE_API void AddInstanceTransform(const FTransform& InstanceTransform);
	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | Instanced")
	UE_API void AddInstanceTransforms(const TArray<FTransform>& InInstanceTransforms);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | Instanced")
	UE_API void GetInstanceTransforms(TArray<FTransform>& OutInstanceTransforms) const;

	/** Get which asset, if any, a scene node is instantiating. Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | Instanced")
	UE_API bool GetCustomInstancedAssetUid(FString& AttributeValue) const;

	/** Add an asset for this scene node to instantiate. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component | Instanced")
	UE_API bool SetCustomInstancedAssetUid(const FString& AttributeValue);

private:
	//Instanced StaticMesh Component's Instances.
	UE::Interchange::TArrayAttributeHelper<FTransform> InstanceTransforms;

	//To track/connect StaticMesh.
	const UE::Interchange::FAttributeKey Macro_CustomAssetInstanceUidKey = UE::Interchange::FAttributeKey(TEXT("AssetInstanceUid"));
};

#undef UE_API
