// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Optional.h"
#include "Nodes/InterchangeBaseNode.h"
#include "InterchangeAnimationTrackSetNode.h"

#include "InterchangeSceneNode.generated.h"

#define UE_API INTERCHANGENODES_API

class UInterchangeBaseNodeContainer;
//Interchange namespace
namespace UE
{
	namespace Interchange
	{
		struct FSceneNodeStaticData : public FBaseNodeStaticData
		{
			static UE_API const FAttributeKey& GetNodeSpecializeTypeBaseKey();
			static UE_API const FAttributeKey& GetMaterialDependencyUidsBaseKey();
			static UE_API const FString& GetMeshToGlobalBindPoseReferencesString();
			static UE_API const FString& GetSlotMaterialDependenciesString();
			static UE_API const FString& GetMorphTargetCurveWeightsKey();
			static UE_API const FString& GetLayerNamesKey();
			static UE_API const FString& GetTagsKey();
			static UE_API const FString& GetCurveAnimationTypesKey();
			static UE_API const FString& GetComponentUidsKey();

			UE_DEPRECATED(5.8, "Use typed class UInterchangeJointNode instead")
			static UE_API const FString& GetJointSpecializeTypeString();

			UE_DEPRECATED(5.8, "Use UInterchangeMeshLODContainerNode class instead")
			static UE_API const FString& GetLodGroupSpecializeTypeString();
		};

	}//ns Interchange
}//ns UE

/**
 * The scene node represents a transform node in the scene.
 * Scene nodes can have user-defined attributes. Use UInterchangeUserDefinedAttributesAPI to get and set user-defined attribute data.
 */
UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSceneNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:
	UE_API UInterchangeSceneNode();

	/**
	 * Override Serialize() to restore SlotMaterialDependencies on load.
	 */
	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);

		if (Ar.IsLoading() && bIsInitialized)
		{
			SlotMaterialDependencies.RebuildCache();
		}
	}

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	UE_API virtual FString GetTypeName() const override;

#if WITH_EDITOR
	UE_API virtual FString GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;

	UE_API virtual FString GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;
#endif //WITH_EDITOR
	/**
	 * Icon names are created by adding "InterchangeIcon_" in front of the specialized type. If there is no special type, the function will return NAME_None, which will use the default icon.
	 */
	UE_API virtual FName GetIconName() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool IsSpecializedTypeContains(const FString& SpecializedType) const;

	/** Get the specialized type this scene node represents (for example, Joint or LODGroup). */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API int32 GetSpecializedTypeCount() const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API void GetSpecializedType(const int32 Index, FString& OutSpecializedType) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API void GetSpecializedTypes(TArray<FString>& OutSpecializedTypes) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool AddSpecializedType(const FString& SpecializedType);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool RemoveSpecializedType(const FString& SpecializedType);

	/** Get which asset, if any, a scene node is instantiating. Return false if the Attribute was not set previously.*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool GetCustomAssetInstanceUid(FString& AttributeValue) const;

	/** Add an asset for this scene node to instantiate. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool SetCustomAssetInstanceUid(const FString& AttributeValue);


	/**
	 * Get the default scene node local transform.
	 * The default transform is the local transform of the node (no bind pose, no time evaluation).
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool GetCustomLocalTransform(FTransform& AttributeValue) const;

	/**
	 * Set the default scene node local transform.
	 * The default transform is the local transform of the node (no bind pose, no time evaluation).
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool SetCustomLocalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& AttributeValue);

	UE_DEPRECATED(5.8, "Use the overload without the bResetCache parameter; it is no longer needed.")
	UE_API bool SetCustomLocalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& AttributeValue, bool bResetCache);

	/** Get the default scene node global transform. This value is computed from the local transforms of all parent scene nodes. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool GetCustomGlobalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& GlobalOffsetTransform, FTransform& AttributeValue) const;

	UE_DEPRECATED(5.8, "Use the overload without the bForceRecache parameter; it is no longer needed.")
	UE_API bool GetCustomGlobalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& GlobalOffsetTransform, FTransform& AttributeValue, bool bForceRecache) const;


	/** Get the geometric offset. Any mesh attached to this scene node will be offset using this transform. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool GetCustomGeometricTransform(FTransform& AttributeValue) const;

	/** Set the geometric offset. Any mesh attached to this scene node will be offset using this transform. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool SetCustomGeometricTransform(const FTransform& AttributeValue);

	/** Get the node pivot geometric offset. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool GetCustomPivotNodeTransform(FTransform& AttributeValue) const;

	/** Set the node pivot geometric offset. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool SetCustomPivotNodeTransform(const FTransform& AttributeValue);

	/** Gets whether components spawned from this node should be visible */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool GetCustomComponentVisibility(bool& bOutIsVisible) const;

	/** Sets whether components spawned from this node should be visible */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool SetCustomComponentVisibility(bool bInIsVisible);

	/** Gets whether actors spawned from this node should be visible */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool GetCustomActorVisibility(bool& bOutIsVisible) const;

	/** Sets whether actors spawned from this node should be visible */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool SetCustomActorVisibility(bool bInIsVisible);

	/** This static function ensures all the global transform caches are reset for all the UInterchangeSceneNode nodes in the UInterchangeBaseNodeContainer. */
	static UE_API void ResetAllGlobalTransformCaches(const UInterchangeBaseNodeContainer* BaseNodeContainer);

	/** This static function ensures all the global transform caches are reset for all the UInterchangeSceneNode nodes children in the UInterchangeBaseNodeContainer. */
	static UE_API void ResetGlobalTransformCachesOfNodeAndAllChildren(const UInterchangeBaseNodeContainer* BaseNodeContainer, const UInterchangeBaseNode* ParentNode);

	/**
	 * Retrieve the correspondence table between slot names and assigned materials for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Meshes")
	UE_API void GetSlotMaterialDependencies(TMap<FString, FString>& OutMaterialDependencies) const;

	/**
	 * Retrieve the Material dependency for a given slot of this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Meshes")
	UE_API bool GetSlotMaterialDependencyUid(const FString& SlotName, FString& OutMaterialDependency) const;

	/**
	 * Add the specified Material dependency to a specific slot name of this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Meshes")
	UE_API bool SetSlotMaterialDependencyUid(const FString& SlotName, const FString& MaterialDependencyUid);

	/**
	 * Remove the Material dependency associated with the given slot name from this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Meshes")
	UE_API bool RemoveSlotMaterialDependencyUid(const FString& SlotName);

	/** Set MorphTarget with given weight. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	UE_API bool SetMorphTargetCurveWeight(const FString& MorphTargetName, const float& Weight);

	/** Get MorphTargets and their weights. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	UE_API void GetMorphTargetCurveWeights(TMap<FString, float>& OutMorphTargetCurveWeights) const;


	/** Set the Animation Asset To Play by this Scene Node. Only relevant for SkeletalMeshActors (that is, SceneNodes that are instantiating Skeletal Meshes). */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	UE_API bool SetCustomAnimationAssetUidToPlay(const FString& AttributeValue);
	/** Get the Animation Asset To Play by this Scene Node. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SkeletalMesh")
	UE_API bool GetCustomAnimationAssetUidToPlay(FString& AttributeValue) const;

	/** Gets the LayerNames that this SceneNode (Actor) is supposed to be part of. */
	UE_API void GetLayerNames(TArray<FString>& OutLayerNames) const;

	/** Add LayerName that this SceneNode (Actor) is supposed to be part of. */
	UE_API bool AddLayerName(const FString& LayerName);

	/** Remove LayerName that this SceneNode (Actor) is supposed to be part of. */
	UE_API bool RemoveLayerName(const FString& LayerName);

	/** Gets the Tags that this SceneNode (Actor) is supposed to have. */
	UE_API void GetTags(TArray<FString>& OutTags) const;

	/** Add Tag that this SceneNode (Actor) is supposed to have. */
	UE_API bool AddTag(const FString& Tag);

	/** Remove Tag that this SceneNode (Actor) is supposed to have. */
	UE_API bool RemoveTag(const FString& Tag);
	
	/** Sets the Animation Curve Type for the given CurveName (StepCurve or Curve). (Mostly used for tracking Custom Attributes' Animation Types) */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint")
	UE_API bool SetAnimationCurveTypeForCurveName(const FString& CurveName, const EInterchangeAnimationPayLoadType& AnimationCurveType);

	/** Gets the Animation Curve Type for the given CurveName. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint")
	UE_API bool GetAnimationCurveTypeForCurveName(const FString& CurveName, EInterchangeAnimationPayLoadType& OutCurveAnimationType) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component")
	UE_API bool AddComponentUid(const FString& ComponentUid);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component")
	UE_API void GetComponentUids(TArray<FString>& OutComponentUids) const;

	/** Sets IsSceneRootNode, which indicates if the SceneNode is a Scene Root Node. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component")
	UE_API bool SetCustomIsSceneRoot(const bool& IsSceneRoot);

	/** Gets IsSceneRoot, which indicates if the SceneNode is a Scene Root Node. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Scene | Component")
	UE_API bool GetCustomIsSceneRoot(bool& IsSceneRoot) const;

	/** Gets whether the scene node was part of a Named LOD Group, indicating if it should show up in the scene or not. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool GetCustomDoNotInstantiateInLevel(bool& bOutDoNotInstantiateInLevel) const;

	/** Sets whether the scene node was part of a Named LOD Group, indicating if it should show up in the scene or not. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Scene")
	UE_API bool SetCustomDoNotInstantiateInLevel(bool bInDoNotInstantiateInLevel);

	/***********************************************************************************************
	* UInterchangeJointNode Deprecation Begin
	*/

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint", meta = (DeprecatedFunction, DeprecationMessage = "Moved to UInterchangeJointNode."))
	UE_DEPRECATED(5.8, "This function moved to UInterchangeJointNode::GetBindPoseLocalTransform")
	UE_API bool GetCustomBindPoseLocalTransform(FTransform& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint", meta = (DeprecatedFunction, DeprecationMessage = "Moved to UInterchangeJointNode."))
	UE_DEPRECATED(5.8, "This function moved to UInterchangeJointNode::SetBindPoseLocalTransform")
	UE_API bool SetCustomBindPoseLocalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& AttributeValue, bool bResetCache = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint", meta = (DeprecatedFunction, DeprecationMessage = "Moved to UInterchangeJointNode."))
	UE_DEPRECATED(5.8, "This function moved to UInterchangeJointNode::GetBindPoseGlobalTransform")
	UE_API bool GetCustomBindPoseGlobalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& GlobalOffsetTransform, FTransform& AttributeValue, bool bForceRecache = false) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint", meta = (DeprecatedFunction, DeprecationMessage = "Moved to UInterchangeJointNode."))
	UE_DEPRECATED(5.8, "This function moved to UInterchangeJointNode::GetTimeZeroLocalTransform")
	UE_API bool GetCustomTimeZeroLocalTransform(FTransform& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint", meta = (DeprecatedFunction, DeprecationMessage = "Moved to UInterchangeJointNode."))
	UE_DEPRECATED(5.8, "This function moved to UInterchangeJointNode::SetTimeZeroLocalTransform")
	UE_API bool SetCustomTimeZeroLocalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& AttributeValue, bool bResetCache = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint", meta = (DeprecatedFunction, DeprecationMessage = "Moved to UInterchangeJointNode."))
	UE_DEPRECATED(5.8, "This function moved to UInterchangeJointNode::GetTimeZeroGlobalTransform")
	UE_API bool GetCustomTimeZeroGlobalTransform(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FTransform& GlobalOffsetTransform, FTransform& AttributeValue, bool bForceRecache = false) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint", meta = (DeprecatedFunction, DeprecationMessage = "Moved to UInterchangeJointNode."))
	UE_DEPRECATED(5.8, "This function moved to UInterchangeJointNode::SetMeshUIDToGlobalBindPoseReferenceMap")
	UE_API void SetGlobalBindPoseReferenceForMeshUIDs(const TMap<FString, FMatrix>& GlobalBindPoseReferenceForMeshUIDs);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint", meta = (DeprecatedFunction, DeprecationMessage = "Moved to UInterchangeJointNode."))
	UE_DEPRECATED(5.8, "This function moved to UInterchangeJointNode::GetGlobalBindPoseReferenceFromMeshUID")
	UE_API bool GetGlobalBindPoseReferenceForMeshUID(const FString& MeshUID, FMatrix& GlobalBindPoseReference) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint", meta = (DeprecatedFunction, DeprecationMessage = "Moved to UInterchangeJointNode."))
	UE_DEPRECATED(5.8, "This function moved to UInterchangeJointNode::SetHasInvalidBindPose")
	UE_API bool SetCustomHasBindPose(const bool& bHasBindPose);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint", meta = (DeprecatedFunction, DeprecationMessage = "Moved to UInterchangeJointNode."))
	UE_DEPRECATED(5.8, "This function moved to UInterchangeJointNode::SetHasInvalidBindPose")
	UE_API bool GetCustomHasBindPose(bool& bHasBindPose) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint", meta = (DeprecatedFunction, DeprecationMessage = "Moved to UInterchangeJointNode."))
	UE_DEPRECATED(5.8, "This function moved to UInterchangeJointNode::SetGlobalMatrixForT0Rebinding")
	UE_API bool SetCustomGlobalMatrixForT0Rebinding(const FMatrix& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Joint", meta = (DeprecatedFunction, DeprecationMessage = "Moved to UInterchangeJointNode."))
	UE_DEPRECATED(5.8, "This function moved to UInterchangeJointNode::GetGlobalMatrixForT0Rebinding")
	UE_API bool GetCustomGlobalMatrixForT0Rebinding(FMatrix& AttributeValue) const;

	/*
	* UInterchangeJointNode Deprecation End
	***********************************************************************************************/

protected:
	UE_API bool GetGlobalTransformWithOffsetInternal(const UE::Interchange::FAttributeKey LocalTransformKey
		, const UInterchangeBaseNodeContainer* BaseNodeContainer
		, const FTransform& GlobalOffsetTransform
		, FTransform& OutAttributeValue) const;

	UE_API void ResetGlobalTransformCacheInternal(const UInterchangeBaseNodeContainer* BaseNodeContainer, const UE::Interchange::FAttributeKey LocalTransformKey) const;

private:
	UE_API bool GetGlobalTransformInternal(const UE::Interchange::FAttributeKey LocalTransformKey
		, const UInterchangeBaseNodeContainer* BaseNodeContainer
		, FTransform& OutAttributeValue) const;

	//Scene node default local transforms.
	IMPLEMENT_NODE_ATTRIBUTE_KEY(LocalTransform);

	//A scene node can have a transform apply to the mesh it reference.
	IMPLEMENT_NODE_ATTRIBUTE_KEY(GeometricTransform);

	//A scene node can have a pivot transform apply to the mesh it reference (use this pivot only if you are not baking the vertices of the mesh).
	IMPLEMENT_NODE_ATTRIBUTE_KEY(PivotNodeTransform);

	//A scene node can be invisible, but still be imported. We have two of these to match how you can separately animate actor and component visibility
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ComponentVisibility);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ActorVisibility);

	//A scene node can reference an asset. Asset can be Mesh, Light, camera...
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AssetInstanceUid);

	//A scene node can represent many special types
	UE::Interchange::TArrayAttributeHelper<FString> NodeSpecializeTypes;

	//A scene node can have is own set of materials for the mesh it reference.
	UE::Interchange::TMapAttributeHelper<FString, FString> SlotMaterialDependencies;

	//A scene node can have different MorphTarget curve settings:
	UE::Interchange::TMapAttributeHelper<FString, float> MorphTargetCurveWeights;

	//A scene node can be part of multiple Layers.
	UE::Interchange::TArrayAttributeHelper<FString> LayerNames;

	//A scene node can have multiple Tags.
	UE::Interchange::TArrayAttributeHelper<FString> Tags;

	//A scene node can have Attributes defining Curves. (Mostly used for tracking Custom Attributes' Animation Types).
	UE::Interchange::TMapAttributeHelper<FString, EInterchangeAnimationPayLoadType> CurveAnimationTypes;

	//A scene node can represent many special types
	UE::Interchange::TArrayAttributeHelper<FString> ComponentUids;

	//A scene node can reference an animation asset on top of base asset:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AnimationAssetUidToPlay);

	//For storing flag in case SceneNode is a Scene Root Node.
	IMPLEMENT_NODE_ATTRIBUTE_KEY(IsSceneRoot);

	//For storing flag in case SceneNode is part of a Named LOD Group.
	IMPLEMENT_NODE_ATTRIBUTE_KEY(DoNotInstantiateInLevel);

protected:

	//Depending on its usage, A SceneNode can be part of multiple spaces (transform, time zero, and bind pose)
	//We want to be able to cache any global transform calculated for any space
	mutable TMap<UE::Interchange::FAttributeKey, FTransform> GlobalTransformCache;
};

#undef UE_API
