// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once
 
#include "Nodes/InterchangeBaseNode.h"
#include "UObject/Object.h"
 
#include "InterchangeShaderGraphNode.generated.h"

#define UE_API INTERCHANGENODES_API

class UInterchangeBaseNodeContainer;

/**
 * The Shader Ports API manages a set of inputs and outputs attributes.
 * This API can be used over any InterchangeBaseNode that wants to support shader ports as attributes.
 */
UCLASS(MinimalAPI, BlueprintType)
class UInterchangeShaderPortsAPI : public UObject
{
	GENERATED_BODY()
 
public:
	/**
	 * Makes an attribute key to represent a node being connected to an input (that is, Inputs:InputName:Connect).
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	static UE_API FString MakeInputConnectionKey(const FString& InputName);
 
	/**
	 * Makes an attribute key to represent a value being given to an input (that is, Inputs:InputName:Value).
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	static UE_API FString MakeInputValueKey(const FString& InputName);

	/**
	 * Makes an attribute key to represent a parameter being given to an input (that is, Inputs:InputName:Parameter).
	 * This is more relevant to Materials, but could be used to differentiate between constant values and parameters.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	static UE_API FString MakeInputParameterKey(const FString& InputName);

	/**
	 * From an attribute key associated with an input (that is, Inputs:InputName:Value), retrieves the input name.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	static UE_API FString MakeInputName(const FString& InputKey);
	
	/**
	 * Returns true if the attribute key is associated with an input (starts with "Inputs:").
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	static UE_API bool IsAnInput(const FString& AttributeKey);

	/**
	 * Returns true if the attribute key is an input that represents parameters (ends with ":Parameter").
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	static UE_API bool IsAParameter(const FString& AttributeKey);
 
	/**
	 * Checks whether a particular input exists on a given node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	static UE_API bool HasInput(const UInterchangeBaseNode* InterchangeNode, const FName& InInputName);
 
	/**
	 * Checks whether a particular input exists as a parameter on a given node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	static UE_API bool HasParameter(const UInterchangeBaseNode* InterchangeNode, const FName& InInputName);

	/**
	 * Retrieves the names of all the inputs for a given node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	static UE_API void GatherInputs(const UInterchangeBaseNode* InterchangeNode, TArray<FString>& OutInputNames);
 
	/**
	 * Adds an input connection attribute.
	 * @param InterchangeNode	The node to create the input on.
	 * @param InputName			The name to give to the input.
	 * @param ExpressionUid		The unique ID of the node to connect to the input.
	 * @return					True if the input connection was successfully added to the node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	static UE_API bool ConnectDefaultOuputToInput(UInterchangeBaseNode* InterchangeNode, const FString& InputName, const FString& ExpressionUid);
 
	/**
	 * Remove an input connection attribute.
	 * @param InterchangeNode	The node to disconnect the input from.
	 * @param InputName			The name of the input to disconnect.
	 * @return					True if the input connection was successfully removed from the node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	static UE_API bool DisconnectInput(UInterchangeBaseNode* InterchangeNode, const FString& InputName);
 
	/**
	 * Remove an input connection attribute by also retrieving the node unique id and the output name connected to the given input, if any.
	 * @param InterchangeNode	The node to disconnect the input from.
	 * @param InputName			The name of the input to disconnect.
	 * @param OutExpressionUid  The Uid of the node that was disconnected from the input.
	 * @param OutputName        The name of node output.
	 * @return					True if the input connection was successfully removed from the node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	static UE_API bool DisconnectInputFromOutputNode(UInterchangeBaseNode* InterchangeNode, const FString& InputName, FString& OutExpressionUid, FString& OutputName);

	/**
	 * Adds an input connection attribute.
	 * @param InterchangeNode	The node to create the input on.
	 * @param InputName			The name to give to the input.
	 * @param ExpressionUid		The unique ID of the node to connect to the input.
	 * @param OutputName		The name of the output from ExpressionUid to connect to the input.
	 * @return					True if the input connection was successfully added to the node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	static UE_API bool ConnectOuputToInputByName(UInterchangeBaseNode* InterchangeNode, const FString& InputName, const FString& ExpressionUid, const FString& OutputName);
	UE_DEPRECATED(5.3, "This function is replace by ConnectOuputToInputByName.")
	static bool ConnectOuputToInput(UInterchangeBaseNode* InterchangeNode, const FString& InputName, const FString& ExpressionUid, const FString& OutputName)
	{
		return ConnectOuputToInputByName(InterchangeNode, InputName, ExpressionUid, OutputName);
	}

	/**
	 * Adds an input connection attribute.
	 * @param InterchangeNode	The node to create the input on.
	 * @param InputName			The name to give to the input.
	 * @param ExpressionUid		The unique ID of the node to connect to the input.
	 * @param OutputIndex		The index of the output from ExpressionUid to connect to the input.
	 * @return					True if the input connection was successfully added to the node.
	 * OutputIndex is encoded in a string in the following pattern: ExpressionUid:OutputByIndex:FString::FromInt(OutputIndex)
	 * The index should be retrieved using UInterchangeShaderPortsAPI::GetOutputIndexFromName().
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	static UE_API bool ConnectOuputToInputByIndex(UInterchangeBaseNode* InterchangeNode, const FString& InputName, const FString& ExpressionUid, int32 OutputIndex);
 
	/**
	 * Retrieves the node unique id and the output name connected to a given input, if any.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	static UE_API bool GetInputConnection(const UInterchangeBaseNode* InterchangeNode, const FString& InputName, FString& OutExpressionUid, FString& OutputName);

	/**
	 * For an input with a value, returns the type of the stored value.
	 */
	static UE_API UE::Interchange::EAttributeTypes GetInputType(const UInterchangeBaseNode* InterchangeNode, const FString& InputName, bool bIsAParameter = false);

	/**
	 * Returns INDEX_NONE if OutputName is not an index.
	 */
	static UE_API int32 GetOutputIndexFromName(const FString& OutputName);

private:
	static UE_API const TCHAR* InputPrefix;
	static UE_API const TCHAR* InputSeparator;
	static UE_API const TCHAR* OutputByIndex;

	static UE_API const TCHAR* ParameterSuffix;
};

/**
 * A shader node is a named set of inputs and outputs. It can be connected to other shader nodes and finally to a shader graph input.
 */
UCLASS(MinimalAPI, BlueprintType)
class UInterchangeShaderNode : public UInterchangeBaseNode
{
	GENERATED_BODY()
 
public:
	/**
	 * Build and return a UID name for a shader node.
	 */
	static UE_API FString MakeNodeUid(const FStringView NodeName, const FStringView ParentNodeUid);

	/**
	 * Creates a new UInterchangeShaderNode and adds it to NodeContainer as a translated node.
	 */
	static UE_API UInterchangeShaderNode* Create(UInterchangeBaseNodeContainer* NodeContainer, const FStringView NodeName, const FStringView ParentNodeUid);

	UE_API virtual FString GetTypeName() const override;
 
public:

	/**
	 * Set the Float Attribute on the Shader Node. If bIsAParameter is set to true, it would be treated as a ScalarParameter
	 * when the Material Pipeline creates the materials. Otherwise it would be a constant expression in the shader graph.
	 * Note: It is assumed that the input name would be the parameter name when bIsAParameter is true.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool AddFloatInput(const FString& InputName, const float& AttributeValue, bool bIsAParameter = false);

	/**
	 * Set the Linear Color Attribute on the Shader Node. If bIsAParameter is set to true, it would be treated as a VectorParameter
	 * when the Material Pipeline creates the materials. Otherwise it would be a constant 3 vector expression in the shader graph.
	 * Note: It is assumed that the input name would be the parameter name when bIsAParameter is true.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool AddLinearColorInput(const FString& InputName, const FLinearColor& AttributeValue, bool bIsAParameter = false);

	/**
	 * Set the String Attribute on the Shader Node. If bIsAParameter is set to true, it would be treated as a overridable Texture
	 * or else it should be treated as a LUT Texture.
	 * Note: It is assumed that the input name would be the parameter name when bIsAParameter is true.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool AddStringInput(const FString& InputName, const FString& AttributeValue, bool bIsAParameter = false);


	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomShaderType(FString& AttributeValue) const;
 
	/**
	 * Sets which type of shader this nodes represents. Can be arbitrary or one of the predefined shader types.
	 * The material pipeline handling the shader node should be aware of the shader type that is being set here.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomShaderType(const FString& AttributeValue);
 
private:
	const UE::Interchange::FAttributeKey Macro_CustomShaderTypeKey = UE::Interchange::FAttributeKey(TEXT("ShaderType"));
};

/**
 * A function call shader node has a named set of inputs and outputs which corresponds to the inputs and outputs of the shader function it instances.
 */
UCLASS(MinimalAPI, BlueprintType)
class UInterchangeFunctionCallShaderNode : public UInterchangeShaderNode
{
	GENERATED_BODY()

public:
	UE_API virtual FString GetTypeName() const override;

public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomMaterialFunction(FString& AttributeValue) const;

	/**
	 * Set the unique id of the material function referenced by the function call expression.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomMaterialFunction(const FString& AttributeValue);

private:
	const UE::Interchange::FAttributeKey Macro_CustomMaterialFunctionKey = UE::Interchange::FAttributeKey(TEXT("MaterialFunction"));
};

/**
 * A shader graph has its own set of inputs on which shader nodes can be connected to.
 */
UCLASS(MinimalAPI, BlueprintType)
class UInterchangeShaderGraphNode : public UInterchangeShaderNode
{
	GENERATED_BODY()
 
public:
	/**
	 * Build and return a UID name for a shader graph node.
	 */
	static UE_API FString MakeNodeUid(const FStringView NodeName);

	/**
	 * Create a new UInterchangeShaderGraphNode and add it to NodeContainer as a translated node.
	 */
	static UE_API UInterchangeShaderGraphNode* Create(UInterchangeBaseNodeContainer* NodeContainer, const FStringView NodeName);

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	UE_API virtual FString GetTypeName() const override;

public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomTwoSided(bool& AttributeValue) const;
 
	/**
	 * Set if this shader graph should be rendered two-sided or not. Defaults to off.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomTwoSided(const bool& AttributeValue);

	/**
	* Forces two-sided even for Transmission materials.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomTwoSidedTransmission(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomTwoSidedTransmission(const bool& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomOpacityMaskClipValue(float& AttributeValue) const;

	/**
	 * The shader is transparent if its alpha value is lower than the clip value, or opaque if it is higher.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomOpacityMaskClipValue(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomIsAShaderFunction(bool& AttributeValue) const;

	/**
	 * Set whether this shader graph should be considered as a material (false), or a material function (true).
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomIsAShaderFunction(const bool& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomScreenSpaceReflections(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomScreenSpaceReflections(const bool& AttributeValue);

	/**
	 * Set the Blend Mode using EBlendMode to avoid a dependency on the Engine.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomBlendMode(int& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomBlendMode(int AttributeValue);

	/**
	 * Set the center of the displacement
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomDisplacementCenterMode(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomDisplacementCenterMode(float AttributeValue);

	/**
	 * Set the magnitude of the displacement
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool GetCustomDisplacementMagnitude(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Material")
	UE_API bool SetCustomDisplacementMagnitude(float AttributeValue);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(TwoSided)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(TwoSidedTransmission)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(OpacityMaskClipValue)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(IsAShaderFunction)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ScreenSpaceReflections)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(BlendMode)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(DisplacementCenter)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(DisplacementMagnitude)
};

#undef UE_API
