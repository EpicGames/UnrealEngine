// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeSpecularProfileNode.generated.h"

#define UE_API INTERCHANGENODES_API

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSpecularProfileNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:

	/**
	 * Build and return a UID name for a specular profile node.
	 */
	static UE_API FString MakeNodeUid(const FStringView& NodeName);

	/**
	 * Create a new UInterchangeSpecularProfileNode and add it to NodeContainer as a translated node.
	 */
	static UE_API UInterchangeSpecularProfileNode* Create(UInterchangeBaseNodeContainer* NodeContainer, const FStringView& NodeName);

public:

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	UE_API virtual FString GetTypeName() const override;

	/** Set/Get the format based on ESpecularProfileFormat */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SpecularProfile")
	UE_API bool SetCustomFormat(uint8 Format);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SpecularProfile")
	UE_API bool GetCustomFormat(uint8& Format) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SpecularProfile")
	UE_API bool GetCustomTexture(FString& TextureUid) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SpecularProfile")
	UE_API bool SetCustomTexture(const FString& TextureUid);

private:

	IMPLEMENT_NODE_ATTRIBUTE_KEY(Format)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Texture)
};

#undef UE_API
