// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeTextureNode.h"

#include "InterchangeTexture2DNode.generated.h"

class UInterchangeBaseNodeContainer;

//Interchange namespace
namespace UE::Interchange
{
	struct FTexture2DNodeStaticData : public FBaseNodeStaticData
	{
		static const FAttributeKey& GetBaseSourceBlocksKey()
		{
			static FAttributeKey StringKey(TEXT("SourceBlocks"));
			return StringKey;
		}
	};
}//ns UE::Interchange

UCLASS(BlueprintType, MinimalAPI)
class UInterchangeTexture2DNode : public UInterchangeTextureNode
{
	GENERATED_BODY()

public:

	/**
	 * Build and return a UID name for a texture 2D node.
	 */
	static INTERCHANGENODES_API FString MakeNodeUid(const FStringView NodeName);

	/**
	 * Creates a new UInterchangeTexture2DNode and adds it to NodeContainer as a translated node.
	 */
	static INTERCHANGENODES_API UInterchangeTexture2DNode* Create(UInterchangeBaseNodeContainer* NodeContainer, const FString& TextureNodeUid, const FString& TextureNodeName);

	virtual void PostInitProperties()
	{
		Super::PostInitProperties();
		SourceBlocks.Initialize(Attributes.ToSharedRef(), UE::Interchange::FTexture2DNodeStaticData::GetBaseSourceBlocksKey().ToString());
	}

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("Texture2DNode");
		return TypeName;
	}

	/**
	 * Override Serialize() to restore SourceBlocks on load.
	 */
	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);

		if (Ar.IsLoading() && bIsInitialized)
		{
			SourceBlocks.RebuildCache();
		}
	}

	//////////////////////////////////////////////////////////////////////////
	// UDIMs begin here
	// UDIM base texture use a different model for the source data

	/**
	 * Get the source blocks for the texture.
	 * If the map is empty, the texture is imported as a normal texture using the payload key.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture | UDIMs")
	TMap<int32, FString> GetSourceBlocks() const
	{
		return SourceBlocks.ToMap();
	}

	/**
	 * Set the source blocks for the texture.
	 * Using this suggests to the pipeline to consider this texture as a UDIM. The pipeline can choose whether to pass these blocks to the texture factory node.
	 * @param InSourceBlocks The blocks and their source image that compose the whole texture.
	 * The textures must be of the same format and use the same pixel format.
	 * The first block in the map is used to determine the accepted texture format and pixel format.
	 */
	void SetSourceBlocks(TMap<int32, FString>&& InSourceBlocks)
	{
		SourceBlocks = InSourceBlocks;
	}

	
	/**
	 * Set the source blocks for the texture.
	 * Using this suggests to the pipeline to consider this texture as a UDIM. The pipeline can choose whether to pass these blocks to the texture factory node.
	 * @param InSourceBlocks The blocks and their source image that compose the whole texture.
	 * The textures must be of the same format and use the same pixel format.
	 * The first block in the map is used to determine the accepted texture format and pixel format.
	 */
	void SetSourceBlocks(const TMap<int32, FString>& InSourceBlocks)
	{
		SourceBlocks = InSourceBlocks;
	}

	// UDIMs ends here
	//////////////////////////////////////////////////////////////////////////

protected:

	UE::Interchange::TMapAttributeHelper<int32, FString> SourceBlocks;

// Custom attributes
public:
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomWrapU(const EInterchangeTextureWrapMode& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(WrapU, EInterchangeTextureWrapMode);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomWrapU(EInterchangeTextureWrapMode& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(WrapU, EInterchangeTextureWrapMode);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomWrapV(const EInterchangeTextureWrapMode& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(WrapV, EInterchangeTextureWrapMode);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomWrapV(EInterchangeTextureWrapMode& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(WrapV, EInterchangeTextureWrapMode);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetForceLongLatCubemap(const bool& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ForceLongLatCubemap, bool);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetForceLongLatCubemap(bool& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(ForceLongLatCubemap, bool);
	}

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(WrapU);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(WrapV);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ForceLongLatCubemap);
};
