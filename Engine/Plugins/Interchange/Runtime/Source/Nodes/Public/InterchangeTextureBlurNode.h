// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeTexture2DNode.h"

#include "InterchangeTextureBlurNode.generated.h"

#define UE_API INTERCHANGENODES_API

class UInterchangeBaseNodeContainer;

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeTextureBlurNode : public UInterchangeTexture2DNode
{
	GENERATED_BODY()

public:

	/**
	 * Build and return a UID name for a texture 2D node.
	 */
	static UE_API FString MakeNodeUid(const FStringView NodeName);

	/**
	 * Creates a new UInterchangeTexture2DNode and adds it to NodeContainer as a translated node.
	 */
	static UE_API UInterchangeTextureBlurNode* Create(UInterchangeBaseNodeContainer* NodeContainer, const FStringView TextureNodeName);

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		return TEXT("TextureBlurNode");
	}
};

#undef UE_API
