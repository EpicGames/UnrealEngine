// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeTextureNode.generated.h"

//Interchange namespace
namespace UE
{
	namespace Interchange
	{
		struct FTextureNodeStaticData : public FBaseNodeStaticData
		{
			static const FAttributeKey& PayloadSourceFileKey()
			{
				static FAttributeKey AttributeKey(TEXT("__PayloadSourceFile__"));
				return AttributeKey;
			}
		};
	}//ns Interchange
}//ns UE

UENUM(BlueprintType)
enum class EInterchangeTextureWrapMode : uint8
{
	Wrap,
	Clamp,
	Mirror
};

UENUM(BlueprintType)
enum class EInterchangeTextureFilterMode : uint8
{
	Nearest,
	Bilinear,
	Trilinear,
	/** Use setting from the Texture Group. */
	Default
};

UENUM(BlueprintType)
enum class EInterchangeTextureColorSpace : uint8
{
	TCS_None = 0 UMETA(DisplayName = "None", ToolTip = "No explicit color space definition."),
	TCS_sRGB = 1 UMETA(DisplayName = "sRGB / Rec709", ToolTip = "sRGB / Rec709 (BT.709) color primaries, with D65 white point."),
	TCS_Rec2020 = 2 UMETA(DisplayName = "Rec2020", ToolTip = "Rec2020 (BT.2020) primaries with D65 white point."),
	TCS_ACESAP0 = 3 UMETA(DIsplayName = "ACES AP0", ToolTip = "ACES AP0 wide gamut primaries, with D60 white point."),
	TCS_ACESAP1 = 4 UMETA(DIsplayName = "ACES AP1 / ACEScg", ToolTip = "ACES AP1 / ACEScg wide gamut primaries, with D60 white point."),
	TCS_P3DCI = 5 UMETA(DisplayName = "P3DCI", ToolTip = "P3 (Theater) primaries, with DCI Calibration white point."),
	TCS_P3D65 = 6 UMETA(DisplayName = "P3D65", ToolTip = "P3 (Display) primaries, with D65 white point."),
	TCS_REDWideGamut = 7 UMETA(DisplayName = "RED Wide Gamut", ToolTip = "RED Wide Gamut primaries, with D65 white point."),
	TCS_SonySGamut3 = 8 UMETA(DisplayName = "Sony S-Gamut3", ToolTip = "Sony S-Gamut/S-Gamut3 primaries, with D65 white point."),
	TCS_SonySGamut3Cine = 9 UMETA(DisplayName = "Sony S-Gamut3 Cine", ToolTip = "Sony S-Gamut3 Cine primaries, with D65 white point."),
	TCS_AlexaWideGamut = 10 UMETA(DisplayName = "Alexa Wide Gamut", ToolTip = "Alexa Wide Gamut primaries, with D65 white point."),
	TCS_CanonCinemaGamut = 11 UMETA(DisplayName = "Canon Cinema Gamut", ToolTip = "Canon Cinema Gamut primaries, with D65 white point."),
	TCS_GoProProtuneNative = 12 UMETA(DisplayName = "GoPro Protune Native", ToolTip = "GoPro Protune Native primaries, with D65 white point."),
	TCS_PanasonicVGamut = 13 UMETA(DisplayName = "Panasonic V-Gamut", ToolTip = "Panasonic V-Gamut primaries, with D65 white point."),
	TCS_Custom = 99 UMETA(DisplayName = "Custom", ToolTip = "User defined color space and white point."),
	TCS_MAX,
};

UCLASS(BlueprintType, Abstract, MinimalAPI)
class UInterchangeTextureNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:
	/**
	 * Build and return a UID name for a texture node.
	 */
	static FString MakeNodeUid(const FStringView NodeName)
	{
		return FString(UInterchangeBaseNode::HierarchySeparator) +  TEXT("Textures") + FString(UInterchangeBaseNode::HierarchySeparator) + NodeName;
	}

	/**
	 * Utility function to build a unique and deterministic UID and Name for a given texture file path.
	 */
	static INTERCHANGENODES_API bool ExtractNodeUidAndNameFromFilePath(const FStringView TextureFilePath, FString& OutNodeUid, FString& OutNodeName);

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("TextureNode");
		return TypeName;
	}

	virtual FName GetIconName() const override
	{
		return FName(TEXT("ClassIcon.Texture2D"));
	}

#if WITH_EDITOR
	virtual FString GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override
	{
		FString KeyDisplayName = NodeAttributeKey.ToString();
		if (NodeAttributeKey == UE::Interchange::FTextureNodeStaticData::PayloadSourceFileKey())
		{
			return KeyDisplayName = TEXT("Payload Source Key");
		}
		return Super::GetKeyDisplayName(NodeAttributeKey);
	}
#endif //WITH_EDITOR

	virtual FGuid GetHash() const override
	{
		return Attributes->GetStorageHash();
	}

	/** Texture node Interface Begin */
	virtual const TOptional<FString> GetPayLoadKey() const
	{
		if (!Attributes->ContainAttribute(UE::Interchange::FTextureNodeStaticData::PayloadSourceFileKey()))
		{
			return TOptional<FString>();
		}
		UE::Interchange::FAttributeStorage::TAttributeHandle<FString> AttributeHandle = Attributes->GetAttributeHandle<FString>(UE::Interchange::FTextureNodeStaticData::PayloadSourceFileKey());
		if (!AttributeHandle.IsValid())
		{
			return TOptional<FString>();
		}
		FString PayloadKey;
		UE::Interchange::EAttributeStorageResult Result = AttributeHandle.Get(PayloadKey);
		if (!IsAttributeStorageResultSuccess(Result))
		{
			LogAttributeStorageErrors(Result, TEXT("UInterchangeTextureNode.GetPayLoadKey"), UE::Interchange::FTextureNodeStaticData::PayloadSourceFileKey());
			return TOptional<FString>();
		}
		return TOptional<FString>(PayloadKey);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	virtual void SetPayLoadKey(const FString& PayloadKey)
	{
		UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FTextureNodeStaticData::PayloadSourceFileKey(), PayloadKey);
		if (!IsAttributeStorageResultSuccess(Result))
		{
			LogAttributeStorageErrors(Result, TEXT("UInterchangeTextureNode.SetPayLoadKey"), UE::Interchange::FTextureNodeStaticData::PayloadSourceFileKey());
		}
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomSRGB(bool& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(SRGB, bool);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomSRGB(const bool& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SRGB, bool)
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustombFlipGreenChannel(bool& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(bFlipGreenChannel, bool);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustombFlipGreenChannel(const bool& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(bFlipGreenChannel, bool)
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomFilter(EInterchangeTextureFilterMode& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(Filter, EInterchangeTextureFilterMode);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomFilter(const EInterchangeTextureFilterMode& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Filter, EInterchangeTextureFilterMode)
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool GetCustomColorSpace(EInterchangeTextureColorSpace& AttributeValue) const
	{
		IMPLEMENT_NODE_ATTRIBUTE_GETTER(ColorSpace, EInterchangeTextureColorSpace);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	bool SetCustomColorSpace(const EInterchangeTextureColorSpace& AttributeValue)
	{
		IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ColorSpace, EInterchangeTextureColorSpace)
	}

private:

	IMPLEMENT_NODE_ATTRIBUTE_KEY(SRGB)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(bFlipGreenChannel)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Filter)
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ColorSpace)
};
