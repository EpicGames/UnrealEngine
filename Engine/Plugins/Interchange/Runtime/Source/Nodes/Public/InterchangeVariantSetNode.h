// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeVariantSetNode.generated.h"

#define UE_API INTERCHANGENODES_API

/**
 * Class to represent a set of variants.
 */
UCLASS(MinimalAPI, BlueprintType)
class UInterchangeVariantSetNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

	UE_API UInterchangeVariantSetNode();

public:
	static FStringView StaticAssetTypeName()
	{
		return TEXT("VariantSet");
	}

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("VariantSetNode");
		return TypeName;
	}

	/**
	 * Retrieve the text that is displayed in the UI for this VariantSet.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | VariantSet")
	UE_API bool GetCustomDisplayText(FString& AttributeValue) const;

	/**
	 * Set the text to be displayed in the UI for this VariantSet.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | VariantSet")
	UE_API bool SetCustomDisplayText(const FString& AttributeValue);

	/**
	 * Get the payload key needed to retrieve the variants for this VariantSet.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | VariantSet")
	UE_API bool GetCustomVariantsPayloadKey(FString& PayloadKey) const;

	/**
	 * Set the payload key needed to retrieve the variants for this VariantSet.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | VariantSet")
	UE_API bool SetCustomVariantsPayloadKey(const FString& PayloadKey);

	/**
	 * Retrieve the number of translated node's unique IDs for this VariantSet.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SceneVariantSet")
	UE_API int32 GetCustomDependencyUidCount() const;

	/**
	 * Retrieve all the translated node's unique IDs for this VariantSet.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SceneVariantSet")
	UE_API void GetCustomDependencyUids(TArray<FString>& OutDependencyUids) const;

	/**
	 * Retrieve the specified translated node's unique ID for this VariantSet.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SceneVariantSet")
	UE_API void GetCustomDependencyUid(const int32 Index, FString& OutDependencyUid) const;

	/**
	 * Add the specified translated node's unique ID to this VariantSet.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SceneVariantSet")
	UE_API bool AddCustomDependencyUid(const FString& DependencyUid);

	/**
	 * Remove the specified translated node's unique ID from this VariantSet.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SceneVariantSet")
	UE_API bool RemoveCustomDependencyUid(const FString& DependencyUid);

private:
	const UE::Interchange::FAttributeKey Macro_CustomDisplayTextKey = UE::Interchange::FAttributeKey(TEXT("DisplayText"));
	const UE::Interchange::FAttributeKey Macro_CustomVariantsPayloadKey = UE::Interchange::FAttributeKey(TEXT("VariantsPayload"));
	UE::Interchange::TArrayAttributeHelper<FString> CustomDependencyUids;
};

/**
 * Class to represent a set of VariantSet nodes
 */
UCLASS(MinimalAPI, BlueprintType)
class UInterchangeSceneVariantSetsNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

	UE_API UInterchangeSceneVariantSetsNode();

public:
	static FStringView StaticAssetTypeName()
	{
		return TEXT("SceneVariantSet");
	}

	/**
	* Return the node type name of the class. This is used when reporting errors.
	*/
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("SceneVariantSetNode");
		return TypeName;
	}

	/**
	 * Retrieve the number of VariantSets for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SceneVariantSet")
	UE_API int32 GetCustomVariantSetUidCount() const;

	/**
	 * Retrieve all the VariantSets' unique ids for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SceneVariantSet")
	UE_API void GetCustomVariantSetUids(TArray<FString>& OutVariantUids) const;

	/**
	 * Retrieve the specified VariantSet's unique id for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SceneVariantSet")
	UE_API void GetCustomVariantSetUid(const int32 Index, FString& OutVariantUid) const;

	/**
	 * Add the specified VariantSet's unique id to this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SceneVariantSet")
	UE_API bool AddCustomVariantSetUid(const FString& VariantUid);

	/**
	 * Remove the specified VariantSet's unique id from this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | SceneVariantSet")
	UE_API bool RemoveCustomVariantSetUid(const FString& VariantUid);

private:
	UE::Interchange::TArrayAttributeHelper<FString> CustomVariantSetUids;
};

#undef UE_API
