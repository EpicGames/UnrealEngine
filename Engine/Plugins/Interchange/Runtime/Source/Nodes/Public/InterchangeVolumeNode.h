// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeVolumeNode.generated.h"

#define UE_API INTERCHANGENODES_API

namespace UE::Interchange
{
	struct FInterchangeVolumeNodeStaticData : public FBaseNodeStaticData
	{
		static UE_API const FAttributeKey& GetCustomGridDependeciesBaseKey();
		static UE_API const FAttributeKey& GetCustomFrameIndicesInAnimationBaseKey();
	};
}

/**
 * Represents a file that contains volume data in the form of (potentially multiple) grids,
 * which are represented as UInterchangeVolumeGridNode dependencies
 */
UCLASS(MinimalAPI, BlueprintType)
class UInterchangeVolumeNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:
	UE_API UInterchangeVolumeNode();

	UE_API virtual FString GetTypeName() const override;

public:
	/**
	 * Gets the filename of the file with volume data (e.g. "C:/MyFolder/File.vdb").
	 *
	 * This is stored on the translated node as well as the source data, as a volume import may discover other
	 * additional files in order to handle animated volume imports.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool GetCustomFileName(FString& AttributeValue) const;

	/**
	 * Sets the filename of the file with volume data (e.g. "C:/MyFolder/File.vdb").
	 *
	 * This is stored on the translated node as well as the source data, as a volume import may discover other
	 * additional files in order to handle animated volume imports.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool SetCustomFileName(const FString& AttributeValue);

	/**
	 * Gets the number of UInterchangeVolumeGridNodes declared as dependencies by this volume node
	 * (in other words, returns the number of volume grids contained in this file)
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API int32 GetCustomGridDependeciesCount() const;

	/** Gets the Node IDs of UInterchangeVolumeGridNodes declared as dependencies by this volume node */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API void GetCustomGridDependecies(TArray<FString>& OutDependencies) const;

	/** Gets the Node ID of an UInterchangeVolumeGridNode dependency */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API void GetCustomGridDependency(const int32 Index, FString& OutDependency) const;

	/** Sets the Node ID of an UInterchangeVolumeGridNode dependency */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool AddCustomGridDependency(const FString& DependencyUid);

	/** Removes the Node ID of an UInterchangeVolumeGridNode dependency */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool RemoveCustomGridDependency(const FString& DependencyUid);

	/**
	 * Gets an identifier that is shared by all volume nodes that correspond to the same animation
	 * (i.e. every volume node within the animation will have the same AnimationID)
	 */
	 UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	 UE_API bool GetCustomAnimationID(FString& AttributeValue) const;

	 /**
	  * Sets an identifier that is shared by all volume nodes that correspond to the same animation
	  * (i.e. every volume node within the animation will have the same AnimationID)
	  */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool SetCustomAnimationID(const FString& AttributeValue);

	/**
	 * Get the frame indices for which this volume is displayed within the animation that it belongs to, if any
	 * (e.g. if this had [2, 3] then frames 2 and 3 of the animation with AnimationID should display this volume)
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API void GetCustomFrameIndicesInAnimation(TArray<int32>& OutAnimationIndices) const;

	/** Gets one of the frame indices for which this volume is displayed within the animation that it belongs to, if any */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API void GetCustomFrameIndexInAnimation(int32 IndexIndex, int32& OutIndex) const;

	/** Adds a frame index for which this volume is displayed within the animation that it belongs to, if any */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool AddCustomFrameIndexInAnimation(int32 Index);

	/** Removes a frame index for which this volume is displayed within the animation that it belongs to, if any */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool RemoveCustomFrameIndexInAnimation(int32 Index);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(FileName);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AnimationID);

	UE::Interchange::TArrayAttributeHelper<FString> GridDependencies;
	UE::Interchange::TArrayAttributeHelper<int32> IndexInVolumeAnimation;
};

UENUM(Blueprintable)
enum class EVolumeGridElementType : uint8
{
	Unknown,
	Half,
	Float,
	Double
};

/**
 * Represents a single grid (essentially a 3d texture) within a volumetric file
 */
UCLASS(MinimalAPI, BlueprintType)
class UInterchangeVolumeGridNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:
	UE_API virtual FString GetTypeName() const override;

public:
	/** Gets the datatype of the value of each voxel in the grid */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Texture")
	UE_API bool GetCustomElementType(EVolumeGridElementType& AttributeValue) const;

	/** Sets the datatype of the value of each voxel in the grid */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool SetCustomElementType(const EVolumeGridElementType& AttributeValue);

	/** Gets the number of components of each voxel of the grid (e.g. 3 components for a vector grid) */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool GetCustomNumComponents(int32& NumComponents) const;

	/** Sets the number of components of each voxel of the grid (e.g. 3 components for a vector grid) */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool SetCustomNumComponents(const int32& NumComponents);

	/** Gets the grid transform contained in the volume file */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool GetCustomGridTransform(FTransform& AttributeValue) const;

	/** Sets the grid transform contained in the volume file */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool SetCustomGridTransform(const FTransform& AttributeValue);

	/** Gets the min X, Y and Z of the grid's active axis-aligned bounding box. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool GetCustomGridActiveAABBMin(FIntVector& AttributeValue) const;

	/** Sets the min X, Y and Z of the grid's active axis-aligned bounding box. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool SetCustomGridActiveAABBMin(const FIntVector& AttributeValue);

	/** Gets the max X, Y and Z of the grid's active axis-aligned bounding box. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool GetCustomGridActiveAABBMax(FIntVector& AttributeValue) const;

	/** Sets the max X, Y and Z of the grid's active axis-aligned bounding box. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool SetCustomGridActiveAABBMax(const FIntVector& AttributeValue);

	/** Gets the size of the grid, in voxels. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool GetCustomGridActiveDimensions(FIntVector& AttributeValue) const;

	/** Sets the size of the grid, in voxels. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Volume")
	UE_API bool SetCustomGridActiveDimensions(const FIntVector& AttributeValue);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ElementType);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(NumComponents);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(GridTransform);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(GridActiveAABBMin);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(GridActiveAABBMax);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(GridActiveDim);
};

#undef UE_API
