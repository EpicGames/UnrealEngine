// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeTextureNode.h"

#include "InterchangeVolumeTextureNode.generated.h"


UCLASS(MinimalAPI, BlueprintType)
class UInterchangeVolumeTextureNode : public UInterchangeTextureNode
{
	GENERATED_BODY()

public:

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("VolumeTextureNode");
		return TypeName;
	}
};
