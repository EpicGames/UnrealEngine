// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class InterchangeCommonParser : ModuleRules
	{
		public InterchangeCommonParser(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"InterchangeCore",
					"InterchangeNodes",
				}
			);

			if (Target.bCompileAgainstEngine)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[] {
						"Engine",
						"Json",
					}
				);
			}
		}
	}
}
