// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeCommonAnimationPayload.h"

#include "CoreMinimal.h"
#if WITH_ENGINE
#include "Animation/AnimTypes.h"
#endif

#include "InterchangeHelper.h"

#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeCommonAnimationPayload)

namespace UE::Interchange
{
	FString Private::HashString(const FString& String)
	{
		TArray<uint8> TempBytes;
		TempBytes.Reserve(64);
		//The archive is flagged as persistent so that machines of different endianness produce identical binary results.
		FMemoryWriter Ar(TempBytes, /*bIsPersistent=*/ true);
		//Hack because serialization do not support const
		FString& NonConst = *const_cast<FString*>(&String);
		Ar << NonConst;
		FSHA1 Sha;
		Sha.Update(TempBytes.GetData(), TempBytes.Num() * TempBytes.GetTypeSize());
		Sha.Final();
		// Retrieve the hash and use it to construct a pseudo-GUID.
		uint32 Hash[5];
		Sha.GetHash((uint8*)Hash);
		FGuid Guid = FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);
		return Guid.ToString(EGuidFormats::Base36Encoded);
	}

	void FAnimationPayloadData::SerializeBaked(FArchive& Ar)
	{
		Ar << BakeFrequency;
		Ar << RangeStartTime;
		Ar << RangeEndTime;
		Ar << Transforms;
	}

	void FAnimationPayloadData::CalculateDataFor(const EInterchangeAnimationPayLoadType& ToType, const FTransform& DefaultTransform)
	{
#if WITH_ENGINE
		if (PayloadKey.Type == EInterchangeAnimationPayLoadType::CURVE
			&& ToType == EInterchangeAnimationPayLoadType::STEPCURVE)
		{

			StepCurves.Reserve(Curves.Num());
			for (const FRichCurve& RichCurve : Curves)
			{
				FInterchangeStepCurve& StepCurve = StepCurves.AddDefaulted_GetRef();
				const int32 KeyCount = RichCurve.GetNumKeys();
				StepCurve.KeyTimes.AddZeroed(KeyCount);
				TArray<float> KeyValues;
				KeyValues.AddZeroed(KeyCount);
				int32 KeyIndex = 0;
				for (FKeyHandle KeyHandle = RichCurve.GetFirstKeyHandle(); KeyHandle != FKeyHandle::Invalid(); KeyHandle = RichCurve.GetNextKey(KeyHandle), ++KeyIndex)
				{
					StepCurve.KeyTimes[KeyIndex] = RichCurve.GetKeyTime(KeyHandle);
					KeyValues[KeyIndex] = RichCurve.GetKeyValue(KeyHandle);
				}
				StepCurve.FloatKeyValues = MoveTemp(KeyValues);
			}
			AdditionalSupportedType = ToType;
		}
		else if (PayloadKey.Type == EInterchangeAnimationPayLoadType::CURVE
				 && ToType == EInterchangeAnimationPayLoadType::BAKED)
		{
			if (Curves.Num() != 9)
			{
				return;
			}

			//calculate RangeEnd -> 'BakeKeyCount'
			RangeEndTime = -FLT_MAX;
			for (const FRichCurve& Curve : Curves)
			{
				const TArray<FRichCurveKey>& CurveKeys = Curve.GetConstRefOfKeys();
				for (const FRichCurveKey& CurveKey : CurveKeys)
				{
					if (RangeEndTime < CurveKey.Time)
					{
						RangeEndTime = CurveKey.Time;
					}
				}
			}
			if (RangeEndTime < 0)
			{
				RangeEndTime = 0;
			}

			const double BakeInterval = 1.0 / (BakeFrequency != 0.0 ? BakeFrequency : DEFAULT_SAMPLERATE);
			const double SequenceLength = FMath::Max<double>(FMath::Max<double>(RangeEndTime - RangeStartTime, BakeInterval), MINIMUM_ANIMATION_LENGTH);
			int32 BakeKeyCount = FMath::RoundToInt32(SequenceLength * BakeFrequency) + 1;

			auto EvaluateCurve = [this](const int32& CurveIndex, double CurrentTime, float DefaultValue)
			{
				if (Curves[CurveIndex].IsEmpty())
				{
					return DefaultValue;
				}
				else
				{
					return Curves[CurveIndex].Eval(CurrentTime);
				}
			};

			FVector DefaultTranslation = DefaultTransform.GetTranslation();
			FVector DefaultEuler = DefaultTransform.GetRotation().Euler();
			FVector DefaultScale = DefaultTransform.GetScale3D();

			double CurrentTime = 0;
			for (int32 BakeIndex = 0; BakeIndex < BakeKeyCount; BakeIndex++, CurrentTime += BakeInterval)
			{
				FVector Translation;
				Translation.X = EvaluateCurve(0, CurrentTime, DefaultTranslation.X);
				Translation.Y = EvaluateCurve(1, CurrentTime, DefaultTranslation.Y);
				Translation.Z = EvaluateCurve(2, CurrentTime, DefaultTranslation.Z);

				FVector RotationEuler;
				RotationEuler.X = EvaluateCurve(3, CurrentTime, DefaultEuler.X);
				RotationEuler.Y = EvaluateCurve(4, CurrentTime, DefaultEuler.Y);
				RotationEuler.Z = EvaluateCurve(5, CurrentTime, DefaultEuler.Z);

				FVector Scale3d;
				Scale3d.X = EvaluateCurve(6, CurrentTime, DefaultScale.X);
				Scale3d.Y = EvaluateCurve(7, CurrentTime, DefaultScale.Y);
				Scale3d.Z = EvaluateCurve(8, CurrentTime, DefaultScale.Z);

				FTransform AnimKeyTransform(FRotator::MakeFromEuler(RotationEuler), Translation, Scale3d);

				Transforms.Add(AnimKeyTransform);
			}

			AdditionalSupportedType = ToType;
		}
#endif
	}

	FString FAnimationPayloadQuery::GetHashString() const
	{
		if (!HashStringCache.IsSet())
		{
			FString ResultPayloadUniqueId = PayloadKey.UniqueId + FString::FromInt(TimeDescription.GetHash());
			HashStringCache = Private::HashString(ResultPayloadUniqueId);
		}

		return HashStringCache.GetValue();
	}

	FString FAnimationPayloadQuery::ToJson() const
	{
		TSharedPtr<FJsonObject> QueryObject = MakeShared<FJsonObject>();

		QueryObject->SetStringField(TEXT("SceneNodeUniqueID"), SceneNodeUniqueID);
		
		QueryObject->SetStringField(TEXT("PayloadKey.UniqueID"), PayloadKey.UniqueId);
		QueryObject->SetNumberField(TEXT("PayloadKey.Type"), (uint8)PayloadKey.Type);

		QueryObject->SetNumberField(TEXT("TimeDescription.BakeFrequency"), TimeDescription.BakeFrequency);
		QueryObject->SetNumberField(TEXT("TimeDescription.RangeStartSecond"), TimeDescription.RangeStartSecond);
		QueryObject->SetNumberField(TEXT("TimeDescription.RangeStopSecond"), TimeDescription.RangeStopSecond);
		
		FString JsonString;
		TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&JsonString);
		if (!FJsonSerializer::Serialize(QueryObject.ToSharedRef(), JsonWriter))
		{
			// Error creating the json string 
			return FString();
		}

		return JsonString;
	}

	void FAnimationPayloadQuery::FromJson(const FString& JsonString)
	{
		TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(JsonString);

		TSharedPtr<FJsonObject> QueryObject;
		if (!FJsonSerializer::Deserialize(Reader, QueryObject) || !QueryObject.IsValid())
		{
			// Cannot read the json file
			return;
		}

		QueryObject->TryGetStringField(TEXT("SceneNodeUniqueID"), SceneNodeUniqueID);

		QueryObject->TryGetStringField(TEXT("PayloadKey.UniqueId"), PayloadKey.UniqueId);
		uint8 Type;
		if (QueryObject->TryGetNumberField(TEXT("PayloadKey.Type"), Type))
		{
			PayloadKey.Type = EInterchangeAnimationPayLoadType(Type);
		}

		QueryObject->TryGetNumberField(TEXT("TimeDescription.BakeFrequency"), TimeDescription.BakeFrequency);
		QueryObject->TryGetNumberField(TEXT("TimeDescription.RangeStartSecond"), TimeDescription.RangeStartSecond);
		QueryObject->TryGetNumberField(TEXT("TimeDescription.RangeStopSecond"), TimeDescription.RangeStopSecond);
	}

	FString FAnimationPayloadQuery::ToJson(const TArray<FAnimationPayloadQuery>& Queries)
	{
		TArray<TSharedPtr<FJsonValue>> QueriesJsonArray;
		for (const FAnimationPayloadQuery& Query : Queries)
		{
			
			QueriesJsonArray.Add(MakeShared<FJsonValueString>(Query.ToJson()));
		}

		FString JsonString;
		TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&JsonString);
		if (!FJsonSerializer::Serialize(QueriesJsonArray, JsonWriter))
		{
			// Error creating the json string 
			return FString();
		}
		return JsonString;
	}

	void FAnimationPayloadQuery::FromJson(const FString& JsonString, TArray<FAnimationPayloadQuery>& Queries)
	{
		TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(JsonString);

		TArray<TSharedPtr<FJsonValue>> JsonValueArray;
		if (!FJsonSerializer::Deserialize(Reader, JsonValueArray))
		{
			// Cannot read the json file
			return;
		}

		Queries.Reserve(JsonValueArray.Num());

		for (size_t QueryIndex = 0; QueryIndex < JsonValueArray.Num(); QueryIndex++)
		{
			FString QueryJsonString = JsonValueArray[QueryIndex]->AsString();

			FAnimationPayloadQuery Query;
			Query.FromJson(QueryJsonString);
			Queries.Add(Query);
		}
	}
}

#if WITH_ENGINE
void FInterchangeCurveKey::ToRichCurveKey(FRichCurveKey& RichCurveKey) const
{
	RichCurveKey.Time = Time;
	RichCurveKey.Value = Value;
	switch (InterpMode)
	{
		case EInterchangeCurveInterpMode::Constant:
			RichCurveKey.InterpMode = ERichCurveInterpMode::RCIM_Constant;
			break;
		case EInterchangeCurveInterpMode::Cubic:
			RichCurveKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
			break;
		case EInterchangeCurveInterpMode::Linear:
			RichCurveKey.InterpMode = ERichCurveInterpMode::RCIM_Linear;
			break;
		default:
			RichCurveKey.InterpMode = ERichCurveInterpMode::RCIM_None;
			break;
	}
	switch (TangentMode)
	{
		case EInterchangeCurveTangentMode::Auto:
			RichCurveKey.TangentMode = ERichCurveTangentMode::RCTM_Auto;
			break;
		case EInterchangeCurveTangentMode::Break:
			RichCurveKey.TangentMode = ERichCurveTangentMode::RCTM_Break;
			break;
		case EInterchangeCurveTangentMode::User:
			RichCurveKey.TangentMode = ERichCurveTangentMode::RCTM_User;
			break;
		default:
			RichCurveKey.TangentMode = ERichCurveTangentMode::RCTM_None;
			break;
	}
	switch (TangentWeightMode)
	{
		case EInterchangeCurveTangentWeightMode::WeightedArrive:
			RichCurveKey.TangentWeightMode = ERichCurveTangentWeightMode::RCTWM_WeightedArrive;
			break;
		case EInterchangeCurveTangentWeightMode::WeightedBoth:
			RichCurveKey.TangentWeightMode = ERichCurveTangentWeightMode::RCTWM_WeightedBoth;
			break;
		case EInterchangeCurveTangentWeightMode::WeightedLeave:
			RichCurveKey.TangentWeightMode = ERichCurveTangentWeightMode::RCTWM_WeightedLeave;
			break;
		default:
			RichCurveKey.TangentWeightMode = ERichCurveTangentWeightMode::RCTWM_WeightedNone;
			break;
	}
	RichCurveKey.ArriveTangent = ArriveTangent;
	RichCurveKey.ArriveTangentWeight = ArriveTangentWeight;
	RichCurveKey.LeaveTangent = LeaveTangent;
	RichCurveKey.LeaveTangentWeight = LeaveTangentWeight;

}
#endif //WITH_ENGINE

void FInterchangeCurveKey::Serialize(FArchive& Ar)
{
	Ar << InterpMode;
	Ar << TangentMode;
	Ar << TangentWeightMode;
	Ar << Time;
	Ar << Value;
	Ar << ArriveTangent;
	Ar << ArriveTangentWeight;
	Ar << LeaveTangent;
	Ar << LeaveTangentWeight;
}

#if WITH_ENGINE
void FInterchangeCurve::ToRichCurve(FRichCurve& OutRichCurve) const
{
	OutRichCurve.Keys.Reserve(Keys.Num());
	for (const FInterchangeCurveKey& CurveKey : Keys)
	{
		FKeyHandle RichCurveKeyHandle = OutRichCurve.AddKey(CurveKey.Time, CurveKey.Value);
		CurveKey.ToRichCurveKey(OutRichCurve.GetKey(RichCurveKeyHandle));
	}
	OutRichCurve.AutoSetTangents();
}
#endif //WITH_ENGINE

void FInterchangeCurve::Serialize(FArchive& Ar)
{
	Ar << Keys;
}

void FInterchangeStepCurve::RemoveRedundantKeys(float Threshold)
{
	const int32 KeyCount = KeyTimes.Num();
	if (KeyCount < 2)
	{
		//Nothing to optimize
		return;
	}

	if (FloatKeyValues.IsSet())
	{
		InternalRemoveRedundantKey<float>(FloatKeyValues.GetValue(), [Threshold](const float& ValueA, const float& ValueB)
			{
				return FMath::IsNearlyEqual(ValueA, ValueB, Threshold);
			});
	}
	else if (IntegerKeyValues.IsSet())
	{
		InternalRemoveRedundantKey<int32>(IntegerKeyValues.GetValue(), [](const int32& ValueA, const int32& ValueB)
			{
				return ValueA == ValueB;
			});
	}
	else if (StringKeyValues.IsSet())
	{
		InternalRemoveRedundantKey<FString>(StringKeyValues.GetValue(), [](const FString& ValueA, const FString& ValueB)
			{
				return ValueA == ValueB;
			});
	}
	else if(BooleanKeyValues.IsSet())
	{
		InternalRemoveRedundantKey<bool>(BooleanKeyValues.GetValue(), [](bool ValueA, bool ValueB)
		{
			return ValueA == ValueB;
		});
	}
	else if(ByteKeyValues.IsSet())
	{
		InternalRemoveRedundantKey<uint8>(ByteKeyValues.GetValue(), [](uint8 ValueA, uint8 ValueB)
		{
			return ValueA == ValueB;
		});
	}
}

void FInterchangeStepCurve::Serialize(FArchive& Ar)
{
	Ar << KeyTimes;
	Ar << FloatKeyValues;
	Ar << ByteKeyValues;
	Ar << BooleanKeyValues;
	Ar << IntegerKeyValues;
	Ar << StringKeyValues;
}

template<typename ValueType>
void FInterchangeStepCurve::InternalRemoveRedundantKey(TArray<ValueType>& Values, TFunctionRef<bool(const ValueType& ValueA, const ValueType& ValueB)> CompareFunction)
{
	const int32 KeyCount = Values.Num();
	if (KeyCount == 0)
	{
		return;
	}

	TArray<float> NewKeyTimes;
	NewKeyTimes.Reserve(KeyCount);
	TArray<ValueType> NewValues;
	NewValues.Reserve(KeyCount);
	ValueType LastValue = Values[0];
	//Add the first key
	NewKeyTimes.Add(KeyTimes[0]);
	NewValues.Add(Values[0]);
	for (int32 KeyIndex = 1; KeyIndex < KeyCount; ++KeyIndex)
	{
		//Only add the not equal keys
		if (!CompareFunction(LastValue, Values[KeyIndex]))
		{
			LastValue = Values[KeyIndex];
			NewKeyTimes.Add(KeyTimes[KeyIndex]);
			NewValues.Add(Values[KeyIndex]);
		}
	}
	NewKeyTimes.Shrink();
	NewValues.Shrink();
	KeyTimes = MoveTemp(NewKeyTimes);
	Values = MoveTemp(NewValues);
}

template void FInterchangeStepCurve::InternalRemoveRedundantKey<float>(TArray<float>& Values, TFunctionRef<bool(const float& ValueA, const float& ValueB)> CompareFunction);
template void FInterchangeStepCurve::InternalRemoveRedundantKey<int32>(TArray<int32>& Values, TFunctionRef<bool(const int32& ValueA, const int32& ValueB)> CompareFunction);
template void FInterchangeStepCurve::InternalRemoveRedundantKey<FString>(TArray<FString>& Values, TFunctionRef<bool(const FString& ValueA, const FString& ValueB)> CompareFunction);