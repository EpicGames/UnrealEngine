// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeCommonParserModule.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"



FInterchangeCommonParserModule& FInterchangeCommonParserModule::Get()
{
	return FModuleManager::LoadModuleChecked< FInterchangeCommonParserModule >(INTERCHANGECOMMONPARSER_MODULE_NAME);
}

bool FInterchangeCommonParserModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded(INTERCHANGECOMMONPARSER_MODULE_NAME);
}

void FInterchangeCommonParserModule::StartupModule()
{
}

IMPLEMENT_MODULE(FInterchangeCommonParserModule, InterchangeCommonParser);
