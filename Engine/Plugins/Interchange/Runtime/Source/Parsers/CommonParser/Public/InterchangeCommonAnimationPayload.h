// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Transform.h"
#include "Serialization/Archive.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "InterchangeAnimationTrackSetNode.h"

#if WITH_ENGINE
#include "Curves/RichCurve.h"
#endif

#include "InterchangeCommonAnimationPayload.generated.h"

namespace UE::Interchange
{
	namespace Private
	{
		INTERCHANGECOMMONPARSER_API FString HashString(const FString& String);
	};

	struct FAnimationPayloadData
	{
		FString SceneNodeUniqueID;

#if WITH_ENGINE
		//CURVE
		TArray<FRichCurve> Curves;

		//In-between blend shape data
		TArray<FString> InbetweenCurveNames;
		TArray<float> InbetweenFullWeights;
#endif
		//STEP CURVE
		TArray<FInterchangeStepCurve> StepCurves;

		//BAKED TRANSFORMS
		/**
		 * This payload class part is used to get a scene node bake transform payload
		 * The translator should bake the scene node transform using the bake settings provided by the factory.
		 */
		double BakeFrequency = 30.0;
		double RangeStartTime = 0.0;
		double RangeEndTime = 1.0 / BakeFrequency;
		TArray<FTransform> Transforms;

		//PayloadKey related:
		FInterchangeAnimationPayLoadKey PayloadKey;
		EInterchangeAnimationPayLoadType AdditionalSupportedType = EInterchangeAnimationPayLoadType::NONE;

		//
		FAnimationPayloadData(const FString& InSceneNodeUID, const FInterchangeAnimationPayLoadKey& InPayloadKey)
			: SceneNodeUniqueID(InSceneNodeUID)
			, PayloadKey(InPayloadKey)
		{
		}

		INTERCHANGECOMMONPARSER_API void SerializeBaked(FArchive& Ar);

		//Conversions:
		INTERCHANGECOMMONPARSER_API void CalculateDataFor(const EInterchangeAnimationPayLoadType& ToType, const FTransform& DefaultTransform = FTransform());
	};
	
	struct FAnimationTimeDescription
	{
		double BakeFrequency;
		double RangeStartSecond;
		double RangeStopSecond;

		FAnimationTimeDescription()
			: BakeFrequency(0)
			, RangeStartSecond(0)
			, RangeStopSecond(0)
		{
		}

		FAnimationTimeDescription(double InBakeFequency, double InRangeStartSecond, double InRangeStopSecond)
			: BakeFrequency(InBakeFequency)
			, RangeStartSecond(InRangeStartSecond)
			, RangeStopSecond(InRangeStopSecond)
		{

		}

		uint32 GetHash() const
		{
			return HashCombine(HashCombine(GetTypeHash(BakeFrequency), GetTypeHash(RangeStartSecond)), GetTypeHash(RangeStopSecond));
		}
	};

	struct FAnimationPayloadQuery
	{
		FString SceneNodeUniqueID;
		FInterchangeAnimationPayLoadKey PayloadKey;
		FAnimationTimeDescription TimeDescription;

	private:
		FAnimationPayloadQuery()
		{
		}

	public:
		FAnimationPayloadQuery(const FString& InSceneNodeUniqueID, const FInterchangeAnimationPayLoadKey& InPayloadKey, double InBakeFequency = 0, double InRangeStartSecond = 0, double InRangeStopSecond = 0)
			: SceneNodeUniqueID(InSceneNodeUniqueID)
			, PayloadKey(InPayloadKey)
			, TimeDescription(InBakeFequency, InRangeStartSecond, InRangeStopSecond)
		{

		}

		INTERCHANGECOMMONPARSER_API FString GetHashString() const;

		INTERCHANGECOMMONPARSER_API FString ToJson() const;
		INTERCHANGECOMMONPARSER_API void FromJson(const FString& JsonString);

		static INTERCHANGECOMMONPARSER_API FString ToJson(const TArray<FAnimationPayloadQuery>& Queries);
		static INTERCHANGECOMMONPARSER_API void FromJson(const FString& JsonString, TArray<FAnimationPayloadQuery>& Queries);
	private:
		mutable TOptional<FString> HashStringCache;
	};
}

/** If using Cubic, this enum describes how the tangents should be controlled. */
UENUM()
enum class EInterchangeCurveInterpMode : uint8
{
	/** Use linear interpolation between values. */
	Linear,
	/** Use a constant value. Represents stepped values. */
	Constant,
	/** Cubic interpolation. See TangentMode for different cubic interpolation options. */
	Cubic,
	/** No interpolation. */
	None
};

/** If using Cubic interpolation mode, this enum describes how the tangents should be controlled. */
UENUM()
enum class EInterchangeCurveTangentMode : uint8
{
	/** Automatically calculates tangents to create smooth curves between values. */
	Auto,
	/** User specifies the tangent as a unified tangent where the two tangents are locked to each other, presenting a consistent curve before and after. */
	User,
	/** User specifies the tangent as two separate broken tangents on each side of the key which can allow a sharp change in evaluation before or after. */
	Break,
	/** No tangents. */
	None
};


/** Enumerates tangent weight modes. */
UENUM()
enum class EInterchangeCurveTangentWeightMode : uint8
{
	/** Don't take tangent weights into account. */
	WeightedNone,
	/** Only take the arrival tangent weight into account for evaluation. */
	WeightedArrive,
	/** Only take the leaving tangent weight into account for evaluation. */
	WeightedLeave,
	/** Take both the arrival and leaving tangent weights into account for evaluation. */
	WeightedBoth
};

/**
* This struct contains only the key data, this is only used to pass animation data from translators to factories
*/
USTRUCT()
struct FInterchangeCurveKey
{
	GENERATED_BODY()

	/** Interpolation mode between this key and the next */
	UPROPERTY()
	EInterchangeCurveInterpMode InterpMode = EInterchangeCurveInterpMode::None;

	/** Mode for tangents at this key */
	UPROPERTY()
	EInterchangeCurveTangentMode TangentMode = EInterchangeCurveTangentMode::None;

	/** If either tangent at this key is 'weighted' */
	UPROPERTY()
	EInterchangeCurveTangentWeightMode TangentWeightMode = EInterchangeCurveTangentWeightMode::WeightedNone;

	/** Time at this key */
	UPROPERTY()
	float Time = 0.0f;

	/** Value at this key */
	UPROPERTY()
	float Value = 0.0f;

	/** If RCIM_Cubic, the arriving tangent at this key */
	UPROPERTY()
	float ArriveTangent = 0.0f;

	/** If RCTWM_WeightedArrive or RCTWM_WeightedBoth, the weight of the left tangent */
	UPROPERTY()
	float ArriveTangentWeight = 0.0f;

	/** If RCIM_Cubic, the leaving tangent at this key */
	UPROPERTY()
	float LeaveTangent = 0.0f;

	/** If RCTWM_WeightedLeave or RCTWM_WeightedBoth, the weight of the right tangent */
	UPROPERTY()
	float LeaveTangentWeight = 0.0f;

#if WITH_ENGINE
	/** Conversion to FRichCurve */
	INTERCHANGECOMMONPARSER_API void ToRichCurveKey(FRichCurveKey& OutKey) const;
#endif

	INTERCHANGECOMMONPARSER_API void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FInterchangeCurveKey& InterchangeCurveKey)
	{
		InterchangeCurveKey.Serialize(Ar);
		return Ar;
	}
};

/**
* This struct contains only the key data, this is only used to pass animation data from interchange worker process translators to factories.
*/
USTRUCT()
struct FInterchangeCurve
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FInterchangeCurveKey> Keys;

#if WITH_ENGINE
	/** Conversion to FRichCurve */
	INTERCHANGECOMMONPARSER_API void ToRichCurve(FRichCurve& OutKey) const;
#endif

	INTERCHANGECOMMONPARSER_API void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FInterchangeCurve& InterchangeCurve)
	{
		InterchangeCurve.Serialize(Ar);
		return Ar;
	}
};

/**
* This struct contains only the key data, this is only used to pass animation data from translators to factories.
*/
USTRUCT()
struct FInterchangeStepCurve
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<float> KeyTimes;
	TOptional<TArray<float>> FloatKeyValues;
	TOptional<TArray<bool>> BooleanKeyValues;
	TOptional<TArray<uint8>> ByteKeyValues;
	TOptional<TArray<int32>> IntegerKeyValues;
	TOptional<TArray<FString>> StringKeyValues;

	INTERCHANGECOMMONPARSER_API void RemoveRedundantKeys(float Threshold);

	INTERCHANGECOMMONPARSER_API void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FInterchangeStepCurve& InterchangeStepCurve)
	{
		InterchangeStepCurve.Serialize(Ar);
		return Ar;
	}

private:
	template<typename ValueType>
	void InternalRemoveRedundantKey(TArray<ValueType>& Values, TFunctionRef<bool(const ValueType& ValueA, const ValueType& ValueB)> CompareFunction);
};
