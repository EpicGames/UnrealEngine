// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Modules/ModuleInterface.h"

#define INTERCHANGECOMMONPARSER_MODULE_NAME TEXT("InterchangeCommonParser")

/**
 * This module is used to add common structure between the translators and the parsers (i.e. payload temporary structure)
 */
class FInterchangeCommonParserModule : public IModuleInterface
{
public:
	static FInterchangeCommonParserModule& Get();
	static bool IsAvailable();

private:
	virtual void StartupModule() override;
};
