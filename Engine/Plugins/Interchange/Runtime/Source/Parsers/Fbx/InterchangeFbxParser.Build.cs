// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class InterchangeFbxParser : ModuleRules
	{
		public InterchangeFbxParser(ReadOnlyTargetRules Target) : base(Target)
		{
			// see _UCRT_LEGACY_INFINITY below. Generate own pch to have this define used
			PCHUsage = PCHUsageMode.NoPCHs;

			if (Target.Type != TargetType.Editor && Target.Type != TargetType.Program)
			{
				throw new BuildException("InterchangeFbxParser module can be build only on editor or program target.");
			}

			if (Target.Platform != UnrealTargetPlatform.Win64 &&
				Target.Platform != UnrealTargetPlatform.Linux &&
				Target.Platform != UnrealTargetPlatform.Mac)
			{
				throw new BuildException("InterchangeFbxParser module do not support target platform {0}.", Target.Platform.ToString());
			}

			bEnableExceptions = true;

			bDisableAutoRTFMInstrumentation = true; // AutoRTFM cannot be used with exceptions

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"InterchangeCore",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DeveloperSettings",
					"ImageCore",
					"ImageWrapper",
					"InterchangeCommonParser",
					"InterchangeMessages",
					"InterchangeNodes",
					"MeshDescription",
					"SkeletalMeshDescription",
					"StaticMeshDescription", 
				}
			);

			if (Target.bCompileAgainstEngine)
			{
				PrivateDefinitions.Add("WITH_MESH_DESCRIPTION_BUILDER");

				PrivateDependencyModuleNames.AddRange(
					new string[] {
						"Engine",
						"InterchangeEngine",
						"MeshConversion",
					}
				);

				PrivateIncludePathModuleNames.AddRange(
					new string[] {
						"InterchangeImport",
					}
				);
				
			}

			AddEngineThirdPartyPrivateStaticDependencies(Target, "FBX");

			// ufbx
			string UfbxDir = Path.GetFullPath(Path.Combine(ModuleDirectory, "..", "ThirdParty", "uFBX", "0.21.2"));
			PublicSystemIncludePaths.Add(UfbxDir);
			PublicDebugVisualizerPaths.Add(Path.Combine(UfbxDir, "misc", "ufbx.natvis"));
			// Allow compilation of ufbx.c with newer Windows SDK(10.0.26100) where INFINITY is defined as (float)1e+300 which caused
			// error C4756: overflow in constant arithmetic
			PrivateDefinitions.Add("_UCRT_LEGACY_INFINITY");

		}
	}
}
