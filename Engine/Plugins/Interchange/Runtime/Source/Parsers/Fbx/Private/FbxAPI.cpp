// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxAPI.h"

#include "CoreMinimal.h"
#include "FbxCamera.h"
#include "FbxConvert.h"
#include "FbxHelper.h"
#include "FbxInclude.h"
#include "FbxLight.h"
#include "FbxMaterial.h"
#include "FbxMesh.h"
#include "FbxScene.h"
#include "InterchangeHelper.h"
#include "InterchangeTextureNode.h"
#if WITH_ENGINE
#include "Mesh/InterchangeMeshPayload.h"
#endif
#include "Misc/Paths.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeSourceNode.h"
#include "Misc/SecureHash.h"
#include "InterchangeCommonAnimationPayload.h"
#include "Serialization/LargeMemoryWriter.h"

#include "FbxAnimation.h"

#define LOCTEXT_NAMESPACE "InterchangeFbxParser"

#define DESTROY_FBX_OBJECT(Object) \
if(Object) \
{ \
	Object->Destroy(); \
	Object = nullptr; \
}

namespace UE
{
	namespace Interchange
	{
		namespace Private
		{
			FFbxParser::~FFbxParser()
			{
				FbxHelper = nullptr;
				Reset();
			}
			
			void FFbxParser::Reset()
			{
				PayloadContexts.Reset();

				DESTROY_FBX_OBJECT(SDKImporter);
				DESTROY_FBX_OBJECT(SDKScene);
				if (SDKGeometryConverter)
				{
					delete SDKGeometryConverter;
					SDKGeometryConverter = nullptr;
				}
				DESTROY_FBX_OBJECT(SDKIoSettings);
				DESTROY_FBX_OBJECT(SDKManager);
				if (FbxHelper.IsValid())
				{
					FbxHelper->Reset();
				}
			}

			const TSharedPtr<FFbxHelper> FFbxParser::GetFbxHelper()
			{
				if (!FbxHelper.IsValid())
				{
					FbxHelper = MakeShared<FFbxHelper>(bKeepFbxNamespace);
				}
				check(FbxHelper.IsValid());
				return FbxHelper;
			}

			bool FFbxParser::LoadFbxFile(const FString& Filename, UInterchangeBaseNodeContainer& NodeContainer)
			{
				SourceFilename = Filename;
				int32 SDKMajor, SDKMinor, SDKRevision;
				
				//The first thing to do is to create the FBX Manager which is the object allocator for almost all the classes in the SDK
				SDKManager = FbxManager::Create();
				if (!SDKManager)
				{
					UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
					Message->Text = LOCTEXT("CannotCreateFBXManager", "Cannot create FBX SDK manager.");
					return false;
				}

				//Create an IOSettings object. This object holds all import/export settings.
				SDKIoSettings = FbxIOSettings::Create(SDKManager, IOSROOT);
				SDKIoSettings->SetBoolProp(IMP_FBX_MATERIAL, true);
				SDKIoSettings->SetBoolProp(IMP_FBX_TEXTURE, true);
				SDKIoSettings->SetBoolProp(IMP_FBX_LINK, true);
				SDKIoSettings->SetBoolProp(IMP_FBX_SHAPE, true);
				SDKIoSettings->SetBoolProp(IMP_FBX_GOBO, true);
				SDKIoSettings->SetBoolProp(IMP_FBX_ANIMATION, true);
				SDKIoSettings->SetBoolProp(IMP_SKINS, true);
				SDKIoSettings->SetBoolProp(IMP_DEFORMATION, true);
				SDKIoSettings->SetBoolProp(IMP_FBX_GLOBAL_SETTINGS, true);
				SDKIoSettings->SetBoolProp(IMP_TAKE, true);
				SDKManager->SetIOSettings(SDKIoSettings);

				SDKGeometryConverter = new FbxGeometryConverter(SDKManager);

				//Create an FBX scene. This object holds most objects imported/exported from/to files.
				SDKScene = FbxScene::Create(SDKManager, "My Scene");
				if (!SDKScene)
				{
					UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
					Message->Text = LOCTEXT("CannotCreateFBXScene", "Cannot create FBX SDK scene.");
					return false;
				}
				
				// Create an importer.
				SDKImporter = FbxImporter::Create(SDKManager, "");

				// Get the version number of the FBX files generated by the
				// version of FBX SDK that you are using.
				FbxManager::GetFileFormatVersion(SDKMajor, SDKMinor, SDKRevision);

				// Initialize the importer by providing a filename.
				auto ReportOpenError = [this, &Filename]()
					{
						FFormatNamedArguments FilenameText
						{
							{ TEXT("Filename"), FText::FromString(Filename) }
						};
						UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
						Message->Text = FText::Format(LOCTEXT("CannotOpenFBXFile", "Cannot open FBX file '{Filename}'."), FilenameText);
					};

				const bool bImportStatus = SDKImporter->Initialize(TCHAR_TO_UTF8(*Filename));
				if (!bImportStatus)
				{
					ReportOpenError();
					return false;
				}

				bool bStatus = SDKImporter->Import(SDKScene);
				if (!bStatus)
				{
					ReportOpenError();
					return false;
				}

				//We always convert scene to UE axis and units
				FbxAMatrix AxisConversionInverseMatrix;
				FFbxConvert::ConvertScene(SDKScene, bConvertScene, bForceFrontXAxis, bConvertSceneUnit, FileDetails.AxisDirection, FileDetails.UnitSystem, AxisConversionInverseMatrix, JointOrientationMatrix);

				//Save the AxisConversionInverseTransform into InterchangeSourceNode (so that socket transport can use it accordingly).
				FTransform AxisConversionInverseTransform = FFbxConvert::ConvertTransform<FTransform, FVector, FQuat>(AxisConversionInverseMatrix);
				UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(&NodeContainer);
				SourceNode->SetCustomAxisConversionInverseTransform(AxisConversionInverseTransform);

				//Store the fbx frame rate
				{
					FrameRate = FbxTime::GetFrameRate(SDKScene->GetGlobalSettings().GetTimeMode());
					FileDetails.FrameRate = FString::Printf(TEXT("%.2f"), FrameRate);
					SourceNode->SetCustomSourceFrameRateNumerator(FrameRate);
					constexpr double Denominator = 1.0;
					SourceNode->SetCustomSourceFrameRateDenominator(Denominator);
				}

				// Fbx legacy has a special way to bake the skeletal mesh that do not fit the interchange standard
				// The interchange skeletal mesh factory will read this to use the proper bake transform so it match legacy behavior.
				// This fix the issue with blender armature bone skip
				SourceNode->SetCustomAlignSkeletalMeshInScene(true);

				//Fbx legacy does not allow Scene Root Nodes to be part of the skeletons (to be joints).
				SourceNode->SetCustomAllowSceneRootAsJoint(false);

				// Get the version number of the FBX file format.
				int32 FileMajor, FileMinor, FileRevision;
				SDKImporter->GetFileVersion(FileMajor, FileMinor, FileRevision);
				FileDetails.FbxFileVersion = FString::Printf(TEXT("%d.%d.%d"), FileMajor, FileMinor, FileRevision);

				// Get The Creator of the FBX File.
				FileDetails.FbxFileCreator = UTF8_TO_TCHAR(SDKImporter->GetFileHeaderInfo()->mCreator.Buffer());
				{
					//Example of creator file info string
					//Blender (stable FBX IO) - 2.78 (sub 0) - 3.7.7
					//Maya and Max use the same string where they specify the fbx sdk version, so we cannot know it is coming from which software
					//We need blender creator when importing skeletal mesh containing the "armature" dummy node as the parent of the root joint. We want to remove this dummy "armature" node
					bCreatorIsBlender = FileDetails.FbxFileCreator.StartsWith(TEXT("Blender"));
				}

				FbxDocumentInfo* DocInfo = SDKImporter->GetSceneInfo();
				if (DocInfo)
				{
					FString LastSavedVendor(UTF8_TO_TCHAR(DocInfo->LastSaved_ApplicationVendor.Get().Buffer()));
					FString LastSavedAppName(UTF8_TO_TCHAR(DocInfo->LastSaved_ApplicationName.Get().Buffer()));
					FString LastSavedAppVersion(UTF8_TO_TCHAR(DocInfo->LastSaved_ApplicationVersion.Get().Buffer()));

					FileDetails.ApplicationVendor = LastSavedVendor;
					FileDetails.ApplicationName = LastSavedAppName;
					FileDetails.ApplicationVersion = LastSavedAppVersion;
				}
				else
				{
					FileDetails.ApplicationVendor = TEXT("");
					FileDetails.ApplicationName = TEXT("");
					FileDetails.ApplicationVersion = TEXT("");
				}

				return true;
			}

			void FFbxParser::FillContainerWithFbxScene(UInterchangeBaseNodeContainer& NodeContainer)
			{
				CleanupFbxData();

				FFbxMaterial FbxMaterial(*this);
				FbxMaterial.AddAllTextures(SDKScene, NodeContainer);
				FbxMaterial.AddAllMaterials(SDKScene, NodeContainer);

				FFbxMesh FbxMesh(*this);
				FbxMesh.AddAllMeshes(SDKScene, SDKGeometryConverter, NodeContainer, PayloadContexts);

				FFbxLight FbxLight(*this);
				FbxLight.AddAllLights(SDKScene, NodeContainer);

				FFbxCamera FbxCamera(*this);
				FbxCamera.AddAllCameras(SDKScene, NodeContainer);

				FFbxScene FbxScene(*this);
				FbxScene.AddHierarchy(SDKScene, NodeContainer, PayloadContexts);
				FbxScene.AddAnimation(SDKScene, NodeContainer, PayloadContexts);
				FbxScene.AddMorphTargetAnimations(SDKScene, NodeContainer, PayloadContexts, FbxMesh.GetMorphTargetAnimationsBuildingData());

				ProcessExtraInformation(NodeContainer);
			}

			bool FFbxParser::FetchPayloadData(const FString& PayloadKey, const FString& PayloadFilepath)
			{
				if (!PayloadContexts.Contains(PayloadKey))
				{
					UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
					Message->Text = LOCTEXT("CannotRetrievePayload", "Cannot retrieve payload; payload key doesn't have any context.");
					return false;
				}

				{
					//Critical section to force payload to be fetch one by one with no concurrency.
					FScopeLock Lock(&PayloadCriticalSection);
					TSharedPtr<FPayloadContextBase>& PayloadContext = PayloadContexts.FindChecked(PayloadKey);
					return PayloadContext->FetchPayloadToFile(*this, PayloadFilepath);
				}
			}

			bool FFbxParser::FetchMeshPayloadData(const FString& PayloadKey, const FTransform& MeshGlobalTransform, const FString& PayloadFilepath)
			{
				if (!PayloadContexts.Contains(PayloadKey))
				{
					UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
					Message->Text = LOCTEXT("CannotRetrievePayload", "Cannot retrieve payload; payload key doesn't have any context.");
					return false;
				}

				{
					//Critical section to force payload to be fetch one by one with no concurrency.
					FScopeLock Lock(&PayloadCriticalSection);
					TSharedPtr<FPayloadContextBase>& PayloadContext = PayloadContexts.FindChecked(PayloadKey);
					return PayloadContext->FetchMeshPayloadToFile(*this, MeshGlobalTransform, PayloadFilepath);
				}
			}

#if WITH_ENGINE
			bool FFbxParser::FetchMeshPayloadData(const FString& PayloadKey, const FTransform& MeshGlobalTransform, FMeshPayloadData& OutMeshPayloadData)
			{
				if (!PayloadContexts.Contains(PayloadKey))
				{
					UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
					Message->Text = LOCTEXT("CannotRetrievePayload", "Cannot retrieve payload; payload key doesn't have any context.");
					return false;
				}

				{
					//Critical section to force payload to be fetch one by one with no concurrency.
					FScopeLock Lock(&PayloadCriticalSection);
					TSharedPtr<FPayloadContextBase>& PayloadContext = PayloadContexts.FindChecked(PayloadKey);
					return PayloadContext->FetchMeshPayload(*this, MeshGlobalTransform, OutMeshPayloadData);
				}
			}
#endif

			bool FFbxParser::FetchAnimationBakeTransformPayload(const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries, const FString& ResultFolder, FCriticalSection* ResultPayloadsCriticalSection, TAtomic<int64>& UniqueIdCounter, TMap<FString, FString>& ResultPayloads/*PayloadUniqueID to FilePath*/)
			{
				//Critical section to force payload to be fetch one by one with no concurrency.
				FScopeLock Lock(&PayloadCriticalSection);

				TMap<uint32, TArray<const UE::Interchange::FAnimationPayloadQuery*>> PayloadQueriesGrouped;

				for (const UE::Interchange::FAnimationPayloadQuery& PayloadQuery : PayloadQueries)
				{
					TArray<const UE::Interchange::FAnimationPayloadQuery*>& PayloadQueriesForHash = PayloadQueriesGrouped.FindOrAdd(PayloadQuery.TimeDescription.GetHash());
					PayloadQueriesForHash.Add(&PayloadQuery);
				}

				TArray<FText> OutErrorMessages;

				bool bResult = true;
				for (const TPair<uint32, TArray<const UE::Interchange::FAnimationPayloadQuery*>>& Group : PayloadQueriesGrouped)
				{
					bResult = FFbxAnimation::FetchAnimationBakeTransformPayload(*this, GetSDKScene(), PayloadContexts, Group.Value, ResultFolder, ResultPayloadsCriticalSection, UniqueIdCounter, ResultPayloads, OutErrorMessages) && bResult;
				}

				for (const FText& ErrorMessage : OutErrorMessages)
				{
					UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
					Message->Text = ErrorMessage;
				}

				return bResult;
			}

			void FFbxParser::CleanupFbxData()
			{
				auto ReportAndSetNoName = [this](FbxObject* Object, const FString& ObjectName)
					{
						Object->SetName(TCHAR_TO_UTF8(*ObjectName));
						if (!GIsAutomationTesting)
						{
							UInterchangeResultDisplay_Generic* Message = AddMessage<UInterchangeResultDisplay_Generic>();
							Message->Text = FText::Format(LOCTEXT("CleanupFbxData_NoObjectName", "Interchange FBX file Loading: Found object with no name, new object name is '{0}'"), FText::FromString(ObjectName));
						}
					};

				//bUseNodeRules: 
				// - we don't manage the namespace as each namespace is expected to be in separate assets
				// - name sanitization will be part of the uniqueness check, but we don't set the sanitized name to the object, 
				//		as node names will have their namespace managed and sanitized when we create the names for Interchange nodes specifically.
				auto MakeFbxObjectNameUnique = [this](FbxObject* Object, TMap<FString, int32>& Names, bool bUseNCL = true, bool bUseNodeRules = false)
					{
						FString NodeName = UTF8_TO_TCHAR(Object->GetName());
						FString UniqueNodeName = NodeName;

						if (!bUseNodeRules)
						{
							//Manage namespaces should be done before sanitizing since it relies on the ":" character
							GetFbxHelper()->ManageNamespaceAndRenameObject(UniqueNodeName, Object);
						}

						//Make sure to use the sanitized name to avoid name collisions later on
						UE::Interchange::SanitizeName(UniqueNodeName);

						if (int32* Count = Names.Find(UniqueNodeName))
						{
							(*Count)++;
							if (bUseNodeRules)
							{
								//if we are using node rules then its expected to fix the name facing towards Interchange, instead of right in place (in FBX struct).
								UniqueNodeName = NodeName + (bUseNCL ? TEXT("_ncl_") : TEXT("")) + FString::FromInt(*Count);
								Object->SetName(TCHAR_TO_UTF8(*UniqueNodeName));
							}
							else
							{
								UniqueNodeName += (bUseNCL ? TEXT("_ncl_") : TEXT("")) + FString::FromInt(*Count);
								Object->SetName(TCHAR_TO_UTF8(*UniqueNodeName));
							}

							if (!GIsAutomationTesting)
							{
								UInterchangeResultDisplay_Generic* Message = AddMessage<UInterchangeResultDisplay_Generic>();
								Message->Text = FText::Format(LOCTEXT("CleanupFbxData_NodeNameClash", "FBX File Loading: Found name clash, object '{0}' was renamed to '{1}'"), FText::FromString(NodeName), FText::FromString(UniqueNodeName));
							}
						}
						else
						{
							Names.Add(UniqueNodeName, 0);
						}
					};

				//////////////////////////////////////////////////////////////////////////
				// Ensure Node Name Validity (uniqueness)
				// Name clash must be global because unreal bones do not support name conflict (they are stored in an array, no hierarchy)
				TMap<FString, int32> NodeNames;
				for (int32 NodeIndex = 0; NodeIndex < SDKScene->GetNodeCount(); ++NodeIndex)
				{
					FbxNode* Node = SDKScene->GetNode(NodeIndex);
					FString NodeName = UTF8_TO_TCHAR(Node->GetName());
					bool bUseNCL = false;
					if (NodeName.IsEmpty())
					{
						bUseNCL = true;
						ReportAndSetNoName(Node, TEXT("Node"));
					}
					MakeFbxObjectNameUnique(Node, NodeNames, bUseNCL, true);
				}

				//////////////////////////////////////////////////////////////////////////
				// Ensure Mesh Name Validity (uniqueness)
				// Name clash must be global because we will build Unique ID from the mesh name
				TMap<FString, int32> MeshNames;
				for (int32 GeometryIndex = 0; GeometryIndex < SDKScene->GetGeometryCount(); ++GeometryIndex)
				{
					FbxGeometry* Geometry = SDKScene->GetGeometry(GeometryIndex);
					if (Geometry->GetAttributeType() != FbxNodeAttribute::eMesh)
					{
						continue;
					}
					FbxMesh* Mesh = static_cast<FbxMesh*>(Geometry);
					if (!Mesh)
					{
						continue;
					}
					FString MeshName = UTF8_TO_TCHAR(Mesh->GetName());
					if (MeshName.IsEmpty())
					{
						ReportAndSetNoName(Mesh, TEXT("Mesh"));
					}
					MakeFbxObjectNameUnique(Mesh, MeshNames);
				}

				/////////////////////////////////////////////////////////////////////////
				// Ensure Material Name Validity (uniqueness)
				// Name clash must be global because we will build Unique ID from the material name
				TMap<FString, int32> MaterialNames;
				for (int32 MaterialIndex = 0; MaterialIndex < SDKScene->GetMaterialCount(); ++MaterialIndex)
				{
					FbxSurfaceMaterial* Material = SDKScene->GetMaterial(MaterialIndex);
					FString MaterialName = UTF8_TO_TCHAR(Material->GetName());
					if (MaterialName.IsEmpty())
					{
						ReportAndSetNoName(Material, TEXT("Material"));
					}
					MakeFbxObjectNameUnique(Material, MaterialNames);
				}
			}

			void FFbxParser::ProcessExtraInformation(UInterchangeBaseNodeContainer& NodeContainer)
			{
				UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(&NodeContainer);

				SourceNode->SetExtraInformation(TEXT("File Version"), FileDetails.FbxFileVersion);
				SourceNode->SetExtraInformation(TEXT("File Creator"), FileDetails.FbxFileCreator);
				SourceNode->SetExtraInformation(TEXT("File Units"), FileDetails.UnitSystem);
				SourceNode->SetExtraInformation(TEXT("File Axis Direction"), FileDetails.AxisDirection);
				SourceNode->SetExtraInformation(TEXT("File Frame Rate"), FileDetails.FrameRate);

				// Analytics Data
				{
					using namespace UE::Interchange;
					if (!FileDetails.ApplicationVendor.IsEmpty())
					{
						SourceNode->SetExtraInformation(FSourceNodeExtraInfoStaticData::GetApplicationVendorExtraInfoKey(), FileDetails.ApplicationVendor);
					}

					if (!FileDetails.ApplicationName.IsEmpty())
					{
						SourceNode->SetExtraInformation(FSourceNodeExtraInfoStaticData::GetApplicationNameExtraInfoKey(), FileDetails.ApplicationName);
					}

					if (!FileDetails.ApplicationVersion.IsEmpty())
					{
						SourceNode->SetExtraInformation(FSourceNodeExtraInfoStaticData::GetApplicationVersionExtraInfoKey(), FileDetails.ApplicationVersion);
					}
				}
			}
		} //ns Private
	} //ns Interchange
} //ns UE

#undef LOCTEXT_NAMESPACE
