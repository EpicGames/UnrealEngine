// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FbxInclude.h"
#include "HAL/CriticalSection.h"
#include "InterchangeResultsContainer.h"


#define FBX_METADATA_PREFIX TEXT("FBX.")
#define INVALID_UNIQUE_ID 0xFFFFFFFFFFFFFFFF

namespace UE
{
	namespace Interchange
	{
		struct FImportImage;
		struct FAnimationPayloadQuery;
		struct FAnimationPayloadData;
		namespace Private
		{
			class FPayloadContextBase;
			struct FFbxHelper;
		}
#if WITH_ENGINE
		struct FMeshPayloadData;
#endif
	}
}
class UInterchangeBaseNodeContainer;

namespace UE
{
	namespace Interchange
	{
		namespace Private
		{
			class IFbxParser
			{
			public:
				virtual ~IFbxParser() = default;

				virtual const TCHAR* GetName() = 0;

				virtual bool IsThreadSafe() { return false; }

				virtual void SetResultContainer(UInterchangeResultsContainer* Result) = 0;
				virtual void SetConvertSettings(const bool InbConvertScene, const bool InbForceFrontXAxis, const bool InbConvertSceneUnit, const bool InbKeepFbxNamespace, const bool InbConsiderClusterBeforePoseForMeshBindPose) = 0;

				/* Load an fbx file into the fbx sdk, return false if the file could not be load. */
				virtual bool LoadFbxFile(const FString& Filename, UInterchangeBaseNodeContainer& NodeContainer) = 0;

				/* Extract the fbx data from the sdk into our node container */
				virtual void FillContainerWithFbxScene(UInterchangeBaseNodeContainer& NodeContainer)  = 0;

				/* Extract the fbx data from the sdk into our node container */
				virtual bool FetchPayloadData(const FString& PayloadKey, const FString& PayloadFilepath)  = 0;

				/* Extract the fbx mesh data from the sdk into our node container */
				virtual bool FetchMeshPayloadData(const FString& PayloadKey, const FTransform& MeshGlobalTransform, const FString& PayloadFilepath)  = 0;
#if WITH_ENGINE
				virtual bool FetchMeshPayloadData(const FString& PayloadKey, const FTransform& MeshGlobalTransform, FMeshPayloadData& OutMeshPayloadData)  = 0;
				virtual bool FetchTexturePayload(const FString& PayloadKey, TOptional<TArray64<uint8>>& OutTexturePayloadData) { return false; }
#endif

				/* Extract the fbx data from the sdk into our node container
				* @Param PayloadQueries - Will be grouped based on their TimeDescription Hashes (so that we acquire the same timings in one iteration, avoiding cache rebuilds)
				*/
				virtual bool FetchAnimationBakeTransformPayload(const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries, const FString& ResultFolder, FCriticalSection* ResultPayloadsCriticalSection, TAtomic<int64>& UniqueIdCounter, TMap<FString, FString>& ResultPayloads/*PayloadUniqueID to FilePath*/)  = 0;
				virtual void Reset() = 0;
			};

			class FFbxParser: public IFbxParser
			{
			public:
				explicit FFbxParser(TWeakObjectPtr<UInterchangeResultsContainer> InResultsContainer)
					: ResultsContainer(InResultsContainer)
				{}

				virtual const TCHAR* GetName() override
				{
					return TEXT("FBXSDK");
				}

				~FFbxParser();

				virtual void Reset() override;

				virtual void SetResultContainer(UInterchangeResultsContainer* Result) override
				{
					ResultsContainer = Result;
				}

				virtual void SetConvertSettings(const bool InbConvertScene, const bool InbForceFrontXAxis, const bool InbConvertSceneUnit, const bool InbKeepFbxNamespace, const bool InbConsiderClusterBeforePoseForMeshBindPose) override
				{
					bConvertScene = InbConvertScene;
					bForceFrontXAxis = InbForceFrontXAxis;
					bConvertSceneUnit = InbConvertSceneUnit;
					bKeepFbxNamespace = InbKeepFbxNamespace;
					bConsiderClusterBeforePoseForMeshBindPose = InbConsiderClusterBeforePoseForMeshBindPose;
				}

				//return the fbx helper for this parser
				const TSharedPtr<FFbxHelper> GetFbxHelper();

				/* Load an fbx file into the fbx sdk, return false if the file could not be load. */
				virtual bool LoadFbxFile(const FString& Filename, UInterchangeBaseNodeContainer& NodeContainer) override;

				/* Extract the fbx data from the sdk into our node container */
				virtual void FillContainerWithFbxScene(UInterchangeBaseNodeContainer& NodeContainer) override;

				/* Extract the fbx data from the sdk into our node container */
				virtual bool FetchPayloadData(const FString& PayloadKey, const FString& PayloadFilepath) override;

				/* Extract the fbx mesh data from the sdk into our node container */
				virtual bool FetchMeshPayloadData(const FString& PayloadKey, const FTransform& MeshGlobalTransform, const FString& PayloadFilepath) override;
#if WITH_ENGINE
				virtual bool FetchMeshPayloadData(const FString& PayloadKey, const FTransform& MeshGlobalTransform, FMeshPayloadData& OutMeshPayloadData) override;
#endif

				/* Extract the fbx data from the sdk into our node container
				* @Param PayloadQueries - Will be grouped based on their TimeDescription Hashes (so that we acquire the same timings in one iteration, avoiding cache rebuilds)
				*/
				virtual bool FetchAnimationBakeTransformPayload(const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries, const FString& ResultFolder, FCriticalSection* ResultPayloadsCriticalSection, TAtomic<int64>& UniqueIdCounter, TMap<FString, FString>& ResultPayloads/*PayloadUniqueID to FilePath*/) override;
				
				/**
				 * This function is used to add the given message object directly into the results for this operation.
				 */
				template <typename T>
				T* AddMessage() const
				{
					check(ResultsContainer.IsValid());
					T* Item = ResultsContainer->Add<T>();
					Item->SourceAssetName = SourceFilename;
					return Item;
				}


				void AddMessage(UInterchangeResult* Item) const
				{
					check(ResultsContainer.IsValid());
					ResultsContainer->Add(Item);
					Item->SourceAssetName = SourceFilename;
				}

				FbxScene* GetSDKScene() { return SDKScene; }

				double GetFrameRate() { return FrameRate; }

				bool IsCreatorBlender() { return bCreatorIsBlender; }

				const FString& GetSourceFilename() const { return SourceFilename; }

				bool GetUseFbxClustersForBindPoseMatrices() const { return bConsiderClusterBeforePoseForMeshBindPose; }

				FbxAMatrix JointOrientationMatrix;

				/**
				 * Critical section to avoid getting multiple payload in same time.
				 * The FBX evaluator use a cache mechanism for evaluating global transform that is not thread safe.
				 * There si other stuff in the sdk which are not thread safe, so all fbx payload should be fetch one by one
				 */
				FCriticalSection PayloadCriticalSection;
			private:

				void EnsureNodeNameAreValid(const FString& BaseFilename);
				void CleanupFbxData();
				void ProcessExtraInformation(UInterchangeBaseNodeContainer& NodeContainer);

				TWeakObjectPtr<UInterchangeResultsContainer> ResultsContainer;
				FbxManager* SDKManager = nullptr;
				FbxScene* SDKScene = nullptr;
				FbxImporter* SDKImporter = nullptr;
				FbxGeometryConverter* SDKGeometryConverter = nullptr;
				FbxIOSettings* SDKIoSettings = nullptr;
				FString SourceFilename;
				TMap<FString, TSharedPtr<FPayloadContextBase>> PayloadContexts;
				TSharedPtr<FFbxHelper> FbxHelper;

				//For PivotReset and Animation Conversion:
				double FrameRate = 30.0;

				//Convert settings
				bool bConvertScene = true;
				bool bForceFrontXAxis = false;
				bool bConvertSceneUnit = true;
				bool bKeepFbxNamespace = false;
				bool bCreatorIsBlender = false;
				bool bConsiderClusterBeforePoseForMeshBindPose = false;

				struct FileDetails
				{
					FString FbxFileVersion;
					FString FbxFileCreator;
					FString ApplicationName;
					FString ApplicationVersion;
					FString ApplicationVendor;
					FString UnitSystem;
					FString AxisDirection;
					FString FrameRate;
				} FileDetails;
				
			};
		}//ns Private
	}//ns Interchange
}//ns UE
