// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxAnimation.h"

#include "Async/ParallelFor.h"
#include "CoreMinimal.h"
#include "FbxAPI.h"
#include "FbxConvert.h"
#include "FbxHelper.h"
#include "FbxInclude.h"
#include "FbxMesh.h"
#include "Fbx/InterchangeFbxMessages.h"
#include "InterchangeAnimationTrackSetNode.h"
#include "InterchangeCommonAnimationPayload.h"
#include "InterchangeMeshNode.h"
#include "InterchangeResultsContainer.h"
#include "InterchangeSceneNode.h"
#include "MeshDescription.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopeLock.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Serialization/LargeMemoryWriter.h"
#include "SkeletalMeshAttributes.h"
#include "StaticMeshAttributes.h"

#define LOCTEXT_NAMESPACE "InterchangeFbxMesh"

namespace UE::Interchange::Private
{
	bool IsStepCurve(FbxAnimCurveNode* AnimCurveNode)
	{
		const uint32 ChannelCount = AnimCurveNode->GetChannelsCount();
		for (uint32 ChannelIndex = 0; ChannelIndex < ChannelCount; ++ChannelIndex)
		{
			const uint32 ChannelCurveCount = AnimCurveNode->GetCurveCount(ChannelIndex);
			for (uint32 CurveIndex = 0; CurveIndex < ChannelCurveCount; ++CurveIndex)
			{
				if (FbxAnimCurve* CurrentAnimCurve = AnimCurveNode->GetCurve(ChannelIndex, CurveIndex))
				{
					int32 KeyCount = CurrentAnimCurve->KeyGetCount();
					for (int32 KeyIndex = 0; KeyIndex < KeyCount; ++KeyIndex)
					{
						FbxAnimCurveKey Key = CurrentAnimCurve->KeyGet(KeyIndex);

						FbxAnimCurveDef::EInterpolationType KeyInterpMode = Key.GetInterpolation();

						if (KeyInterpMode != FbxAnimCurveDef::EInterpolationType::eInterpolationConstant)
						{
							return false;
						}
					}
				}
			}
		}

		return true;
	}

	bool ImportCurve(const FbxAnimCurve* SourceFloatCurves, const float ScaleValue, TArray<FInterchangeCurveKey>& DestinationFloatCurve)
	{
		if (!SourceFloatCurves)
		{
			return true;
		}
		const float DefaultCurveWeight = FbxAnimCurveDef::sDEFAULT_WEIGHT;
		//We use the non const to query the left and right derivative of the key, for whatever reason those FBX API functions are not const
		FbxAnimCurve* NonConstSourceFloatCurves = const_cast<FbxAnimCurve*>(SourceFloatCurves);
		int32 KeyCount = SourceFloatCurves->KeyGetCount();
		for (int32 KeyIndex = 0; KeyIndex < KeyCount; ++KeyIndex)
		{
			FbxAnimCurveKey Key = SourceFloatCurves->KeyGet(KeyIndex);
			FbxTime KeyTime = Key.GetTime();
			const float KeyTimeValue = static_cast<float>(KeyTime.GetSecondDouble());
			float Value = Key.GetValue() * ScaleValue;
			FInterchangeCurveKey& InterchangeCurveKey = DestinationFloatCurve.AddDefaulted_GetRef();
			InterchangeCurveKey.Time = KeyTimeValue;
			InterchangeCurveKey.Value = Value;

			const bool bIncludeOverrides = true;
			FbxAnimCurveDef::ETangentMode KeyTangentMode = Key.GetTangentMode(bIncludeOverrides);
			FbxAnimCurveDef::EInterpolationType KeyInterpMode = Key.GetInterpolation();
			FbxAnimCurveDef::EWeightedMode KeyTangentWeightMode = Key.GetTangentWeightMode();

			EInterchangeCurveInterpMode NewInterpMode = EInterchangeCurveInterpMode::Linear;
			EInterchangeCurveTangentMode NewTangentMode = EInterchangeCurveTangentMode::Auto;
			EInterchangeCurveTangentWeightMode NewTangentWeightMode = EInterchangeCurveTangentWeightMode::WeightedNone;

			float RightTangent = NonConstSourceFloatCurves->KeyGetRightDerivative(KeyIndex) * ScaleValue;
			float LeftTangent = NonConstSourceFloatCurves->KeyGetLeftDerivative(KeyIndex) * ScaleValue;
			float RightTangentWeight = 0.0f;
			float LeftTangentWeight = 0.0f; //This one is dependent on the previous key.
			bool bLeftWeightActive = false;
			bool bRightWeightActive = false;

			const bool bPreviousKeyValid = KeyIndex > 0;
			const bool bNextKeyValid = KeyIndex < KeyCount - 1;
			float PreviousValue = 0.0f;
			float PreviousKeyTimeValue = 0.0f;
			float NextValue = 0.0f;
			float NextKeyTimeValue = 0.0f;
			if (bPreviousKeyValid)
			{
				FbxAnimCurveKey PreviousKey = SourceFloatCurves->KeyGet(KeyIndex - 1);
				FbxTime PreviousKeyTime = PreviousKey.GetTime();
				PreviousKeyTimeValue = static_cast<float>(PreviousKeyTime.GetSecondDouble());
				PreviousValue = PreviousKey.GetValue() * ScaleValue;
				//The left tangent is driven by the previous key. If the previous key have a the NextLeftweight or both flag weighted mode, it mean the next key is weighted on the left side
				bLeftWeightActive = (PreviousKey.GetTangentWeightMode() & FbxAnimCurveDef::eWeightedNextLeft) > 0;
				if (bLeftWeightActive)
				{
					LeftTangentWeight = PreviousKey.GetDataFloat(FbxAnimCurveDef::eNextLeftWeight);
				}
			}
			if (bNextKeyValid)
			{
				FbxAnimCurveKey NextKey = SourceFloatCurves->KeyGet(KeyIndex + 1);
				FbxTime NextKeyTime = NextKey.GetTime();
				NextKeyTimeValue = static_cast<float>(NextKeyTime.GetSecondDouble());
				NextValue = NextKey.GetValue() * ScaleValue;

				bRightWeightActive = (KeyTangentWeightMode & FbxAnimCurveDef::eWeightedRight) > 0;
				if (bRightWeightActive)
				{
					//The right tangent weight should be use only if we are not the last key since the last key do not have a right tangent.
					//Use the current key to gather the right tangent weight
					RightTangentWeight = Key.GetDataFloat(FbxAnimCurveDef::eRightWeight);
				}
			}

			// When this flag is true, the tangent is flat if the value has the same value as the previous or next key.
			const bool bTangentGenericClamp = (KeyTangentMode & FbxAnimCurveDef::eTangentGenericClamp);

			//Time independent tangent this is consider has a spline tangent key
			const bool bTangentGenericTimeIndependent = (KeyTangentMode & FbxAnimCurveDef::ETangentMode::eTangentGenericTimeIndependent);

			// When this flag is true, the tangent is flat if the value is outside of the [previous key, next key] value range.
			//Clamp progressive is (eTangentGenericClampProgressive |eTangentGenericTimeIndependent)
			const bool bTangentGenericClampProgressive = (KeyTangentMode & FbxAnimCurveDef::ETangentMode::eTangentGenericClampProgressive) == FbxAnimCurveDef::ETangentMode::eTangentGenericClampProgressive;

			if (KeyTangentMode & FbxAnimCurveDef::eTangentGenericBreak)
			{
				NewTangentMode = EInterchangeCurveTangentMode::Break;
			}
			else if (KeyTangentMode & FbxAnimCurveDef::eTangentUser)
			{
				NewTangentMode = EInterchangeCurveTangentMode::User;
			}

			switch (KeyInterpMode)
			{
			case FbxAnimCurveDef::eInterpolationConstant://! Constant value until next key.
				NewInterpMode = EInterchangeCurveInterpMode::Constant;
				break;
			case FbxAnimCurveDef::eInterpolationLinear://! Linear progression to next key.
				NewInterpMode = EInterchangeCurveInterpMode::Linear;
				break;
			case FbxAnimCurveDef::eInterpolationCubic://! Cubic progression to next key.
				NewInterpMode = EInterchangeCurveInterpMode::Cubic;
				// get tangents
				{
					bool bIsFlatTangent = false;
					bool bIsComputedTangent = false;
					if (bTangentGenericClampProgressive)
					{
						if (bPreviousKeyValid && bNextKeyValid)
						{
							const float PreviousNextHalfDelta = (NextValue - PreviousValue) * 0.5f;
							const float PreviousNextAverage = PreviousValue + PreviousNextHalfDelta;
							// If the value is outside of the previous-next value range, the tangent is flat.
							bIsFlatTangent = FMath::Abs(Value - PreviousNextAverage) >= FMath::Abs(PreviousNextHalfDelta);
						}
						else
						{
							//Start/End tangent with the ClampProgressive flag are flat.
							bIsFlatTangent = true;
						}
					}
					else if (bTangentGenericClamp && (bPreviousKeyValid || bNextKeyValid))
					{
						if (bPreviousKeyValid && PreviousValue == Value)
						{
							bIsFlatTangent = true;
						}
						if (bNextKeyValid)
						{
							bIsFlatTangent |= Value == NextValue;
						}
					}
					else if (bTangentGenericTimeIndependent)
					{
						//Spline tangent key, because bTangentGenericClampProgressive include bTangentGenericTimeIndependent, we must treat this case after bTangentGenericClampProgressive
						if (KeyCount == 1)
						{
							bIsFlatTangent = true;
						}
						else
						{
							//Spline tangent key must be User mode since we want to keep the tangents provide by the fbx key left and right derivatives
							NewTangentMode = EInterchangeCurveTangentMode::User;
						}
					}

					if (bIsFlatTangent)
					{
						RightTangent = 0;
						LeftTangent = 0;
						//To force flat tangent we need to set the tangent mode to user
						NewTangentMode = EInterchangeCurveTangentMode::User;
					}

				}
				break;
			}

			//auto with weighted give the wrong result, so when auto is weighted we set user mode and set the Right tangent equal to the left tangent.
			//Auto has only the left tangent set
			if (NewTangentMode == EInterchangeCurveTangentMode::Auto && (bLeftWeightActive || bRightWeightActive))
			{

				NewTangentMode = EInterchangeCurveTangentMode::User;
				RightTangent = LeftTangent;
			}

			if (NewTangentMode != EInterchangeCurveTangentMode::Auto)
			{
				const bool bEqualTangents = FMath::IsNearlyEqual(LeftTangent, RightTangent);
				//If tangents are different then broken.
				if (bEqualTangents)
				{
					NewTangentMode = EInterchangeCurveTangentMode::User;
				}
				else
				{
					NewTangentMode = EInterchangeCurveTangentMode::Break;
				}
			}

			//Only cubic interpolation allow weighted tangents
			if (KeyInterpMode == FbxAnimCurveDef::eInterpolationCubic)
			{
				if (bLeftWeightActive && bRightWeightActive)
				{
					NewTangentWeightMode = EInterchangeCurveTangentWeightMode::WeightedBoth;
				}
				else if (bLeftWeightActive)
				{
					NewTangentWeightMode = EInterchangeCurveTangentWeightMode::WeightedArrive;
					RightTangentWeight = DefaultCurveWeight;
				}
				else if (bRightWeightActive)
				{
					NewTangentWeightMode = EInterchangeCurveTangentWeightMode::WeightedLeave;
					LeftTangentWeight = DefaultCurveWeight;
				}
				else
				{
					NewTangentWeightMode = EInterchangeCurveTangentWeightMode::WeightedNone;
					LeftTangentWeight = DefaultCurveWeight;
					RightTangentWeight = DefaultCurveWeight;
				}

				auto ComputeWeightInternal = [](float TimeA, float TimeB, const float TangentSlope, const float TangentWeight)
				{
					const float X = TimeA - TimeB;
					const float Y = TangentSlope * X;
					return FMath::Sqrt(X * X + Y * Y) * TangentWeight;
				};

				if (!FMath::IsNearlyZero(LeftTangentWeight))
				{
					if (bPreviousKeyValid)
					{
						LeftTangentWeight = ComputeWeightInternal(KeyTimeValue, PreviousKeyTimeValue, LeftTangent, LeftTangentWeight);
					}
					else
					{
						LeftTangentWeight = 0.0f;
					}
				}

				if (!FMath::IsNearlyZero(RightTangentWeight))
				{
					if (bNextKeyValid)
					{
						RightTangentWeight = ComputeWeightInternal(NextKeyTimeValue, KeyTimeValue, RightTangent, RightTangentWeight);
					}
					else
					{
						RightTangentWeight = 0.0f;
					}
				}
			}

			const bool bForceDisableTangentRecompute = false; //No need to recompute all the tangents of the curve every time we change de key.
			InterchangeCurveKey.InterpMode = NewInterpMode;
			InterchangeCurveKey.TangentMode = NewTangentMode;
			InterchangeCurveKey.TangentWeightMode = NewTangentWeightMode;

			InterchangeCurveKey.ArriveTangent = LeftTangent;
			InterchangeCurveKey.LeaveTangent = RightTangent;
			InterchangeCurveKey.ArriveTangentWeight = LeftTangentWeight;
			InterchangeCurveKey.LeaveTangentWeight = RightTangentWeight;
		}
		return true;
	}

	template<typename AttributeType>
	void FillStepCurveAttribute(TArray<float>& OutFrameTimes, TArray<AttributeType>& OutFrameValues, const FbxAnimCurve* FbxCurve, TFunctionRef<AttributeType(const FbxAnimCurveKey*, const FbxTime*)> EvaluationFunction)
	{
		const int32 KeyCount = FbxCurve ? FbxCurve->KeyGetCount() : 0;

		if (KeyCount > 0)
		{
			OutFrameTimes.Reserve(KeyCount);
			OutFrameValues.Reserve(KeyCount);
			const FbxTime StartTime = FbxCurve->KeyGet(0).GetTime();

			for (int32 KeyIndex = 0; KeyIndex < KeyCount; ++KeyIndex)
			{
				FbxAnimCurveKey Key = FbxCurve->KeyGet(KeyIndex);
				FbxTime KeyTime = Key.GetTime() - StartTime;

				OutFrameTimes.Add(KeyTime.GetSecondDouble());
				OutFrameValues.Add(EvaluationFunction(&Key, &KeyTime));
			}
		}
		else
		{
			OutFrameTimes.Add(0);
			OutFrameValues.Add(EvaluationFunction(nullptr, nullptr));
		}
	}

	template<typename T>
	void ImportStepCurve(const FbxAnimCurve* SourceCurves, FbxProperty& Property, FInterchangeStepCurve& DestinationCurve)
	{
		TArray<T> StepCurveValues;
		FillStepCurveAttribute<T>(DestinationCurve.KeyTimes, StepCurveValues, SourceCurves, [&Property](const FbxAnimCurveKey* Key, const FbxTime* KeyTime)
		{
			if(Key)
			{
				return static_cast<T>(Key->GetValue());
			}
			else
			{
				return static_cast<T>(Property.Get<T>());
			}
		});

		if constexpr(std::is_same_v<T, bool>)
		{
			DestinationCurve.BooleanKeyValues = StepCurveValues;
		}
		else if constexpr(std::is_floating_point_v<T>)
		{
			check(false); //Float curve payload should be extract as FInterchangeCurve since we can interpolate them
		}
		else if constexpr(sizeof(T) == sizeof(uint8))
		{
			DestinationCurve.ByteKeyValues = StepCurveValues;
		}
		else if constexpr(std::is_integral_v<T> && sizeof(T) > sizeof(uint8))
		{
			DestinationCurve.IntegerKeyValues = StepCurveValues;
		}
	}

	void ImportStringStepCurve(const FbxAnimCurve* SourceCurves, FbxProperty& Property, FInterchangeStepCurve& DestinationCurve)
	{
		TArray<FString> StepCurveValues;
		FillStepCurveAttribute<FString>(DestinationCurve.KeyTimes, StepCurveValues, SourceCurves, [&Property](const FbxAnimCurveKey* Key, const FbxTime* KeyTime)
			{
				if (KeyTime)
				{
					FbxPropertyValue& EvaluatedValue = Property.EvaluateValue(*KeyTime);
					FbxString StringValue;
					EvaluatedValue.Get(&StringValue, EFbxType::eFbxString);
					return FString(UTF8_TO_TCHAR(StringValue));
				}
				else
				{
					return FString(UTF8_TO_TCHAR(Property.Get<FbxString>()));
				}
			});
		DestinationCurve.StringKeyValues = StepCurveValues;
	}

	struct FGetFbxTransformCurvesParameters
	{
		FGetFbxTransformCurvesParameters(FbxScene* InSDKScene, FbxNode* InNode)
		{
			SDKScene = InSDKScene;
			check(SDKScene);
			Node = InNode;
			check(Node);
		}

		FbxScene* SDKScene = nullptr;
		FbxNode* Node = nullptr;
		bool IsNodeAnimated = false;
		FbxAnimStack* CurrentAnimStack = nullptr;
	};

	void GetFbxTransformCurves(FGetFbxTransformCurvesParameters& Parameters, const int32& AnimationIndex)
	{
		if (!ensure(Parameters.SDKScene) || !ensure(Parameters.Node))
		{
			return;
		}
		//Get the node transform curve keys, the transform components are separate into float curve
		//Translation X
		//Translation Y
		//Translation Z
		//Euler X
		//Euler Y
		//Euler Z
		//Scale X
		//Scale Y
		//Scale Z

		Parameters.IsNodeAnimated = false;

		int32 NumAnimations = Parameters.SDKScene->GetSrcObjectCount<FbxAnimStack>();
		if (AnimationIndex >= NumAnimations)
		{
			return;
		}

		int32 NumberOfChannels = 9;

		Parameters.CurrentAnimStack = (FbxAnimStack*)Parameters.SDKScene->GetSrcObject<FbxAnimStack>(AnimationIndex);
		
		int32 NumLayers = Parameters.CurrentAnimStack->GetMemberCount();
		for (int LayerIndex = 0; LayerIndex < NumLayers && !Parameters.IsNodeAnimated; LayerIndex++)
		{
			FbxAnimLayer* AnimLayer = (FbxAnimLayer*)Parameters.CurrentAnimStack->GetMember(LayerIndex);

			TArray<FbxAnimCurve*> TransformChannelCurves;
			TransformChannelCurves.Reserve(NumberOfChannels);

			// Display curves specific to properties
			TransformChannelCurves.Add(Parameters.Node->LclTranslation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_X, false));
			TransformChannelCurves.Add(Parameters.Node->LclTranslation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Y, false));
			TransformChannelCurves.Add(Parameters.Node->LclTranslation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Z, false));

			TransformChannelCurves.Add(Parameters.Node->LclRotation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_X, false));
			TransformChannelCurves.Add(Parameters.Node->LclRotation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Y, false));
			TransformChannelCurves.Add(Parameters.Node->LclRotation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Z, false));

			TransformChannelCurves.Add(Parameters.Node->LclScaling.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_X, false));
			TransformChannelCurves.Add(Parameters.Node->LclScaling.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Y, false));
			TransformChannelCurves.Add(Parameters.Node->LclScaling.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Z, false));

			if (!Parameters.IsNodeAnimated)
			{
				for (int32 ChannelIndex = 0; ChannelIndex < NumberOfChannels; ++ChannelIndex)
				{
					if (TransformChannelCurves[ChannelIndex])
					{
						Parameters.IsNodeAnimated = true;
						break;
					}
				}
			}
		}
	}

	bool FAnimationPayloadContext::FetchPayloadToFile(FFbxParser& Parser, const FString& PayloadFilepath)
	{
		if (AttributeFetchPayloadData.IsSet() || AttributeNodeTransformFetchPayloadData.IsSet())
		{
			return InternalFetchCurveNodePayloadToFile(Parser, PayloadFilepath);
		}
		else if (MorphTargetFetchPayloadData.IsSet())
		{
			return InternalFetchMorphTargetCurvePayloadToFile(Parser, PayloadFilepath);
		}
		return false;
	}

	bool FAnimationPayloadContext::FetchAnimationBakeTransformPayloadForTime(FFbxParser& Parser, const FbxTime Currenttime, FTransform& OutLocalTransform)
	{
		if (!ensure(NodeTransformFetchPayloadData.IsSet()))
		{
			UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
			Message->Text = LOCTEXT("NodeTransformFetchPayloadData_NotSet", "Cannot fetch FBX animation transform payload because the FBX FNodeTransformFetchPayloadData is not set.");
			return false;
		}

		FNodeTransformFetchPayloadData& FetchPayloadData = NodeTransformFetchPayloadData.GetValue();
		if (!ensure(FetchPayloadData.Node))
		{
			UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
			Message->InterchangeKey = Parser.GetFbxHelper()->GetFbxNodeHierarchyName(FetchPayloadData.Node);
			Message->Text = LOCTEXT("FBXNodeNull", "Cannot fetch FBX animation transform payload because the FBX node is null.");
			return false;
		}

		bool bNanErrorLogged = false;
		auto LogNanError = [&bNanErrorLogged, &FetchPayloadData, &Parser]()
			{
				if (bNanErrorLogged)
				{
					return;
				}
				UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
				Message->InterchangeKey = Parser.GetFbxHelper()->GetFbxNodeHierarchyName(FetchPayloadData.Node);
				Message->Text = LOCTEXT("BoneTransformNan", "Error when fetching FBX animation bake transforms payload, some transform contain NAN.");
				bNanErrorLogged = true;
			};

		FbxNode* ParentNode = FetchPayloadData.Node->GetParent();
		if (ParentNode)
		{
			FbxAMatrix NodeTransform = FetchPayloadData.Node->EvaluateGlobalTransform(Currenttime);
			NodeTransform = NodeTransform * Parser.JointOrientationMatrix;

			FTransform GlobalTransform = UE::Interchange::Private::FFbxConvert::ConvertTransform<FTransform, FVector, FQuat>(NodeTransform);
			if (GlobalTransform.ContainsNaN())
			{
				LogNanError();
				GlobalTransform.SetIdentity();
			}

			FbxAMatrix ParentTransform = ParentNode->EvaluateGlobalTransform(Currenttime);
			if (!FetchPayloadData.bIsRootJoint)
			{
				ParentTransform = ParentTransform * Parser.JointOrientationMatrix;
			}

			FTransform ParentGlobalTransform = UE::Interchange::Private::FFbxConvert::ConvertTransform<FTransform, FVector, FQuat>(ParentTransform);
			if (ParentGlobalTransform.ContainsNaN())
			{
				LogNanError();
				ParentGlobalTransform.SetIdentity();
			}

			OutLocalTransform = GlobalTransform.GetRelativeTransform(ParentGlobalTransform);
		}
		else
		{
			FbxAMatrix LocalMatrix = FetchPayloadData.Node->EvaluateLocalTransform(Currenttime);

			FbxVector4 NewLocalT = LocalMatrix.GetT();
			FbxVector4 NewLocalS = LocalMatrix.GetS();
			FbxQuaternion NewLocalQ = LocalMatrix.GetQ();

			OutLocalTransform.SetTranslation(UE::Interchange::Private::FFbxConvert::ConvertPos<FVector>(NewLocalT));
			OutLocalTransform.SetScale3D(UE::Interchange::Private::FFbxConvert::ConvertScale<FVector>(NewLocalS));
			OutLocalTransform.SetRotation(UE::Interchange::Private::FFbxConvert::ConvertRotToQuat<FQuat>(NewLocalQ));
		}

		return true;
	}

	FbxAnimStack* FAnimationPayloadContext::GetAnimStack()
	{
		if (NodeTransformFetchPayloadData.IsSet())
		{
			return NodeTransformFetchPayloadData->CurrentAnimStack;
		}

		return nullptr;
	}

	bool FAnimationPayloadContext::InternalFetchCurveNodePayloadToFile(FFbxParser& Parser, const FString& PayloadFilepath)
	{
		if (AttributeFetchPayloadData.IsSet())
		{
			FAttributeFetchPayloadData& FetchPayloadData = AttributeFetchPayloadData.GetValue();
			if (!ensure(FetchPayloadData.Node != nullptr))
			{
				UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
				Message->InterchangeKey = Parser.GetFbxHelper()->GetFbxNodeHierarchyName(FetchPayloadData.Node);
				Message->Text = LOCTEXT("InternalFetchCurveNodePayloadToFile_FBXNodeNull", "Cannot fetch FBX animation curve payload because the FBX node is null.");
				return false;
			}

			if (!ensure(FetchPayloadData.AnimCurves != nullptr))
			{
				UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
				Message->InterchangeKey = Parser.GetFbxHelper()->GetFbxNodeHierarchyName(FetchPayloadData.Node);
				Message->Text = LOCTEXT("InternalFetchCurveNodePayloadToFile_FBXCurveNull", "Cannot fetch FBX user attribute animation curve payload because the FBX anim curve node is null.");
				return false;
			}
			if (FetchPayloadData.bAttributeTypeIsStepCurveAnimation)
			{
				//Fetch TArray<FInterchangeStepCurve> step curve, no interpolation
				TArray<FInterchangeStepCurve> InterchangeStepCurves;
				const uint32 ChannelCount = FetchPayloadData.AnimCurves->GetChannelsCount();
				for (uint32 ChannelIndex = 0; ChannelIndex < ChannelCount; ++ChannelIndex)
				{
					const uint32 ChannelCurveCount = FetchPayloadData.AnimCurves->GetCurveCount(ChannelIndex);
					for (uint32 CurveIndex = 0; CurveIndex < ChannelCurveCount; ++CurveIndex)
					{
						if (FbxAnimCurve* CurrentAnimCurve = FetchPayloadData.AnimCurves->GetCurve(ChannelIndex, CurveIndex))
						{
							switch (FetchPayloadData.PropertyType)
							{
							case EFbxType::eFbxBool:
								ImportStepCurve<bool>(CurrentAnimCurve, FetchPayloadData.Property, InterchangeStepCurves.AddDefaulted_GetRef());
							case EFbxType::eFbxChar:
							case EFbxType::eFbxUChar:
							case EFbxType::eFbxEnum:
								ImportStepCurve<uint8>(CurrentAnimCurve, FetchPayloadData.Property, InterchangeStepCurves.AddDefaulted_GetRef());
							case EFbxType::eFbxShort:
							case EFbxType::eFbxUShort:
							case EFbxType::eFbxInt:
							case EFbxType::eFbxUInt:
							case EFbxType::eFbxLongLong:
							case EFbxType::eFbxULongLong:
								ImportStepCurve<int32>(CurrentAnimCurve, FetchPayloadData.Property, InterchangeStepCurves.AddDefaulted_GetRef());
								break;
							case EFbxType::eFbxHalfFloat:
							case EFbxType::eFbxFloat:
							case EFbxType::eFbxDouble:
							case EFbxType::eFbxDouble2:
							case EFbxType::eFbxDouble3:
							case EFbxType::eFbxDouble4:
								check(false); //Float curve payload should be extract as FInterchangeCurve since we can interpolate them
								break;
							case EFbxType::eFbxString:
								ImportStringStepCurve(CurrentAnimCurve, FetchPayloadData.Property, InterchangeStepCurves.AddDefaulted_GetRef());
								break;
							}
						}
					}
				}
				{
					FLargeMemoryWriter Ar;
					Ar << InterchangeStepCurves;
					uint8* ArchiveData = Ar.GetData();
					int64 ArchiveSize = Ar.TotalSize();
					TArray64<uint8> Buffer(ArchiveData, ArchiveSize);
					FFileHelper::SaveArrayToFile(Buffer, *PayloadFilepath);
				}
			}
			else
			{
				//Fetch TArray<FInterchangeCurve> which are float curves with interpolation
				TArray<FInterchangeCurve> InterchangeCurves;
				const uint32 ChannelCount = FetchPayloadData.AnimCurves->GetChannelsCount();
				for (uint32 ChannelIndex = 0; ChannelIndex < ChannelCount; ++ChannelIndex)
				{
					const uint32 ChannelCurveCount = FetchPayloadData.AnimCurves->GetCurveCount(ChannelIndex);
					for (uint32 CurveIndex = 0; CurveIndex < ChannelCurveCount; ++CurveIndex)
					{
						if (FbxAnimCurve* CurrentAnimCurve = FetchPayloadData.AnimCurves->GetCurve(ChannelIndex, CurveIndex))
						{
							ImportCurve(CurrentAnimCurve, 1.0f, InterchangeCurves.AddDefaulted_GetRef().Keys);
						}
					}
				}
				{
					FLargeMemoryWriter Ar;
					Ar << InterchangeCurves;
					uint8* ArchiveData = Ar.GetData();
					int64 ArchiveSize = Ar.TotalSize();
					TArray64<uint8> Buffer(ArchiveData, ArchiveSize);
					FFileHelper::SaveArrayToFile(Buffer, *PayloadFilepath);
				}
			}
			return true;
		}
		
		if (AttributeNodeTransformFetchPayloadData.IsSet())
		{
			FAttributeNodeTransformFetchPayloadData& FetchPayloadData = AttributeNodeTransformFetchPayloadData.GetValue();
			//Fetch TArray<FInterchangeCurve> which are float curves with interpolation
			TArray<FInterchangeCurve> InterchangeCurves;

			auto ParseInterchangeCurveCurve = [&InterchangeCurves](FbxAnimCurveNode* AnimCurveNode, const char* pChannelName, float Scale = 1.0f)
			{
				bool bInterchangeCurveAdded = false;
				if (AnimCurveNode)
				{
					int ChannelIndex = AnimCurveNode->GetChannelIndex(pChannelName);
					if (ChannelIndex != -1)
					{
						if (FbxAnimCurve* CurrentAnimCurve = AnimCurveNode->GetCurve(ChannelIndex))
						{
							ImportCurve(CurrentAnimCurve, Scale, InterchangeCurves.AddDefaulted_GetRef().Keys);
							bInterchangeCurveAdded = true;
						}
					}
				}
				
				if (!bInterchangeCurveAdded)
				{
					InterchangeCurves.AddDefaulted();
				}
			};

			auto ResetPivotsPrePostRotationsAndSetRotationOrder = [](FbxNode* Node, double FrameRate)
			{
				//This function now clears out all pivots, post and pre rotations and set's the
				//RotationOrder to XYZ.
				//Updated per the latest documentation
				//https://help.autodesk.com/view/FBX/2017/ENU/?guid=__files_GUID_C35D98CB_5148_4B46_82D1_51077D8970EE_htm

				if (!ensure(Node))
				{
					return;
				}

				// Activate pivot converting 
				Node->SetPivotState(FbxNode::eSourcePivot, FbxNode::ePivotActive);
				Node->SetPivotState(FbxNode::eDestinationPivot, FbxNode::ePivotActive);

				FbxVector4 Zero(0, 0, 0);

				// We want to set all these to 0 and bake them into the transforms. 
				Node->SetPostRotation(FbxNode::eDestinationPivot, Zero);
				Node->SetPreRotation(FbxNode::eDestinationPivot, Zero);
				Node->SetRotationOffset(FbxNode::eDestinationPivot, Zero);
				Node->SetScalingOffset(FbxNode::eDestinationPivot, Zero);
				Node->SetRotationPivot(FbxNode::eDestinationPivot, Zero);
				Node->SetScalingPivot(FbxNode::eDestinationPivot, Zero);

				Node->SetRotationOrder(FbxNode::eDestinationPivot, eEulerXYZ);
				//When  we support other orders do this.
				//EFbxRotationOrder ro;
				//Node->GetRotationOrder(FbxNode::eSourcePivot, ro);
				//Node->SetRotationOrder(FbxNode::eDestinationPivot, ro);

				//Most DCC's don't have this but 3ds Max does
				Node->SetGeometricTranslation(FbxNode::eDestinationPivot, Zero);
				Node->SetGeometricRotation(FbxNode::eDestinationPivot, Zero);
				Node->SetGeometricScaling(FbxNode::eDestinationPivot, Zero);
				//NOTE THAT ConvertPivotAnimationRecursive did not seem to work when getting the local transform values!!!
				Node->ConvertPivotAnimationRecursive(nullptr, FbxNode::eDestinationPivot, FrameRate);
			};

			ResetPivotsPrePostRotationsAndSetRotationOrder(FetchPayloadData.Node, FetchPayloadData.FrameRate);

			ParseInterchangeCurveCurve(FetchPayloadData.TranlsationCurveNode, FBXSDK_CURVENODE_COMPONENT_X);
			ParseInterchangeCurveCurve(FetchPayloadData.TranlsationCurveNode, FBXSDK_CURVENODE_COMPONENT_Y, -1.0f); //FBX to UE conversion with scale -1.0
			ParseInterchangeCurveCurve(FetchPayloadData.TranlsationCurveNode, FBXSDK_CURVENODE_COMPONENT_Z);

			ParseInterchangeCurveCurve(FetchPayloadData.RotationCurveNode, FBXSDK_CURVENODE_COMPONENT_X);
			ParseInterchangeCurveCurve(FetchPayloadData.RotationCurveNode, FBXSDK_CURVENODE_COMPONENT_Y, -1.0f); //FBX to UE conversion with scale -1.0
			ParseInterchangeCurveCurve(FetchPayloadData.RotationCurveNode, FBXSDK_CURVENODE_COMPONENT_Z, -1.0f); //FBX to UE conversion with scale -1.0

			ParseInterchangeCurveCurve(FetchPayloadData.ScaleCurveNode, FBXSDK_CURVENODE_COMPONENT_X);
			ParseInterchangeCurveCurve(FetchPayloadData.ScaleCurveNode, FBXSDK_CURVENODE_COMPONENT_Y);
			ParseInterchangeCurveCurve(FetchPayloadData.ScaleCurveNode, FBXSDK_CURVENODE_COMPONENT_Z);

			{
				FLargeMemoryWriter Ar;
				Ar << InterchangeCurves;
				uint8* ArchiveData = Ar.GetData();
				int64 ArchiveSize = Ar.TotalSize();
				TArray64<uint8> Buffer(ArchiveData, ArchiveSize);
				FFileHelper::SaveArrayToFile(Buffer, *PayloadFilepath);
			}

			return true;
		}
		return false;
	}

	bool FAnimationPayloadContext::InternalFetchMorphTargetCurvePayloadToFile(FFbxParser& Parser, const FString& PayloadFilepath)
	{
		check(MorphTargetFetchPayloadData.IsSet());
		FMorphTargetFetchPayloadData& FetchPayloadData = MorphTargetFetchPayloadData.GetValue();
		
		TArray<FInterchangeCurve> InterchangeCurves;
		if (InternalFetchMorphTargetCurvePayload(Parser, InterchangeCurves))
		{
			FLargeMemoryWriter Ar;
			Ar << InterchangeCurves;
			Ar << FetchPayloadData.InbetweenCurveNames;
			Ar << FetchPayloadData.InbetweenFullWeights;
			uint8* ArchiveData = Ar.GetData();
			int64 ArchiveSize = Ar.TotalSize();
			TArray64<uint8> Buffer(ArchiveData, ArchiveSize);
			FFileHelper::SaveArrayToFile(Buffer, *PayloadFilepath);
			return true;
		}
		return false;
	}

	bool FAnimationPayloadContext::InternalFetchMorphTargetCurvePayload(FFbxParser& Parser, TArray<FInterchangeCurve>& InterchangeCurves)
	{
		check(MorphTargetFetchPayloadData.IsSet());
		FMorphTargetFetchPayloadData& FetchPayloadData = MorphTargetFetchPayloadData.GetValue();
	
		if (!FetchPayloadData.SDKScene)
		{
			UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
			Message->Text = LOCTEXT("InternalFetchMorphTargetCurvePayloadToFile_FBXSDKSceneNull", "InternalFetchMorphTargetCurvePayloadToFile, fbx sdk is nullptr.");
			return false;
		}

		FbxGeometry* Geometry = FetchPayloadData.SDKScene->GetGeometry(FetchPayloadData.GeometryIndex);

		if (!Geometry)
		{
			UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
			Message->Text = LOCTEXT("InternalFetchMorphTargetCurvePayloadToFile_FBXGeometryNull", "Cannot fetch FBX geometry from the scene.");
			return false;
		}
		FbxAnimCurve* AnimCurve = Geometry->GetShapeChannel(FetchPayloadData.MorphTargetIndex, FetchPayloadData.ChannelIndex, FetchPayloadData.AnimLayer);

		//Morph target curve in fbx are between 0 and 100, in Unreal we are between 0 and 1, so we must scale
		//The curve with 0.01
		constexpr float ScaleCurve = 0.01f;
		return ImportCurve(AnimCurve, ScaleCurve, InterchangeCurves.AddDefaulted_GetRef().Keys);
	}

	bool FFbxAnimation::AddSkeletalTransformAnimation(UInterchangeBaseNodeContainer& NodeContainer
		, FbxScene* SDKScene
		, FFbxParser& Parser
		, FbxNode* Node
		, UInterchangeSceneNode* SceneNode
		, TMap<FString, TSharedPtr<FPayloadContextBase>>& PayloadContexts
		, UInterchangeSkeletalAnimationTrackNode* SkeletalAnimationTrackNode
		, const int32& AnimationIndex
		, bool bIsRootJoint)
	{
		FGetFbxTransformCurvesParameters Parameters(SDKScene, Node);
		GetFbxTransformCurves(Parameters, AnimationIndex);
		if (!Parameters.IsNodeAnimated)
		{
			//If we have a joint under the root skeleton and there is some animation in the parent hierarchy
			//we have to enable IsNodeAnimated so it get bake correctly and generate the appropriate curves
			FString RootSkeletonNodeUid;
			if (SkeletalAnimationTrackNode->GetCustomSkeletonNodeUid(RootSkeletonNodeUid))
			{
				const UInterchangeSceneNode* ParentNode = Cast<UInterchangeSceneNode>(NodeContainer.GetNode(SceneNode->GetParentUid()));
				//Search up the hierarchy if we found any animated parent we have to animate the children
				while (ParentNode)
				{
					if (SkeletalAnimationTrackNode->IsNodeAnimatedWithBakedCurve(ParentNode->GetUniqueID()))
					{
						Parameters.IsNodeAnimated = true;
						break;
					}
					if (ParentNode->GetUniqueID().Equals(RootSkeletonNodeUid))
					{
						break;
					}
					ParentNode = Cast<UInterchangeSceneNode>(NodeContainer.GetNode(ParentNode->GetParentUid()));
				}
			}
		}

		if (Parameters.IsNodeAnimated)
		{
			FString PayLoadKey = Parser.GetFbxHelper()->GetFbxNodeHierarchyName(Node) + TEXT("_") + FString::FromInt(AnimationIndex) + TEXT("_SkeletalAnimationPayloadKey");
			if (ensure(!PayloadContexts.Contains(PayLoadKey)))
			{
				TSharedPtr<FAnimationPayloadContext> AnimPayload = MakeShared<FAnimationPayloadContext>();
				FNodeTransformFetchPayloadData FetchPayloadData;
				FetchPayloadData.Node = Node;
				FetchPayloadData.CurrentAnimStack = Parameters.CurrentAnimStack;
				FetchPayloadData.bIsRootJoint = bIsRootJoint;

				AnimPayload->NodeTransformFetchPayloadData = FetchPayloadData;
				PayloadContexts.Add(PayLoadKey, AnimPayload);
			}
			SkeletalAnimationTrackNode->SetAnimationPayloadKeyForSceneNodeUid(SceneNode->GetUniqueID(), PayLoadKey, EInterchangeAnimationPayLoadType::BAKED);
		}

		return Parameters.IsNodeAnimated;
	}

	void FFbxAnimation::AddNodeAttributeCurvesAnimation(FFbxParser& Parser
		, FbxNode* Node
		, FbxProperty& Property
		, FbxAnimCurveNode* AnimCurveNode
		, TMap<FString, TSharedPtr<FPayloadContextBase>>& PayloadContexts
		, EFbxType PropertyType
		, TOptional<FString>& OutPayloadKey
		, TOptional<bool>& OutIsStepCurve)
	{
		const FString PropertyName = Parser.GetFbxHelper()->GetFbxPropertyName(Property);
		const FString PayLoadKey = Parser.GetFbxHelper()->GetFbxNodeHierarchyName(Node) + PropertyName + TEXT("_AnimationPayloadKey");
		if (ensure(!PayloadContexts.Contains(PayLoadKey)))
		{
			TSharedPtr<FAnimationPayloadContext> AnimPayload = MakeShared<FAnimationPayloadContext>();
			FAttributeFetchPayloadData FetchPayloadData;
			FetchPayloadData.Node = Node;
			FetchPayloadData.AnimCurves = AnimCurveNode;
			
			//Only curves with Constant interpolation on all keys are deemed as StepCurves.
			OutIsStepCurve = !IsFbxPropertyTypeDecimal(PropertyType) && IsStepCurve(AnimCurveNode);
			FetchPayloadData.bAttributeTypeIsStepCurveAnimation = OutIsStepCurve.GetValue();
			FetchPayloadData.PropertyType = PropertyType;
			FetchPayloadData.Property = Property;
			AnimPayload->AttributeFetchPayloadData = FetchPayloadData;
			PayloadContexts.Add(PayLoadKey, AnimPayload);
			OutPayloadKey = PayLoadKey;
		}
	}

	void FFbxAnimation::AddRigidTransformAnimation(FFbxParser& Parser
		, FbxNode* Node
		, const int32& AnimationIndex
		, FbxAnimCurveNode* TranslationCurveNode
		, FbxAnimCurveNode* RotationCurveNode
		, FbxAnimCurveNode* ScaleCurveNode
		, TMap<FString, TSharedPtr<FPayloadContextBase>>& PayloadContexts
		, TOptional<FString>& OutPayloadKey)
	{
		const FString PayloadKeySuffix = FString::Printf(TEXT("_RigidAnimationPayloadKey_%d"), AnimationIndex);
		const FString PayLoadKey = Parser.GetFbxHelper()->GetFbxNodeHierarchyName(Node) + PayloadKeySuffix;
		if (ensure(!PayloadContexts.Contains(PayLoadKey)))
		{
			TSharedPtr<FAnimationPayloadContext> AnimPayload = MakeShared<FAnimationPayloadContext>();
			FAttributeNodeTransformFetchPayloadData FetchPayloadData;
			FetchPayloadData.FrameRate = Parser.GetFrameRate();
			FetchPayloadData.Node = Node;
			FetchPayloadData.TranlsationCurveNode = TranslationCurveNode;
			FetchPayloadData.RotationCurveNode = RotationCurveNode;
			FetchPayloadData.ScaleCurveNode = ScaleCurveNode;

			//Any property that is not decimal should import with step curve
			AnimPayload->AttributeNodeTransformFetchPayloadData = FetchPayloadData;
			PayloadContexts.Add(PayLoadKey, AnimPayload);
			OutPayloadKey = PayLoadKey;
		}
	}

	void FFbxAnimation::AddMorphTargetCurvesAnimation(FbxScene* SDKScene
		, FFbxParser& Parser
		, UInterchangeSkeletalAnimationTrackNode* SkeletalAnimationTrackNode
		, TMap<FString, TSharedPtr<FPayloadContextBase>>& PayloadContexts
		, const FMorphTargetAnimationBuildingData& MorphTargetAnimationBuildingData)
	{
		FString PayLoadKey = MorphTargetAnimationBuildingData.MorphTargetNodeUid
			+ TEXT("\\") + (MorphTargetAnimationBuildingData.InterchangeMeshNode ? MorphTargetAnimationBuildingData.InterchangeMeshNode->GetUniqueID() : FString()) //Same shape can be animated on different mesh node
			+ TEXT("\\") + FString::FromInt(MorphTargetAnimationBuildingData.AnimationIndex)
			+ TEXT("\\") + FString::FromInt(MorphTargetAnimationBuildingData.MorphTargetIndex)
			+ TEXT("\\") + FString::FromInt(MorphTargetAnimationBuildingData.ChannelIndex)
			+ TEXT("_CurveAnimationPayloadKey");

		if (ensure(!PayloadContexts.Contains(PayLoadKey)))
		{
			TSharedPtr<FAnimationPayloadContext> AnimPayload = MakeShared<FAnimationPayloadContext>();
			FMorphTargetFetchPayloadData FetchPayloadData;
			FetchPayloadData.SDKScene = SDKScene;
			FetchPayloadData.GeometryIndex = MorphTargetAnimationBuildingData.GeometryIndex;
			FetchPayloadData.MorphTargetIndex = MorphTargetAnimationBuildingData.MorphTargetIndex;
			FetchPayloadData.ChannelIndex = MorphTargetAnimationBuildingData.ChannelIndex;
			FetchPayloadData.AnimLayer = MorphTargetAnimationBuildingData.AnimLayer;
			FetchPayloadData.InbetweenCurveNames = MorphTargetAnimationBuildingData.InbetweenCurveNames;
			FetchPayloadData.InbetweenFullWeights = MorphTargetAnimationBuildingData.InbetweenFullWeights;

			AnimPayload->MorphTargetFetchPayloadData = FetchPayloadData;

			PayloadContexts.Add(PayLoadKey, AnimPayload);
		}

		SkeletalAnimationTrackNode->SetAnimationPayloadKeyForMorphTargetNodeUid(MorphTargetAnimationBuildingData.MorphTargetNodeUid, PayLoadKey, EInterchangeAnimationPayLoadType::MORPHTARGETCURVE);
	}

	bool FFbxAnimation::IsFbxPropertyTypeSupported(EFbxType PropertyType)
	{
		switch (PropertyType)
		{
		case EFbxType::eFbxBool:
		case EFbxType::eFbxChar:		//!< 8 bit signed integer.
		case EFbxType::eFbxUChar:		//!< 8 bit unsigned integer.
		case EFbxType::eFbxShort:		//!< 16 bit signed integer.
		case EFbxType::eFbxUShort:		//!< 16 bit unsigned integer.
		case EFbxType::eFbxInt:			//!< 32 bit signed integer.
		case EFbxType::eFbxUInt:		//!< 32 bit unsigned integer.
		case EFbxType::eFbxLongLong:	//!< 64 bit signed integer.
		case EFbxType::eFbxULongLong:	//!< 64 bit unsigned integer.
		case EFbxType::eFbxHalfFloat:	//!< 16 bit floating point.
		case EFbxType::eFbxFloat:		//!< Floating point value.
		case EFbxType::eFbxDouble:	//!< Double width floating point value.
		case EFbxType::eFbxDouble2:	//!< Vector of two double values.
		case EFbxType::eFbxDouble3:	//!< Vector of three double values.
		case EFbxType::eFbxDouble4:	//!< Vector of four double values.
		case EFbxType::eFbxEnum:		//!< Enumeration.
		case EFbxType::eFbxString:	//!< String.
			return true;
		}

		return false;
	};

	bool FFbxAnimation::IsFbxPropertyTypeDecimal(EFbxType PropertyType)
	{
		switch (PropertyType)
		{
		case EFbxType::eFbxHalfFloat:
		case EFbxType::eFbxFloat:
		case EFbxType::eFbxDouble:
		case EFbxType::eFbxDouble2:
		case EFbxType::eFbxDouble3:
		case EFbxType::eFbxDouble4:
			return true;
		}

		return false;
	}

	//PayloadQueries arriving here should be of the same start/stop and frequency:
	bool FFbxAnimation::FetchAnimationBakeTransformPayload(FFbxParser& Parser, FbxScene* SDKScene,
		TMap<FString, TSharedPtr<FPayloadContextBase>>& PayloadContexts,
		const TArray<const UE::Interchange::FAnimationPayloadQuery*>& PayloadQueries,
		const FString& ResultFolder, FCriticalSection* ResultPayloadsCriticalSection,
		TAtomic<int64>& UniqueIdCounter, TMap<FString, FString>& ResultPayloads,
		TArray<FText>& OutErrorMessages)
	{
		//Helper Structs:
		struct FPayloadQueryHelper
		{
			const UE::Interchange::FAnimationPayloadQuery* PayloadQuery;
			TSharedPtr<FPayloadContextBase> PayloadContext;
			FPayloadQueryHelper(const UE::Interchange::FAnimationPayloadQuery* InPayloadQuery, TSharedPtr<FPayloadContextBase> InPayloadContext)
				: PayloadQuery(InPayloadQuery)
				, PayloadContext(InPayloadContext)
			{
			}
		};
		struct FPayloadDataHelper
		{
			FString QueryHashString; //Used for payload file name
			FAnimationPayloadData PayloadData;
			FPayloadDataHelper(const FString& InHashString, const FString& InSceneNodeUID, const FInterchangeAnimationPayLoadKey& InPayloadKey)
				: QueryHashString(InHashString)
				, PayloadData(InSceneNodeUID, InPayloadKey)
			{
			}
		};

		//If no PayloadQueries then return
		if (PayloadQueries.Num() == 0)
		{
			return true;
		}

		//Get Timings and Number of Frames for the current PayloadQueries
		//		Note: PayloadQueries arriving here(FetchAnimationBakeTransformPayloadInternal) should be of the same start/stop and frequency, so we can grab the first one's:
		FAnimationTimeDescription TimeDescription = PayloadQueries[0]->TimeDescription;
		if (!ensure(!FMath::IsNearlyZero(TimeDescription.BakeFrequency)))
		{
			OutErrorMessages.Add(LOCTEXT("BakeFrequencyZero", "Cannot fetch FBX animation bake transforms payload because the bake frequency is zero."));
			return false;
		}
		FbxTime StartTime;
		StartTime.SetSecondDouble(TimeDescription.RangeStartSecond);
		FbxTime EndTime;
		EndTime.SetSecondDouble(TimeDescription.RangeStopSecond);
		if (!ensure(TimeDescription.RangeStopSecond > TimeDescription.RangeStartSecond))
		{
			OutErrorMessages.Add(LOCTEXT("InvalidRange", "Cannot fetch FBX animation bake transforms payload because the bake range is invalid."));
			return false;
		}
		const double TimeStepSecond = 1.0 / TimeDescription.BakeFrequency;
		FbxTime TimeStep = 0;
		TimeStep.SetSecondDouble(TimeStepSecond);

		const double SequenceLength = FMath::Max<double>(TimeDescription.RangeStopSecond - TimeDescription.RangeStartSecond, TimeStepSecond);
		const int32 NumFrame = FMath::RoundToInt32(SequenceLength * TimeDescription.BakeFrequency);
		int32 BakeKeyCount = NumFrame + 1;

		ensure(NumFrame >= 0);

		//Acquire PayloadContexts and AnimationStacks for the PayloadQueries (also group the FPayloadQueryHelper per AnimationStacks):
		//		PayloadQueryHelpers maps AnimStacks to FPaloayQueryHelper (which contains a PayloadQuery and the PayloadContext that belongs to it)
		TMap<FbxAnimStack*, TArray<FPayloadQueryHelper>> PayloadQueryHelpers;
		{
			for (const UE::Interchange::FAnimationPayloadQuery* PayloadQuery : PayloadQueries)
			{
				if (!PayloadContexts.Contains(PayloadQuery->PayloadKey.UniqueId))
				{
					OutErrorMessages.Add(FText::Format(LOCTEXT("CannotRetrievePayload", "Cannot retrieve payload; payload key['{0}'] doesn't have any context."), FText::FromString(PayloadQuery->PayloadKey.UniqueId)));
					continue;
				}

				TSharedPtr<FPayloadContextBase> PayloadContext = PayloadContexts.FindChecked(PayloadQuery->PayloadKey.UniqueId);

				FbxAnimStack* AnimStack = PayloadContext->GetAnimStack();
				if (!AnimStack)
				{
					continue;
				}

				FString ResultPayloadUniqueId = PayloadQuery->GetHashString();

				{
					FScopeLock Lock(ResultPayloadsCriticalSection);
					//If we already have extract this mesh, no need to extract again
					if (ResultPayloads.Contains(ResultPayloadUniqueId))
					{
						continue;
					}
				}

				TArray<FPayloadQueryHelper>& PayloadHelpersPerAnimStack = PayloadQueryHelpers.FindOrAdd(AnimStack);
				PayloadHelpersPerAnimStack.Add(FPayloadQueryHelper(PayloadQuery, PayloadContexts.FindChecked(PayloadQuery->PayloadKey.UniqueId)));
			}
		}

		//Iterate on the AnimStacked groups:
		for (TPair<FbxAnimStack*, TArray<FPayloadQueryHelper>>& AnimStackPayloadQueryHelpersPair : PayloadQueryHelpers)
		{
			SDKScene->SetCurrentAnimationStack(AnimStackPayloadQueryHelpersPair.Key);

			TArray<FPayloadQueryHelper>& PayloadQueryHelpersForAnimStack = AnimStackPayloadQueryHelpersPair.Value;

			//Initialize PayloadDataHelpers (contains the HashString for the Query and the FAnimationPayloadData
			TArray<FPayloadDataHelper> PayloadDataHelpersForAnimStack;
			PayloadDataHelpersForAnimStack.Empty(PayloadQueryHelpersForAnimStack.Num());
			for (FPayloadQueryHelper& PayloadQueryHelper : PayloadQueryHelpersForAnimStack)
			{
				FPayloadDataHelper PayloadDataHelper(PayloadQueryHelper.PayloadQuery->GetHashString(), PayloadQueryHelper.PayloadQuery->SceneNodeUniqueID, PayloadQueryHelper.PayloadQuery->PayloadKey);

				PayloadDataHelper.PayloadData.BakeFrequency = PayloadQueryHelper.PayloadQuery->TimeDescription.BakeFrequency;
				PayloadDataHelper.PayloadData.RangeStartTime = PayloadQueryHelper.PayloadQuery->TimeDescription.RangeStartSecond;
				PayloadDataHelper.PayloadData.RangeEndTime = PayloadQueryHelper.PayloadQuery->TimeDescription.RangeStopSecond;

				PayloadDataHelper.PayloadData.Transforms.SetNum(BakeKeyCount);

				PayloadDataHelpersForAnimStack.Add(PayloadDataHelper);
			}

			//Acquire Bone Transforms:
			//Time iteration
			FbxTime CurrentTime = StartTime;
			for (size_t FrameIndex = 0; FrameIndex < BakeKeyCount; FrameIndex++, CurrentTime+=TimeStep)
			{

				//Bone iteration:
				for (size_t PayloadDataIndex = 0; PayloadDataIndex < PayloadDataHelpersForAnimStack.Num(); PayloadDataIndex++)
				{
					FPayloadQueryHelper& PayloadQueryHelper = PayloadQueryHelpersForAnimStack[PayloadDataIndex];
					FPayloadDataHelper& PayloadDataHelper = PayloadDataHelpersForAnimStack[PayloadDataIndex];

					FTransform Transform;
					if (PayloadQueryHelper.PayloadContext->FetchAnimationBakeTransformPayloadForTime(Parser, CurrentTime, Transform))
					{
						PayloadDataHelper.PayloadData.Transforms[FrameIndex] = Transform;
					}
					else
					{
						if (FrameIndex == 0)
						{
							PayloadDataHelper.PayloadData.Transforms[FrameIndex] = FTransform::Identity;
						}
						else
						{
							PayloadDataHelper.PayloadData.Transforms[FrameIndex] = PayloadDataHelper.PayloadData.Transforms[FrameIndex - 1];
						}
					}
				}
			}

			//Write out results:
			ParallelFor(PayloadDataHelpersForAnimStack.Num(), [&PayloadDataHelpersForAnimStack, &ResultPayloads, &ResultPayloadsCriticalSection, &ResultFolder, &UniqueIdCounter](int32 PayloadDataHelperIndex)
				{
					FPayloadDataHelper& PayloadDataHelper = PayloadDataHelpersForAnimStack[PayloadDataHelperIndex];
					FString QueryHashString = PayloadDataHelper.QueryHashString; //PayloadQuery's GetHashString()

					FString PayloadFilepathCopy;
					{
						FScopeLock Lock(ResultPayloadsCriticalSection);
						//If we already have extract this mesh, no need to extract again
						if (ResultPayloads.Contains(QueryHashString))
						{
							return;
						}

						FString& PayloadFilepath = ResultPayloads.FindOrAdd(QueryHashString);
						//To avoid file path with too many character, we hash the payloadKey so we have a deterministic length for the file path.
						PayloadFilepath = ResultFolder + TEXT("/") + QueryHashString + FString::FromInt(UniqueIdCounter.IncrementExchange()) + TEXT(".payload");

						//Copy the map filename key because we are multithreaded and the TMap can be reallocated
						PayloadFilepathCopy = PayloadFilepath;
					}

					FString PayloadFilepath = PayloadFilepathCopy;

					FLargeMemoryWriter Ar;
					PayloadDataHelper.PayloadData.SerializeBaked(Ar);
					uint8* ArchiveData = Ar.GetData();
					int64 ArchiveSize = Ar.TotalSize();
					TArray64<uint8> Buffer(ArchiveData, ArchiveSize);
					FFileHelper::SaveArrayToFile(Buffer, *PayloadFilepath);
				});
		}

		return true;
	}
}//ns UE::Interchange::Private

#undef LOCTEXT_NAMESPACE
