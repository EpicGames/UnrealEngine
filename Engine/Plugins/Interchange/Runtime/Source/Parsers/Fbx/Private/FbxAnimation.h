// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FbxAPI.h"
#include "FbxHelper.h"
#include "FbxInclude.h"

/** Forward declarations */
struct FInterchangeCurve;
struct FMeshDescription;
class UInterchangeBaseNodeContainer;
class UInterchangeMeshNode;
class UInterchangeSceneNode;
class UInterchangeSkeletalAnimationTrackNode;

namespace UE::Interchange
{
	struct FAnimationPayloadQuery;
}

namespace UE::Interchange::Private
{
	struct FMorphTargetAnimationBuildingData;

	//Skeletal/Rigged Animation
	struct FNodeTransformFetchPayloadData
	{
		FbxNode* Node = nullptr;
		FbxAnimStack* CurrentAnimStack = nullptr;
		bool bIsRootJoint = false;
	};

	//User defined animations:
	struct FAttributeFetchPayloadData
	{
		FbxNode* Node = nullptr;
		FbxAnimCurveNode* AnimCurves = nullptr;
		
		//If true the payload is TArray<FInterchangeStepCurve>, if false the payload is TArray<FInterchangeCurve>
		bool bAttributeTypeIsStepCurveAnimation = false;
		fbxsdk::EFbxType PropertyType = fbxsdk::EFbxType::eFbxUndefined;
		FbxProperty Property;
	};

	//Rigid Animation
	struct FAttributeNodeTransformFetchPayloadData
	{
		double FrameRate = 30.0;
		FbxNode* Node = nullptr;
		FbxAnimCurveNode* TranlsationCurveNode = nullptr;
		FbxAnimCurveNode* RotationCurveNode = nullptr;
		FbxAnimCurveNode* ScaleCurveNode = nullptr;
	};

	struct FMorphTargetFetchPayloadData
	{
		//We cannot just store the FbxAnimCurve
		// because! it is referenced from FbxMesh which gets replaced when we are triangulating it upon PayloadData acquisition
		// for that reason we need to re-acquire it all the way from the Scene, based on GeometryIndex, MorphTargetIndex, ChannelIndex and AnimLayer.
		// (This is happening because upon Triangulation we are passing bReplace = true, see @FMeshDescriptionImporter::FillMeshDescriptionFromFbxMesh)
		//FbxAnimCurve* MorphTargetAnimCurve = nullptr;

		FbxScene* SDKScene;

		int32 GeometryIndex;
		int32 MorphTargetIndex;
		int32 ChannelIndex;
		FbxAnimLayer* AnimLayer;

		//In between blend shape animation support
		//The count of the curve names should match the in between full weights
		TArray<FString> InbetweenCurveNames;
		TArray<float> InbetweenFullWeights;
	};

	

	class FAnimationPayloadContext : public FPayloadContextBase
	{
	public:
		virtual ~FAnimationPayloadContext() = default;
		virtual FString GetPayloadType() const override { return TEXT("TransformAnimation-PayloadContext"); }
		virtual bool FetchPayloadToFile(FFbxParser& Parser, const FString& PayloadFilepath) override;
		virtual bool FetchAnimationBakeTransformPayloadForTime(FFbxParser& Parser, const FbxTime Currenttime, FTransform& Out) override;
		virtual FbxAnimStack* GetAnimStack() override;
		
		TOptional<FNodeTransformFetchPayloadData> NodeTransformFetchPayloadData;
		TOptional<FAttributeFetchPayloadData> AttributeFetchPayloadData;
		TOptional<FAttributeNodeTransformFetchPayloadData> AttributeNodeTransformFetchPayloadData;
		TOptional<FMorphTargetFetchPayloadData> MorphTargetFetchPayloadData;
	private:
		bool InternalFetchCurveNodePayloadToFile(FFbxParser& Parser, const FString& PayloadFilepath);
		bool InternalFetchMorphTargetCurvePayloadToFile(FFbxParser& Parser, const FString& PayloadFilepath);
		bool InternalFetchMorphTargetCurvePayload(FFbxParser& Parser, TArray<FInterchangeCurve>& InterchangeCurves);
	};

	class FFbxAnimation
	{
	public:
		/** This function add the payload key if the scene node transform is animated. */
		static bool AddSkeletalTransformAnimation(UInterchangeBaseNodeContainer& NodeContainer
			, FbxScene* SDKScene
			, FFbxParser& Parser
			, FbxNode* Node
			, UInterchangeSceneNode* UnrealNode
			, TMap<FString, TSharedPtr<FPayloadContextBase>>& PayloadContexts
			, UInterchangeSkeletalAnimationTrackNode* SkeletalAnimationTrackNode
			, const int32& AnimationIndex
			, bool bIsRootJoint);

		/** This function add the payload key for an animated node user attribute. */
		static void AddNodeAttributeCurvesAnimation(FFbxParser& Parser
			, FbxNode* Node
			, FbxProperty& Property
			, FbxAnimCurveNode* AnimCurveNode
			, TMap<FString, TSharedPtr<FPayloadContextBase>>& PayloadContexts
			, EFbxType PropertyType
			, TOptional<FString>& OutPayloadKey
			, TOptional<bool>& OutIsStepCurve);

		static void AddRigidTransformAnimation(FFbxParser& Parser
			, FbxNode* Node
			, const int32& AnimationIndex
			, FbxAnimCurveNode* TranslationCurveNode
			, FbxAnimCurveNode* RotationCurveNode
			, FbxAnimCurveNode* ScaleCurveNode
			, TMap<FString, TSharedPtr<FPayloadContextBase>>& PayloadContexts
			, TOptional<FString>& OutPayloadKey);

		/** This function add the payload key for an animated curve for morph target. */
		static void AddMorphTargetCurvesAnimation(FbxScene* SDKScene
			, FFbxParser& Parser
			, UInterchangeSkeletalAnimationTrackNode* SkeletalAnimationTrackNode
			, TMap<FString, TSharedPtr<FPayloadContextBase>>& PayloadContexts
			, const FMorphTargetAnimationBuildingData& MorphTargetAnimationBuildingData);

		static bool IsFbxPropertyTypeSupported(EFbxType PropertyType);
		static bool IsFbxPropertyTypeDecimal(EFbxType PropertyType);
		
		//PayloadQueries arriving here should be of the same start/stop and frequency:
		static bool FetchAnimationBakeTransformPayload(FFbxParser& Parser, FbxScene* SDKScene,
			TMap<FString, TSharedPtr<FPayloadContextBase>>& PayloadContexts,
			const TArray<const UE::Interchange::FAnimationPayloadQuery*>& PayloadQueries, 
			const FString& ResultFolder, FCriticalSection* ResultPayloadsCriticalSection, 
			TAtomic<int64>& UniqueIdCounter, TMap<FString, FString>& ResultPayloads,
			TArray<FText>& OutErrorMessages);
	};
}//ns UE::Interchange::Private
