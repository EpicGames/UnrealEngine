// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxCamera.h"

#include "CoreMinimal.h"
#include "FbxAPI.h"
#include "FbxHelper.h"
#include "FbxInclude.h"
#include "InterchangeCameraNode.h"
#include "InterchangeResultsContainer.h"
#include "Math/UnitConversion.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#define LOCTEXT_NAMESPACE "InterchangeFbxCamera"

namespace UE
{
	namespace Interchange
	{
		namespace Private
		{
			void FillCameraNode(UInterchangePhysicalCameraNode* CameraNode, FbxCamera& SourceCamera)
			{
				if (!CameraNode)
				{
					return;
				}

				float FocalLength;
				if (SourceCamera.GetApertureMode() == FbxCamera::eFocalLength)
				{
					FocalLength = SourceCamera.FocalLength.Get();
				}
				else
				{
					FocalLength = SourceCamera.ComputeFocalLength(SourceCamera.FieldOfView.Get());
				}

				CameraNode->SetCustomFocalLength(FocalLength); //Both FBX and UE have their focal length in mm
				CameraNode->SetCustomSensorHeight(FUnitConversion::Convert(SourceCamera.GetApertureHeight(), EUnit::Inches, EUnit::Millimeters));
				CameraNode->SetCustomSensorWidth(FUnitConversion::Convert(SourceCamera.GetApertureWidth(), EUnit::Inches, EUnit::Millimeters));
			}

			UInterchangePhysicalCameraNode* FFbxCamera::CreateCameraNode(UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeUid, const FString& NodeName)
			{
				UInterchangePhysicalCameraNode* CameraNode = NewObject<UInterchangePhysicalCameraNode>(&NodeContainer, NAME_None);
				if (!ensure(CameraNode))
				{
					UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
					Message->Text = LOCTEXT("CannotAllocateNode", "Cannot allocate a node when importing FBX.");
					return nullptr;
				}

				NodeContainer.SetupNode(CameraNode, NodeUid, NodeName, EInterchangeNodeContainerType::TranslatedAsset);

				return CameraNode;
			}

			void FFbxCamera::AddCamerasRecursively(FbxNode* Node, UInterchangeBaseNodeContainer& NodeContainer)
			{
				int32 AttributeCount = Node->GetNodeAttributeCount();
				for (int32 AttributeIndex = 0; AttributeIndex < AttributeCount; ++AttributeIndex)
				{
					FbxNodeAttribute* NodeAttribute = Node->GetNodeAttributeByIndex(AttributeIndex);

					if (NodeAttribute && NodeAttribute->GetAttributeType() == FbxNodeAttribute::eCamera)
					{
						FString NodeName = Parser.GetFbxHelper()->GetNodeAttributeName(NodeAttribute, UInterchangePhysicalCameraNode::StaticAssetTypeName());
						FString NodeUid = Parser.GetFbxHelper()->GetNodeAttributeUniqueID(NodeAttribute, UInterchangePhysicalCameraNode::StaticAssetTypeName());

						const UInterchangePhysicalCameraNode* CameraNode = Cast<const UInterchangePhysicalCameraNode>(NodeContainer.GetNode(NodeUid));

						if (!CameraNode)
						{
							UInterchangePhysicalCameraNode* NewCameraNode = CreateCameraNode(NodeContainer, NodeUid, NodeName);
							FillCameraNode(NewCameraNode, static_cast<FbxCamera&>(*NodeAttribute));
						}
					}
				}

				const int32 ChildCount = Node->GetChildCount();
				for (int32 ChildIndex = 0; ChildIndex < ChildCount; ++ChildIndex)
				{
					FbxNode* ChildNode = Node->GetChild(ChildIndex);
					AddCamerasRecursively(ChildNode, NodeContainer);
				}
			}

			void FFbxCamera::AddAllCameras(FbxScene* SDKScene, UInterchangeBaseNodeContainer& NodeContainer)
			{
				AddCamerasRecursively(SDKScene->GetRootNode(), NodeContainer);
			}
		} //ns Private
	} //ns Interchange
}//ns UE

#undef LOCTEXT_NAMESPACE
