// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FbxInclude.h"

class UInterchangeBaseNodeContainer;
class UInterchangePhysicalCameraNode;

namespace UE
{
	namespace Interchange
	{
		namespace Private
		{
			class FFbxParser;

			class FFbxCamera
			{
			public:
				explicit FFbxCamera(FFbxParser& InParser)
					: Parser(InParser)
				{}

				/**
				 * Create a UInterchangePhysicalCameraNode and add it to the NodeContainer for all FbxNodeAttributes of type eCamera that the fbx file contains.
				 * 
				 * @note - Any node that already exist in the NodeContainer will not be created or modified.
				 */
				void AddAllCameras(FbxScene* SDKScene, UInterchangeBaseNodeContainer& NodeContainer);

			protected:
				UInterchangePhysicalCameraNode* CreateCameraNode(UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeUID, const FString& NodeName);
				void AddCamerasRecursively(FbxNode* Node, UInterchangeBaseNodeContainer& NodeContainer);

			private:
				FFbxParser& Parser;
			};
		}//ns Private
	}//ns Interchange
}//ns UE
