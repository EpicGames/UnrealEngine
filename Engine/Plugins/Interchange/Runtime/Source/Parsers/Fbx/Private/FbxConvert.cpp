// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxConvert.h"

#include "CoreMinimal.h"
#include "FbxInclude.h"

#define CONVERT_TO_FRONT_X 0

namespace UE
{
	namespace Interchange
	{
		namespace Private
		{
			FString GetFileAxisDirection(FbxAxisSystem FileAxisSystem)
			{
				FString AxisDirection;
				int32 Sign = 1;
				switch (FileAxisSystem.GetUpVector(Sign))
				{
					case FbxAxisSystem::eXAxis:
						{
							AxisDirection += TEXT("X");
						}
						break;
					case FbxAxisSystem::eYAxis:
						{
							AxisDirection += TEXT("Y");
						}
						break;
					case FbxAxisSystem::eZAxis:
						{
							AxisDirection += TEXT("Z");
						}
						break;
				}

				//Negative sign mean down instead of up
				AxisDirection += Sign == 1 ? TEXT("-UP") : TEXT("-DOWN");

				switch (FileAxisSystem.GetCoorSystem())
				{
					case FbxAxisSystem::eLeftHanded:
						{
							AxisDirection += TEXT(" (LH)");
						}
						break;
					case FbxAxisSystem::eRightHanded:
						{
							AxisDirection += TEXT(" (RH)");
						}
						break;
				}
				return AxisDirection;
			}

			void FFbxConvert::ConvertScene(FbxScene* SDKScene, 
				const bool bConvertScene, const bool bForceFrontXAxis, const bool bConvertSceneUnit, 
				FString& FileSystemDirection, FString& FileUnitSystem, 
				FbxAMatrix& AxisConversionInverseMatrix, FbxAMatrix& JointOrientationMatrix)
			{
				if (!ensure(SDKScene))
				{
					//Cannot convert a null scene
					return;
				}

				const FbxGlobalSettings& GlobalSettings = SDKScene->GetGlobalSettings();
				FbxTime::EMode TimeMode = GlobalSettings.GetTimeMode();
				//Set the original framerate from the current fbx file
				float FbxFramerate = FbxTime::GetFrameRate(TimeMode);

				//Apply any curve filter here, we currently do not apply any
				//The unroll curve filter was apply in legacy fbx importer if there was more then one FbxAnimStack.
				//The unroll curve filter can obliterate curve keys if for example a key do a complete rotation (360 degree in euler)

				FbxAxisSystem FileAxisSystem = SDKScene->GetGlobalSettings().GetAxisSystem();
				FileSystemDirection = GetFileAxisDirection(FileAxisSystem);

				AxisConversionInverseMatrix.SetIdentity();
				JointOrientationMatrix.SetIdentity();

				if (bConvertScene)
				{
					//UE is: z up, front x, left handed
					FbxAxisSystem::EUpVector UpVector = FbxAxisSystem::EUpVector::eZAxis;
					FbxAxisSystem::EFrontVector FrontVector = (FbxAxisSystem::EFrontVector)(bForceFrontXAxis ? FbxAxisSystem::eParityEven : -FbxAxisSystem::eParityOdd);
					FbxAxisSystem::ECoordSystem CoordSystem = FbxAxisSystem::ECoordSystem::eRightHanded;
					FbxAxisSystem UnrealImportAxis(UpVector, FrontVector, CoordSystem);

					if (FileAxisSystem != UnrealImportAxis)
					{
						FbxRootNodeUtility::RemoveAllFbxRoots(SDKScene);
						UnrealImportAxis.ConvertScene(SDKScene);

						FbxAMatrix SourceMatrix;
						FileAxisSystem.GetMatrix(SourceMatrix);
						FbxAMatrix UnrealMatrix;
						UnrealImportAxis.GetMatrix(UnrealMatrix);

						FbxAMatrix AxisConversionMatrix;
						AxisConversionMatrix = SourceMatrix.Inverse() * UnrealMatrix;
						AxisConversionInverseMatrix = AxisConversionMatrix.Inverse();

						if (bForceFrontXAxis)
						{
							JointOrientationMatrix.SetR(FbxVector4(-90.0, -90.0, 0.0));
						}
					}
				}

				FbxSystemUnit OriginalFileUnitSystem = SDKScene->GetGlobalSettings().GetSystemUnit();
				FileUnitSystem = FString(UTF8_TO_TCHAR(OriginalFileUnitSystem.GetScaleFactorAsString(false).Buffer()));

				if (bConvertSceneUnit)
				{
					if (OriginalFileUnitSystem != FbxSystemUnit::cm)
					{
						FbxSystemUnit::cm.ConvertScene(SDKScene);
					}
				}

				//Reset all the transform evaluation cache since we change some node transform
				SDKScene->GetAnimationEvaluator()->Reset();
			}


			FTransform FFbxConvert::AdjustCameraTransform(const FTransform& Transform)
			{
				//Add a roll of -90 degree locally for every cameras. Camera up vector differ from fbx to unreal
				const FRotator AdditionalRotation(0.0f, 0.0f, -90.0f);
				FTransform CameraTransform = FTransform(AdditionalRotation) * Transform;

				//Remove the scale of the node holding a camera (the mesh is provide by the engine and can be different in size)
				CameraTransform.SetScale3D(FVector::OneVector);
				
				return CameraTransform;
			}

			FTransform FFbxConvert::AdjustLightTransform(const FTransform& Transform)
			{
				//Add the z rotation of 90 degree locally for every light. Light direction differ from fbx to unreal 
				const FRotator AdditionalRotation(0.0f, 90.0f, 0.0f);
				FTransform LightTransform = FTransform(AdditionalRotation) * Transform;
				return LightTransform;
			}

			FLinearColor FFbxConvert::ConvertColor(const FbxDouble3& Color)
			{
				FLinearColor LinearColor;
				LinearColor.R =(float)Color[0];
				LinearColor.G =(float)Color[1];
				LinearColor.B =(float)Color[2];
				LinearColor.A = 1.f;

				return LinearColor;
			}

			/**
			 * Convert UTF8 char to a FString using ANSI_TO_TCHAR macro
			 */
			FString FFbxConvert::MakeString(const ANSICHAR* Name)
			{
				return FString(UTF8_TO_TCHAR(Name));
			}
		}//ns Private
	}//ns Interchange
}//ns UE
