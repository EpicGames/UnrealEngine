// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FbxInclude.h"

namespace UE
{
	namespace Interchange
	{
		namespace Private
		{
			struct FFbxConvert
			{
			public:

				//////////////////////////////////////////////////////////////////////////
				/** Transform Conversion API Begin */

				/* Return a FTransform float or double*/
				template<typename TransformType, typename VectorType, typename QuatType>
				static TransformType ConvertTransform(const FbxAMatrix& Matrix)
				{
					TransformType Out;

					Out.SetTranslation(ConvertPos<VectorType>(Matrix.GetT()));
					Out.SetScale3D(ConvertScale<VectorType>(Matrix.GetS()));
					Out.SetRotation(ConvertRotToQuat<QuatType>(Matrix.GetQ()));

					return Out;
				}

				/* Return a FMatrix float or double*/
				template<typename MatrixType>
				static MatrixType ConvertMatrix(const FbxAMatrix& Matrix)
				{
					MatrixType UEMatrix;

					for (int i = 0; i < 4; ++i)
					{
						const FbxVector4 Row = Matrix.GetRow(i);
						if (i == 1)
						{
							UEMatrix.M[i][0] = -Row[0];
							UEMatrix.M[i][1] = Row[1];
							UEMatrix.M[i][2] = -Row[2];
							UEMatrix.M[i][3] = -Row[3];
						}
						else
						{
							UEMatrix.M[i][0] = Row[0];
							UEMatrix.M[i][1] = -Row[1];
							UEMatrix.M[i][2] = Row[2];
							UEMatrix.M[i][3] = Row[3];
						}
					}

					VerifyFiniteMatrix<MatrixType>(UEMatrix);
					return UEMatrix;
				}

				/* Take a FMatrix float or double and return a fbx affine matrix*/
				template<typename MatrixType>
				static FbxAMatrix ConvertMatrix(const MatrixType& UEMatrix)
				{
					FbxAMatrix FbxMatrix;

					for (int i = 0; i < 4; ++i)
					{
						FbxVector4 Row;
						if (i == 1)
						{
							Row[0] = -UEMatrix.M[i][0];
							Row[1] = UEMatrix.M[i][1];
							Row[2] = -UEMatrix.M[i][2];
							Row[3] = -UEMatrix.M[i][3];
						}
						else
						{
							Row[0] = UEMatrix.M[i][0];
							Row[1] = -UEMatrix.M[i][1];
							Row[2] = UEMatrix.M[i][2];
							Row[3] = UEMatrix.M[i][3];
						}
						FbxMatrix.SetRow(i, Row);
					}

					return FbxMatrix;
				}

				/* Return a FQuat float or double */
				template<typename QuatType>
				static QuatType ConvertRotToQuat(FbxQuaternion Quaternion)
				{
					QuatType UnrealQuat;
					UnrealQuat.X = Quaternion[0];
					UnrealQuat.Y = -Quaternion[1];
					UnrealQuat.Z = Quaternion[2];
					UnrealQuat.W = -Quaternion[3];
					VerifyFiniteQuat<QuatType>(UnrealQuat);
					return UnrealQuat;
				}

				/* Return a FRotator float or double*/
				template<typename RotatorType, typename VectorType>
				static RotatorType ConvertEuler(FbxDouble3 Euler)
				{
					return RotatorType::MakeFromEuler(VectorType(Euler[0], -Euler[1], Euler[2]));
				}
				
				/* Return a FVector float or double */
				template<typename VectorType>
				static VectorType ConvertScale(FbxVector4 Vector)
				{
					VectorType Out;
					Out[0] = Vector[0];
					Out[1] = Vector[1];
					Out[2] = Vector[2];
					VerifyFiniteVector<VectorType>(Out);
					return Out;
				}
				
				/* Return a FRotator float or double */
				template<typename RotatorType, typename QuatType>
				static RotatorType ConvertRotation(FbxQuaternion Quaternion)
				{
					RotatorType Out(ConvertRotToQuat<QuatType>(Quaternion));
					return Out;
				}

				/* Return a FVector float or double */
				template<typename VectorType>
				static VectorType ConvertPos(const FbxVector4& Vector)
				{
					VectorType Pos(Vector[0], -Vector[1], Vector[2]);
					VerifyFiniteVector<VectorType>(Pos);
					return Pos;
				}

				/* Return a FVector float or double */
				template<typename VectorType>
				static VectorType ConvertDir(const FbxVector4& Vector)
				{
					VectorType Dir(Vector[0], -Vector[1], Vector[2]);
					VerifyFiniteVector<VectorType>(Dir);
					return Dir;
				}

				static FLinearColor ConvertColor(const FbxDouble3& Color);

				static FTransform AdjustCameraTransform(const FTransform& Transform);
				static FTransform AdjustLightTransform(const FTransform& Transform);

				/** Transform Conversion API End */
				//////////////////////////////////////////////////////////////////////////

				
				//////////////////////////////////////////////////////////////////////////
				/** Scene Conversion API Begin */

				/**
				 * Convert a fbx scene 
				 */
				static void ConvertScene(FbxScene* SDKScene, 
					const bool bConvertScene, const bool bForceFrontXAxis, const bool bConvertSceneUnit, 
					FString& FileSystemDirection, FString& FileUnitSystem, 
					FbxAMatrix& AxisConversionInverseMatrix, FbxAMatrix& JointOrientationMatrix);
				
				/** Scene Conversion API End */
				//////////////////////////////////////////////////////////////////////////


				//////////////////////////////////////////////////////////////////////////
				/** String Conversion API Begin */

				/**
				 * Convert UTF8 char to a FString using ANSI_TO_TCHAR macro
				 */
				static FString MakeString(const ANSICHAR* Name);

				/** String Conversion API End */
				//////////////////////////////////////////////////////////////////////////

			private:
				
				template<typename ScalarType>
				static void VerifyFiniteScalar(ScalarType& Value)
				{
					if (!FMath::IsFinite(Value))
					{
						Value = 0.0f;
					}
				}

				template<typename VectorType>
				static void VerifyFiniteVector(VectorType& Value)
				{
					if (Value.ContainsNaN())
					{
						Value.Set(0.0, 0.0, 0.0);
					}
				}

				template<typename QuatType>
				static void VerifyFiniteQuat(QuatType& Value)
				{
					if (Value.ContainsNaN())
					{
						Value = QuatType::Identity;
					}
				}

				template<typename MatrixType>
				static void VerifyFiniteMatrix(MatrixType& Value)
				{
					if (Value.ContainsNaN())
					{
						Value.SetIdentity();
					}
				}
			};
		}//ns Private
	}//ns Interchange
}//ns UE
