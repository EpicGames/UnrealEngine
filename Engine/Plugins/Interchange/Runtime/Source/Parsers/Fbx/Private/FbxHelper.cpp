// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxHelper.h"

#include "CoreMinimal.h"
#include "FbxAPI.h"
#include "FbxConvert.h"
#include "FbxInclude.h"
#include "InterchangeHelper.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"

#define GeneratedLODNameSuffix "_GeneratedLOD_"

namespace UE
{
	namespace Interchange
	{
		namespace Private
		{
			FString FFbxHelper::GetMeshName(FbxGeometryBase* Mesh) const
			{
				if (!Mesh)
				{
					return {};
				}

				FString DefaultPrefix;
				if (Mesh->GetAttributeType() == FbxNodeAttribute::eMesh)
				{
					DefaultPrefix = TEXT("Mesh");
				}
				else if (Mesh->GetAttributeType() == FbxNodeAttribute::eShape)
				{
					DefaultPrefix = TEXT("Shape");
				}

				return GetNodeAttributeName(Mesh, DefaultPrefix);
			}

			FString FFbxHelper::GetMeshUniqueID(FbxGeometryBase* Mesh) const
			{
				if (!Mesh)
				{
					return {};
				}

				FString MeshUniqueID;
				if (Mesh->GetAttributeType() == FbxNodeAttribute::eMesh)
				{
					MeshUniqueID = TEXT("Mesh");
				}
				else if (Mesh->GetAttributeType() == FbxNodeAttribute::eShape)
				{
					MeshUniqueID = TEXT("Shape");
				}

				return GetNodeAttributeUniqueID(Mesh, MeshUniqueID);
			}

			FString FFbxHelper::GetNodeAttributeName(FbxNodeAttribute* NodeAttribute, const FStringView DefaultNamePrefix) const
			{
				FString NodeAttributeName = FFbxHelper::GetFbxObjectName(NodeAttribute);
				if (NodeAttributeName.IsEmpty())
				{
					if (NodeAttribute->GetNodeCount() > 0)
					{
						NodeAttributeName = FString(DefaultNamePrefix) + TEXT("_") + FFbxHelper::GetFbxObjectName(NodeAttribute->GetNode(0));
					}
					else
					{
						uint64 UniqueFbxObjectID = NodeAttribute->GetUniqueID();
						NodeAttributeName += GetUniqueIDString(UniqueFbxObjectID);
					}
				}
				return NodeAttributeName;
			}

			FString FFbxHelper::GetNodeAttributeUniqueID(FbxNodeAttribute* NodeAttribute, const FStringView Prefix) const
			{
				if (!NodeAttribute)
				{
					return {};
				}

				FString NodeAttributeUniqueID = FString(TEXT("\\")) + Prefix + TEXT("\\");
				FString NodeAttributeName = FFbxHelper::GetFbxObjectName(NodeAttribute);

				if (NodeAttributeName.IsEmpty())
				{
					if (NodeAttribute->GetNodeCount() > 0)
					{
						NodeAttributeName = FFbxHelper::GetFbxNodeHierarchyName(NodeAttribute->GetNode(0));
					}
					else
					{
						NodeAttributeName = GetNodeAttributeName(NodeAttribute, Prefix);
					}
				}

				NodeAttributeUniqueID += NodeAttributeName;

				return NodeAttributeUniqueID;
			}

			FString FFbxHelper::GetFbxPropertyName(const FbxProperty& Property) const
			{
				FString PropertyName = FFbxConvert::MakeString(Property.GetName());
				UE::Interchange::SanitizeName(PropertyName);

				return PropertyName;
			}

			FString FFbxHelper::GetFbxObjectName(const FbxObject* Object, bool bIsJoint) const
			{
				if (!Object)
				{
					return FString();
				}

				FString ObjName = FFbxConvert::MakeString(Object->GetName());

				ManageNamespace(ObjName);

				UE::Interchange::SanitizeName(ObjName, bIsJoint);

				return ObjName;
			}

			FString FFbxHelper::GetFbxNodeHierarchyName(const FbxNode* Node) const
			{
				if (!Node)
				{
					return FString();
				}
				TArray<FString> UniqueIDTokens;
				const FbxNode* ParentNode = Node;
				while (ParentNode)
				{
					//As we don't want the name to have the namespace removed,
					//we manually generate the name here:

					FString ParentNodeName = FFbxConvert::MakeString(ParentNode->GetName());
					UE::Interchange::SanitizeName(ParentNodeName);

					UniqueIDTokens.Add(ParentNodeName);
					ParentNode = ParentNode->GetParent();
				}
				FString UniqueID;
				for (int32 TokenIndex = UniqueIDTokens.Num() - 1; TokenIndex >= 0; TokenIndex--)
				{
					UniqueID += UniqueIDTokens[TokenIndex];
					if (TokenIndex > 0)
					{
						UniqueID += TEXT(".");
					}
				}
				return UniqueID;
			}

			void FFbxHelper::ManageNamespace(FString& ObjectName) const
			{
				if (bKeepFbxNamespace)
				{
					if (ObjectName.Contains(TEXT(":")))
					{
						ObjectName.ReplaceInline(TEXT(":"), TEXT("_"));
					}
				}
				else
				{
					// Remove namespaces
					int32 LastNamespaceTokenIndex = INDEX_NONE;
					if (ObjectName.FindLastChar(TEXT(':'), LastNamespaceTokenIndex))
					{
						//+1 to remove the ':' character we found
						ObjectName.RightChopInline(LastNamespaceTokenIndex + 1, EAllowShrinking::Yes);
					}
				}
			}

			void FFbxHelper::ManageNamespaceAndRenameObject(FString& ObjectName, FbxObject* Object) const
			{
				if (bKeepFbxNamespace)
				{
					if (ObjectName.Contains(TEXT(":")))
					{
						ObjectName.ReplaceInline(TEXT(":"), TEXT("_"));
						Object->SetName(TCHAR_TO_UTF8(*ObjectName));
					}
				}
				else
				{
					// Remove namespaces
					int32 LastNamespaceTokenIndex = INDEX_NONE;
					if (ObjectName.FindLastChar(TEXT(':'), LastNamespaceTokenIndex))
					{
						//+1 to remove the ':' character we found
						ObjectName.RightChopInline(LastNamespaceTokenIndex + 1, EAllowShrinking::Yes);
						Object->SetName(TCHAR_TO_UTF8(*ObjectName));
					}
				}
			}

			FString FFbxHelper::GetUniqueIDString(const uint64 UniqueID) const
			{
				FStringFormatNamedArguments FormatArguments;
				FormatArguments.Add(TEXT("UniqueID"), UniqueID);
				return FString::Format(TEXT("{UniqueID}"), FormatArguments);
			}

			void ProcessCustomAttribute(FFbxParser& Parser, UInterchangeBaseNode* UnrealNode, FbxProperty Property, const TOptional<FString>& PayloadKey)
			{
				FString PropertyName = Parser.GetFbxHelper()->GetFbxPropertyName(Property);

				switch (Property.GetPropertyDataType().GetType())
				{
					case EFbxType::eFbxBool:
						{
							bool PropertyValue = Property.Get<bool>();
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
					case EFbxType::eFbxChar:
						{
							int8 PropertyValue = Property.Get<int8>();
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
					case EFbxType::eFbxUChar:
						{
							uint8 PropertyValue = Property.Get<uint8>();
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
					case EFbxType::eFbxShort:
						{
							int16 PropertyValue = Property.Get<int16>();
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
					case EFbxType::eFbxUShort:
						{
							uint16 PropertyValue = Property.Get<uint16>();
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
					case EFbxType::eFbxInt:
						{
							int32 PropertyValue = Property.Get<int32>();
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
					case EFbxType::eFbxUInt:
						{
							uint32 PropertyValue = Property.Get<uint32>();
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
					case EFbxType::eFbxLongLong:
						{
							int64 PropertyValue = Property.Get<int64>();
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
					case EFbxType::eFbxULongLong:
						{
							uint64 PropertyValue = Property.Get<uint64>();
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
					case EFbxType::eFbxHalfFloat:
						{
							FbxHalfFloat HalfFloat = Property.Get<FbxHalfFloat>();
							FFloat16 PropertyValue = FFloat16(HalfFloat.value());
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
					case EFbxType::eFbxFloat:
						{
							float PropertyValue = Property.Get<float>();
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
					case EFbxType::eFbxDouble:
						{
							double PropertyValue = Property.Get<double>();
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
					case EFbxType::eFbxDouble2:
						{
							FbxDouble2 Vec = Property.Get<FbxDouble2>();
							FVector2D PropertyValue = FVector2D(Vec[0], Vec[1]);
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
					case EFbxType::eFbxDouble3:
						{
							FbxDouble3 Vec = Property.Get<FbxDouble3>();
							FVector3d PropertyValue = FVector3d(Vec[0], Vec[1], Vec[2]);
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
					case EFbxType::eFbxDouble4:
						{
							FbxDouble4 Vec = Property.Get<FbxDouble4>();
							FVector4d PropertyValue = FVector4d(Vec[0], Vec[1], Vec[2], Vec[3]);
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
					case EFbxType::eFbxEnum:
						{
							//Convert enum to uint8
							FbxEnum EnumValue = Property.Get<FbxEnum>();
							uint8 PropertyValue = static_cast<uint8>(EnumValue);
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
					case EFbxType::eFbxString:
						{
							FbxString StringValue = Property.Get<FbxString>();
							FString PropertyValue = FFbxConvert::MakeString(StringValue.Buffer());
							UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UnrealNode, PropertyName, PropertyValue, PayloadKey);
						}
						break;
				}
			}

			void ProcessCustomAttributes(FFbxParser& Parser, FbxObject* Object, UInterchangeBaseNode* UnrealNode)
			{
				FbxProperty Property = Object->GetFirstProperty();

				//Add all custom Attributes for the node
				while (Property.IsValid())
				{
					EFbxType PropertyType = Property.GetPropertyDataType().GetType();
					if (Property.GetFlag(FbxPropertyFlags::eUserDefined))
					{
						ProcessCustomAttribute(Parser, UnrealNode, Property);
					}

					//Inspect next node property
					Property = Object->GetNextProperty(Property);
				}
			}
		}//ns Private
	}//ns Interchange
}//ns UE
