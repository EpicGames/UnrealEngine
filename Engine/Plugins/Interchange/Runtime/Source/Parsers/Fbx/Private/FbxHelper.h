// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "FbxInclude.h"

class UInterchangeBaseNode;

namespace UE::Interchange
{
#if WITH_ENGINE
	struct FMeshPayloadData;
#endif

	namespace Private
	{
		class FFbxParser;

		class FPayloadContextBase
		{
		public:
			virtual ~FPayloadContextBase() = default;
			virtual FString GetPayloadType() const { return FString(); }
			virtual bool FetchPayloadToFile(FFbxParser& Parser, const FString& PayloadFilepath) { return false; }
			virtual bool FetchMeshPayloadToFile(FFbxParser& Parser, const FTransform& MeshGlobalTransform, const FString& PayloadFilepath) { return false; }
#if WITH_ENGINE
			virtual bool FetchMeshPayload(FFbxParser& Parser, const FTransform& MeshGlobalTransform, FMeshPayloadData& OutMeshPayloadData) { return false; };
#endif
			virtual bool FetchAnimationBakeTransformPayloadForTime(FFbxParser& Parser, const FbxTime Currenttime, FTransform& Out) { return false; }

			virtual FbxAnimStack* GetAnimStack() { return nullptr; }
		};

		struct FFbxHelper
		{
		public:
			explicit FFbxHelper(bool bInKeepFbxNamespace)
				: bKeepFbxNamespace(bInKeepFbxNamespace)
			{
			}

			void Reset() { MaterialNameClashMap.Reset(); }
			FString GetMeshName(FbxGeometryBase* Mesh) const;
			FString GetMeshUniqueID(FbxGeometryBase* Mesh) const;

			FString GetNodeAttributeName(FbxNodeAttribute* NodeAttribute, const FStringView DefaultNamePrefix) const;
			FString GetNodeAttributeUniqueID(FbxNodeAttribute* NodeAttribute, const FStringView Prefix) const;

			/**
				* Return the name of an FbxProperty, return empty string if the property is null.
				*/
			FString GetFbxPropertyName(const FbxProperty& Property) const;

			/**
				* Return the name of an FbxObject, return empty string if the object is null.
				*/
			FString GetFbxObjectName(const FbxObject* Object, bool bIsJoint = false) const;

			/**
				* Return a string with the name of all the parent in the hierarchy separate by a dot( . ) from the fbx root node to the specified node.
				* This is a way to have a valid unique ID for a fbx node that will be the same if the fbx change when we re-import.
				* Using the fbx sdk int32 uniqueID is not valid anymore if the fbx is re-exported.
				*/
			FString GetFbxNodeHierarchyName(const FbxNode* Node) const;

			void ManageNamespace(FString& ObjectName) const;
			void ManageNamespaceAndRenameObject(FString& ObjectName, FbxObject* Object) const;
		protected:
			FString GetUniqueIDString(const uint64 UniqueID) const;

		private:
			mutable TMap<FString, const FbxObject*> MaterialNameClashMap;
			bool bKeepFbxNamespace;
		};

		void ProcessCustomAttribute(FFbxParser& Parser, UInterchangeBaseNode* UnrealNode, FbxProperty Property, const TOptional<FString>& PayloadKey = TOptional<FString>());
		
		//Without AnimationProcessing
		void ProcessCustomAttributes(FFbxParser& Parser, FbxObject* Object, UInterchangeBaseNode* UnrealNode);
	}//ns Private
}//ns UE::Interchange
