// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// Temporarily disable a few warnings due to virtual function abuse in FBX source files
#pragma warning( push )

#pragma warning( disable : 4263 ) // 'function' : member function does not override any base class virtual member function
#pragma warning( disable : 4264 ) // 'virtual_function' : no override available for virtual member function from base 'class'; function is hidden

// Include the fbx sdk header
// temp undef/redef of _O_RDONLY because kfbxcache.h (included by fbxsdk.h) does
// a weird use of these identifiers inside an enum.
#ifdef _O_RDONLY
#define TMP_UNFBX_BACKUP_O_RDONLY _O_RDONLY
#define TMP_UNFBX_BACKUP_O_WRONLY _O_WRONLY
#undef _O_RDONLY
#undef _O_WRONLY
#endif

#if PLATFORM_MAC
#include <mach/vm_types.h> // This needs to be included outside of #pragma pack
#endif

//Robert G. : Packing was only set for the 64bits platform, but we also need it for 32bits.
//This was found while trying to trace a loop that iterate through all character links.
//The memory didn't match what the debugger displayed, obviously since the packing was not right.
#pragma pack(push,8)

#if PLATFORM_WINDOWS
// _CRT_SECURE_NO_DEPRECATE is defined but is not enough to suppress the deprecation
// warning for vsprintf and stricmp in VS2010.  Since FBX is able to properly handle the non-deprecated
// versions on the appropriate platforms, _CRT_SECURE_NO_DEPRECATE is temporarily undefined before
// including the FBX headers

// The following is a hack to make the FBX header files compile correctly under Visual Studio 2012 and Visual Studio 2013
#if _MSC_VER >= 1700
#define FBX_DLL_MSC_VER 1600
#endif


#endif // PLATFORM_WINDOWS

// FBX casts null pointer to a reference
THIRD_PARTY_INCLUDES_START
#include <fbxsdk.h>
THIRD_PARTY_INCLUDES_END

#pragma pack(pop)

#ifdef TMP_UNFBX_BACKUP_O_RDONLY
#define _O_RDONLY TMP_FBX_BACKUP_O_RDONLY
#define _O_WRONLY TMP_FBX_BACKUP_O_WRONLY
#undef TMP_UNFBX_BACKUP_O_RDONLY
#undef TMP_UNFBX_BACKUP_O_WRONLY
#endif

#pragma warning( pop )
