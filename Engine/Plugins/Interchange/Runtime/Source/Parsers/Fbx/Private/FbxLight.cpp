// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxLight.h"

#include "CoreMinimal.h"
#include "FbxAPI.h"
#include "FbxHelper.h"
#include "FbxInclude.h"
#include "InterchangeLightNode.h"
#include "InterchangeResultsContainer.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#define LOCTEXT_NAMESPACE "InterchangeFbxLight"

namespace UE
{
	namespace Interchange
	{
		namespace Private
		{
			UInterchangeBaseLightNode* FFbxLight::CreateLightNode(UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeUid, const FString& NodeName, const FbxLight& LightAttribute)
			{
				UClass* LightClass;

				switch(LightAttribute.LightType.Get())
				{
				case FbxLight::ePoint:
				case FbxLight::eVolume:
					LightClass = UInterchangePointLightNode::StaticClass();
					break;
				case FbxLight::eDirectional:
					LightClass = UInterchangeDirectionalLightNode::StaticClass();
					break;
				case FbxLight::eSpot:
					LightClass = UInterchangeSpotLightNode::StaticClass();
					break;
				case FbxLight::eArea:
					LightClass = UInterchangeRectLightNode::StaticClass();
					break;
				default:
					LightClass = UInterchangePointLightNode::StaticClass();
					break;
				}

				UInterchangeBaseLightNode* BaseLightNode = NewObject<UInterchangeBaseLightNode>(&NodeContainer, LightClass, NAME_None);
				if (!ensure(BaseLightNode))
				{
					UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
					Message->Text = LOCTEXT("CannotAllocateNode", "Cannot allocate a node when importing FBX.");
					return nullptr;
				}
				NodeContainer.SetupNode(BaseLightNode, NodeUid, NodeName, EInterchangeNodeContainerType::TranslatedAsset);

				const FbxDouble3 LightColor = LightAttribute.Color.Get();
				const FColor UnrealLightColor(uint8(255.0 * LightColor[0]), uint8(255.0 * LightColor[1]), uint8(255.0 * LightColor[2]));
				BaseLightNode->SetCustomLightColor(UnrealLightColor);
				
				const FbxDouble Intensity = LightAttribute.Intensity.Get();
				BaseLightNode->SetCustomIntensity((float)Intensity);

				if (UInterchangeLightNode* InterchangeLightNode = Cast<UInterchangeLightNode>(BaseLightNode))
				{
					InterchangeLightNode->SetCustomAttenuationRadius(LightAttribute.DecayStart.Get());
					const FbxLight::EDecayType Decay = LightAttribute.DecayType.Get();
					if (Decay == FbxLight::EDecayType::eNone)
					{
						InterchangeLightNode->SetCustomAttenuationRadius(FBXSDK_FLOAT_MAX);
					}
				}

				// NOTE: Area Lights do not have dimensions property on FBX Lights to be able to set the width and height of the lights.				
				switch (LightAttribute.LightType.Get())
				{
				case FbxLight::eSpot:
					if (UInterchangeSpotLightNode* SpotLightNode = Cast<UInterchangeSpotLightNode>(BaseLightNode))
					{
						SpotLightNode->SetCustomInnerConeAngle(LightAttribute.InnerAngle.Get());
						SpotLightNode->SetCustomOuterConeAngle(LightAttribute.OuterAngle.Get());
					}
					break;
				default:
					break;
				}

				return BaseLightNode;
			}

			void FFbxLight::AddLightsRecursively(FbxNode* Node, UInterchangeBaseNodeContainer& NodeContainer)
			{
				int32 AttributeCount = Node->GetNodeAttributeCount();
				for (int32 AttributeIndex = 0; AttributeIndex < AttributeCount; ++AttributeIndex)
				{
					FbxNodeAttribute* NodeAttribute = Node->GetNodeAttributeByIndex(AttributeIndex);

					if (NodeAttribute && NodeAttribute->GetAttributeType() == FbxNodeAttribute::eLight)
					{
						FString NodeName = Parser.GetFbxHelper()->GetNodeAttributeName(NodeAttribute, UInterchangeBaseLightNode::StaticAssetTypeName());
						FString NodeUid = Parser.GetFbxHelper()->GetNodeAttributeUniqueID(NodeAttribute, UInterchangeBaseLightNode::StaticAssetTypeName());

						const UInterchangeBaseLightNode* LightNode = Cast<const UInterchangeBaseLightNode>(NodeContainer.GetNode(NodeUid));

						if (!LightNode)
						{
							CreateLightNode(NodeContainer, NodeUid, NodeName, static_cast<FbxLight&>(*NodeAttribute));
						}
					}
				}

				const int32 ChildCount = Node->GetChildCount();
				for (int32 ChildIndex = 0; ChildIndex < ChildCount; ++ChildIndex)
				{
					FbxNode* ChildNode = Node->GetChild(ChildIndex);
					AddLightsRecursively(ChildNode, NodeContainer);
				}
			}

			void FFbxLight::AddAllLights(FbxScene* SDKScene, UInterchangeBaseNodeContainer& NodeContainer)
			{
				AddLightsRecursively(SDKScene->GetRootNode(), NodeContainer);
			}
		} //ns Private
	} //ns Interchange
}//ns UE

#undef LOCTEXT_NAMESPACE
