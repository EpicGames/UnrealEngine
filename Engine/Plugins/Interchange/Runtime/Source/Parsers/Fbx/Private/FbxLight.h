// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FbxInclude.h"

class UInterchangeBaseNodeContainer;
class UInterchangeBaseLightNode;

namespace UE
{
	namespace Interchange
	{
		namespace Private
		{
			class FFbxParser;

			class FFbxLight
			{
			public:
				explicit FFbxLight(FFbxParser& InParser)
					: Parser(InParser)
				{}

				/**
				 * Create a UInterchangeBaseLightNode and add it to the NodeContainer for all FbxNodeAttributes of type eLight the fbx file contains.
				 * 
				 * @note - Any node that already exist in the NodeContainer will not be created or modified.
				 */
				void AddAllLights(FbxScene* SDKScene, UInterchangeBaseNodeContainer& NodeContainer);

			protected:
				UInterchangeBaseLightNode* CreateLightNode(UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeUID, const FString& NodeName, const FbxLight& LightAttribute);
				void AddLightsRecursively(FbxNode* Node, UInterchangeBaseNodeContainer& NodeContainer);

			private:
				FFbxParser& Parser;
			};
		}//ns Private
	}//ns Interchange
}//ns UE
