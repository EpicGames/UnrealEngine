// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxMaterial.h"

#include "FbxAPI.h"
#include "FbxHelper.h"
#include "FbxInclude.h"
#include "IImageWrapperModule.h"
#include "Fbx/InterchangeFbxMessages.h"
#include "InterchangeMaterialDefinitions.h"
#include "InterchangeMaterialInstanceNode.h"
#include "InterchangeResultsContainer.h"
#include "InterchangeSceneNode.h"
#include "InterchangeTexture2DNode.h"
#include "InterchangeTextureNode.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#define LOCTEXT_NAMESPACE "InterchangeFbxMaterial"

namespace UE
{
	namespace Interchange
	{
		namespace Private
		{
			FString FindTextureFilePath(const FFbxParser& Parser, FbxFileTexture& FbxTexture)
			{
				// try opening from absolute path
				FString AbsoluteTextureFilepath = UTF8_TO_TCHAR(FbxTexture.GetFileName());
				FPaths::NormalizeFilename(AbsoluteTextureFilepath);
				if (FPaths::FileExists(AbsoluteTextureFilepath))
				{
					return AbsoluteTextureFilepath;
				}

				FString FileBasePath = FPaths::GetPath(Parser.GetSourceFilename());

				FString RelativeToFileBasePath = FileBasePath / FPaths::GetCleanFilename(AbsoluteTextureFilepath);
				FPaths::NormalizeFilename(RelativeToFileBasePath);
				if (FPaths::FileExists(RelativeToFileBasePath))
				{
					return RelativeToFileBasePath;
				}

				// try fbx file base path + relative path
				FString RelativeToFBXTextureFilepath = FileBasePath / UTF8_TO_TCHAR(FbxTexture.GetRelativeFileName());
				FPaths::NormalizeFilename(RelativeToFBXTextureFilepath);
				if (FPaths::FileExists(RelativeToFBXTextureFilepath))
				{
					return RelativeToFBXTextureFilepath;
				}

				// Some fbx files do not store the actual absolute filename as absolute and it is actually relative.  Try to get it relative to the FBX file we are importing
				FString AbosluteAsRelativeToFBXTextureFilepath = FileBasePath / UTF8_TO_TCHAR(FbxTexture.GetFileName());
				FPaths::NormalizeFilename(AbosluteAsRelativeToFBXTextureFilepath);
				if (FPaths::FileExists(AbosluteAsRelativeToFBXTextureFilepath))
				{
					return AbosluteAsRelativeToFBXTextureFilepath;
				}

				return FString();
			}

			UInterchangeMaterialInstanceNode* FFbxMaterial::CreateMaterialInstanceNode(UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeUID, const FString& NodeName)
			{
				UInterchangeMaterialInstanceNode* MaterialInstanceNode = NewObject<UInterchangeMaterialInstanceNode>(&NodeContainer);
				NodeContainer.SetupNode(MaterialInstanceNode, NodeUID, NodeName, EInterchangeNodeContainerType::TranslatedAsset);

				return MaterialInstanceNode;
			}

			const UInterchangeTexture2DNode* FFbxMaterial::CreateTexture2DNode(UInterchangeBaseNodeContainer& NodeContainer, const FString& TextureFilePath)
			{
				if (TextureFilePath.IsEmpty())
				{
					return nullptr;
				}
				FString NormalizeFilePath = TextureFilePath;
				FPaths::NormalizeFilename(NormalizeFilePath);
				if (!FPaths::FileExists(NormalizeFilePath))
				{
					return nullptr;
				}

				FString TextureName;
				FString TextureNodeUid;
				
				ensure(UInterchangeTextureNode::ExtractNodeUidAndNameFromFilePath(NormalizeFilePath, TextureNodeUid, TextureName));

				if (const UInterchangeTexture2DNode* TextureNode = Cast<const UInterchangeTexture2DNode>(NodeContainer.GetNode(TextureNodeUid)))
				{
					return TextureNode;
				}

				UInterchangeTexture2DNode* NewTextureNode = UInterchangeTexture2DNode::Create(&NodeContainer, TextureNodeUid, TextureName);

				//All texture translator expect a file as the payload key
				NewTextureNode->SetPayLoadKey(NormalizeFilePath);

				return NewTextureNode;
			}

			const UInterchangeTexture2DNode* FFbxMaterial::CreateTexture2DNode(FbxFileTexture* FbxTexture, UInterchangeBaseNodeContainer& NodeContainer, UInterchangeMaterialInstanceNode* MaterialInstanceNode)
			{
				if (!FbxTexture)
				{
					return nullptr;
				}

				const FString TextureFilename = FindTextureFilePath(Parser, *FbxTexture);

				// Return incomplete texture sampler if texture file does not exist
				if (TextureFilename.IsEmpty() || !FPaths::FileExists(TextureFilename))
				{
					if (!GIsAutomationTesting)
					{
						UInterchangeResultTextureDisplay_TextureFileDoNotExist* Message = Parser.AddMessage<UInterchangeResultTextureDisplay_TextureFileDoNotExist>();
						Message->TextureName = TextureFilename;
						Message->MaterialName = MaterialInstanceNode->GetDisplayLabel();
					}

					return nullptr;
				}

				const UInterchangeTexture2DNode* TextureNode = CreateTexture2DNode(NodeContainer, TextureFilename);
				return TextureNode;
			}

			void FFbxMaterial::ConvertValuePropertyToMaterialInstanceNode(UInterchangeMaterialInstanceNode* MaterialInstanceNode, FbxProperty& Property, float Factor, FName InputName, const TVariant<FLinearColor, float>& DefaultValue, bool bInverse)
			{
				using namespace Materials::Standard::Nodes;

				const FString InputAttributeKey = InputName.ToString();

				const EFbxType DataType = Property.GetPropertyDataType().GetType();
				if (DataType == eFbxDouble || DataType == eFbxFloat || DataType == eFbxInt)
				{
					const float PropertyValue = Property.Get<float>() * Factor;
					MaterialInstanceNode->AddScalarParameterValue(InputAttributeKey, bInverse ? 1.f - PropertyValue : PropertyValue);
				}
				else if (DataType == eFbxDouble3 || DataType == eFbxDouble4)
				{
					FbxDouble3 Color = DataType == eFbxDouble3 ? Property.Get<FbxDouble3>() : Property.Get<FbxDouble4>();
					FVector3f FbxValue = FVector3f(Color[0], Color[1], Color[2]) * Factor;
					FLinearColor PropertyValue = bInverse ? FVector3f::OneVector - FbxValue : FbxValue;

					if (DefaultValue.IsType<FLinearColor>())
					{
						MaterialInstanceNode->AddVectorParameterValue(InputAttributeKey, PropertyValue);
					}
					else if (DefaultValue.IsType<float>())
					{
						// We're connecting a linear color to a float input. Ideally, we'd go through a desaturate, but for now we'll just take the red channel and ignore the rest.
						MaterialInstanceNode->AddScalarParameterValue(InputAttributeKey, PropertyValue.R);
					}
				}
			}

			const UInterchangeTexture2DNode* FFbxMaterial::ConvertTexturePropertyToMaterialInstanceNode(UInterchangeBaseNodeContainer& NodeContainer, UInterchangeMaterialInstanceNode* MaterialInstanceNode, FName InputName, FbxFileTexture* FbxTexture)
			{
				if (const UInterchangeTexture2DNode* TextureNode = CreateTexture2DNode(FbxTexture, NodeContainer, MaterialInstanceNode))
				{
					const FString TextureMapInput = InputName.ToString() + TEXT("Map");
					MaterialInstanceNode->AddTextureParameterValue(TextureMapInput, TextureNode->GetUniqueID());
					MaterialInstanceNode->AddScalarParameterValue(TextureMapInput + TEXT("Weight"), 1.f);
					MaterialInstanceNode->AddScalarParameterValue(TextureMapInput + TEXT("TilingU"), FbxTexture->GetScaleU()); //-V595
					MaterialInstanceNode->AddScalarParameterValue(TextureMapInput + TEXT("TilingV"), FbxTexture->GetScaleV()); //-V595
					return TextureNode;
				}

				if (!GIsAutomationTesting)
				{
					UInterchangeResultTextureDisplay_TextureFileDoNotExist* Message = Parser.AddMessage<UInterchangeResultTextureDisplay_TextureFileDoNotExist>();
					Message->TextureName = FbxTexture ? UTF8_TO_TCHAR(FbxTexture->GetFileName()) : TEXT("Undefined");
					Message->MaterialName = MaterialInstanceNode->GetDisplayLabel();
				}
				return nullptr;
			}

			bool FFbxMaterial::ConvertPropertyToMaterialInstanceNode(UInterchangeBaseNodeContainer& NodeContainer, UInterchangeMaterialInstanceNode* MaterialInstanceNode, FbxProperty& Property, float Factor, FName InputName, const TVariant<FLinearColor, float>& DefaultValue, bool bInverse)
			{
				if (Property.GetSrcObjectCount<FbxFileTexture>() == 0)
				{
					ConvertValuePropertyToMaterialInstanceNode(MaterialInstanceNode, Property, Factor, InputName, DefaultValue, bInverse);
					return true;
				}

				// Handles max one texture per property.
				FbxFileTexture* FbxTexture = Property.GetSrcObject<FbxFileTexture>(0);
				return ConvertTexturePropertyToMaterialInstanceNode(NodeContainer, MaterialInstanceNode, InputName, FbxTexture) != nullptr;
			}

			bool DoesTextureHaveAlpha(const FString& TextureFilename)
			{
				IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));

				TArray64<uint8> ImageData;

				if (!FFileHelper::LoadFileToArray(ImageData, *TextureFilename) || ImageData.IsEmpty())
				{
					return false;
				}

				switch (ImageWrapperModule.DetectImageFormat(ImageData.GetData(), ImageData.Num()))
				{
					case EImageFormat::Invalid:
						// Early exit for Jpeg, which doesn't support alpha
					case EImageFormat::JPEG:
					case EImageFormat::GrayscaleJPEG:
					case EImageFormat::UEJPEG:
					case EImageFormat::GrayscaleUEJPEG:
						return false;
				}

				FImage LoadedImage;
				if (ImageWrapperModule.DecompressImage(ImageData.GetData(), ImageData.Num(), LoadedImage))
				{
					return FImageCore::DetectAlphaChannel(LoadedImage);
				}

				return false;
			}

			const UInterchangeMaterialInstanceNode* FFbxMaterial::AddMaterialInstanceNode(FbxSurfaceMaterial* SurfaceMaterial, UInterchangeBaseNodeContainer& NodeContainer)
			{
				using namespace UE::Interchange::Materials;

				if (!SurfaceMaterial)
				{
					return nullptr;
				}

				//Create a material node
				FString MaterialName = Parser.GetFbxHelper()->GetFbxObjectName(SurfaceMaterial);
				FString NodeUid = TEXT("\\Material\\") + MaterialName;
				const UInterchangeMaterialInstanceNode* ExistingMaterialInstanceNode = Cast<const UInterchangeMaterialInstanceNode>(NodeContainer.GetNode(NodeUid));
				if (ExistingMaterialInstanceNode)
				{
					return ExistingMaterialInstanceNode;
				}

				FString ShadingModel;
				FString BumpMaterial;

				UInterchangeMaterialInstanceNode* MaterialInstanceNode = CreateMaterialInstanceNode(NodeContainer, NodeUid, MaterialName);
				if (MaterialInstanceNode == nullptr)
				{
					FFormatNamedArguments Args
					{
						{ TEXT("MaterialName"), FText::FromString(MaterialName) }
					};
					UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
					Message->Text = FText::Format(LOCTEXT("CannotCreateFBXMaterial", "Cannot create FBX material '{MaterialName}'."), Args);
					return nullptr;
				}

				ProcessCustomAttributes(Parser, SurfaceMaterial, MaterialInstanceNode);

				auto ShouldConvertProperty = [](FBXSDK_NAMESPACE::FbxProperty& MaterialProperty) -> bool
				{
					bool bShouldConvertProperty = false;

					if (MaterialProperty.IsValid())
					{
						// FbxProperty::HasDefaultValue(..) can return true while the property has textures attached to it.
						bShouldConvertProperty = MaterialProperty.GetSrcObjectCount<FBXSDK_NAMESPACE::FbxTexture>() > 0
							|| !FBXSDK_NAMESPACE::FbxProperty::HasDefaultValue(MaterialProperty);
					}

					return bShouldConvertProperty;
				};

				auto GetFactor = [&SurfaceMaterial](const char* FactorName)
				{
					FBXSDK_NAMESPACE::FbxProperty Property = SurfaceMaterial->FindProperty(FactorName);
					return Property.IsValid() ? (float)Property.Get<FbxDouble>() : 1.;
				};

				auto ConnectInput = [&](FName InputName, const char* FbxPropertyName, float Factor, TVariant<FLinearColor, float>& DefaultValue, bool bInverse) -> bool
				{
					FBXSDK_NAMESPACE::FbxProperty MaterialProperty = SurfaceMaterial->FindProperty(FbxPropertyName);
					if (ShouldConvertProperty(MaterialProperty))
					{
						return ConvertPropertyToMaterialInstanceNode(NodeContainer, MaterialInstanceNode, MaterialProperty, Factor, InputName, DefaultValue, bInverse);
					}

					return false;
				};

				bool bHasInput = false;

				// Diffuse
				{
					const float Factor = GetFactor(FbxSurfaceMaterial::sDiffuseFactor);
					TVariant<FLinearColor, float> DefaultValue;
					DefaultValue.Set<FLinearColor>(FLinearColor::Black);
					bHasInput |= ConnectInput(Phong::Parameters::DiffuseColor, FbxSurfaceMaterial::sDiffuse, Factor, DefaultValue, false);
				}

				// Ambient
				{
					const float Factor = GetFactor(FbxSurfaceMaterial::sAmbientFactor);
					TVariant<FLinearColor, float> DefaultValue;
					DefaultValue.Set<FLinearColor>(FLinearColor::Black);
					bHasInput |= ConnectInput(Phong::Parameters::AmbientColor, FbxSurfaceMaterial::sAmbient, Factor, DefaultValue, false);
				}

				// Emissive
				{
					const float Factor = GetFactor(FbxSurfaceMaterial::sEmissiveFactor);
					TVariant<FLinearColor, float> DefaultValue;
					DefaultValue.Set<FLinearColor>(FLinearColor::Black);
					bHasInput |= ConnectInput(Phong::Parameters::EmissiveColor, FbxSurfaceMaterial::sEmissive, Factor, DefaultValue, false);
				}

				// Normal
				{
					const float FactorNormal = GetFactor(FbxSurfaceMaterial::sNormalMap);
					TVariant<FLinearColor, float> DefaultValue;
					DefaultValue.Set<FLinearColor>(FLinearColor{ FVector::UpVector });
					bool bHasNormal = ConnectInput(Phong::Parameters::Normal, FbxSurfaceMaterial::sNormalMap, FactorNormal, DefaultValue, false);
					bHasInput |= bHasNormal;
					if (!bHasNormal)
					{
						const float FactorBump = GetFactor(FbxSurfaceMaterial::sBumpFactor);
						DefaultValue.Set<float>(0.f);
						bool bHasBump = ConnectInput(Phong::Parameters::Normal, FbxSurfaceMaterial::sBump, FactorBump, DefaultValue, false);
						if (bHasBump)
						{
							BumpMaterial = TEXT("Bump");
						}
						bHasInput |= bHasBump;
					}
				}

				// Opacity
				// Connect only if transparency is either a texture or different from 0.f
				{
					FBXSDK_NAMESPACE::FbxProperty MaterialProperty = SurfaceMaterial->FindProperty(FbxSurfaceMaterial::sTransparentColor);
					if (ShouldConvertProperty(MaterialProperty))
					{
						const float Factor = GetFactor(FbxSurfaceMaterial::sTransparencyFactor);

						if (MaterialProperty.GetSrcObjectCount<FbxFileTexture>() > 0)
						{
							TVariant<FLinearColor, float> DefaultValue;
							DefaultValue.Set<float>(0.f); // Opaque
							FName InputName = Phong::Parameters::Opacity;
							EBlendMode BlendMode = EBlendMode::BLEND_Translucent;							

							// The texture is hooked to the OpacityMask when transparency is with a texture
							if (MaterialProperty.Get<FbxDouble>() == 0.)
							{
								InputName = Phong::Parameters::OpacityMask;
								BlendMode = EBlendMode::BLEND_Masked;
							}

							MaterialInstanceNode->SetCustomBlendMode(BlendMode);

							if (MaterialProperty.GetSrcObjectCount<FbxFileTexture>() == 0)
							{
								ConvertValuePropertyToMaterialInstanceNode(MaterialInstanceNode, MaterialProperty, Factor, InputName, DefaultValue, true);
								bHasInput = true;
							}
							else
							{
								FbxFileTexture* FbxTexture = MaterialProperty.GetSrcObject<FbxFileTexture>(0);
								if (const UInterchangeTexture2DNode* Texture2DNode = ConvertTexturePropertyToMaterialInstanceNode(NodeContainer, MaterialInstanceNode, InputName, FbxTexture))
								{
									bHasInput = true;

									// When alpha is present in the image use it as opacity, instead of RGB
									if (DoesTextureHaveAlpha(*Texture2DNode->GetPayLoadKey()))
									{
										MaterialInstanceNode->AddScalarParameterValue(TEXT("OpacitySourceChannel"), 1);
									}
								}
							}
						}
						else
						{
							const float OpacityScalar = 1.f - Factor;
							if (OpacityScalar < 1.f)
							{
								MaterialInstanceNode->SetCustomBlendMode(EBlendMode::BLEND_Translucent);
								MaterialInstanceNode->AddScalarParameterValue(Phong::Parameters::Opacity.ToString(), OpacityScalar);
							}
						}
					}
				}

				// Default to Phong, if there's no specular, no shininess then it's a Lambert material
				bool bIsPhong = true;

				// Specular
				{
					const float Factor = GetFactor(FbxSurfaceMaterial::sSpecularFactor);
					TVariant<FLinearColor, float> DefaultValue;
					DefaultValue.Set<FLinearColor>(FLinearColor::Black);
					bIsPhong &= ConnectInput(Phong::Parameters::SpecularColor, FbxSurfaceMaterial::sSpecular, Factor, DefaultValue, false);
					bHasInput |= bIsPhong;
				}

				//Shininess
				{
					FBXSDK_NAMESPACE::FbxProperty ShininessProperty = SurfaceMaterial->FindProperty(FbxSurfaceMaterial::sShininess);
					if (ShininessProperty.IsValid())
					{
						TVariant<FLinearColor, float> DefaultValue;
						DefaultValue.Set<float>(20.f);
						bIsPhong |= ConnectInput(Phong::Parameters::Shininess, FbxSurfaceMaterial::sShininess, 1.f, DefaultValue, false);
						bHasInput |= bIsPhong;
					}
				}

				ShadingModel = bIsPhong ? TEXT("Phong") : TEXT("Lambert");
				FString ParentMaterial{ TEXT("FBXLegacy") + ShadingModel + TEXT("SurfaceMaterial") + BumpMaterial };
				MaterialInstanceNode->SetCustomParent(TEXT("/InterchangeAssets/Materials/") + ParentMaterial + TEXT('.') + ParentMaterial);

				// If no valid property found, create a material anyway
				TArray<FString> InputNames;
				if (!bHasInput)
				{
					FLinearColor BaseColor;
					BaseColor.R = 0.7f;
					BaseColor.G = 0.7f;
					BaseColor.B = 0.7f;

					const FString InputValueKey = Phong::Parameters::DiffuseColor.ToString();
					MaterialInstanceNode->AddVectorParameterValue(InputValueKey, BaseColor);
				}

				return MaterialInstanceNode;
			}

			void FFbxMaterial::AddAllTextures(FbxScene* SDKScene, UInterchangeBaseNodeContainer& NodeContainer)
			{
				int32 TextureCount = SDKScene->GetSrcObjectCount<FbxFileTexture>();
				for (int32 TextureIndex = 0; TextureIndex < TextureCount; ++TextureIndex)
				{
					if (FbxFileTexture* Texture = SDKScene->GetSrcObject<FbxFileTexture>(TextureIndex))
					{
						const FString TextureFilename = FindTextureFilePath(Parser, *Texture);
						
						//Only import texture that exist on disk
						if (TextureFilename.IsEmpty() || !FPaths::FileExists(TextureFilename))
						{
							if (!GIsAutomationTesting)
							{
								UInterchangeResultTextureDisplay_TextureFileDoNotExist* Message = Parser.AddMessage<UInterchangeResultTextureDisplay_TextureFileDoNotExist>();
								Message->TextureName = TextureFilename;
								Message->MaterialName.Empty();
							}
							continue;
						}

						//Create a texture node and make it child of the material node
						const FString TextureName = FPaths::GetBaseFilename(TextureFilename);
						const UInterchangeTexture2DNode* TextureNode = Cast<UInterchangeTexture2DNode>(NodeContainer.GetNode(UInterchangeTextureNode::MakeNodeUid(TextureName)));
						if (!TextureNode)
						{
							CreateTexture2DNode(NodeContainer, TextureFilename);
						}
					}
				}
			}
			
			void FFbxMaterial::AddAllNodeMaterials(UInterchangeSceneNode* SceneNode, FbxNode* ParentFbxNode, UInterchangeBaseNodeContainer& NodeContainer)
			{
				int32 MaterialCount = ParentFbxNode->GetMaterialCount();
				TMap<FbxSurfaceMaterial*, int32> UniqueSlotNames;
				UniqueSlotNames.Reserve(MaterialCount);
				for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex)
				{
					if (FbxSurfaceMaterial* SurfaceMaterial = ParentFbxNode->GetMaterial(MaterialIndex))
					{
						const UInterchangeMaterialInstanceNode* MaterialInstanceNode = AddMaterialInstanceNode(SurfaceMaterial, NodeContainer);
						
						int32& SlotMaterialCount = UniqueSlotNames.FindOrAdd(SurfaceMaterial);
						FString MaterialSlotName = Parser.GetFbxHelper()->GetFbxObjectName(SurfaceMaterial);
						if (SlotMaterialCount > 0)
						{
							MaterialSlotName += TEXT("_Section") + FString::FromInt(SlotMaterialCount);
						}
						SceneNode->SetSlotMaterialDependencyUid(MaterialSlotName, MaterialInstanceNode->GetUniqueID());
						SlotMaterialCount++;
					}
				}
			}

			void FFbxMaterial::AddAllMaterials(FbxScene* SDKScene, UInterchangeBaseNodeContainer& NodeContainer)
			{
				int32 MaterialCount = SDKScene->GetMaterialCount();
				for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex)
				{
					if (FbxSurfaceMaterial* SurfaceMaterial = SDKScene->GetMaterial(MaterialIndex))
					{
						AddMaterialInstanceNode(SurfaceMaterial, NodeContainer);
					}
				}
			}
		} //ns Private
	} //ns Interchange
}//ns UE

#undef LOCTEXT_NAMESPACE
