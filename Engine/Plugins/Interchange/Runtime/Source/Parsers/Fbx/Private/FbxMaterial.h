// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FbxInclude.h"
#include "Misc/TVariant.h"

/** Forward declarations */
class UInterchangeBaseNode;
class UInterchangeBaseNodeContainer;
class UInterchangeMaterialInstanceNode;
class UInterchangeTexture2DNode;
class UInterchangeSceneNode;

namespace UE
{
	namespace Interchange
	{
		namespace Private
		{
			class FFbxParser;

			// same enum as the one in EngineTypes, required because InterchangeWorker does not compile with Engine
			enum EBlendMode : int
			{
				BLEND_Opaque,
				BLEND_Masked,
				BLEND_Translucent,
				BLEND_Additive,
				BLEND_Modulate,
				BLEND_AlphaComposite,
				BLEND_AlphaHoldout,
				BLEND_TranslucentColoredTransmittance, /*Substrate only */
				BLEND_MAX,
				// Renamed blend modes. These blend modes are remapped onto legacy ones and kept hidden for not confusing users in legacy mode, while allowing to use the new blend mode names into code.
				BLEND_TranslucentGreyTransmittance = BLEND_Translucent, /*Substrate only */
				BLEND_ColoredTransmittanceOnly = BLEND_Modulate, /*Substrate only */
			};

			class FFbxMaterial
			{
			public:
				explicit FFbxMaterial(FFbxParser& InParser)
					: Parser(InParser)
				{}

				/**
				 * Create a UInterchangeFextureNode and add it to the NodeContainer for each texture of type FbxFileTexture that the FBX file contains.
				 *
				 * @note - Any node that already exists in the NodeContainer will not be created or modified.
				 */
				void AddAllTextures(FbxScene* SDKScene, UInterchangeBaseNodeContainer& NodeContainer);
				
				/**
				 * Create a UInterchangeMaterialNode and add it to the NodeContainer for each material of type FbxSurfaceMaterial that the FBX file contains.
				 * 
				 * @note - Any node that already exists in the NodeContainer will not be created or modified.
				 */
				void AddAllMaterials(FbxScene* SDKScene, UInterchangeBaseNodeContainer& NodeContainer);

				/**
				 * Create a UInterchangeMaterialNode and add it to the NodeContainer for each material of type FbxSurfaceMaterial that the FBX ParentFbxNode contains.
				 * Also set the dependencies of the node materials on the Interchange ParentNode.
				 * 
				 * @note - Any material node that already exists in the NodeContainer will be added as a dependency.
				 */
				void AddAllNodeMaterials(UInterchangeSceneNode* SceneNode, FbxNode* ParentFbxNode, UInterchangeBaseNodeContainer& NodeContainer);

			protected:
				UInterchangeMaterialInstanceNode* CreateMaterialInstanceNode(UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeUID, const FString& NodeName);
				const UInterchangeTexture2DNode* CreateTexture2DNode(UInterchangeBaseNodeContainer& NodeContainer, const FString& TextureFilePath);
				const UInterchangeTexture2DNode* CreateTexture2DNode(FbxFileTexture* FbxTexture, UInterchangeBaseNodeContainer& NodeContainer, UInterchangeMaterialInstanceNode* MaterialInstanceNode);
				bool ConvertPropertyToMaterialInstanceNode(UInterchangeBaseNodeContainer& NodeContainer, UInterchangeMaterialInstanceNode* MaterialInstanceNode,
				                                           FbxProperty& Property, float Factor, FName InputName, const TVariant<FLinearColor, float>& DefaultValue, bool bInverse = false);
				void ConvertValuePropertyToMaterialInstanceNode(UInterchangeMaterialInstanceNode* MaterialInstanceNode,
					FbxProperty& Property, float Factor, FName InputName,
					const TVariant<FLinearColor, float>& DefaultValue, bool bInverse);
				const UInterchangeTexture2DNode* ConvertTexturePropertyToMaterialInstanceNode(UInterchangeBaseNodeContainer& NodeContainer,
				                                                  UInterchangeMaterialInstanceNode* MaterialInstanceNode,
				                                                  FName InputName, FbxFileTexture* FbxTexture);

			private:
				const UInterchangeMaterialInstanceNode* AddMaterialInstanceNode(FbxSurfaceMaterial* SurfaceMaterial, UInterchangeBaseNodeContainer& NodeContainer);

				FFbxParser& Parser;
			};
		}//ns Private
	}//ns Interchange
}//ns UE
