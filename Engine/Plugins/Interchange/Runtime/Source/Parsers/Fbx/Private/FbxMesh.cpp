// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxMesh.h"

#include "CoreMinimal.h"
#include "FbxAnimation.h"
#include "FbxAPI.h"
#include "FbxConvert.h"
#include "FbxHelper.h"
#include "FbxInclude.h"
#include "Fbx/InterchangeFbxMessages.h"
#include "InterchangeFbxParserLog.h"
#include "InterchangeMeshNode.h"
#include "InterchangeResultsContainer.h"
#include "MeshDescription.h"
#if WITH_ENGINE
#include "Mesh/InterchangeMeshPayload.h"
#endif
#include "Misc/FileHelper.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Serialization/LargeMemoryWriter.h"
#include "SkeletalMeshAttributes.h"
#include "StaticMeshAttributes.h"

#if WITH_ENGINE
#include "InterchangeProjectSettings.h"
#endif
#define LOCTEXT_NAMESPACE "InterchangeFbxMesh"

namespace UE::Interchange::Private
{

	//Import vertex Attribute from other vertex color layer
	void GetVertexAttributeFromMeshVertexColor(FMeshDescription& MeshDescription, FbxMesh* Mesh)
	{
		//
		// Get the vertex attribute layers from all layers, even the first layer which may be used as a vertex color layer.
		// Currently we're only interested in alpha-only layers, since those are the only layer types the engine
		// currently exposes for vertex attributes. Internally we can store 1-4 components, but there's no tooling for that
		// 2-4 channels as of yet.
		//
		struct FNamedVertexAttribute
		{
			FNamedVertexAttribute(FString&& InAttributeName, TArray<float>&& InAttributeValues, const int32 InComponentCount)
				: AttributeName(InAttributeName)
				, AttributeValues(InAttributeValues)
				, ComponentCount(InComponentCount)
			{}

			FString AttributeName;
			TArray<float> AttributeValues;
			int32 ComponentCount;
		};
		int32 ControlPointsCount = Mesh->GetControlPointsCount();
		int32 TriangleCount = Mesh->GetPolygonCount();
		int32 MeshLayerCount = Mesh->GetLayerCount();
		TArray<FNamedVertexAttribute> NamedVertexAttributes;

		for (int32 LayerIndex = 0; LayerIndex < MeshLayerCount; LayerIndex++)
		{
			FbxLayerElementVertexColor* LayerElementVertexAttribute = Mesh->GetLayer(LayerIndex)->GetVertexColors();
			if (!LayerElementVertexAttribute)
			{
				continue;
			}

			// Check if this is an alpha-only attribute, by ensuring the RGB values are all zero, otherwise skip.
			bool bIsValidAttribute = true;
			const FbxLayerElementArrayTemplate<FbxColor>& AttributeValues = LayerElementVertexAttribute->GetDirectArray();
			for (int32 Index = 0; Index < AttributeValues.GetCount(); Index++)
			{
				// We do an exact comparison, since that's how empty channels would be represented in the FBX file.
				const FbxColor& Value = AttributeValues.GetAt(Index);
				if (Value.mRed != 0.0 || Value.mGreen != 0.0 || Value.mBlue != 0.0)
				{
					bIsValidAttribute = false;
					break;
				}
			}

			// We can only do attributes that are mapped per-vertex.
			if (!bIsValidAttribute)
			{
				continue;
			}

			const int32 AttributeComponentCount = 1;	// Number of component values per vertex. See comment above. 
			TArray<float> AttributeComponentValues;

			switch (LayerElementVertexAttribute->GetMappingMode())
			{
			case FbxLayerElement::eByControlPoint:
			{
				AttributeComponentValues.AddZeroed(ControlPointsCount);

				if (LayerElementVertexAttribute->GetReferenceMode() == FbxLayerElement::eDirect)
				{
					for (int32 Index = 0; Index < AttributeValues.GetCount(); Index++)
					{
						AttributeComponentValues[Index] = AttributeValues.GetAt(Index).mAlpha;
					}
				}
				else // LayerElementVertexAttribute->GetReferenceMode() == FbxLayerElement::eIndexToDirect
				{
					const FbxLayerElementArrayTemplate<int>& IndexArray = LayerElementVertexAttribute->GetIndexArray();
					for (int32 Index = 0; Index < IndexArray.GetCount(); Index++)
					{
						AttributeComponentValues[Index] = AttributeValues.GetAt(IndexArray[Index]).mAlpha;
					}
				}
			}
			break;
			case FbxLayerElement::eByPolygonVertex:
			{
				// Vertex attributes are stored per-vertex, not per-vertex instance. To work around this we average
				// together values that share a vertex.
				TArray<int32> SharedVertexCount;
				SharedVertexCount.AddZeroed(ControlPointsCount);
				AttributeComponentValues.AddZeroed(ControlPointsCount);

				const FbxLayerElementArrayTemplate<int>* IndexArray = nullptr;
				if (LayerElementVertexAttribute->GetReferenceMode() == FbxLayerElement::eIndexToDirect)
				{
					IndexArray = &LayerElementVertexAttribute->GetIndexArray();
				}

				const int* PolygonControlPointIndexes = Mesh->GetPolygonVertices();

				for (int32 TriangleIndex = 0; TriangleIndex < TriangleCount; TriangleIndex++)
				{
					for (int32 InnerIndex = 0; InnerIndex < 3; InnerIndex++)
					{
						const int32 PolygonVertexIndex = TriangleIndex * 3 + InnerIndex;;
						const int32 PointIndex = PolygonControlPointIndexes[PolygonVertexIndex];

						AttributeComponentValues[PointIndex] +=
							AttributeValues.GetAt(IndexArray ? IndexArray->GetAt(PolygonVertexIndex) : PolygonVertexIndex).mAlpha;
						SharedVertexCount[PointIndex]++;
					}
				}

				for (int32 PointIndex = 0; PointIndex < ControlPointsCount; PointIndex++)
				{
					if (SharedVertexCount[PointIndex] > 1)
					{
						AttributeComponentValues[PointIndex] /= static_cast<float>(SharedVertexCount[PointIndex]);
					}
				}
			}
			break;
			default:
				break;
			}

			if (!AttributeComponentValues.IsEmpty())
			{
				FString AttributeName(UTF8_TO_TCHAR(LayerElementVertexAttribute->GetName()));
				NamedVertexAttributes.Emplace(MoveTemp(AttributeName), MoveTemp(AttributeComponentValues), AttributeComponentCount);
			}
		}
		if(NamedVertexAttributes.Num() > 0)
		{
			FSkeletalMeshAttributes MeshAttributes(MeshDescription);
			MeshAttributes.Register(true);
			TVertexAttributesRef<FVector3f> VertexPositions = MeshAttributes.GetVertexPositions();

			TMap<FString, FName> ValidAttributes;
			for (int32 AttributeIndex = 0; AttributeIndex < NamedVertexAttributes.Num(); AttributeIndex++)
			{
				const FNamedVertexAttribute& NamedVertexAttribute = NamedVertexAttributes[AttributeIndex];
				const FString& VertexAttributeName = NamedVertexAttribute.AttributeName;
				if (!ensure(NamedVertexAttribute.AttributeValues.Num() == (VertexPositions.GetNumElements() * NamedVertexAttribute.ComponentCount)))
				{
					continue;
				}

				EMeshAttributeFlags DefaultAttributeFlags = EMeshAttributeFlags::Mergeable | EMeshAttributeFlags::Lerpable;

				FName RegisteredName(VertexAttributeName);

				// Ignore attributes with reserved names. This should have been handled at import time or when the attribute
				// was created/renamed.
				if (!ensure(!FSkeletalMeshAttributes::IsReservedAttributeName(RegisteredName)))
				{
					continue;
				}

				switch (NamedVertexAttribute.ComponentCount)
				{
				case 1:
					MeshDescription.VertexAttributes().RegisterAttribute<float>(RegisteredName, 1, 0.0f, DefaultAttributeFlags);
					break;
				case 2:
					MeshDescription.VertexAttributes().RegisterAttribute<FVector2f>(RegisteredName, 1, FVector2f::Zero(), DefaultAttributeFlags);
					break;
				case 3:
					MeshDescription.VertexAttributes().RegisterAttribute<FVector3f>(RegisteredName, 1, FVector3f::Zero(), DefaultAttributeFlags);
					break;
				case 4:
					MeshDescription.VertexAttributes().RegisterAttribute<FVector4f>(RegisteredName, 1, FVector4f::Zero(), DefaultAttributeFlags);
					break;
				default:
					continue;
				}

				ValidAttributes.Add(VertexAttributeName, RegisteredName);

				switch (NamedVertexAttribute.ComponentCount)
				{
				case 1:
				{
					TVertexAttributesRef<float> AttributeRef = MeshDescription.VertexAttributes().GetAttributesRef<float>(RegisteredName);
					for (int32 Index = 0; Index < NamedVertexAttribute.AttributeValues.Num(); Index++)
					{
						AttributeRef.Set(FVertexID(Index), NamedVertexAttribute.AttributeValues[Index]);
					}
					break;
				}
				case 2:
				{
					TVertexAttributesRef<FVector2f> AttributeRef = MeshDescription.VertexAttributes().GetAttributesRef<FVector2f>(RegisteredName);
					for (int32 Index = 0; Index < NamedVertexAttribute.AttributeValues.Num(); Index += 2)
					{
						AttributeRef.Set(FVertexID(Index / 2),
							FVector2f(NamedVertexAttribute.AttributeValues[Index], NamedVertexAttribute.AttributeValues[Index + 1]));
					}
					break;
				}
				case 3:
				{
					TVertexAttributesRef<FVector3f> AttributeRef = MeshDescription.VertexAttributes().GetAttributesRef<FVector3f>(RegisteredName);
					for (int32 Index = 0; Index < NamedVertexAttribute.AttributeValues.Num(); Index += 3)
					{
						AttributeRef.Set(FVertexID(Index / 3),
							FVector3f(NamedVertexAttribute.AttributeValues[Index], NamedVertexAttribute.AttributeValues[Index + 1], NamedVertexAttribute.AttributeValues[Index + 2]));
					}
					break;
				}
				case 4:
				{
					TVertexAttributesRef<FVector4f> AttributeRef = MeshDescription.VertexAttributes().GetAttributesRef<FVector4f>(RegisteredName);
					for (int32 Index = 0; Index < NamedVertexAttribute.AttributeValues.Num(); Index += 4)
					{
						AttributeRef.Set(FVertexID(Index / 4),
							FVector4f(
								NamedVertexAttribute.AttributeValues[Index], NamedVertexAttribute.AttributeValues[Index + 1],
								NamedVertexAttribute.AttributeValues[Index + 2], NamedVertexAttribute.AttributeValues[Index + 3]));
					}
					break;
				}
				default:
					checkNoEntry();
				}
			}
		}
	}

	void ExtractMeshMaterials(FFbxParser& Parser, FbxMesh* Mesh, FbxNode* MeshNode, TFunction<void(const FString& MaterialName, const FString& MaterialUid, const int32 MeshMaterialIndex)> CollectMaterial)
	{
		if (!Mesh || !MeshNode)
		{
			return;
		}

		//Grab all Material indexes use by the mesh
		TArray<int32> MaterialIndexes;
		int32 PolygonCount = Mesh->GetPolygonCount();
		if (FbxGeometryElementMaterial* GeometryElementMaterial = Mesh->GetElementMaterial())
		{
			FbxLayerElementArrayTemplate<int32>& IndexArray = GeometryElementMaterial->GetIndexArray();
			switch (GeometryElementMaterial->GetMappingMode())
			{
			case FbxGeometryElement::eByPolygon:
			{
				if (IndexArray.GetCount() == PolygonCount)
				{
					for (int32 PolygonIndex = 0; PolygonIndex < PolygonCount; ++PolygonIndex)
					{
						MaterialIndexes.AddUnique(IndexArray.GetAt(PolygonIndex));
					}
				}
			}
			break;

			case FbxGeometryElement::eAllSame:
			{
				if (IndexArray.GetCount() > 0)
				{
					MaterialIndexes.AddUnique(IndexArray.GetAt(0));
				}
			}
			break;
			}
		}
		const int32 MaterialCount = MeshNode->GetMaterialCount();
		TMap<FbxSurfaceMaterial*, int32> UniqueSlotNames;
		UniqueSlotNames.Reserve(MaterialCount);
		bool bAddAllNodeMaterials = (MaterialIndexes.Num() == 0);
		for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex)
		{
			if (FbxSurfaceMaterial* FbxMaterial = MeshNode->GetMaterial(MaterialIndex))
			{
				int32& SlotMaterialCount = UniqueSlotNames.FindOrAdd(FbxMaterial);
				FString MaterialName = Parser.GetFbxHelper()->GetFbxObjectName(FbxMaterial);
				FString MaterialUid = TEXT("\\Material\\") + MaterialName;
				if (bAddAllNodeMaterials || MaterialIndexes.Contains(MaterialIndex))
				{
					if (SlotMaterialCount > 0)
					{
						MaterialName += TEXT("_Section") + FString::FromInt(SlotMaterialCount);
					}
					SlotMaterialCount++;
					CollectMaterial(MaterialName, MaterialUid, MaterialIndex);
				}
			}
		}
	}

// Wraps some common code useful for multiple fbx import code path
struct FFBXUVs
{
	// constructor
	FFBXUVs(FMeshDescriptionImporter& InMeshDescriptionImporter, FbxMesh* Mesh)
		: MeshDescriptionImporter(InMeshDescriptionImporter)
		, UniqueUVCount(0)
	{
		check(Mesh);

		//
		//	store the UVs in arrays for fast access in the later looping of triangles 
		//
		// mapping from UVSets to Fbx LayerElementUV
		// Fbx UVSets may be duplicated, remove the duplicated UVSets in the mapping 
		int32 LayerCount = Mesh->GetLayerCount();
		if (LayerCount > 0)
		{
			int32 UVLayerIndex;
			for (UVLayerIndex = 0; UVLayerIndex < LayerCount; UVLayerIndex++)
			{
				FbxLayer* lLayer = Mesh->GetLayer(UVLayerIndex);
				int UVSetCount = lLayer->GetUVSetCount();
				if (UVSetCount)
				{
					FbxArray<FbxLayerElementUV const*> EleUVs = lLayer->GetUVSets();
					for (int UVIndex = 0; UVIndex < UVSetCount; UVIndex++)
					{
						FbxLayerElementUV const* ElementUV = EleUVs[UVIndex];
						if (ElementUV)
						{
							const char* UVSetName = ElementUV->GetName();
							FString LocalUVSetName = UTF8_TO_TCHAR(UVSetName);
							if (LocalUVSetName.IsEmpty())
							{
								LocalUVSetName = TEXT("UVmap_") + FString::FromInt(UVLayerIndex);
							}

							UVSets.AddUnique(LocalUVSetName);
						}
					}
				}
			}
		}


		// If the the UV sets are named using the following format (UVChannel_X; where X ranges from 1 to 4)
		// we will re-order them based on these names.  Any UV sets that do not follow this naming convention
		// will be slotted into available spaces.
		if (UVSets.Num())
		{
			for (int32 ChannelNumIdx = 0; ChannelNumIdx < 4; ChannelNumIdx++)
			{
				FString ChannelName = FString::Printf(TEXT("UVChannel_%d"), ChannelNumIdx + 1);
				int32 SetIdx = UVSets.Find(ChannelName);

				// If the specially formatted UVSet name appears in the list and it is in the wrong spot,
				// we will swap it into the correct spot.
				if (SetIdx != INDEX_NONE && SetIdx != ChannelNumIdx)
				{
					// If we are going to swap to a position that is outside the bounds of the
					// array, then we pad out to that spot with empty data.
					for (int32 ArrSize = UVSets.Num(); ArrSize < ChannelNumIdx + 1; ArrSize++)
					{
						UVSets.Add(FString(TEXT("")));
					}
					//Swap the entry into the appropriate spot.
					UVSets.Swap(SetIdx, ChannelNumIdx);
				}
			}
		}
	}

	void Phase2(FbxMesh* Mesh)
	{
		//
		//	store the UVs in arrays for fast access in the later looping of triangles 
		//
		UniqueUVCount = UVSets.Num();
		if (UniqueUVCount > 0)
		{
			LayerElementUV.Reserve(UniqueUVCount);
			UVReferenceMode.Reserve(UniqueUVCount);
			UVMappingMode.Reserve(UniqueUVCount);
		}
		for (int32 UVIndex = 0; UVIndex < UniqueUVCount; UVIndex++)
		{
			for (int32 UVLayerIndex = 0, LayerCount = Mesh->GetLayerCount(); UVLayerIndex < LayerCount; UVLayerIndex++)
			{
				FbxLayer* lLayer = Mesh->GetLayer(UVLayerIndex);
				int UVSetCount = lLayer->GetUVSetCount();
				if (UVSetCount)
				{
					FbxArray<FbxLayerElementUV const*> EleUVs = lLayer->GetUVSets();
					for (int32 FbxUVIndex = 0; FbxUVIndex < UVSetCount; FbxUVIndex++)
					{
						FbxLayerElementUV const* ElementUV = EleUVs[FbxUVIndex];
						if (ElementUV)
						{
							const char* UVSetName = ElementUV->GetName();
							FString LocalUVSetName = UTF8_TO_TCHAR(UVSetName);
							if (LocalUVSetName.IsEmpty())
							{
								LocalUVSetName = TEXT("UVmap_") + FString::FromInt(UVLayerIndex);
							}
							if (LocalUVSetName == UVSets[UVIndex])
							{
								LayerElementUV.Add(ElementUV);
								UVReferenceMode.Add(ElementUV->GetReferenceMode());
								UVMappingMode.Add(ElementUV->GetMappingMode());
								break;
							}
						}
					}
				}
			}
		}
		UniqueUVCount = LayerElementUV.Num();

		if (UniqueUVCount > MAX_MESH_TEXTURE_COORDS_MD)
		{
			UInterchangeResultMeshWarning_TooManyUVs* Message = MeshDescriptionImporter.AddMessage<UInterchangeResultMeshWarning_TooManyUVs>(Mesh);
			Message->ExcessUVs = UniqueUVCount - MAX_MESH_TEXTURE_COORDS_MD;
		}

		UniqueUVCount = FMath::Min<int32>(UniqueUVCount, MAX_MESH_TEXTURE_COORDS_MD);
	}

	// @param FaceCornerIndex usually TriangleIndex * 3 + CornerIndex but more complicated for mixed n-gons
	int32 ComputeUVIndex(int32 UVLayerIndex, int32 lControlPointIndex, int32 FaceCornerIndex) const
	{
		int32 UVMapIndex = (UVMappingMode[UVLayerIndex] == FbxLayerElement::eByControlPoint) ? lControlPointIndex : FaceCornerIndex;

		int32 Ret;

		if (UVReferenceMode[UVLayerIndex] == FbxLayerElement::eDirect)
		{
			Ret = UVMapIndex;
		}
		else
		{
			FbxLayerElementArrayTemplate<int>& Array = LayerElementUV[UVLayerIndex]->GetIndexArray();
			Ret = Array.GetAt(UVMapIndex);
		}

		return Ret;
	}

	// todo: is that needed? could the dtor do it?
	void Cleanup()
	{
		//
		// clean up.  This needs to happen before the mesh is destroyed
		//
		LayerElementUV.Empty();
		UVReferenceMode.Empty();
		UVMappingMode.Empty();
	}

	FMeshDescriptionImporter& MeshDescriptionImporter;
	TArray<FString> UVSets;
	TArray<FbxLayerElementUV const*> LayerElementUV;
	TArray<FbxLayerElement::EReferenceMode> UVReferenceMode;
	TArray<FbxLayerElement::EMappingMode> UVMappingMode;
	int32 UniqueUVCount;
};


FMeshDescriptionImporter::FMeshDescriptionImporter(FFbxParser& InParser, FMeshDescription* InMeshDescription, FbxScene* InSDKScene, FbxGeometryConverter* InSDKGeometryConverter)
	: Parser(InParser)
	, MeshDescription(InMeshDescription)
	, SDKScene(InSDKScene)
	, SDKGeometryConverter(InSDKGeometryConverter)
{
	bInitialized = ensure(MeshDescription) && ensure(SDKScene) && ensure(SDKGeometryConverter);
}
			
bool FMeshDescriptionImporter::FillStaticMeshDescriptionFromFbxMesh(FbxMesh* Mesh, const FTransform& MeshGlobalTransform)
{
	if (!ensure(bInitialized) || !ensure(MeshDescription) || !ensure(Mesh))
	{
		return false;
	}
	TArray<FString> UnusedArray;
	return FillMeshDescriptionFromFbxMesh(Mesh, MeshGlobalTransform, UnusedArray, EMeshType::Static);
}

bool FMeshDescriptionImporter::FillSkinnedMeshDescriptionFromFbxMesh(FbxMesh* Mesh, const FTransform& MeshGlobalTransform, TArray<FString>& OutJointUniqueNames)
{
	if (!ensure(bInitialized) || !ensure(MeshDescription) || !ensure(Mesh) || !ensure(Mesh->GetDeformerCount() > 0))
	{
		return false;
	}

	return FillMeshDescriptionFromFbxMesh(Mesh, MeshGlobalTransform, OutJointUniqueNames, EMeshType::Skinned);
}

bool FMeshDescriptionImporter::FillMeshDescriptionFromFbxShape(FbxShape* Shape, const FTransform& MeshGlobalTransform)
{
	//For the shape we just need the vertex positions
	if (!ensure(bInitialized) || !ensure(MeshDescription))
	{
		return false;
	}
	FStaticMeshAttributes Attributes(*MeshDescription);

	//The shape should be for a specified mesh
	if (!ensure(Shape))
	{
		return false;
	}

	//Extract the points into a simplified MeshDescription
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GatherMorphTargetPoints);
		MeshDescription->SuspendVertexIndexing();

		// Construct the matrices for the conversion from right handed to left handed system
		FbxAMatrix TotalMatrix = FFbxConvert::ConvertMatrix(MeshGlobalTransform.ToMatrixWithScale());
		FbxAMatrix TotalMatrixForNormal;
		TotalMatrixForNormal = TotalMatrix.Inverse();
		TotalMatrixForNormal = TotalMatrixForNormal.Transpose();

		FbxGeometryBase* GeoBase = (FbxGeometryBase*)Shape;

		int32 VertexCount = GeoBase->GetControlPointsCount();

		TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
		int32 VertexOffset = MeshDescription->Vertices().Num();
		// The below code expects Num() to be equivalent to GetArraySize(), i.e. that all added elements are appended, not inserted into existing gaps
		check(VertexOffset == MeshDescription->Vertices().GetArraySize());
		//Fill the vertex array
		auto GetFinalPosition = [&TotalMatrix](FbxVector4& FbxPosition)
		{
			FbxPosition = TotalMatrix.MultT(FbxPosition);
			return FFbxConvert::ConvertPos<FVector3f>(FbxPosition);
		};

		MeshDescription->ReserveNewVertices(VertexCount);
		for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
		{
			int32 RealVertexIndex = VertexOffset + VertexIndex;
			const FVector3f VertexPosition = GetFinalPosition(GeoBase->GetControlPoints()[VertexIndex]);
			FVertexID AddedVertexId = MeshDescription->CreateVertex();
			if (AddedVertexId.GetValue() != RealVertexIndex)
			{
				UInterchangeResultMeshError_Generic* Message = AddMessage<UInterchangeResultMeshError_Generic>(Shape);
				Message->Text = LOCTEXT("CantCreateVertex", "Cannot create valid vertex for the mesh '{MeshName}'.");

				return false;
			}
			//Add the delta position, so we do not have to recompute it later
			VertexPositions[AddedVertexId] = VertexPosition;// -MeshVertexPosition;
		}
		MeshDescription->ResumeVertexIndexing();
	}
	return true;
}

bool FMeshDescriptionImporter::FillMeshDescriptionFromFbxMesh(FbxMesh* Mesh, const FTransform& MeshGlobalTransform, TArray<FString>& OutJointUniqueNames, EMeshType MeshType)
{
	if (!ensure(bInitialized) || !ensure(MeshDescription) || !ensure(Mesh))
	{
		return false;
	}

	FStaticMeshAttributes Attributes(*MeshDescription);
	constexpr bool bKeepExistingAttribute = true;
	Attributes.Register(bKeepExistingAttribute);
	
	// We want to store the source mesh name with each triangle so that we can identify individual geometry pieces in the combined static/skeletal mesh.
	Attributes.RegisterPolygonObjectNameAttribute();

	bool bStaticMeshUseSmoothEdgesIfSmoothingInformationIsMissing = true;

#if WITH_ENGINE
	//Get the default hard/smooth edges from the project settings
	const UInterchangeProjectSettings* InterchangeProjectSettings = GetDefault<UInterchangeProjectSettings>();
	bStaticMeshUseSmoothEdgesIfSmoothingInformationIsMissing = InterchangeProjectSettings->bStaticMeshUseSmoothEdgesIfSmoothingInformationIsMissing;
#endif

	//Get the base layer of the mesh
	FbxLayer* BaseLayer = Mesh->GetLayer(0);
	if (BaseLayer == NULL)
	{
		UInterchangeResultMeshError_Generic* Message = AddMessage<UInterchangeResultMeshError_Generic>(Mesh);
		Message->Text = LOCTEXT("CantGetLayer", "Unable to get FBX base layer for the mesh '{MeshName}'.");
		return false;
	}
	
	FFBXUVs FBXUVs(*this, Mesh);

	FbxNode* MeshNode = Mesh->GetNodeCount() > 0 ? Mesh->GetNode() : nullptr; //Get the first node using the mesh to setup default asset materials

	// Grab the mesh node's name, we use that to fill the object name attribute. The mesh object name itself is of no use since it's the same
	// for all meshes.
	const FName MeshNodeName(Parser.GetFbxHelper()->GetFbxObjectName(MeshNode));
	
	//
	// create materials
	//
	//Create a material name array in the node order, also fill the Meshdescription PolygonGroup
	TArray<FName> MaterialNames;
	TArray<int32> MaterialRemap;
	int32 MaterialCount = (MeshNode != nullptr) ? MeshNode->GetMaterialCount() : 1;
	if (MeshNode)
	{
		MaterialNames.Reserve(MaterialCount);
		MaterialRemap.Reserve(MaterialCount);
		for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex)
		{
			MaterialRemap.Add(MaterialIndex);
		}

		ExtractMeshMaterials(Parser, Mesh, MeshNode, [&MaterialNames, &MaterialRemap](const FString& MaterialName, const FString& MaterialUid, const int32 MeshMaterialIndex)
			{
				MaterialRemap[MeshMaterialIndex] = MaterialNames.Add(*MaterialName);
			});
		MaterialCount = MaterialNames.Num();
	}

	// Must do this before triangulating the mesh due to an FBX bug in TriangulateMeshAdvance
	int32 LayerSmoothingCount = Mesh->GetLayerCount(FbxLayerElement::eSmoothing);
	if (LayerSmoothingCount == 0 && !GIsAutomationTesting)
	{
		UInterchangeResultMeshWarning_Generic* Message = AddMessage<UInterchangeResultMeshWarning_Generic>(Mesh);
		Message->Text = LOCTEXT("MissingSmoothGroup", "No smoothing group information was found for this mesh '{MeshName}' in the FBX file. Please make sure to enable the 'Export Smoothing Groups' option in the FBX Exporter before exporting the file.");
	}

	for (int32 SmoothingLayerIndex = 0; SmoothingLayerIndex < LayerSmoothingCount; SmoothingLayerIndex++)
	{
		FbxLayerElementSmoothing const* SmoothingInfoTmp = Mesh->GetLayer(SmoothingLayerIndex)->GetSmoothing();
		if (SmoothingInfoTmp && SmoothingInfoTmp->GetMappingMode() != FbxLayerElement::eByPolygon)
		{
			SDKGeometryConverter->ComputePolygonSmoothingFromEdgeSmoothing(Mesh, SmoothingLayerIndex);
		}
	}

	//Mesh must be triangulate when creating the payload context, we cannot change the Mesh pointer after
	if(!Mesh->IsTriangleMesh())
	{
		//Since we want to avoid deleting the pointer, we set the bReplace to false
		constexpr bool bReplace = false;
		FbxNodeAttribute* ConvertedNode = SDKGeometryConverter->Triangulate(Mesh, bReplace);

		if (ConvertedNode != NULL && ConvertedNode->GetAttributeType() == FbxNodeAttribute::eMesh)
		{
			Mesh = (fbxsdk::FbxMesh*)ConvertedNode;
		}
		else
		{
			UInterchangeResultMeshError_Generic* Message = AddMessage<UInterchangeResultMeshError_Generic>(Mesh);
			Message->Text = LOCTEXT("TriangulationError", "Unable to triangulate the mesh '{MeshName}'.");

			MeshDescription->Empty();
			return false;
		}
	}

	// renew the base layer
	BaseLayer = Mesh->GetLayer(0);

	//
	// get the smoothing group layer
	//
	bool bSmoothingAvailable = false;

	FbxLayerElementSmoothing* SmoothingInfo = BaseLayer->GetSmoothing();
	FbxLayerElement::EReferenceMode SmoothingReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode SmoothingMappingMode(FbxLayerElement::eByEdge);
	if (SmoothingInfo)
	{
		if (SmoothingInfo->GetMappingMode() == FbxLayerElement::eByPolygon)
		{
			//Convert the base layer to edge smoothing
			for (int32 SmoothingLayerIndex = 0; SmoothingLayerIndex < LayerSmoothingCount; SmoothingLayerIndex++)
			{
				FbxLayerElementSmoothing const* SmoothingInfoTmp = Mesh->GetLayer(SmoothingLayerIndex)->GetSmoothing();
				if (SmoothingInfoTmp && SmoothingInfoTmp->GetMappingMode() != FbxLayerElement::eByEdge)
				{
					SDKGeometryConverter->ComputeEdgeSmoothingFromPolygonSmoothing(Mesh, SmoothingLayerIndex);
				}
			}
			
			BaseLayer = Mesh->GetLayer(0);
			SmoothingInfo = BaseLayer->GetSmoothing();
		}

		if (SmoothingInfo->GetMappingMode() == FbxLayerElement::eByEdge)
		{
			bSmoothingAvailable = true;
		}

		SmoothingReferenceMode = SmoothingInfo->GetReferenceMode();
		SmoothingMappingMode = SmoothingInfo->GetMappingMode();

		//If all smooth group are 0's, we can say that there is no valid smoothing data, but only for skeletal mesh
		//Static mesh will be faceted in this case since we will not compute smooth group from normals like we do here
		//This is legacy code to get the same result has before
		if (MeshType == EMeshType::Skinned)
		{
			// Check and see if the smooothing data is valid.  If not generate it from the normals
			BaseLayer = Mesh->GetLayer(0);
			if (BaseLayer)
			{
				const FbxLayerElementSmoothing* SmoothingLayer = BaseLayer->GetSmoothing();

				if (SmoothingLayer)
				{
					bool bValidSmoothingData = false;
					FbxLayerElementArrayTemplate<int32>& Array = SmoothingLayer->GetDirectArray();
					for (int32 SmoothingIndex = 0; SmoothingIndex < Array.GetCount(); ++SmoothingIndex)
					{
						if (Array[SmoothingIndex] != 0)
						{
							bValidSmoothingData = true;
							break;
						}
					}

					if (!bValidSmoothingData && Mesh->GetPolygonVertexCount() > 0)
					{
						SDKGeometryConverter->ComputeEdgeSmoothingFromNormals(Mesh);
						BaseLayer = Mesh->GetLayer(0);
						SmoothingInfo = BaseLayer->GetSmoothing();
					}
				}
			}
		}
	}

	//
	//	get the "material index" layer.  Do this AFTER the triangulation step as that may reorder material indices
	//
	FbxLayerElementMaterial* LayerElementMaterial = BaseLayer->GetMaterials();
	FbxLayerElement::EMappingMode MaterialMappingMode = LayerElementMaterial ?
		LayerElementMaterial->GetMappingMode() : FbxLayerElement::eByPolygon;

	//	todo second phase UV, ok to put in first phase?
	FBXUVs.Phase2(Mesh);

	//
	// get the first vertex color layer
	//
	FbxLayerElementVertexColor* LayerElementVertexColor = BaseLayer->GetVertexColors();
	FbxLayerElement::EReferenceMode VertexColorReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode VertexColorMappingMode(FbxLayerElement::eByControlPoint);
	if (LayerElementVertexColor)
	{
		VertexColorReferenceMode = LayerElementVertexColor->GetReferenceMode();
		VertexColorMappingMode = LayerElementVertexColor->GetMappingMode();
	}

	//
	// get the first normal layer
	//
	FbxLayerElementNormal* LayerElementNormal = BaseLayer->GetNormals();
	FbxLayerElementTangent* LayerElementTangent = BaseLayer->GetTangents();
	FbxLayerElementBinormal* LayerElementBinormal = BaseLayer->GetBinormals();

	//whether there is normal, tangent and binormal data in this mesh
	bool bHasNTBInformation = LayerElementNormal && LayerElementTangent && LayerElementBinormal;

	FbxLayerElement::EReferenceMode NormalReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode NormalMappingMode(FbxLayerElement::eByControlPoint);
	if (LayerElementNormal)
	{
		NormalReferenceMode = LayerElementNormal->GetReferenceMode();
		NormalMappingMode = LayerElementNormal->GetMappingMode();
	}

	FbxLayerElement::EReferenceMode TangentReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode TangentMappingMode(FbxLayerElement::eByControlPoint);
	if (LayerElementTangent)
	{
		TangentReferenceMode = LayerElementTangent->GetReferenceMode();
		TangentMappingMode = LayerElementTangent->GetMappingMode();
	}

	FbxLayerElement::EReferenceMode BinormalReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode BinormalMappingMode(FbxLayerElement::eByControlPoint);
	if (LayerElementBinormal)
	{
		BinormalReferenceMode = LayerElementBinormal->GetReferenceMode();
		BinormalMappingMode = LayerElementBinormal->GetMappingMode();
	}

	bool bHasNonDegeneratePolygons = false;
	{
		MeshDescription->SuspendVertexInstanceIndexing();
		MeshDescription->SuspendEdgeIndexing();
		MeshDescription->SuspendPolygonIndexing();
		MeshDescription->SuspendPolygonGroupIndexing();
		MeshDescription->SuspendUVIndexing();

		TRACE_CPUPROFILER_EVENT_SCOPE(Interchange_ImportMeshDescription);

		// Construct the matrices for the conversion from right handed to left handed system
		FbxAMatrix TotalMatrix = FFbxConvert::ConvertMatrix(MeshGlobalTransform.ToMatrixWithScale());
		FbxAMatrix TotalMatrixForNormal;
		TotalMatrixForNormal = TotalMatrix.Inverse();
		TotalMatrixForNormal = TotalMatrixForNormal.Transpose();
		int32 PolygonCount = Mesh->GetPolygonCount();

		if (PolygonCount == 0)
		{
			UInterchangeResultMeshError_Generic* Message = AddMessage<UInterchangeResultMeshError_Generic>(Mesh);
			Message->Text = LOCTEXT("NoPolygons", "No polygons were found in the mesh '{MeshName}'.");

			return false;
		}

		int32 VertexCount = Mesh->GetControlPointsCount();
		//Todo implement the odd negative scale, see legacy fbx 
		bool OddNegativeScale = IsOddNegativeScale(TotalMatrix);

		TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
		TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
		TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
		TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
		TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
		TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
		TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();
		TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
		TPolygonAttributesRef<FName> PolygonObjectNames = Attributes.GetPolygonObjectNames();

		int32 VertexOffset = MeshDescription->Vertices().Num();
		int32 VertexInstanceOffset = MeshDescription->VertexInstances().Num();
		int32 PolygonOffset = MeshDescription->Polygons().Num();

		// The below code expects Num() to be equivalent to GetArraySize(), i.e. that all added elements are appended, not inserted into existing gaps
		check(VertexOffset == MeshDescription->Vertices().GetArraySize());
		check(VertexInstanceOffset == MeshDescription->VertexInstances().GetArraySize());
		check(PolygonOffset == MeshDescription->Polygons().GetArraySize());

		TMap<int32, FPolygonGroupID> PolygonGroupMapping;

		//Ensure the polygon groups are create in the same order has the imported materials order
		for (int32 FbxMeshMaterialIndex = 0; FbxMeshMaterialIndex < MaterialNames.Num(); ++FbxMeshMaterialIndex)
		{
			FName MaterialName = MaterialNames[FbxMeshMaterialIndex];
			if (!PolygonGroupMapping.Contains(FbxMeshMaterialIndex))
			{
				FPolygonGroupID ExistingPolygonGroup = INDEX_NONE;
				for (const FPolygonGroupID PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
				{
					if (PolygonGroupImportedMaterialSlotNames[PolygonGroupID] == MaterialName)
					{
						ExistingPolygonGroup = PolygonGroupID;
						break;
					}
				}
				if (ExistingPolygonGroup == INDEX_NONE)
				{
					ExistingPolygonGroup = MeshDescription->CreatePolygonGroup();
					PolygonGroupImportedMaterialSlotNames[ExistingPolygonGroup] = MaterialName;
				}
				PolygonGroupMapping.Add(FbxMeshMaterialIndex, ExistingPolygonGroup);
			}
		}

		// When importing multiple mesh pieces to the same static mesh.  Ensure each mesh piece has the same number of Uv's
		int32 ExistingUVCount = VertexInstanceUVs.GetNumChannels();

		int32 NumUVs = FMath::Max(FBXUVs.UniqueUVCount, ExistingUVCount);
		NumUVs = FMath::Min<int32>(MAX_MESH_TEXTURE_COORDS_MD, NumUVs);
		// At least one UV set must exist.  
		// @todo: this needn't be true; we should be able to handle zero UV channels
		NumUVs = FMath::Max(1, NumUVs);

		//Make sure all Vertex instance have the correct number of UVs
		VertexInstanceUVs.SetNumChannels(NumUVs);
		MeshDescription->SetNumUVChannels(NumUVs);

		TArray<int32> UVOffsets;
		UVOffsets.SetNumUninitialized(NumUVs);
		for (int UVChannel = 0; UVChannel < NumUVs; UVChannel++)
		{
			UVOffsets[UVChannel] = MeshDescription->UVs(UVChannel).GetArraySize();
		}

		//Fill the vertex array
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Interchange_ImportVertices);
			MeshDescription->ReserveNewVertices(VertexCount);
			for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
			{
				int32 RealVertexIndex = VertexOffset + VertexIndex;
				FbxVector4 FbxPosition = Mesh->GetControlPoints()[VertexIndex];
				FbxPosition = TotalMatrix.MultT(FbxPosition);
				const FVector3f VertexPosition = FFbxConvert::ConvertPos<FVector3f>(FbxPosition);

				FVertexID AddedVertexId = MeshDescription->CreateVertex();
				VertexPositions[AddedVertexId] = VertexPosition;
				if (AddedVertexId.GetValue() != RealVertexIndex)
				{
					UInterchangeResultMeshError_Generic* Message = AddMessage<UInterchangeResultMeshError_Generic>(Mesh);
					Message->Text = LOCTEXT("CannotCreateVertex", "Cannot create valid vertex for mesh '{MeshName}'.");

					return false;
				}
			}
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Interchange_ImportUVs);
			// Fill the UV arrays
			for (int32 UVLayerIndex = 0; UVLayerIndex < FBXUVs.UniqueUVCount; UVLayerIndex++)
			{
				check(FBXUVs.LayerElementUV[UVLayerIndex]);
				if (FBXUVs.LayerElementUV[UVLayerIndex] != nullptr)
				{
					int32 UVCount = FBXUVs.LayerElementUV[UVLayerIndex]->GetDirectArray().GetCount();
					if (UVCount == 0 && !GIsAutomationTesting)
					{
						UInterchangeResultMeshWarning_Generic* Message = AddMessage<UInterchangeResultMeshWarning_Generic>(Mesh);
						Message->Text = LOCTEXT("CreateUVs_UVCorrupted", "Found invalid UVs value when importing mesh '{MeshName}'.");
					}

					TUVAttributesRef<FVector2f> UVCoordinates = MeshDescription->UVAttributes(UVLayerIndex).GetAttributesRef<FVector2f>(MeshAttribute::UV::UVCoordinate);
					MeshDescription->ReserveNewUVs(UVCount, UVLayerIndex);
					for (int32 UVIndex = 0; UVIndex < UVCount; UVIndex++)
					{
						FUVID UVID = MeshDescription->CreateUV(UVLayerIndex);
						FbxVector2 UVVector = FBXUVs.LayerElementUV[UVLayerIndex]->GetDirectArray().GetAt(UVIndex);
						UVCoordinates[UVID] = FVector2f(static_cast<float>(UVVector[0]), 1.0f - static_cast<float>(UVVector[1]));	// flip the Y of UVs for DirectX
					}
				}
			}
		}

		TMap<uint64, int32> RemapEdgeID;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Interchange_ImportEdgeVertices);
			Mesh->BeginGetMeshEdgeVertices();

			//Fill the edge array
			int32 FbxEdgeCount = Mesh->GetMeshEdgeCount();
			RemapEdgeID.Reserve(FbxEdgeCount * 2);
			for (int32 FbxEdgeIndex = 0; FbxEdgeIndex < FbxEdgeCount; ++FbxEdgeIndex)
			{
				int32 EdgeStartVertexIndex = -1;
				int32 EdgeEndVertexIndex = -1;
				Mesh->GetMeshEdgeVertices(FbxEdgeIndex, EdgeStartVertexIndex, EdgeEndVertexIndex);
				// Skip invalid edges, i.e. one of the ends is invalid, or degenerated ones
				if (EdgeStartVertexIndex == -1 || EdgeEndVertexIndex == -1 || EdgeStartVertexIndex == EdgeEndVertexIndex)
				{
					UInterchangeResultMeshWarning_Generic* Message = AddMessage<UInterchangeResultMeshWarning_Generic>(Mesh);
					Message->Text = LOCTEXT("SkippingEdge", "Skipping invalid edge on mesh '{MeshName}'.");
					continue;
				}
				FVertexID EdgeVertexStart(EdgeStartVertexIndex + VertexOffset);
				check(MeshDescription->Vertices().IsValid(EdgeVertexStart));
				FVertexID EdgeVertexEnd(EdgeEndVertexIndex + VertexOffset);
				check(MeshDescription->Vertices().IsValid(EdgeVertexEnd));
				uint64 CompactedKey = (((uint64)EdgeVertexStart.GetValue()) << 32) | ((uint64)EdgeVertexEnd.GetValue());
				RemapEdgeID.Add(CompactedKey, FbxEdgeIndex);
				//Add the other edge side
				CompactedKey = (((uint64)EdgeVertexEnd.GetValue()) << 32) | ((uint64)EdgeVertexStart.GetValue());
				RemapEdgeID.Add(CompactedKey, FbxEdgeIndex);
			}
			//Call this after all GetMeshEdgeIndexForPolygon call this is for optimization purpose.
			Mesh->EndGetMeshEdgeVertices();
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Interchange_ImportPolygons);

			// Compute and reserve memory to be used for vertex instances
			{
				int32 TotalVertexCount = 0;
				for (int32 PolygonIndex = 0; PolygonIndex < PolygonCount; PolygonIndex++)
				{
					TotalVertexCount += Mesh->GetPolygonSize(PolygonIndex);
				}

				MeshDescription->ReserveNewPolygons(PolygonCount);
				MeshDescription->ReserveNewVertexInstances(TotalVertexCount);
				MeshDescription->ReserveNewEdges(TotalVertexCount);
			}

			bool  bBeginGetMeshEdgeIndexForPolygonCalled = false;
			bool  bBeginGetMeshEdgeIndexForPolygonRequired = true;
			int32 CurrentVertexInstanceIndex = 0;

			// keep those for all iterations to avoid heap allocations
			TArray<FVertexInstanceID> CornerInstanceIDs;
			TArray<FVertexID> CornerVerticesIDs;
			TArray<FVector3f, TInlineAllocator<3>> P;

			bool bUnsupportedSmoothingGroupErrorDisplayed = false;
			bool bCorruptedMsgDone = false;
			//Polygons
			for (int32 PolygonIndex = 0; PolygonIndex < PolygonCount; PolygonIndex++)
			{
				int32 PolygonVertexCount = Mesh->GetPolygonSize(PolygonIndex);

				int32 RealPolygonIndex = PolygonOffset + PolygonIndex;
				CornerInstanceIDs.Reset();
				CornerInstanceIDs.AddUninitialized(PolygonVertexCount);
				CornerVerticesIDs.Reset();
				CornerVerticesIDs.AddUninitialized(PolygonVertexCount);
				for (int32 CornerIndex = 0; CornerIndex < PolygonVertexCount; CornerIndex++)
				{
					int32 VertexInstanceIndex = VertexInstanceOffset + CurrentVertexInstanceIndex + CornerIndex;
					int32 RealFbxVertexIndex = CurrentVertexInstanceIndex + CornerIndex;
					const FVertexInstanceID VertexInstanceID(VertexInstanceIndex);
					CornerInstanceIDs[CornerIndex] = VertexInstanceID;
					const int32 ControlPointIndex = Mesh->GetPolygonVertex(PolygonIndex, CornerIndex);
					const FVertexID VertexID(VertexOffset + ControlPointIndex);
					CornerVerticesIDs[CornerIndex] = VertexID;

					FVertexInstanceID AddedVertexInstanceId = MeshDescription->CreateVertexInstance(VertexID);

					//Make sure the Added vertex instance ID is matching the expected vertex instance ID
					check(AddedVertexInstanceId == VertexInstanceID);

					if (AddedVertexInstanceId.GetValue() != VertexInstanceIndex)
					{
						UInterchangeResultMeshError_Generic* Message = AddMessage<UInterchangeResultMeshError_Generic>(Mesh);
						Message->Text = LOCTEXT("CannotCreateVertexInstance", "Cannot create valid vertex instance for the mesh '{MeshName}'.");

						return false;
					}

					//UVs attributes
					for (int32 UVLayerIndex = 0; UVLayerIndex < FBXUVs.UniqueUVCount; UVLayerIndex++)
					{
						FVector2D FinalUVVector(0.0f, 0.0f);
						if (FBXUVs.LayerElementUV[UVLayerIndex] != NULL)
						{
							int UVMapIndex = (FBXUVs.UVMappingMode[UVLayerIndex] == FbxLayerElement::eByControlPoint) ? ControlPointIndex : RealFbxVertexIndex;
							int32 UVIndex = (FBXUVs.UVReferenceMode[UVLayerIndex] == FbxLayerElement::eDirect) ?
								UVMapIndex : FBXUVs.LayerElementUV[UVLayerIndex]->GetIndexArray().GetAt(UVMapIndex);

							FbxVector2	UVVector = FBXUVs.LayerElementUV[UVLayerIndex]->GetDirectArray().GetAt(UVIndex);
							FinalUVVector.X = static_cast<float>(UVVector[0]);
							FinalUVVector.Y = 1.f - static_cast<float>(UVVector[1]);   //flip the Y of UVs for DirectX
						}
						VertexInstanceUVs.Set(AddedVertexInstanceId, UVLayerIndex, FVector2f(FinalUVVector));	// LWC_TODO: Precision loss
					}

					//Color attribute
					if (LayerElementVertexColor)
					{
						int32 VertexColorMappingIndex = (VertexColorMappingMode == FbxLayerElement::eByControlPoint) ?
							Mesh->GetPolygonVertex(PolygonIndex, CornerIndex) : (RealFbxVertexIndex);

						int32 VectorColorIndex = (VertexColorReferenceMode == FbxLayerElement::eDirect) ? //-V516 PVS 7.41 bug
							VertexColorMappingIndex : LayerElementVertexColor->GetIndexArray().GetAt(VertexColorMappingIndex);

						FbxColor VertexColor = LayerElementVertexColor->GetDirectArray().GetAt(VectorColorIndex);

						// Clamp manually to avoid wrapping around because of truncation for values outside of the 0.0 to 1.0 range
						FColor VertexInstanceColor(
							uint8(255.f * FMath::Clamp(VertexColor.mRed, 0.0, 1.0)),
							uint8(255.f * FMath::Clamp(VertexColor.mGreen, 0.0, 1.0)),
							uint8(255.f * FMath::Clamp(VertexColor.mBlue,0.0, 1.0)),
							uint8(255.f * FMath::Clamp(VertexColor.mAlpha, 0.0, 1.0))
						);
						VertexInstanceColors[AddedVertexInstanceId] = FVector4f(FLinearColor(VertexInstanceColor));
					}

					if (LayerElementNormal)
					{
						//normals may have different reference and mapping mode than tangents and binormals
						int NormalMapIndex = (NormalMappingMode == FbxLayerElement::eByControlPoint) ?
							ControlPointIndex : RealFbxVertexIndex;
						int NormalValueIndex = (NormalReferenceMode == FbxLayerElement::eDirect) ? //-V516 PVS 7.41 bug
							NormalMapIndex : LayerElementNormal->GetIndexArray().GetAt(NormalMapIndex);

						FbxVector4 TempValue = LayerElementNormal->GetDirectArray().GetAt(NormalValueIndex);
						TempValue = TotalMatrixForNormal.MultT(TempValue);
						FVector3f TangentZ = FFbxConvert::ConvertDir<FVector3f>(TempValue);
						VertexInstanceNormals[AddedVertexInstanceId] = TangentZ.GetSafeNormal();
						//tangents and binormals share the same reference, mapping mode and index array
						if (bHasNTBInformation)
						{
							int TangentMapIndex = (TangentMappingMode == FbxLayerElement::eByControlPoint) ?
								ControlPointIndex : RealFbxVertexIndex;
							int TangentValueIndex = (TangentReferenceMode == FbxLayerElement::eDirect) ? //-V516 PVS 7.41 bug
								TangentMapIndex : LayerElementTangent->GetIndexArray().GetAt(TangentMapIndex);

							TempValue = LayerElementTangent->GetDirectArray().GetAt(TangentValueIndex);
							TempValue = TotalMatrixForNormal.MultT(TempValue);
							FVector3f TangentX = FFbxConvert::ConvertDir<FVector3f>(TempValue);
							VertexInstanceTangents[AddedVertexInstanceId] = TangentX.GetSafeNormal();

							int BinormalMapIndex = (BinormalMappingMode == FbxLayerElement::eByControlPoint) ?
								ControlPointIndex : RealFbxVertexIndex;
							int BinormalValueIndex = (BinormalReferenceMode == FbxLayerElement::eDirect) ? //-V516 PVS 7.41 bug
								BinormalMapIndex : LayerElementBinormal->GetIndexArray().GetAt(BinormalMapIndex);

							TempValue = LayerElementBinormal->GetDirectArray().GetAt(BinormalValueIndex);
							TempValue = TotalMatrixForNormal.MultT(TempValue);
							FVector3f TangentY = -FFbxConvert::ConvertDir<FVector3f>(TempValue);
							VertexInstanceBinormalSigns[AddedVertexInstanceId] = FbxGetBasisDeterminantSign(TangentX.GetSafeNormal(), TangentY.GetSafeNormal(), TangentZ.GetSafeNormal());
						}
					}
				}

				// Check if the polygon just discovered is non-degenerate if we haven't found one yet
				//TODO check all polygon vertex, not just the first 3 vertex
				if (!bHasNonDegeneratePolygons)
				{
					float TriangleComparisonThreshold = THRESH_POINTS_ARE_SAME;
					FVector3f VertexPosition[3];
					VertexPosition[0] = VertexPositions[CornerVerticesIDs[0]];
					VertexPosition[1] = VertexPositions[CornerVerticesIDs[1]];
					VertexPosition[2] = VertexPositions[CornerVerticesIDs[2]];
					if (!(VertexPosition[0].Equals(VertexPosition[1], TriangleComparisonThreshold)
						|| VertexPosition[0].Equals(VertexPosition[2], TriangleComparisonThreshold)
						|| VertexPosition[1].Equals(VertexPosition[2], TriangleComparisonThreshold)))
					{
						bHasNonDegeneratePolygons = true;
					}
				}

				//
				// material index
				//
				int32 MaterialIndex = 0;
				if (MaterialCount > 0)
				{
					if (LayerElementMaterial)
					{
						int32 MaterialRemapIndex = 0;
						switch (MaterialMappingMode)
						{
							// material index is stored in the IndexArray, not the DirectArray (which is irrelevant with 2009.1)
							case FbxLayerElement::eAllSame:
							{
								MaterialRemapIndex = LayerElementMaterial->GetIndexArray().GetAt(0);
							}
							break;
							case FbxLayerElement::eByPolygon:
							{
								MaterialRemapIndex = LayerElementMaterial->GetIndexArray().GetAt(PolygonIndex);
							}
							break;
						}

						// Follows same behaviour as Legacy Import, i.e. forces index to zero if negative.
						MaterialIndex = 0;
						if (MaterialRemap.IsValidIndex(MaterialRemapIndex))
						{
							MaterialIndex = MaterialRemap[MaterialRemapIndex];
						}
					}
				}

				if (MaterialIndex < 0)
				{
					//TODO log an error message
					MaterialIndex = 0;
				}
				//Add missing material name with generated name
				while (MaterialIndex >= MaterialNames.Num())
				{
					FString MaterialName = TEXT("Material_") + FString::FromInt(MaterialNames.Num());
					MaterialNames.Add((*MaterialName));
				}

				//Material name should always exist since we create up to index materials
				check(MaterialNames.IsValidIndex(MaterialIndex));
				{
					//Create a polygon with the 3 vertex instances Add it to the material group
					FName MaterialName = MaterialNames[MaterialIndex];
					if (!PolygonGroupMapping.Contains(MaterialIndex))
					{
						FPolygonGroupID ExistingPolygonGroup = INDEX_NONE;
						for (const FPolygonGroupID PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
						{
							if (PolygonGroupImportedMaterialSlotNames[PolygonGroupID] == MaterialName)
							{
								ExistingPolygonGroup = PolygonGroupID;
								break;
							}
						}
						if (ExistingPolygonGroup == INDEX_NONE)
						{
							ExistingPolygonGroup = MeshDescription->CreatePolygonGroup();
							PolygonGroupImportedMaterialSlotNames[ExistingPolygonGroup] = MaterialName;
						}
						PolygonGroupMapping.Add(MaterialIndex, ExistingPolygonGroup);
					}
				}

				// Create polygon edges
				{
					// Add the edges of this polygon
					for (uint32 PolygonEdgeNumber = 0; PolygonEdgeNumber < (uint32)PolygonVertexCount; ++PolygonEdgeNumber)
					{
						//Find the matching edge ID
						uint32 CornerIndices[2];
						CornerIndices[0] = (PolygonEdgeNumber + 0) % PolygonVertexCount;
						CornerIndices[1] = (PolygonEdgeNumber + 1) % PolygonVertexCount;

						FVertexID EdgeVertexIDs[2];
						EdgeVertexIDs[0] = CornerVerticesIDs[CornerIndices[0]];
						EdgeVertexIDs[1] = CornerVerticesIDs[CornerIndices[1]];

						FEdgeID MatchEdgeId = MeshDescription->GetVertexPairEdge(EdgeVertexIDs[0], EdgeVertexIDs[1]);
						if (MatchEdgeId == INDEX_NONE)
						{
							MatchEdgeId = MeshDescription->CreateEdge(EdgeVertexIDs[0], EdgeVertexIDs[1]);
						}

						//RawMesh do not have edges, so by ordering the edge with the triangle construction we can ensure back and forth conversion with RawMesh
						//When raw mesh will be completely remove we can create the edges right after the vertex creation.
						int32 EdgeIndex = INDEX_NONE;
						uint64 CompactedKey = (((uint64)EdgeVertexIDs[0].GetValue()) << 32) | ((uint64)EdgeVertexIDs[1].GetValue());
						if (RemapEdgeID.Contains(CompactedKey))
						{
							EdgeIndex = RemapEdgeID[CompactedKey];
						}
						else
						{
							// Call BeginGetMeshEdgeIndexForPolygon lazily only if we enter the code path calling GetMeshEdgeIndexForPolygon
							if (bBeginGetMeshEdgeIndexForPolygonRequired)
							{
								//Call this before all GetMeshEdgeIndexForPolygon for optimization purpose.
								//But do not spend time precomputing stuff if the mesh has no edge since 
								//GetMeshEdgeIndexForPolygon is going to always returns -1 anyway without going into the slow path.
								if (Mesh->GetMeshEdgeCount() > 0)
								{
									Mesh->BeginGetMeshEdgeIndexForPolygon();
									bBeginGetMeshEdgeIndexForPolygonCalled = true;
								}
								bBeginGetMeshEdgeIndexForPolygonRequired = false;
							}

							EdgeIndex = Mesh->GetMeshEdgeIndexForPolygon(PolygonIndex, PolygonEdgeNumber);
						}

						if (!EdgeHardnesses[MatchEdgeId])
						{
							if (bSmoothingAvailable && SmoothingInfo && SmoothingMappingMode == FbxLayerElement::eByEdge)
							{
								int32 lSmoothingIndex = (SmoothingReferenceMode == FbxLayerElement::eDirect) ? EdgeIndex : SmoothingInfo->GetIndexArray().GetAt(EdgeIndex); //-V516 PVS 7.41 bug
								//Set the hard edges
								int32 SmoothingFlag = SmoothingInfo->GetDirectArray().GetAt(lSmoothingIndex);
								EdgeHardnesses[MatchEdgeId] = (SmoothingFlag == 0);
							}
							else
							{
								if (!bUnsupportedSmoothingGroupErrorDisplayed && SmoothingMappingMode != FbxLayerElement::eByEdge)
								{
									bUnsupportedSmoothingGroupErrorDisplayed = true;
									UInterchangeResultMeshError_Generic* Message = AddMessage<UInterchangeResultMeshError_Generic>(Mesh);
									Message->Text = LOCTEXT("Error_UnsupportedSmoothingGroup", "Unsupported Smoothing group mapping mode on mesh '{MeshName}'.");
								}
								//When there is no smoothing group we set all edge to: hard (faceted mesh) for static mesh and smooth for skinned and rigid
								EdgeHardnesses[MatchEdgeId] = MeshType == EMeshType::Static ? !bStaticMeshUseSmoothEdgesIfSmoothingInformationIsMissing : false;
							}
						}
					}
				}

				FPolygonGroupID PolygonGroupID = PolygonGroupMapping[MaterialIndex];
				// Insert a triangle into the mesh
				// @note: we only ever import triangulated meshes currently. This could easily change as we have the infrastructure set up for arbitrary ngons.
				// However, I think either we need to triangulate in the exact same way as Maya/Max if we do that.
				check(CornerInstanceIDs.Num() == 3);
				check(PolygonVertexCount == 3);
				TArray<FEdgeID> NewEdgeIDs;
				const FTriangleID NewTriangleID = MeshDescription->CreateTriangle(PolygonGroupID, CornerInstanceIDs, &NewEdgeIDs);
				check(NewEdgeIDs.Num() == 0);

				// Set the name of the object that generated this triangle.
				PolygonObjectNames.Set(MeshDescription->GetTrianglePolygon(NewTriangleID), MeshNodeName);

				for (int32 UVLayerIndex = 0; UVLayerIndex < FBXUVs.UniqueUVCount; UVLayerIndex++)
				{
					FUVID UVIDs[3] = { INDEX_NONE, INDEX_NONE, INDEX_NONE };
					if (FBXUVs.LayerElementUV[UVLayerIndex] != NULL)
					{
						for (int32 VertexIndex = 0; VertexIndex < 3; ++VertexIndex)
						{
							int UVMapIndex = (FBXUVs.UVMappingMode[UVLayerIndex] == FbxLayerElement::eByControlPoint)
								? Mesh->GetPolygonVertex(PolygonIndex, VertexIndex)
								: CurrentVertexInstanceIndex + VertexIndex;
							int32 UVIndex = (FBXUVs.UVReferenceMode[UVLayerIndex] == FbxLayerElement::eDirect)
								? UVMapIndex
								: FBXUVs.LayerElementUV[UVLayerIndex]->GetIndexArray().GetAt(UVMapIndex);

							//When we have a valid UV channel but empty or not containing enough value, we do not
							//set any UVIDs value for this corner
							if (UVIndex >= FBXUVs.LayerElementUV[UVLayerIndex]->GetDirectArray().GetCount())
							{
								//The warning was already emit at this time that UVs are invalid for this mesh
								UVIndex = -1;
							}

							if (UVIndex != -1)
							{
								UVIDs[VertexIndex] = UVIndex + UVOffsets[UVLayerIndex];

								check(MeshDescription->VertexInstanceAttributes().GetAttribute<FVector2f>(CornerInstanceIDs[VertexIndex], MeshAttribute::VertexInstance::TextureCoordinate, UVLayerIndex) ==
										MeshDescription->UVAttributes(UVLayerIndex).GetAttribute<FVector2f>(UVIndex + UVOffsets[UVLayerIndex], MeshAttribute::UV::UVCoordinate));
							}
						}
					}

					MeshDescription->SetTriangleUVIndices(NewTriangleID, UVIDs, UVLayerIndex);
				}

				CurrentVertexInstanceIndex += PolygonVertexCount;
			}

			//Call this after all GetMeshEdgeIndexForPolygon call this is for optimization purpose.
			if (bBeginGetMeshEdgeIndexForPolygonCalled)
			{
				Mesh->EndGetMeshEdgeIndexForPolygon();
			}
		}

		if (MeshType == EMeshType::Skinned)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Interchange_ImportSkin);
			FSkeletalMeshAttributes SkeletalMeshAttributes(*MeshDescription);
			SkeletalMeshAttributes.Register(true);

			//Import vertex Attribute from all mesh layer vertex color
			GetVertexAttributeFromMeshVertexColor(*MeshDescription, Mesh);

			using namespace UE::AnimationCore;
			TMap<FVertexID, TArray<FBoneWeight>> RawBoneWeights;
						
			//Add the influence data in the skeletalmesh description
			FSkinWeightsVertexAttributesRef VertexSkinWeights = SkeletalMeshAttributes.GetVertexSkinWeights();

			FbxSkin* Skin = (FbxSkin*)Mesh->GetDeformer(0, FbxDeformer::eSkin);
			if (!ensure(Skin))
			{
				//TODO log an error
				return false;
			}
			const int32 ClusterCount = Skin->GetClusterCount();
			TArray<FbxNode*> SortedJoints;
			SortedJoints.Reserve(ClusterCount);
			OutJointUniqueNames.Reserve(ClusterCount);
			// create influences for each cluster
			for (int32 ClusterIndex = 0; ClusterIndex < ClusterCount; ClusterIndex++)
			{
				FbxCluster* Cluster = Skin->GetCluster(ClusterIndex);
				// When Maya plug-in exports rigid binding, it will generate "CompensationCluster" for each ancestor links.
				// FBX writes these "CompensationCluster" out. The CompensationCluster also has weight 1 for vertices.
				// Unreal importer should skip these clusters.
				if (!Cluster || (FCStringAnsi::Strcmp(Cluster->GetUserDataID(), "Maya_ClusterHint") == 0 && FCStringAnsi::Strcmp(Cluster->GetUserData(), "CompensationCluster") == 0))
				{
					continue;
				}

				FbxNode* Link = Cluster->GetLink();
				// find the bone index
				int32 BoneIndex = -1;
				if (!SortedJoints.Find(Link, BoneIndex))
				{
					BoneIndex = SortedJoints.Add(Link);
					FString JointNodeUniqueID = Parser.GetFbxHelper()->GetFbxObjectName(Link, true);
					OutJointUniqueNames.Add(JointNodeUniqueID);
				}

				//	get the vertex indices
				int32 ControlPointIndicesCount = Cluster->GetControlPointIndicesCount();
				int32* ControlPointIndices = Cluster->GetControlPointIndices();
				double* Weights = Cluster->GetControlPointWeights();

				//	for each vertex index in the cluster
				for (int32 ControlPointIndex = 0; ControlPointIndex < ControlPointIndicesCount; ++ControlPointIndex)
				{
					FVertexID VertexID(ControlPointIndices[ControlPointIndex] + VertexOffset);
					if (!ensure(MeshDescription->IsVertexValid(VertexID)))
					{
						//Invalid Influence
						continue;
					}
					float Weight = static_cast<float>(Weights[ControlPointIndex]);

					TArray<FBoneWeight> *BoneWeights = RawBoneWeights.Find(VertexID);
					if (BoneWeights)
					{
						// Do we already have a weight for this bone? Keep the greater weight.
						bool bShouldAdd = true;
						for (int32 WeightIndex = 0; WeightIndex < BoneWeights->Num(); WeightIndex++)
						{
							FBoneWeight &BoneWeight = (*BoneWeights)[WeightIndex];
							if (BoneWeight.GetBoneIndex() == BoneIndex)
							{
								if (BoneWeight.GetWeight() < Weight)
								{
									BoneWeight.SetWeight(Weight);
								}
								bShouldAdd = false;
								break;
							}
						}
						if (bShouldAdd)
						{
							BoneWeights->Add(FBoneWeight(BoneIndex, Weight));
						}
					}
					else
					{
						RawBoneWeights.Add(VertexID).Add(FBoneWeight(BoneIndex, Weight));
					}
				}
			}

			// Add all the raw bone weights. This will cause the weights to be sorted and re-normalized after culling to max influences.
			for (const TTuple<FVertexID, TArray<FBoneWeight>> &Item: RawBoneWeights)
			{
				VertexSkinWeights.Set(Item.Key, Item.Value);
			}
		}

		MeshDescription->ResumeVertexInstanceIndexing();
		MeshDescription->ResumeEdgeIndexing();
		MeshDescription->ResumePolygonIndexing();
		MeshDescription->ResumePolygonGroupIndexing();
		MeshDescription->ResumeUVIndexing();
	}

	TArray<FPolygonGroupID> EmptyPolygonGroups;
	for (const FPolygonGroupID PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
	{
		if (MeshDescription->GetNumPolygonGroupTriangles(PolygonGroupID) == 0)
		{
			EmptyPolygonGroups.Add(PolygonGroupID);
		}
	}
	if (EmptyPolygonGroups.Num() > 0)
	{
		for (const FPolygonGroupID PolygonGroupID : EmptyPolygonGroups)
		{
			MeshDescription->DeletePolygonGroup(PolygonGroupID);
		}
		FElementIDRemappings OutRemappings;
		MeshDescription->Compact(OutRemappings);
	}
	
	// needed?
	FBXUVs.Cleanup();

	if (!bHasNonDegeneratePolygons)
	{
		//TODO log an error we have imported no valid triangle in this fbx mesh node
		//CreateTokenizedErrorForDegeneratedPart(this, StaticMesh->GetName(), FbxNodeName);
	}

	return bHasNonDegeneratePolygons;
}

bool FMeshDescriptionImporter::IsOddNegativeScale(FbxAMatrix& TotalMatrix)
{
	FbxVector4 Scale = TotalMatrix.GetS();
	int32 NegativeNum = 0;

	if (Scale[0] < 0) NegativeNum++;
	if (Scale[1] < 0) NegativeNum++;
	if (Scale[2] < 0) NegativeNum++;

	return NegativeNum == 1 || NegativeNum == 3;
}


//////////////////////////////////////////////////////////////////////////
/// FMeshPayloadContext implementation

bool FMeshPayloadContext::FetchMeshPayloadInternal(FFbxParser& Parser
	, const FTransform& MeshGlobalTransform
	, FMeshDescription& OutMeshDescription
	, TArray<FString>& OutJointNames)
{
	if (!ensure(SDKScene != nullptr))
	{
		UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
		Message->InterchangeKey = Parser.GetFbxHelper()->GetMeshUniqueID(Mesh);
		Message->Text = LOCTEXT("FetchMeshPayloadInternal_FBXSceneNull_Mesh", "Cannot fetch FBX mesh payload because the FBX scene is null.");
		return false;
	}

	if (!ensure(Mesh != nullptr))
	{
		UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
		Message->InterchangeKey = Parser.GetFbxHelper()->GetMeshUniqueID(Mesh);
		Message->Text = LOCTEXT("FetchMeshPayloadInternal_FBXMeshNull", "Cannot fetch FBX mesh payload because the FBX mesh is null.");
		return false;
	}

	if (!ensure(SDKGeometryConverter != nullptr))
	{
		UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
		Message->InterchangeKey = Parser.GetFbxHelper()->GetMeshUniqueID(Mesh);
		Message->Text = LOCTEXT("FetchMeshPayloadInternal_FBXConverterNull", "Cannot fetch FBX mesh payload because the FBX geometry converter is null.");
		return false;
	}

	if (bIsSkinnedMesh)
	{
		FSkeletalMeshAttributes SkeletalMeshAttribute(OutMeshDescription);
		SkeletalMeshAttribute.Register();
		FMeshDescriptionImporter MeshDescriptionImporter(Parser, &OutMeshDescription, SDKScene, SDKGeometryConverter);
		if (!MeshDescriptionImporter.FillSkinnedMeshDescriptionFromFbxMesh(Mesh, MeshGlobalTransform, OutJointNames))
		{
			UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
			Message->InterchangeKey = Parser.GetFbxHelper()->GetMeshUniqueID(Mesh);
			Message->Text = LOCTEXT("SkinnedMeshDescriptionError", "Cannot fetch skinned mesh payload because there was an error when creating the MeshDescription.");
			return false;
		}
	}
	else
	{
		FStaticMeshAttributes StaticMeshAttribute(OutMeshDescription);
		StaticMeshAttribute.Register();
		FMeshDescriptionImporter MeshDescriptionImporter(Parser, &OutMeshDescription, SDKScene, SDKGeometryConverter);
		if (!MeshDescriptionImporter.FillStaticMeshDescriptionFromFbxMesh(Mesh, MeshGlobalTransform))
		{
			UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
			Message->InterchangeKey = Parser.GetFbxHelper()->GetMeshUniqueID(Mesh);
			Message->Text = LOCTEXT("StaticMeshDescriptionError", "Cannot fetch static mesh payload because there was an error when creating the MeshDescription.");
			return false;
		}
	}
	return true;
}

bool FMeshPayloadContext::FetchMeshPayloadToFile(FFbxParser& Parser, const FTransform& MeshGlobalTransform, const FString& PayloadFilepath)
{
	FMeshDescription MeshDescription;
	TArray<FString> JointNames;
	if (!FetchMeshPayloadInternal(Parser, MeshGlobalTransform, MeshDescription, JointNames))
	{
		return false;
	}

	//Dump the MeshDescription to a file
	{
		FLargeMemoryWriter Ar;
		MeshDescription.Serialize(Ar);

		Ar << bIsSkinnedMesh;
		if (bIsSkinnedMesh)
		{
			//When passing a skinned MeshDescription, We want to pass the joint Node ID so we can know what the influence bone index refer to
			Ar << JointNames;
		}
		uint8* ArchiveData = Ar.GetData();
		int64 ArchiveSize = Ar.TotalSize();
		TArray64<uint8> Buffer(ArchiveData, ArchiveSize);
		FFileHelper::SaveArrayToFile(Buffer, *PayloadFilepath);
	}

	return true;
}
#if WITH_ENGINE
bool FMeshPayloadContext::FetchMeshPayload(FFbxParser& Parser, const FTransform& MeshGlobalTransform, FMeshPayloadData& OutMeshPayloadData)
{
	if (FetchMeshPayloadInternal(Parser, MeshGlobalTransform, OutMeshPayloadData.MeshDescription, OutMeshPayloadData.JointNames))
	{
		return true;
	}
	return false;
}
#endif
//////////////////////////////////////////////////////////////////////////
/// FMorphTargetPayloadContext implementation

bool FMorphTargetPayloadContext::FetchMeshPayloadInternal(FFbxParser& Parser
	, const FTransform& MeshGlobalTransform
	, FMeshDescription& OutMorphTargetMeshDescription)
{
	if (!ensure(SDKScene != nullptr))
	{
		UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
		Message->InterchangeKey = Parser.GetFbxHelper()->GetMeshUniqueID(Shape);
		Message->Text = LOCTEXT("FetchMorphTargetMeshPayloadInternal_FBXSceneNull_Mesh", "Cannot fetch FBX mesh morph shape payload because the FBX scene is null.");
		return false;
	}

	if (!ensure(SDKGeometryConverter != nullptr))
	{
		UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
		Message->InterchangeKey = Parser.GetFbxHelper()->GetMeshUniqueID(Shape);
		Message->Text = LOCTEXT("FetchMorphTargetMeshPayloadInternal_FBXConverterNull", "Cannot fetch FBX mesh morph shape payload because the FBX geometry converter is null.");
		return false;
	}

	if (!ensure(Shape))
	{
		UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
		Message->InterchangeKey = Parser.GetFbxHelper()->GetMeshUniqueID(Shape);
		Message->Text = LOCTEXT("FetchMorphTargetMeshPayloadInternal_FBXMeshNull", "Cannot fetch FBX mesh morph shape payload because the FBX shape is null.");
		return false;
	}

	//Import the MorphTarget
	FStaticMeshAttributes StaticMeshAttribute(OutMorphTargetMeshDescription);
	StaticMeshAttribute.Register();
	FMeshDescriptionImporter MeshDescriptionImporter(Parser, &OutMorphTargetMeshDescription, SDKScene, SDKGeometryConverter);
	if (!MeshDescriptionImporter.FillMeshDescriptionFromFbxShape(Shape, MeshGlobalTransform))
	{
		UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
		Message->InterchangeKey = Parser.GetFbxHelper()->GetMeshUniqueID(Shape);
		Message->Text = LOCTEXT("MeshDescriptionMorphTargetError", "Unable to create MeshDescription from FBX morph target.");

		return false;
	}
	return true;
}

bool FMorphTargetPayloadContext::FetchMeshPayloadToFile(FFbxParser& Parser, const FTransform& MeshGlobalTransform, const FString& PayloadFilepath)
{
	FMeshDescription MorphTargetMeshDescription;
	if(!FetchMeshPayloadInternal(Parser, MeshGlobalTransform, MorphTargetMeshDescription))
	{
		return false;
	}
	//Dump the MeshDescription to a file
	{
		FLargeMemoryWriter Ar;
		MorphTargetMeshDescription.Serialize(Ar);
		uint8* ArchiveData = Ar.GetData();
		int64 ArchiveSize = Ar.TotalSize();
		TArray64<uint8> Buffer(ArchiveData, ArchiveSize);
		FFileHelper::SaveArrayToFile(Buffer, *PayloadFilepath);
	}
	return true;
}

#if WITH_ENGINE
bool FMorphTargetPayloadContext::FetchMeshPayload(FFbxParser& Parser, const FTransform& MeshGlobalTransform, FMeshPayloadData& OutMeshPayloadData)
{
	return FetchMeshPayloadInternal(Parser, MeshGlobalTransform, OutMeshPayloadData.MeshDescription);
}
#endif
//////////////////////////////////////////////////////////////////////////
/// FFbxMesh implementation

void GetMaterialIndex(FbxMesh* Mesh, TArray<int32>& MaterialIndexes)
{
	int32 PolygonCount = Mesh->GetPolygonCount();
	if (FbxGeometryElementMaterial* GeometryElementMaterial = Mesh->GetElementMaterial())
	{
		FbxLayerElementArrayTemplate<int32>& IndexArray = GeometryElementMaterial->GetIndexArray();
		MaterialIndexes.Reset();
		switch (GeometryElementMaterial->GetMappingMode())
		{
			case FbxGeometryElement::eByPolygon:
			{
				if (IndexArray.GetCount() == PolygonCount)
				{
					for (int i = 0; i < PolygonCount; ++i)
					{
						MaterialIndexes.AddUnique(IndexArray.GetAt(i));
					}
				}
			}
			break;
			case FbxGeometryElement::eAllSame:
			{
				MaterialIndexes.AddUnique(IndexArray.GetAt(0));
			}
			break;
		}
	}
}

void FFbxMesh::AddAllMeshes(FbxScene* SDKScene, FbxGeometryConverter* SDKGeometryConverter, UInterchangeBaseNodeContainer& NodeContainer, TMap<FString, TSharedPtr<FPayloadContextBase>>& PayloadContexts)
{
	int32 GeometryCount = SDKScene->GetGeometryCount();
	//Triangulate meshes
	{
		TArray<FbxMesh*> ToTriangulateMeshes;
		ToTriangulateMeshes.Reserve(GeometryCount);
		for (int32 GeometryIndex = 0; GeometryIndex < GeometryCount; ++GeometryIndex)
		{
			FbxGeometry* Geometry = SDKScene->GetGeometry(GeometryIndex);
			if (Geometry->GetAttributeType() != FbxNodeAttribute::eMesh)
			{
				continue;
			}
			FbxMesh* Mesh = static_cast<FbxMesh*>(Geometry);
			if (!Mesh)
			{
				continue;
			}
			if (!Mesh->IsTriangleMesh())
			{
				ToTriangulateMeshes.Add(Mesh);
			}
		}
		for (FbxMesh* ToTriangulateMesh : ToTriangulateMeshes)
		{
			// Must do this before triangulating the mesh due to an FBX bug in Triangulate. Edge hardnees triangulation give wrong edge hardness so we compute them to smooth group during the triangulation.
			int32 LayerSmoothingCount = ToTriangulateMesh->GetLayerCount(FbxLayerElement::eSmoothing);
			for (int32 SmoothingLayerIndex = 0; SmoothingLayerIndex < LayerSmoothingCount; SmoothingLayerIndex++)
			{
				FbxLayerElementSmoothing const* SmoothingInfoTmp = ToTriangulateMesh->GetLayer(SmoothingLayerIndex)->GetSmoothing();
				if (SmoothingInfoTmp && SmoothingInfoTmp->GetMappingMode() != FbxLayerElement::eByPolygon)
				{
					SDKGeometryConverter->ComputePolygonSmoothingFromEdgeSmoothing(ToTriangulateMesh, SmoothingLayerIndex);
				}
			}
			const bool bReplace = true;
			SDKGeometryConverter->Triangulate(ToTriangulateMesh, bReplace);
		}
	}
	//Now requery the triangulated geometries
	GeometryCount = SDKScene->GetGeometryCount();
	for (int32 GeometryIndex = 0; GeometryIndex < GeometryCount; ++GeometryIndex)
	{
		FbxGeometry* Geometry = SDKScene->GetGeometry(GeometryIndex);
		if (Geometry->GetAttributeType() != FbxNodeAttribute::eMesh)
		{
			continue;
		}
		FbxMesh* Mesh = static_cast<FbxMesh*>(Geometry);
		if (!Mesh)
		{
			continue;
		}

		FString MeshName = Parser.GetFbxHelper()->GetMeshName(Mesh);
		FString MeshUniqueID = Parser.GetFbxHelper()->GetMeshUniqueID(Mesh);
		if (!Mesh->IsTriangleMesh())
		{
			//Unable to triangulate this mesh skipping it
			UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
			Message->Text = FText::Format(LOCTEXT("InterchangeFbxSdkMeshTriangulationError", "Fbx sdk is unable to triangulate mesh '{0}': it will be omitted."), FText::FromString(MeshName));
			continue;
		}

		const UInterchangeMeshNode* ExistingMeshNode = Cast<UInterchangeMeshNode>(NodeContainer.GetNode(MeshUniqueID));
		UInterchangeMeshNode* MeshNode = nullptr;
		if (ExistingMeshNode)
		{
			//This mesh node was already created
			continue;
		}

		MeshNode = CreateMeshNode(NodeContainer, MeshName, MeshUniqueID);

		ProcessCustomAttributes(Parser, Mesh, MeshNode);

		if (Geometry->GetDeformerCount(FbxDeformer::eSkin) > 0)
		{
			if (ExtractSkinnedMeshNodeJoints(SDKScene, NodeContainer, Mesh, MeshNode))
			{
				//Set the skinned mesh attribute
				MeshNode->SetSkinnedMesh(true);
			}
			
		}
		Mesh->ComputeBBox();
		const int32 MeshVertexCount = Mesh->GetControlPointsCount();
		MeshNode->SetCustomVertexCount(MeshVertexCount);
		const int32 MeshPolygonCount = Mesh->GetPolygonCount();
		MeshNode->SetCustomPolygonCount(MeshPolygonCount);
		const FBox MeshBoundingBox = FBox(FFbxConvert::ConvertPos<FVector>(Mesh->BBoxMin.Get()), FFbxConvert::ConvertPos<FVector>(Mesh->BBoxMax.Get()));
		MeshNode->SetCustomBoundingBox(MeshBoundingBox);
		const bool bMeshHasVertexNormal = Mesh->GetElementNormalCount() > 0;
		MeshNode->SetCustomHasVertexNormal(bMeshHasVertexNormal);
		const bool bMeshHasVertexBinormal = Mesh->GetElementBinormalCount() > 0;
		MeshNode->SetCustomHasVertexBinormal(bMeshHasVertexBinormal);
		const bool bMeshHasVertexTangent = Mesh->GetElementTangentCount() > 0;
		MeshNode->SetCustomHasVertexTangent(bMeshHasVertexTangent);
		const bool bMeshHasSmoothGroup = Mesh->GetElementSmoothingCount() > 0;
		MeshNode->SetCustomHasSmoothGroup(bMeshHasSmoothGroup);
		const bool bMeshHasVertexColor = Mesh->GetElementVertexColorCount() > 0;
		MeshNode->SetCustomHasVertexColor(bMeshHasVertexColor);
		const int32 MeshUVCount = Mesh->GetElementUVCount();
		MeshNode->SetCustomUVCount(MeshUVCount);

		//Add Material dependencies, we use always the first fbx node that instance the fbx geometry to grab the fbx surface materials.
		ExtractMeshMaterials(Parser, Mesh, Mesh->GetNode(), [&MeshNode](const FString& MaterialName, const FString& MaterialUid, const int32 MeshMaterialIndex)
			{
				MeshNode->SetSlotMaterialDependencyUid(MaterialName, MaterialUid);
			});

		FString PayLoadKey = MeshUniqueID;
		if (ensure(!PayloadContexts.Contains(PayLoadKey)))
		{
			TSharedPtr<FMeshPayloadContext> GeoPayload = MakeShared<FMeshPayloadContext>();
			GeoPayload->Mesh = Mesh;
			GeoPayload->SDKScene = SDKScene;
			GeoPayload->SDKGeometryConverter = SDKGeometryConverter;
			GeoPayload->bIsSkinnedMesh = MeshNode->IsSkinnedMesh();
			PayloadContexts.Add(PayLoadKey, GeoPayload);
		}
		MeshNode->SetPayLoadKey(PayLoadKey, MeshNode->IsSkinnedMesh() ? EInterchangeMeshPayLoadType::SKELETAL : EInterchangeMeshPayLoadType::STATIC);

		int32 NumAnimations = SDKScene->GetSrcObjectCount<FbxAnimStack>();

		//Add all MorphTargets for this geometry node

		TMap<FString, FbxShape*> ShapeNameToFbxShape;
		const int32 MorphTargetCount = Geometry->GetDeformerCount(FbxDeformer::eBlendShape);

		for (int32 MorphTargetIndex = 0; MorphTargetIndex < MorphTargetCount; ++MorphTargetIndex)
		{
			FbxBlendShape* MorphTarget = (FbxBlendShape*)Geometry->GetDeformer(MorphTargetIndex, FbxDeformer::eBlendShape);
			const int32 MorphTargetChannelCount = MorphTarget->GetBlendShapeChannelCount();
			FString MorphTargetName = Parser.GetFbxHelper()->GetFbxObjectName(MorphTarget);
			// see below where this is used for explanation...
			const bool bMightBeBadMAXFile = (MorphTargetName == FString("Morpher"));
			for (int32 ChannelIndex = 0; ChannelIndex < MorphTargetChannelCount; ++ChannelIndex)
			{
				FbxBlendShapeChannel* Channel = MorphTarget->GetBlendShapeChannel(ChannelIndex);
				if (!Channel)
				{
					continue;
				}
				//Find which morph target should we use according to the weight.
				const int32 CurrentChannelMorphTargetCount = Channel->GetTargetShapeCount();
				if (CurrentChannelMorphTargetCount == 0)
				{
					continue;
				}

				FString ChannelName = Parser.GetFbxHelper()->GetFbxObjectName(Channel);
				// Maya adds the name of the MorphTarget and an underscore to the front of the channel name, so remove it
				if (ChannelName.StartsWith(MorphTargetName) && (ChannelName.Compare(MorphTargetName, ESearchCase::IgnoreCase) != 0))
				{
					ChannelName.RightInline(ChannelName.Len() - (MorphTargetName.Len() + 1), EAllowShrinking::No);
				}
				for (int32 ChannelMorphTargetIndex = 0; ChannelMorphTargetIndex < CurrentChannelMorphTargetCount; ++ChannelMorphTargetIndex)
				{
					FbxShape* Shape = Channel->GetTargetShape(ChannelMorphTargetIndex);
					FString ShapeName;
					if (CurrentChannelMorphTargetCount > 1)
					{
						ShapeName = Parser.GetFbxHelper()->GetFbxObjectName(Shape);
					}
					else
					{
						if (bMightBeBadMAXFile)
						{
							ShapeName = Parser.GetFbxHelper()->GetFbxObjectName(Shape);
						}
						else
						{
							// Maya concatenates the number of the shape to the end of its name, so instead use the name of the channel
							ShapeName = ChannelName;
						}
					}

					if (ShapeName.IsEmpty())
					{
						ShapeName = TEXT("MorphTarget_") + FString::FromInt(ChannelIndex) + TEXT("_") + FString::FromInt(ChannelMorphTargetIndex);
					}

					static const FString NameClash = "_ncl_";

					if (ShapeNameToFbxShape.Contains(ShapeName))
					{
						int32 UniqueId = 1;
						FString UniqueShapeName;
						do
						{
							UniqueShapeName = ShapeName + NameClash + FString::FromInt(UniqueId++);
						}
						while (ShapeNameToFbxShape.Contains(UniqueShapeName));

						UE_LOGF(LogInterchangeFbxParser, Warning,
							"Duplicate morph target channel name '%ls' found in FBX file. It will be imported as '%ls'.",
							*ShapeName, *UniqueShapeName);

						ShapeName = UniqueShapeName;
					}
					ensure(!ShapeNameToFbxShape.Contains(ShapeName));

					ShapeNameToFbxShape.Add(ShapeName, Shape);
					FString MorphTargetAttributeName = Parser.GetFbxHelper()->GetMeshName(Shape);
					FString MorphTargetUniqueID = Parser.GetFbxHelper()->GetMeshUniqueID(Shape);
					const UInterchangeMeshNode* ExistingMorphTargetNode = Cast<const UInterchangeMeshNode>(NodeContainer.GetNode(MorphTargetUniqueID));
					if (ExistingMorphTargetNode)
					{
						int32 UniqueId = 1;
						while (ExistingMorphTargetNode)
						{
							MorphTargetUniqueID = Parser.GetFbxHelper()->GetMeshUniqueID(Shape) + NameClash + FString::FromInt(UniqueId++);
							ExistingMorphTargetNode = Cast<const UInterchangeMeshNode>(NodeContainer.GetNode(MorphTargetUniqueID));
						}
					}

					if (!ExistingMorphTargetNode)
					{
						UInterchangeMeshNode* MorphTargetNode = CreateMeshNode(NodeContainer, MorphTargetAttributeName, MorphTargetUniqueID);
						const bool bIsMorphTarget = true;
						MorphTargetNode->SetMorphTarget(bIsMorphTarget);
						MorphTargetNode->SetMorphTargetName(ShapeName);

						//Create a Mesh node dependency, so the mesh node can retrieve is associate morph target
						MeshNode->SetMorphTargetDependencyUid(MorphTargetUniqueID);

						FString MorphTargetPayLoadKey = MorphTargetUniqueID;
						if (ensure(!PayloadContexts.Contains(MorphTargetPayLoadKey)))
						{
							TSharedPtr<FMorphTargetPayloadContext> GeoPayload = MakeShared<FMorphTargetPayloadContext>();
							GeoPayload->Shape = Shape;
							GeoPayload->SDKScene = SDKScene;
							GeoPayload->SDKGeometryConverter = SDKGeometryConverter;
							PayloadContexts.Add(MorphTargetPayLoadKey, GeoPayload);
						}
						MorphTargetNode->SetPayLoadKey(MorphTargetPayLoadKey, EInterchangeMeshPayLoadType::MORPHTARGET);
					}
				}

				FbxShape* Shape = Channel->GetTargetShape(0);
				FString MorphTargetUniqueID = Parser.GetFbxHelper()->GetMeshUniqueID(Shape);
				for (int32 AnimationIndex = 0; AnimationIndex < NumAnimations; AnimationIndex++)
				{
					FbxAnimStack* AnimStack = (FbxAnimStack*)SDKScene->GetSrcObject<FbxAnimStack>(AnimationIndex);
					if (AnimStack)
					{
						FbxAnimLayer* AnimLayer = (FbxAnimLayer*)AnimStack->GetMember(0);

						FbxAnimCurve* Curve = Mesh->GetShapeChannel(MorphTargetIndex, ChannelIndex, AnimLayer);
						if (Curve && Curve->KeyGetCount() > 0)
						{
							FbxTimeSpan TimeInterval;
							Curve->GetTimeInterval(TimeInterval);
							if (CurrentChannelMorphTargetCount == 1)
							{
								MorphTargetAnimationsBuildingData.Add(FMorphTargetAnimationBuildingData(TimeInterval.GetStart().GetSecondDouble()
									, TimeInterval.GetStop().GetSecondDouble()
									, MeshNode
									, GeometryIndex
									, AnimationIndex
									, AnimLayer
									, MorphTargetIndex
									, ChannelIndex
									, MorphTargetUniqueID));
							}
							else
							{
								TArray<FString> InbetweenCurveNames;
								InbetweenCurveNames.Reserve(CurrentChannelMorphTargetCount);

								// in fbx the primary shape is the last shape, however to make
								// the code more similar to usd importer, we deal with the primary shape separately
								InbetweenCurveNames.Add(ChannelName);

								// ignoring the last shape because it is not a inbetween, i.e. it is the primary shape
								int32 InbetweenCount = CurrentChannelMorphTargetCount - 1;

								TArrayView<double> FbxInbetweenFullWeights = { Channel->GetTargetShapeFullWeights(), InbetweenCount };

								TArray<float> InbetweenFullWeights;
								InbetweenFullWeights.Reserve(InbetweenCount);
								/** for some reason blend shape values are coming as 100 scaled, so a transform is needed to scale it to 0-1 **/
								Algo::Transform(FbxInbetweenFullWeights, InbetweenFullWeights, [](double Input) { return Input * 0.01f; });

								// collect inbetween shape names
								for (int32 InbetweenIndex = 0; InbetweenIndex < InbetweenCount; ++InbetweenIndex)
								{
									FbxShape* InbetweenShape = Channel->GetTargetShape(InbetweenIndex);
									InbetweenCurveNames.Add(Parser.GetFbxHelper()->GetFbxObjectName(InbetweenShape));
								}
								MorphTargetAnimationsBuildingData.Add(FMorphTargetAnimationBuildingData(TimeInterval.GetStart().GetSecondDouble()
									, TimeInterval.GetStop().GetSecondDouble()
									, MeshNode
									, GeometryIndex
									, AnimationIndex
									, AnimLayer
									, MorphTargetIndex
									, ChannelIndex
									, MorphTargetUniqueID
									, InbetweenCurveNames
									, InbetweenFullWeights));
							}
						}
					}
				}
			} // for MorphTargetChannelCount
		} // for MorphTargetCount
	} // for GeometryCount
}

bool AcquireBindPoseMatrix(FbxPose* CurrentPose, FbxAMatrix& GlobalBindPoseJointMatrix, FbxNode* Joint)
{
	if (!CurrentPose || !Joint)
	{
		return false;
	}

	int32 PoseLinkIndex = CurrentPose->Find(Joint);
	if (PoseLinkIndex >= 0)
	{
		if (!CurrentPose->IsLocalMatrix(PoseLinkIndex))
		{
			FbxMatrix NoneAffineMatrix = CurrentPose->GetMatrix(PoseLinkIndex);
			GlobalBindPoseJointMatrix = *(FbxAMatrix*)(double*)&NoneAffineMatrix;
			return true;
		}
	}
	return false;
}

bool RetrievePoseFromBindPose(FbxNode* Joint, FbxAMatrix& GlobalBindPoseJointMatrix)
{
	if (!Joint)
	{
		return false;
	}

	FbxScene* SDKScene = Joint->GetScene();
	if (!SDKScene)
	{
		return false;
	}

	const int32 PoseCount = SDKScene->GetPoseCount();
	for (int32 PoseIndex = 0; PoseIndex < PoseCount; PoseIndex++)
	{
		FbxPose* CurrentPose = SDKScene->GetPose(PoseIndex);

		// current pose is bind pose, 
		if (CurrentPose && CurrentPose->IsBindPose())
		{
			FString PoseName = CurrentPose->GetName();
			// all error report status
			FbxStatus Status;

			FbxArray<FbxNode*> pMissingAncestors, pMissingDeformers, pMissingDeformersAncestors, pWrongMatrices;

			if (CurrentPose->IsValidBindPoseVerbose(Joint, pMissingAncestors, pMissingDeformers, pMissingDeformersAncestors, pWrongMatrices, 0.0001, &Status))
			{
				if (AcquireBindPoseMatrix(CurrentPose, GlobalBindPoseJointMatrix, Joint))
				{
					return true;
				}
			}
			else
			{
				// first try to fix up
				// add missing ancestors
				for (int i = 0; i < pMissingAncestors.GetCount(); i++)
				{
					FbxAMatrix mat = pMissingAncestors.GetAt(i)->EvaluateGlobalTransform(FBXSDK_TIME_ZERO);
					CurrentPose->Add(pMissingAncestors.GetAt(i), mat);
				}

				pMissingAncestors.Clear();
				pMissingDeformers.Clear();
				pMissingDeformersAncestors.Clear();
				pWrongMatrices.Clear();

				// check it again
				if (CurrentPose->IsValidBindPose(Joint))
				{
					if (AcquireBindPoseMatrix(CurrentPose, GlobalBindPoseJointMatrix, Joint))
					{
						return true;
					}
				}
				else
				{
					// first try to find parent who is null group and see if you can try test it again
					FbxNode* ParentNode = Joint->GetParent();
					while (ParentNode)
					{
						FbxNodeAttribute* Attr = ParentNode->GetNodeAttribute();
						if (Attr && Attr->GetAttributeType() == FbxNodeAttribute::eNull)
						{
							// found it 
							break;
						}

						// find next parent
						ParentNode = ParentNode->GetParent();
					}

					if (ParentNode && CurrentPose->IsValidBindPose(ParentNode))
					{
						if (AcquireBindPoseMatrix(CurrentPose, GlobalBindPoseJointMatrix, Joint))
						{
							return true;
						}
					}
				}
			}
		}
	}

	return false;
} 

bool FFbxMesh::GetGlobalJointBindPoseTransform(FFbxParser* Parser, FbxScene* SDKScene, FbxNode* Joint, FFbxJointMeshBindPoseGenerator& FbxJointMeshBindPoseGenerator,
	FbxAMatrix& GlobalBindPoseJointMatrix, TMap<FString, FMatrix>& MeshIdToGlobalBindPoseJointMap, TMap<FString, FMatrix>& MeshIdToGlobalBindPoseReferenceMap, 
	bool& bBadBindPoseMessageDisplay, const bool bIsOriginalJoint, const FbxAMatrix& JointOrientationMatrix)
{
	FbxManager* SDKManager = SDKScene->GetFbxManager();
	
	GlobalBindPoseJointMatrix = FbxJointMeshBindPoseGenerator.GetBindPoseFbxAMatrix(Joint, GlobalBindPoseJointMatrix);
	MeshIdToGlobalBindPoseReferenceMap = FbxJointMeshBindPoseGenerator.GetMeshIdToGlobalBindPoseReferenceMap(Joint);
	MeshIdToGlobalBindPoseJointMap = FbxJointMeshBindPoseGenerator.GetMeshIdToGlobalBindPoseJointMap(Joint);
	
	if (MeshIdToGlobalBindPoseReferenceMap.Num() > 0)
	{
		return true;
	}

	// TODO: Do we expect all Joints to have meshes assigned to them?
	if (!(Joint->GetMesh() && Joint->GetMesh()->GetDeformerCount(FbxDeformer::eSkin) > 0))
	{
		return false;
	}

	bool bRetrieveBindPoseResult = RetrievePoseFromBindPose(Joint, GlobalBindPoseJointMatrix);
	// get bind pose
	if (!bRetrieveBindPoseResult)
	{
		// if failed, delete bind pose, and retry.
		const int32 PoseCount = SDKScene->GetPoseCount();
		for (int32 PoseIndex = PoseCount - 1; PoseIndex >= 0; --PoseIndex)
		{
			FbxPose* CurrentPose = SDKScene->GetPose(PoseIndex);

			// current pose is bind pose, 
			if (CurrentPose && CurrentPose->IsBindPose())
			{
				SDKScene->RemovePose(PoseIndex);
				CurrentPose->Destroy();
			}
		}

		SDKManager->CreateMissingBindPoses(SDKScene);
		bRetrieveBindPoseResult = RetrievePoseFromBindPose(Joint, GlobalBindPoseJointMatrix);
	}

	//Only joints should have the JointOrientationMatrix applied here.
	if (bRetrieveBindPoseResult && bIsOriginalJoint)
	{
		GlobalBindPoseJointMatrix = GlobalBindPoseJointMatrix * JointOrientationMatrix;
	}

	return bRetrieveBindPoseResult;
}

bool FFbxMesh::ExtractSkinnedMeshNodeJoints(FbxScene* SDKScene, UInterchangeBaseNodeContainer& NodeContainer, FbxMesh* Mesh, UInterchangeMeshNode* MeshNode)
{
	bool bFoundValidJoint = false;
	TArray<FString> JointNodeUniqueIDs;

	const int32 SkinDeformerCount = Mesh->GetDeformerCount(FbxDeformer::eSkin);
	for (int32 DeformerIndex = 0; DeformerIndex < SkinDeformerCount; DeformerIndex++)
	{
		FbxSkin* Skin = (FbxSkin*)Mesh->GetDeformer(DeformerIndex, FbxDeformer::eSkin);
		if (!ensure(Skin))
		{
			continue;
		}
		const int32 ClusterCount = Skin->GetClusterCount();
		JointNodeUniqueIDs.Reserve(JointNodeUniqueIDs.Num() + ClusterCount);
		for (int32 ClusterIndex = 0; ClusterIndex < ClusterCount; ClusterIndex++)
		{
			FbxCluster* Cluster = Skin->GetCluster(ClusterIndex);
			// When Maya plug-in exports rigid binding, it will generate "CompensationCluster" for each ancestor links.
			// FBX writes these "CompensationCluster" out. The CompensationCluster also has weight 1 for vertices.
			// Unreal importer should skip these clusters.
			if (!Cluster || (FCStringAnsi::Strcmp(Cluster->GetUserDataID(), "Maya_ClusterHint") == 0 && FCStringAnsi::Strcmp(Cluster->GetUserData(), "CompensationCluster") == 0))
			{
				continue;
			}
			FbxNode* Link = Cluster->GetLink();
			if (!Link)
			{
				continue;
			}
			bFoundValidJoint = true;

			FString JointNodeUniqueID = Parser.GetFbxHelper()->GetFbxNodeHierarchyName(Link);

			// find the bone index
			if (!JointNodeUniqueIDs.Contains(JointNodeUniqueID))
			{
				JointNodeUniqueIDs.Add(JointNodeUniqueID);
				MeshNode->SetSkeletonDependencyUid(JointNodeUniqueID);
			}
		}
	}
	return bFoundValidJoint;
}

UInterchangeMeshNode* FFbxMesh::CreateMeshNode(UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeName, const FString& NodeUniqueID)
{
	FString DisplayLabel(NodeName);
	FString NodeUid(NodeUniqueID);
	UInterchangeMeshNode* MeshNode = NewObject<UInterchangeMeshNode>(&NodeContainer, NAME_None);
	if (!ensure(MeshNode))
	{
		UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
		Message->InterchangeKey = NodeUniqueID;
		Message->Text = LOCTEXT("NodeAllocationError", "Mesh node allocation failed when importing FBX.");
		return nullptr;
	}
	// Creating a UMaterialInterface
	NodeContainer.SetupNode(MeshNode, NodeUid, DisplayLabel, EInterchangeNodeContainerType::TranslatedAsset);
	return MeshNode;
}

FFbxJointMeshBindPoseGenerator::FFbxJointMeshBindPoseGenerator(FbxScene* SDKScene, FFbxParser& Parser, const bool bConsiderClusterBeforePoseForMeshBindPose)
{
	//Search all skeletalmesh(FbxGeometry with valid deformer) using this joint and see if there is a valid FbxCluster
	//containing a TransformLinkMatrix for this joint

	TSet<const FbxNode*> ValidationTracker; //Gets reset everytime when a BindPose gets recreated.
	TMap<const FbxNode*, FString> JointToGeometryMap; //Maps Joint to the Geometry that the Bind Pose is being used from.
	bool bClusterVsPoseMissmatch = false;

	const int32 GeometryCount = SDKScene->GetGeometryCount();
	for (int32 GeometryIndex = 0; GeometryIndex < GeometryCount; ++GeometryIndex)
	{
		FbxGeometry* Geometry = SDKScene->GetGeometry(GeometryIndex);
		if (!ensure(Geometry))
		{
			continue;
		}
		FString GeometryName = Parser.GetFbxHelper()->GetFbxObjectName(Geometry);
		FString MeshUniqueID = Parser.GetFbxHelper()->GetMeshUniqueID(Geometry);

		const int32 GeometryDeformerCount = Geometry->GetDeformerCount(FbxDeformer::eSkin);
		for (int32 GeometryDeformerIndex = 0; GeometryDeformerIndex < GeometryDeformerCount; ++GeometryDeformerIndex)
		{
			FbxSkin* Skin = (FbxSkin*)Geometry->GetDeformer(GeometryDeformerIndex, FbxDeformer::eSkin);

			if (!ensure(Skin))
			{
				continue;
			}
			const int32 ClusterCount = Skin->GetClusterCount();
			for (int32 ClusterIndex = 0; ClusterIndex < ClusterCount; ClusterIndex++)
			{
				FbxCluster* Cluster = Skin->GetCluster(ClusterIndex);
				// When Maya plug-in exports rigid binding, it will generate "CompensationCluster" for each ancestor links.
				// FBX writes these "CompensationCluster" out. The CompensationCluster also has weight 1 for vertices.
				// Unreal importer should skip these clusters.
				if (!Cluster || (FCStringAnsi::Strcmp(Cluster->GetUserDataID(), "Maya_ClusterHint") == 0 && FCStringAnsi::Strcmp(Cluster->GetUserData(), "CompensationCluster") == 0))
				{
					continue;
				}

				FbxNode* Joint = Cluster->GetLink();

				if (!Joint)
				{
					continue;
				}

				//Acquire Cluster transforms:
				FbxAMatrix BoneBindMatrix_Cluster;
				FbxAMatrix MeshBindMatrix_Cluster;
				Cluster->GetTransformMatrix(MeshBindMatrix_Cluster);
				Cluster->GetTransformLinkMatrix(BoneBindMatrix_Cluster);

				//Acquire BindPose transforms, reasoning:
				// Here we try fetching the bind pose only from actual FbxPose objects. Interchange traditionally preferred reading the
				// TransformLink directly from the cluster, but it seems that in some edge cases (see UE-273136) that is not the correct thing to do?
				// By checking the pose first, we mirror the FbxLegacy behavior shown at UnFbx::FFbxImporter::ImportBones, with how it first 
				// checks the PoseArray when filling in GlobalsPerLink
				FbxAMatrix BoneBindMatrix_Pose;
				FbxAMatrix MeshBindMatrix_Pose;
				bool bRetrieveBindPoseResult = RetrievePoseFromBindPose(Joint, BoneBindMatrix_Pose);
				if (!bRetrieveBindPoseResult)
				{
					// if failed, delete bind pose, and retry.
					// This is likely not needed as FFbxScene::IsValidBindPose will already create this fallback pose, but just in case
					const int32 PoseCount = SDKScene->GetPoseCount();
					for (int32 PoseIndex = PoseCount - 1; PoseIndex >= 0; --PoseIndex)
					{
						FbxPose* CurrentPose = SDKScene->GetPose(PoseIndex);

						// current pose is bind pose, 
						if (CurrentPose && CurrentPose->IsBindPose())
						{
							SDKScene->RemovePose(PoseIndex);
							CurrentPose->Destroy();

							ValidationTracker.Reset();
						}
					}

					SDKScene->GetFbxManager()->CreateMissingBindPoses(SDKScene);
					bRetrieveBindPoseResult = RetrievePoseFromBindPose(Joint, BoneBindMatrix_Pose);

					if (!bRetrieveBindPoseResult)
					{
						Cluster->GetTransformLinkMatrix(BoneBindMatrix_Pose);
					}
				}
				if (!RetrievePoseFromBindPose(Geometry->GetNode(), MeshBindMatrix_Pose))
				{
					Cluster->GetTransformMatrix(MeshBindMatrix_Pose);
				}

				if (bRetrieveBindPoseResult && !bClusterVsPoseMissmatch)
				{
					// Convert to FMatrix to check equality with tolerance. 
					const FMatrix ClusterMatrix = FFbxConvert::ConvertMatrix<FMatrix>(BoneBindMatrix_Cluster);
					const FMatrix PoseMatrix = FFbxConvert::ConvertMatrix<FMatrix>(BoneBindMatrix_Pose);

					if (!ClusterMatrix.Equals(PoseMatrix, KINDA_SMALL_NUMBER))
					{
						bClusterVsPoseMissmatch = true;
					}
				}

				FbxAMatrix MeshBindMatrix;
				FbxAMatrix BoneBindMatrix;
				if (bConsiderClusterBeforePoseForMeshBindPose)
				{
					MeshBindMatrix = MeshBindMatrix_Cluster;
					BoneBindMatrix = BoneBindMatrix_Cluster;
				}
				else
				{
					MeshBindMatrix = MeshBindMatrix_Pose;
					BoneBindMatrix = BoneBindMatrix_Pose;
				}

				TMap<FString, FMatrix>& MeshIdToGlobalBindPoseJointMap = JointToMeshIdToGlobalBindPoseJointMap.FindOrAdd(Joint);
				MeshIdToGlobalBindPoseJointMap.Add(MeshUniqueID, FFbxConvert::ConvertMatrix<FMatrix>(BoneBindMatrix));

				const FbxAMatrix BindPoseFbxAMatrix = BoneBindMatrix * Parser.JointOrientationMatrix;
				if (!JointToBindPoseMap.Contains(Joint))
				{
					JointToBindPoseMap.Add(Joint, BindPoseFbxAMatrix);
					ValidationTracker.Add(Joint);
					JointToGeometryMap.Add(Joint, GeometryName);
				}
				else
				{
					if (ValidationTracker.Contains(Joint))
					{
						// Convert to FMatrix to check equality with tolerance. 
						const FMatrix ExistingMatrix = FFbxConvert::ConvertMatrix<FMatrix>(JointToBindPoseMap[Joint]);
						const FMatrix NewMatrix = FFbxConvert::ConvertMatrix<FMatrix>(BindPoseFbxAMatrix);

						if (!NewMatrix.Equals(ExistingMatrix, KINDA_SMALL_NUMBER))
						{
							FText JointName = FText::FromString(Parser.GetFbxHelper()->GetFbxObjectName(Joint));
							bool bWarningAdded = false;
							if (JointToGeometryMap.Contains(Joint))
							{
								if (GeometryName != JointToGeometryMap[Joint])
								{
									//Conflicting bind Poses.
									//#interchange_todo: Perhaps in the future we could introduce a Pose selector for Bind Pose calculations. Currently out of scope.

									UInterchangeResultWarning_Generic* Message = Parser.AddMessage<UInterchangeResultWarning_Generic>();
									Message->Text = FText::Format(LOCTEXT("FFbxJointMeshBindPoseGenerator_ConflictingBindPoseMatrices", 
										"Imported SkeletalMesh and/or Skeleton might be incorrect.\nConflicting Bind Pose Matrices found for Joint [{0}].\n   Currently using Geometry [{1}] for Bind Pose Matrix [{2}].\n   However Geometry [{3}] stores a differing Bind Pose Matrix [{4}]."), 
										JointName, 
										FText::FromString(JointToGeometryMap[Joint]), FText::FromString(ExistingMatrix.ToString()),
										FText::FromString(GeometryName), FText::FromString(NewMatrix.ToString()));

									bWarningAdded = true;
								}
							}
							
							if (!bWarningAdded)
							{
								UInterchangeResultWarning_Generic* Message = Parser.AddMessage<UInterchangeResultWarning_Generic>();
								Message->Text = FText::Format(LOCTEXT("FFbxJointMeshBindPoseGenerator_UnexpectedBindPoseMatrices", "Imported SkeletalMesh and/or Skeleton might be incorrect.\nExpected to have same joint [{0}] node in different clusters to have same Bind Pose matrix!"), FText::FromString(Parser.GetFbxHelper()->GetFbxObjectName(Joint)));
							}
						}
					}
					else
					{
						JointToBindPoseMap[Joint] = BindPoseFbxAMatrix;
						JointToGeometryMap.Add(Joint, GeometryName);
					}
				}

				TMap<FString, FMatrix>& MeshIdToGlobalBindPoseReferenceMap = JointToMeshIdToGlobalBindPoseReferenceMap.FindOrAdd(Joint);
				MeshIdToGlobalBindPoseReferenceMap.Add(MeshUniqueID, FFbxConvert::ConvertMatrix<FMatrix>(MeshBindMatrix));
			}
		}
	}

	if (bClusterVsPoseMissmatch)
	{
		UInterchangeResultWarning_Generic* Message = Parser.AddMessage<UInterchangeResultWarning_Generic>();
		Message->Text = LOCTEXT("FFbxJointMeshBindPoseGenerator_NonMatchingBindPoses", "BindPose Matrix generation acquired 2 different Matrices from FbxCluster vs FbxPose.\n If import is incorrect, consider using a different setting for FBX option 'bConsiderClusterBeforePoseForMeshBindPose'.");
	}
}

TMap<FString, FMatrix>& FFbxJointMeshBindPoseGenerator::GetMeshIdToGlobalBindPoseReferenceMap(const FbxNode* Joint)
{
	TMap<FString, FMatrix>* MeshIdToGlobalBindPoseReferenceMap = JointToMeshIdToGlobalBindPoseReferenceMap.Find(Joint);
	if (MeshIdToGlobalBindPoseReferenceMap)
	{
		return *MeshIdToGlobalBindPoseReferenceMap;
	}
	else
	{
		static TMap<FString, FMatrix> EmptyMap;
		return EmptyMap;
	}
}

TMap<FString, FMatrix>& FFbxJointMeshBindPoseGenerator::GetMeshIdToGlobalBindPoseJointMap(const FbxNode* Joint)
{
	TMap<FString, FMatrix>* MeshIdToGlobalBindPoseJointMap = JointToMeshIdToGlobalBindPoseJointMap.Find(Joint);
	if (MeshIdToGlobalBindPoseJointMap)
	{
		return *MeshIdToGlobalBindPoseJointMap;
	}
	else
	{
		static TMap<FString, FMatrix> EmptyMap;
		return EmptyMap;
	}
}

FbxAMatrix FFbxJointMeshBindPoseGenerator::GetBindPoseFbxAMatrix(const FbxNode* Joint, FbxAMatrix& DefaultValue)
{
	FbxAMatrix* Matrix = JointToBindPoseMap.Find(Joint);
	if (Matrix)
	{
		return *Matrix;
	}
	else
	{
		return DefaultValue;
	}
}

} // namespace UE::Interchange::Private


#undef LOCTEXT_NAMESPACE
