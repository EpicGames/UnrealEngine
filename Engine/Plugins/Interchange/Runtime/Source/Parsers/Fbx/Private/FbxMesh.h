// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FbxAPI.h"
#include "FbxHelper.h"
#include "FbxInclude.h"

/** Forward declarations */
struct FMeshDescription;
class UInterchangeBaseNodeContainer;
class UInterchangeMeshNode;

namespace UE
{
	namespace Interchange
	{
#if WITH_ENGINE
		struct FMeshPayloadData;
#endif

		namespace Private
		{
			//Helper to generate and cache per joint MeshIdToGlobalBindPose transforms.
			struct FFbxJointMeshBindPoseGenerator
			{
				TMap<const FbxNode*, TMap<FString, FMatrix>> JointToMeshIdToGlobalBindPoseReferenceMap;
				TMap<const FbxNode*, TMap<FString, FMatrix>> JointToMeshIdToGlobalBindPoseJointMap;
				TMap<const FbxNode*, FbxAMatrix> JointToBindPoseMap;

				FFbxJointMeshBindPoseGenerator(FbxScene* SDKScene, FFbxParser& Parser, const bool bConsiderClusterBeforePoseForMeshBindPose);

				TMap<FString, FMatrix>& GetMeshIdToGlobalBindPoseReferenceMap(const FbxNode* Joint);
				TMap<FString, FMatrix>& GetMeshIdToGlobalBindPoseJointMap(const FbxNode* Joint);
				FbxAMatrix GetBindPoseFbxAMatrix(const FbxNode* Joint, FbxAMatrix& DefaultValue);
			};

			struct FMorphTargetAnimationBuildingData
			{
				double StartTime;
				double StopTime;
				UInterchangeMeshNode* InterchangeMeshNode;
				int32 GeometryIndex;
				int32 AnimationIndex;
				FbxAnimLayer* AnimLayer;
				int32 MorphTargetIndex;
				int32 ChannelIndex;
				FString MorphTargetNodeUid;
				TArray<FString> InbetweenCurveNames;
				TArray<float> InbetweenFullWeights;

				FMorphTargetAnimationBuildingData(double InStartTime
						, double InStopTime
						, UInterchangeMeshNode* InInterchangeMeshNode
						, int32 InGeometryIndex
						, int32 InAnimationIndex
						, FbxAnimLayer* InAnimLayer
						, int32 InMorphTargetIndex
						, int32 InChannelIndex
						, FString InMorphTargetNodeUid)
					: StartTime(InStartTime)
					, StopTime(InStopTime)
					, InterchangeMeshNode(InInterchangeMeshNode)
					, GeometryIndex(InGeometryIndex)
					, AnimationIndex(InAnimationIndex)
					, AnimLayer(InAnimLayer)
					, MorphTargetIndex(InMorphTargetIndex)
					, ChannelIndex(InChannelIndex)
					, MorphTargetNodeUid(InMorphTargetNodeUid)
				{

				}

				FMorphTargetAnimationBuildingData(double InStartTime
						, double InStopTime
						, UInterchangeMeshNode* InInterchangeMeshNode
						, int32 InGeometryIndex
						, int32 InAnimationIndex
						, FbxAnimLayer* InAnimLayer
						, int32 InMorphTargetIndex
						, int32 InChannelIndex
						, FString InMorphTargetNodeUid
						, TArray<FString> InInbetweenCurveNames
						, TArray<float> InInbetweenFullWeights)
					: StartTime(InStartTime)
					, StopTime(InStopTime)
					, InterchangeMeshNode(InInterchangeMeshNode)
					, GeometryIndex(InGeometryIndex)
					, AnimationIndex(InAnimationIndex)
					, AnimLayer(InAnimLayer)
					, MorphTargetIndex(InMorphTargetIndex)
					, ChannelIndex(InChannelIndex)
					, MorphTargetNodeUid(InMorphTargetNodeUid)
					, InbetweenCurveNames(InInbetweenCurveNames)
					, InbetweenFullWeights(InInbetweenFullWeights)
				{

				}
			};

			class FFbxMesh;

			class FMeshDescriptionImporter
			{
			public:
				FMeshDescriptionImporter(FFbxParser& InParser, FMeshDescription* InMeshDescription, FbxScene* InSDKScene, FbxGeometryConverter* InSDKGeometryConverter);
				
				/*
				 * Fill the mesh description using the Mesh parameter.
				 */
				bool FillStaticMeshDescriptionFromFbxMesh(FbxMesh* Mesh, const FTransform& MeshGlobalTransform);
				
				/*
				 * Fill the mesh description using the Mesh parameter, and also fill the OutJointNodeUniqueIDs so the MeshDescription bone Index can be mapped to the correct Interchange joint scene node.
				 */
				bool FillSkinnedMeshDescriptionFromFbxMesh(FbxMesh* Mesh, const FTransform& MeshGlobalTransform, TArray<FString>& OutJointUniqueNames);

				/*
				 * Fill the mesh description using the Shape parameter.
				 */
				bool FillMeshDescriptionFromFbxShape(FbxShape* Shape, const FTransform& MeshGlobalTransform);

				/**
				 * Add messages to the message log.
				 */
				template <typename T>
				T* AddMessage(FbxGeometryBase* FbxNode) const
				{
					T* Item = Parser.AddMessage<T>();
					Item->MeshName = Parser.GetFbxHelper()->GetMeshName(FbxNode);
					Item->InterchangeKey = Parser.GetFbxHelper()->GetMeshUniqueID(FbxNode);
					return Item;
				}

			private:
				
				enum class EMeshType : uint8
				{
					None = 0, //No mesh type to import
					Static = 1, //static mesh
					Skinned = 2, //skinned mesh with joints
				};

				bool FillMeshDescriptionFromFbxMesh(FbxMesh* Mesh, const FTransform& MeshGlobalTransform, TArray<FString>& OutJointUniqueNames, EMeshType MeshType);
				bool IsOddNegativeScale(FbxAMatrix& TotalMatrix);
				
				//TODO move the real function from RenderCore to FVector, so we do not have to add render core to compute such a simple thing
				float FbxGetBasisDeterminantSign(const FVector3f& XAxis, const FVector3f& YAxis, const FVector3f& ZAxis)
				{
					FMatrix44f Basis(
						FPlane4f(XAxis, 0),
						FPlane4f(YAxis, 0),
						FPlane4f(ZAxis, 0),
						FPlane4f(0, 0, 0, 1)
					);
					return (Basis.Determinant() < 0) ? -1.0f : +1.0f;
				}

				FFbxParser& Parser;
				FMeshDescription* MeshDescription;
				FbxScene* SDKScene;
				FbxGeometryConverter* SDKGeometryConverter;
				bool bInitialized = false;
			};

			class FMeshPayloadContext : public FPayloadContextBase
			{
			public:
				virtual ~FMeshPayloadContext() = default;
				virtual FString GetPayloadType() const override { return TEXT("Mesh-PayloadContext"); }
				virtual bool FetchMeshPayloadToFile(FFbxParser& Parser, const FTransform& MeshGlobalTransform, const FString& PayloadFilepath) override;
#if WITH_ENGINE
				virtual bool FetchMeshPayload(FFbxParser& Parser, const FTransform& MeshGlobalTransform, FMeshPayloadData& OutMeshPayloadData) override;
#endif
				bool bIsSkinnedMesh = false;
				FbxMesh* Mesh = nullptr;
				FbxScene* SDKScene = nullptr;
				FbxGeometryConverter* SDKGeometryConverter = nullptr;
			private:
				bool FetchMeshPayloadInternal(FFbxParser& Parser
					, const FTransform& MeshGlobalTransform
					, FMeshDescription& OutMeshDescription
					, TArray<FString>& OutJointNames);
			};

			class FMorphTargetPayloadContext : public FPayloadContextBase
			{
			public:
				virtual ~FMorphTargetPayloadContext() = default;
				virtual FString GetPayloadType() const override { return TEXT("MorphTarget-PayloadContext"); }
				virtual bool FetchMeshPayloadToFile(FFbxParser& Parser, const FTransform& MeshGlobalTransform, const FString& PayloadFilepath) override;
#if WITH_ENGINE
				virtual bool FetchMeshPayload(FFbxParser& Parser, const FTransform& MeshGlobalTransform, FMeshPayloadData& OutMeshPayloadData) override;
#endif
				FbxShape* Shape = nullptr;
				FbxScene* SDKScene = nullptr;
				FbxGeometryConverter* SDKGeometryConverter = nullptr;
			private:
				bool FetchMeshPayloadInternal(FFbxParser& Parser
					, const FTransform& MeshGlobalTransform
					, FMeshDescription& OutMeshDescription);
			};

			class FFbxMesh
			{
			public:
				explicit FFbxMesh(FFbxParser& InParser)
					: Parser(InParser)
				{}

				void AddAllMeshes(FbxScene* SDKScene, FbxGeometryConverter* SDKGeometryConverter, UInterchangeBaseNodeContainer& NodeContainer, TMap<FString, TSharedPtr<FPayloadContextBase>>& PayloadContexts);
				static bool GetGlobalJointBindPoseTransform(FFbxParser* Parser, FbxScene* SDKScene, FbxNode* Joint, FFbxJointMeshBindPoseGenerator& FbxJointMeshBindPoseGenerator,
					FbxAMatrix& GlobalBindPoseJointMatrix, TMap<FString, FMatrix>& MeshIdToGlobalBindPoseJointMap, TMap<FString, FMatrix>& MeshIdToGlobalBindPoseReferenceMap, 
					bool& bBadBindPoseMessageDisplay, const bool bIsOriginalJoint, const FbxAMatrix& JointOrientationMatrix);

				TArray<FMorphTargetAnimationBuildingData>& GetMorphTargetAnimationsBuildingData() { return MorphTargetAnimationsBuildingData; }
			protected:
				/** Add joints to the Interchange mesh node joint dependencies. Return false if there is no valid joint (not a valid skinned mesh). */
				bool ExtractSkinnedMeshNodeJoints(FbxScene* SDKScene, UInterchangeBaseNodeContainer& NodeContainer, FbxMesh* Mesh, UInterchangeMeshNode* MeshNode);
				UInterchangeMeshNode* CreateMeshNode(UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeName, const FString& NodeUniqueID);

			private:
				FFbxParser& Parser;

				//In order to appropriately identify the Skeleton Node Uids we have to process the MorphTarget animations once the hierarchy is processed
				TArray<FMorphTargetAnimationBuildingData> MorphTargetAnimationsBuildingData;
			};
		}//ns Private
	}//ns Interchange
}//ns UE
