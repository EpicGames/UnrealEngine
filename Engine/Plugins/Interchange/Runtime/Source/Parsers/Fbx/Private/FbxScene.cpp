// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxScene.h"

#include "CoreMinimal.h"
#include "FbxAnimation.h"
#include "FbxAPI.h"
#include "FbxConvert.h"
#include "FbxHelper.h"
#include "FbxInclude.h"
#include "FbxMaterial.h"
#include "FbxMesh.h"
#include "InterchangeCameraNode.h"
#include "InterchangeLightNode.h"
#include "InterchangeMeshLODContainerNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangeResultsContainer.h"
#include "InterchangeJointNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeSourceNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"
#include "InterchangeAnimationTrackSetNode.h"
#include "InterchangeAnimationDefinitions.h"
#include "InterchangeFbxSettings.h"
#include "InterchangeHelper.h"

#define LOCTEXT_NAMESPACE "InterchangeFbxScene"

namespace UE
{
	namespace Interchange
	{
		namespace Private
		{
			namespace RecursiveHelper
			{
				void RecursiveFillChildrenFbxNode(FbxNode* Parent, TArray<FbxNode*>& NodeArray)
				{
					if (!Parent)
					{
						return;
					}
					NodeArray.Add(Parent);
					int32 ChildCount = Parent->GetChildCount();
					for (int32 ChildIndex = 0; ChildIndex < ChildCount; ++ChildIndex)
					{
						RecursiveFillChildrenFbxNode(Parent->GetChild(ChildIndex), NodeArray);
					}
				}
			} // ns RecursiveHelper

			FString CreateTrackNodeUid(const FString& JointUid, const int32 AnimationIndex)
			{
				FString TrackNodeUid = TEXT("\\SkeletalAnimation\\") + JointUid + TEXT("\\") + FString::FromInt(AnimationIndex);
				return TrackNodeUid;
			}

			void FFbxScene::CreateMeshNodeReference(UInterchangeSceneNode* UnrealSceneNode, FbxNodeAttribute* NodeAttribute, UInterchangeBaseNodeContainer& NodeContainer, const FTransform& GeometricTransform, const FTransform& PivotNodeTransform)
			{
				const UInterchangeMeshNode* MeshNode = nullptr;
				if (NodeAttribute->GetAttributeType() == FbxNodeAttribute::eMesh)
				{
					FbxMesh* Mesh = static_cast<FbxMesh*>(NodeAttribute);
					if (ensure(Mesh))
					{
						FString MeshRefString = Parser.GetFbxHelper()->GetMeshUniqueID(Mesh);
						MeshNode = Cast<UInterchangeMeshNode>(NodeContainer.GetNode(MeshRefString));
					}
				}
				else if (NodeAttribute->GetAttributeType() == FbxNodeAttribute::eShape)
				{
					//We do not add a dependency for shape on the scene node since shapes are a MeshNode dependency.
				}

				if (MeshNode)
				{
					UnrealSceneNode->SetCustomAssetInstanceUid(MeshNode->GetUniqueID());

					if (!GeometricTransform.Equals(FTransform::Identity))
					{
						UnrealSceneNode->SetCustomGeometricTransform(GeometricTransform);
					}

					if (!PivotNodeTransform.Equals(FTransform::Identity))
					{
						UnrealSceneNode->SetCustomPivotNodeTransform(PivotNodeTransform);
					}

					// @todo: Nothing is using the SceneInstanceUid in the MeshNode. Do we even need to support it?
					// For the moment an ugly const_cast so we can mutate it (it was fetched from the NodeContainer and is hence const).
					// Possible solutions:
					// - keep track in some other way of MeshNodes which we are in the process of maintaining / modifying
					// - a derived UInterchangeMutableBaseNodeContainer which overrides node accessors and makes them mutable, to be passed only to translators
					// - get rid of this attribute, and provide an alternate method for getting the scene instance UIDs which reference the mesh (by iterating scene instance nodes)
					const_cast<UInterchangeMeshNode*>(MeshNode)->SetSceneInstanceUid(UnrealSceneNode->GetUniqueID());
				}
			}

			void CreateAssetNodeReference(FFbxParser& Parser, UInterchangeSceneNode* UnrealSceneNode, FbxNodeAttribute* NodeAttribute, UInterchangeBaseNodeContainer& NodeContainer, const FStringView TypeName)
			{
				const FString AssetUniqueID = Parser.GetFbxHelper()->GetNodeAttributeUniqueID(NodeAttribute, TypeName);

				if (const UInterchangeBaseNode* AssetNode = NodeContainer.GetNode(AssetUniqueID))
				{
					UnrealSceneNode->SetCustomAssetInstanceUid(AssetNode->GetUniqueID());
				}
			}

			void FFbxScene::CreateCameraNodeReference(UInterchangeSceneNode* UnrealSceneNode, FbxNodeAttribute* NodeAttribute, UInterchangeBaseNodeContainer& NodeContainer)
			{
				CreateAssetNodeReference(Parser, UnrealSceneNode, NodeAttribute, NodeContainer, UInterchangePhysicalCameraNode::StaticAssetTypeName());
			}

			void FFbxScene::CreateLightNodeReference(UInterchangeSceneNode* UnrealSceneNode, FbxNodeAttribute* NodeAttribute, UInterchangeBaseNodeContainer& NodeContainer)
			{
				CreateAssetNodeReference(Parser, UnrealSceneNode, NodeAttribute, NodeContainer, UInterchangeLightNode::StaticAssetTypeName());
			}

			bool IsNodeUnderCommonJointRootNode(FbxNode* Node, TMap<FbxNode*, FFbxScene::FRootJointInfo>& CommonJointRootNodes)
			{
				if (!Node || CommonJointRootNodes.IsEmpty())
				{
					return false;
				}

				//Simply go up the hierarchy until we match the CommonJointRootNode
				FbxNode* IterateNode = Node;
				while (IterateNode)
				{
					if (CommonJointRootNodes.Contains(IterateNode))
					{
						return true;
					}
					IterateNode = IterateNode->GetParent();
				}
				return false;
			}

			FTransform GetConvertedTransform(FbxNode* Node, const FbxAMatrix& NewFbxMatrix)
			{
				FTransform Transform = FFbxConvert::ConvertTransform<FTransform, FVector, FQuat>(NewFbxMatrix);

				if (FbxNodeAttribute* NodeAttribute = Node->GetNodeAttribute())
				{
					switch (NodeAttribute->GetAttributeType())
					{
					case FbxNodeAttribute::eCamera:
						Transform = FFbxConvert::AdjustCameraTransform(Transform);
						break;
					case FbxNodeAttribute::eLight:
						Transform = FFbxConvert::AdjustLightTransform(Transform);
						break;
					}
				}

				return Transform;
			};

			void FFbxScene::ParseMesh(UInterchangeBaseNodeContainer& NodeContainer
				, UInterchangeSceneNode* UnrealNode
				, FbxNode* Node
				, FbxNodeAttribute* NodeAttribute)
			{
				if (!UnrealNode)
				{
					return;
				}

				//For Mesh attribute we add the fbx nodes materials
				FFbxMaterial FbxMaterial(Parser);
				FbxMaterial.AddAllNodeMaterials(UnrealNode, Node, NodeContainer);

				//Get the Geometric offset transform and set it in the mesh node
				//The geometric offset is not part of the hierarchy transform, it is not inherited
				FbxAMatrix Geometry;
				FbxVector4 Translation, Rotation, Scaling;
				Translation = Node->GetGeometricTranslation(FbxNode::eSourcePivot);
				Rotation = Node->GetGeometricRotation(FbxNode::eSourcePivot);
				Scaling = Node->GetGeometricScaling(FbxNode::eSourcePivot);
				Geometry.SetT(Translation);
				Geometry.SetR(Rotation);
				Geometry.SetS(Scaling);

				FTransform GeometricTransform = GetConvertedTransform(Node, Geometry);

				//Get the pivot geometry offset 
				FbxAMatrix PivotGeometry;
				FbxVector4 RotationPivot = Node->GetRotationPivot(FbxNode::eSourcePivot);
				FbxVector4 FullPivot;
				FullPivot[0] = -RotationPivot[0];
				FullPivot[1] = -RotationPivot[1];
				FullPivot[2] = -RotationPivot[2];
				PivotGeometry.SetT(FullPivot);
				FTransform PivotNodeTransform = GetConvertedTransform(Node, PivotGeometry);

				CreateMeshNodeReference(UnrealNode, NodeAttribute, NodeContainer, GeometricTransform, PivotNodeTransform);
			}

			void FFbxScene::ParseMeshLODs(UInterchangeMeshLODContainerNode& LODContainerNode
				, int32 LODIndex
				, FbxNode* Node
				, FbxScene* SDKScene
				, const FbxAMatrix& InvTransformMatrix /*Global Inverse Transform Matrix Of LODContainer*/)
			{
				//#interchange_LODRefactor_Note: we could report warnings, if unexpected NodeAttribute is found (for ep eSkeleton etc)

				//#interchange_LODRefactor_Note: Seems the expectation is that the LODIndex is driven by the Child placement (index) on the eLODGroup.
				//									(Not the node names.)

				//Check for Mesh attributes:
				const int32 AttributeCount = Node->GetNodeAttributeCount();
				for (int32 AttributeIndex = 0; AttributeIndex < AttributeCount; ++AttributeIndex)
				{
					FbxNodeAttribute* NodeAttribute = Node->GetNodeAttributeByIndex(AttributeIndex);
					switch (NodeAttribute->GetAttributeType())
					{
						case FbxNodeAttribute::eMesh:
						{
							FbxMesh* Mesh = static_cast<FbxMesh*>(NodeAttribute);
							if (ensure(Mesh))
							{
								FString MeshRefString = Parser.GetFbxHelper()->GetMeshUniqueID(Mesh);

								//Get relative transform to LODContainerNode:
								FbxAMatrix RelativeMatrix = InvTransformMatrix * SDKScene->GetAnimationEvaluator()->GetNodeGlobalTransform(Node, 0);
								FTransform RelativeTransform = GetConvertedTransform(Node, RelativeMatrix);

								//Get the Geometric offset transform and set it in the mesh node
								//The geometric offset is not part of the hierarchy transform, it is not inherited
								FbxAMatrix Geometry;
								FbxVector4 Translation, Rotation, Scaling;
								Translation = Node->GetGeometricTranslation(FbxNode::eSourcePivot);
								Rotation = Node->GetGeometricRotation(FbxNode::eSourcePivot);
								Scaling = Node->GetGeometricScaling(FbxNode::eSourcePivot);
								Geometry.SetT(Translation);
								Geometry.SetR(Rotation);
								Geometry.SetS(Scaling);
								FTransform GeometryTransform = GetConvertedTransform(Node, Geometry);

								//Get the pivot geometry offset 
								FbxAMatrix PivotGeometry;
								FbxVector4 RotationPivot = Node->GetRotationPivot(FbxNode::eSourcePivot);
								FbxVector4 FullPivot;
								FullPivot[0] = -RotationPivot[0];
								FullPivot[1] = -RotationPivot[1];
								FullPivot[2] = -RotationPivot[2];
								PivotGeometry.SetT(FullPivot);
								FTransform PivotTransform = GetConvertedTransform(Node, PivotGeometry);
								
								//Combine relative geometirc and pivot transform matrices.
								FTransform CombinedTransform = PivotTransform * GeometryTransform * RelativeTransform;

								LODContainerNode.AddMeshForLODIndex(LODIndex, MeshRefString, CombinedTransform);
							}
							break;
						}
						default:
							break;
					}
				}

				const int32 ChildCount = Node->GetChildCount();
				for (int32 ChildIndex = 0; ChildIndex < ChildCount; ++ChildIndex)
				{
					FbxNode* ChildNode = Node->GetChild(ChildIndex);
					ParseMeshLODs(LODContainerNode, LODIndex, ChildNode, SDKScene, InvTransformMatrix);
				}
			}

			void FFbxScene::AddHierarchyRecursively(UInterchangeSceneNode* UnrealParentNode
				, FbxNode* Node
				, FbxScene* SDKScene
				, UInterchangeBaseNodeContainer& NodeContainer
				, TMap<FString, TSharedPtr<FPayloadContextBase, ESPMode::ThreadSafe>>& PayloadContexts
				, const TArray<FbxNode*>& ForceJointNodes
				, bool& bBadBindPoseMessageDisplay
				, FFbxJointMeshBindPoseGenerator& FbxJointMeshBindPoseGenerator)
			{	
				UInterchangeSceneNode* UnrealNode = CreateUnrealSceneNode(UnrealParentNode, Node, SDKScene, NodeContainer,
					ForceJointNodes, bBadBindPoseMessageDisplay, FbxJointMeshBindPoseGenerator);

				bool bAllowChildrenParsing = true;

				const int32 AttributeCount = Node->GetNodeAttributeCount();
				for (int32 AttributeIndex = 0; AttributeIndex < AttributeCount; ++AttributeIndex)
				{
					FbxNodeAttribute* NodeAttribute = Node->GetNodeAttributeByIndex(AttributeIndex);
					switch (NodeAttribute->GetAttributeType())
					{
						case FbxNodeAttribute::eUnknown:
						case FbxNodeAttribute::eOpticalReference:
						case FbxNodeAttribute::eOpticalMarker:
						case FbxNodeAttribute::eCachedEffect:
						case FbxNodeAttribute::eMarker:
						case FbxNodeAttribute::eCameraStereo:
						case FbxNodeAttribute::eCameraSwitcher:
						case FbxNodeAttribute::eNurbs:
						case FbxNodeAttribute::ePatch:
						case FbxNodeAttribute::eNurbsCurve:
						case FbxNodeAttribute::eTrimNurbsSurface:
						case FbxNodeAttribute::eBoundary:
						case FbxNodeAttribute::eNurbsSurface:
						case FbxNodeAttribute::eSubDiv:
						case FbxNodeAttribute::eLine:
							//Unsupported attribute
							break;

						case FbxNodeAttribute::eShape:
							//We do not add a dependency for shape on the scene node since shapes are a MeshNode dependency.
							break;

						case FbxNodeAttribute::eNull:
							// eNull were already process at this point.
							break;

						case FbxNodeAttribute::eSkeleton:
							// Joint were already process at this point.
							break;

						case FbxNodeAttribute::eMesh:
						{
							ParseMesh(NodeContainer, UnrealNode, Node, NodeAttribute);
							break;
						}
						case FbxNodeAttribute::eLODGroup:
						{
							//From here on out everything should be procesed into a MeshLODContainer.
							FString LODContainerUid = UInterchangeMeshLODContainerNode::MakeMeshLODContainerUid(UnrealNode->GetUniqueID());

							UInterchangeMeshLODContainerNode* LODContainerNode = NewObject<UInterchangeMeshLODContainerNode>(&NodeContainer);
							if (!ensure(LODContainerNode))
							{
								return;
							}

							NodeContainer.SetupNode(
								LODContainerNode,
								LODContainerUid,
								UnrealNode->GetDisplayLabel(),
								EInterchangeNodeContainerType::TranslatedAsset
							);

							UnrealNode->SetCustomAssetInstanceUid(LODContainerUid);
							
							FbxAMatrix InverseGlobalTransformMatrixOfLODContainer = SDKScene->GetAnimationEvaluator()->GetNodeGlobalTransform(Node, 0).Inverse();

							const int32 ChildCount = Node->GetChildCount();
							for (int32 ChildIndex = 0; ChildIndex < ChildCount; ++ChildIndex)
							{
								FbxNode* ChildNode = Node->GetChild(ChildIndex);
								ParseMeshLODs(*LODContainerNode, ChildIndex, ChildNode, SDKScene, InverseGlobalTransformMatrixOfLODContainer);
							}

							bAllowChildrenParsing = false;

							break;
						}
						case FbxNodeAttribute::eCamera:
						{
							//Add the Camera asset
							CreateCameraNodeReference(UnrealNode, NodeAttribute, NodeContainer);
							break;
						}
						case FbxNodeAttribute::eLight:
						{
							//Add the Light asset
							CreateLightNodeReference(UnrealNode, NodeAttribute, NodeContainer);
							break;
						}
					}
				}

				if (!UnrealNode->IsA(UInterchangeJointNode::StaticClass()))
				{
					//Set the node default transform
					FbxAMatrix GlobalFbxMatrix = Node->EvaluateGlobalTransform();
					FTransform GlobalTransform = GetConvertedTransform(Node, GlobalFbxMatrix);
					if (FbxNode* ParentNode = Node->GetParent())
					{
						FbxAMatrix GlobalFbxParentMatrix = ParentNode->EvaluateGlobalTransform();
						FbxAMatrix	LocalFbxMatrix = GlobalFbxParentMatrix.Inverse() * GlobalFbxMatrix;
						FTransform LocalTransform = GetConvertedTransform(Node, LocalFbxMatrix);
						UnrealNode->SetCustomLocalTransform(&NodeContainer, LocalTransform);
					}
					else
					{
						//No parent, set the same matrix as the global
						UnrealNode->SetCustomLocalTransform(&NodeContainer, GlobalTransform);
					}
				}
				
				auto AddAnimationTrackNode = [&](EInterchangePropertyTracks PropertyTrack, const FString& CurveNodeName, const FString& PayloadKey, EInterchangeAnimationPayLoadType PayloadType)
				{
					UInterchangeAnimationTrackNode* AnimTrackNode = NewObject< UInterchangeAnimationTrackNode >(&NodeContainer);
					const FString AnimTrackNodeName = FString::Printf(TEXT("%s"), *UnrealNode->GetDisplayLabel()) + CurveNodeName;
					const FString AnimTrackNodeUid = TEXT("\\AnimationTrack\\") + AnimTrackNodeName;

					NodeContainer.SetupNode(AnimTrackNode, AnimTrackNodeUid, AnimTrackNodeName, EInterchangeNodeContainerType::TranslatedAsset);

					AnimTrackNode->SetCustomActorDependencyUid(*UnrealNode->GetUniqueID());
					AnimTrackNode->SetCustomAnimationPayloadKey(PayloadKey, PayloadType);
					AnimTrackNode->SetCustomPropertyTrack(PropertyTrack);
				};

				const UInterchangeFbxSettings* InterchangeFbxSettings = GetDefault<UInterchangeFbxSettings>();

				//Add all Node Attributes for the node
				for(int32 i = 0, Count = Node->GetNodeAttributeCount(); i < Count; ++i)
				{
					FbxNodeAttribute* NodeAttribute = Node->GetNodeAttributeByIndex(i);
					FbxProperty Property = NodeAttribute->GetFirstProperty();

					while(Property.IsValid())
					{
						FbxAnimCurveNode* CurveNode = Property.GetCurveNode();
						EFbxType PropertyType = Property.GetPropertyDataType().GetType();
						if(CurveNode && CurveNode->IsAnimated() && FFbxAnimation::IsFbxPropertyTypeSupported(PropertyType))
						{
							TOptional<FString> PayloadKey;
							TOptional<bool> bIsStepCurve;
							//Attribute is animated, add the curves payload key that represent the attribute animation
							FFbxAnimation::AddNodeAttributeCurvesAnimation(Parser, Node, Property, CurveNode, PayloadContexts, PropertyType, PayloadKey, bIsStepCurve);

							if(PayloadKey.IsSet() && bIsStepCurve.IsSet())
							{
								const char* CurveNodeName = CurveNode->GetName();								
								
								EInterchangeAnimationPayLoadType PayloadType = bIsStepCurve.GetValue() ? EInterchangeAnimationPayLoadType::STEPCURVE : EInterchangeAnimationPayLoadType::CURVE;

								if (EInterchangePropertyTracks PropertyTrack = InterchangeFbxSettings->GetPropertyTrack(CurveNodeName); PropertyTrack != EInterchangePropertyTracks::None)
								{
									AddAnimationTrackNode(PropertyTrack, CurveNodeName, *PayloadKey, PayloadType);
								}

								UnrealNode->SetAnimationCurveTypeForCurveName(CurveNodeName, PayloadType);
							}
						}

						Property = NodeAttribute->GetNextProperty(Property);
					}
				}

				FbxProperty Property = Node->GetFirstProperty();

				//Add all custom Attributes for the node
				while (Property.IsValid())
				{
					EFbxType PropertyType = Property.GetPropertyDataType().GetType();
					if (Property.GetFlag(FbxPropertyFlags::eUserDefined) && FFbxAnimation::IsFbxPropertyTypeSupported(PropertyType))
					{
						FbxAnimCurveNode* CurveNode = Property.GetCurveNode();
						TOptional<FString> PayloadKey;
						TOptional<bool> bIsStepCurve;
						if (CurveNode && CurveNode->IsAnimated())
						{
							//Attribute is animated, add the curves payload key that represent the attribute animation
							FFbxAnimation::AddNodeAttributeCurvesAnimation(Parser, Node, Property, CurveNode, PayloadContexts, PropertyType, PayloadKey, bIsStepCurve);

							if(PayloadKey.IsSet() && bIsStepCurve.IsSet())
							{
								const char* CurveNodeName = CurveNode->GetName();
								
								EInterchangeAnimationPayLoadType PayloadType = bIsStepCurve.GetValue() ? EInterchangeAnimationPayLoadType::STEPCURVE : EInterchangeAnimationPayLoadType::CURVE;

								if (EInterchangePropertyTracks PropertyTrack = InterchangeFbxSettings->GetPropertyTrack(CurveNodeName); PropertyTrack != EInterchangePropertyTracks::None)
								{
									AddAnimationTrackNode(PropertyTrack, CurveNodeName, *PayloadKey, PayloadType);
								}

								UnrealNode->SetAnimationCurveTypeForCurveName(CurveNodeName, PayloadType);
							}
						}

						ProcessCustomAttribute(Parser, UnrealNode, Property, PayloadKey);
					}
					//Inspect next node property
					Property = Node->GetNextProperty(Property);
				}

				if (bAllowChildrenParsing)
				{
					const int32 ChildCount = Node->GetChildCount();
					for (int32 ChildIndex = 0; ChildIndex < ChildCount; ++ChildIndex)
					{
						FbxNode* ChildNode = Node->GetChild(ChildIndex);
						AddHierarchyRecursively(UnrealNode, ChildNode, SDKScene, NodeContainer, PayloadContexts, ForceJointNodes, bBadBindPoseMessageDisplay, FbxJointMeshBindPoseGenerator);
					}
				}
			}

			UInterchangeSceneNode* FFbxScene::CreateUnrealSceneNode(UInterchangeSceneNode* UnrealParentNode
				, FbxNode* Node
				, FbxScene* SDKScene
				, UInterchangeBaseNodeContainer& NodeContainer
				, const TArray<FbxNode*>& ForceJointNodes
				, bool& bBadBindPoseMessageDisplay
				, FFbxJointMeshBindPoseGenerator& FbxJointMeshBindPoseGenerator)
			{
				auto ApplySkeletonAttribute = [this, &SDKScene, &Node, &NodeContainer, &bBadBindPoseMessageDisplay, &FbxJointMeshBindPoseGenerator](UInterchangeJointNode* JointNode, bool bIsOriginalJoint)
					{
						if (FRootJointInfo* RootJointInfo = CommonJointRootNodes.Find(Node))
						{
							if (!RootJointInfo->bValidBindPose)
							{
								JointNode->SetHasInvalidBindPose(true);
							}
						}

						bool bIsRootJoint = false;
						if (const UInterchangeSceneNode* UEParentNode = Cast< UInterchangeSceneNode >(NodeContainer.GetNode(JointNode->GetParentUid())))
						{
							if (!UEParentNode->IsA(UInterchangeJointNode::StaticClass()))
							{
								bIsRootJoint = true;
							}
						}

						FbxNode* ParentNode = Node->GetParent();

						{
							//Get the bind pose transform for this joint
							FbxAMatrix GlobalBindPoseJointMatrix = SDKScene->GetAnimationEvaluator()->GetNodeGlobalTransform(Node, 0);
							TMap<FString, FMatrix> MeshIdToGlobalBindPoseReferenceMap;
							TMap<FString, FMatrix> MeshIdToGlobalBindPoseJointMap;
							FFbxMesh::GetGlobalJointBindPoseTransform(&Parser, SDKScene, Node, FbxJointMeshBindPoseGenerator, GlobalBindPoseJointMatrix, MeshIdToGlobalBindPoseJointMap, MeshIdToGlobalBindPoseReferenceMap, bBadBindPoseMessageDisplay, bIsOriginalJoint, Parser.JointOrientationMatrix);

							FTransform GlobalBindPoseJointTransform = GetConvertedTransform(Node, GlobalBindPoseJointMatrix);
							JointNode->SetMeshUIDToGlobalBindPoseReferenceMap(MeshIdToGlobalBindPoseReferenceMap);
							//Add Transform attribute per Geometry
							//Temporary (to fix in 5.6 main branch): We need to make an API like Get/Set GlobalBindPoseReferenceForMeshUIDs for the joint one
							for (const TPair<FString, FMatrix>& MeshIdAndBindPoseJointMatrix : MeshIdToGlobalBindPoseJointMap)
							{
								FString AttributeKey = TEXT("JointBindPosePerMesh_") + MeshIdAndBindPoseJointMatrix.Key;
								JointNode->RegisterAttribute<FMatrix>(FAttributeKey(AttributeKey), MeshIdAndBindPoseJointMatrix.Value);
							}

							if (ParentNode != nullptr)
							{
								FbxAMatrix GlobalFbxParentMatrix = SDKScene->GetAnimationEvaluator()->GetNodeGlobalTransform(ParentNode, 0);
								TMap<FString, FMatrix> ParentMeshIdToGlobalBindPoseReferenceMap;
								TMap<FString, FMatrix> ParentMeshIdToGlobalBindPoseJointMap;
								FFbxMesh::GetGlobalJointBindPoseTransform(&Parser, SDKScene, ParentNode, FbxJointMeshBindPoseGenerator, GlobalFbxParentMatrix, ParentMeshIdToGlobalBindPoseJointMap, ParentMeshIdToGlobalBindPoseReferenceMap, bBadBindPoseMessageDisplay, bIsOriginalJoint, Parser.JointOrientationMatrix);

								FbxAMatrix LocalFbxMatrix = GlobalFbxParentMatrix.Inverse() * GlobalBindPoseJointMatrix;
								FTransform LocalBindPoseJointTransform = GetConvertedTransform(Node, LocalFbxMatrix);

								JointNode->SetBindPoseLocalTransform(&NodeContainer, LocalBindPoseJointTransform);
							}
							else
							{
								//No parent, set the same matrix has the global
								JointNode->SetBindPoseLocalTransform(&NodeContainer, GlobalBindPoseJointTransform);
							}
						}

						//Get time Zero transform for this joint
						{
							//NOTE:
							// Legacy FBX uses the following Matrix calculation for moving Vertices to T0:
							//		VertexTransformMatrix = ((JointReference * JointBindPose.Inverse()) * (JointT0 * GlobalMeshTransformMatrix.Inverse()));
							//					JointReference				:= MeshIdToGlobalBindPoseReferenceMap * GeometricTransform (each mesh part can have a different bind position for the node)
							//					JointBindPose				:= MeshIdToGlobalBindPoseJointMap (each mesh part can have a different bind position for the node)
							//					T0							:= The joint evaluate a time 0
							//					GlobalMeshTransformMatrix	:= Mesh's Node's GlobalTransform * GeometricTransform (Interchange.SceneNodeTransform)

							//Set the global node transform
							FbxAMatrix GlobalFbxMatrix = SDKScene->GetAnimationEvaluator()->GetNodeGlobalTransform(Node, 0);
							//To be able to use the GlobalTimeZero as is (without JointOrientationMatrix application) for T0 Rebinding, we store the Calculated GlobalTimeZero here.
							JointNode->SetGlobalMatrixForT0Rebinding(FFbxConvert::ConvertMatrix<FMatrix>(GlobalFbxMatrix));
							GlobalFbxMatrix = GlobalFbxMatrix * Parser.JointOrientationMatrix;

							FTransform GlobalTransform = GetConvertedTransform(Node, GlobalFbxMatrix);

							if (ParentNode != nullptr)
							{
								FbxAMatrix GlobalFbxParentMatrix = SDKScene->GetAnimationEvaluator()->GetNodeGlobalTransform(ParentNode, 0);

								if (!bIsRootJoint)
								{
									GlobalFbxParentMatrix = GlobalFbxParentMatrix * Parser.JointOrientationMatrix;
								}

								FbxAMatrix	LocalFbxMatrix = GlobalFbxParentMatrix.Inverse() * GlobalFbxMatrix;
								FTransform LocalTransform = GetConvertedTransform(Node, LocalFbxMatrix);
								JointNode->SetTimeZeroLocalTransform(&NodeContainer, LocalTransform);
							}
							else
							{
								//No parent, set the same matrix has the global
								JointNode->SetTimeZeroLocalTransform(&NodeContainer, GlobalTransform);
							}
						}

						{
							//Set Default transforms with JointOrientation applied:
							FbxAMatrix GlobalFbxMatrix = Node->EvaluateGlobalTransform() * Parser.JointOrientationMatrix;
							if (ParentNode)
							{
								FbxAMatrix GlobalFbxParentMatrix = ParentNode->EvaluateGlobalTransform();
								if (!bIsRootJoint)
								{
									GlobalFbxParentMatrix = GlobalFbxParentMatrix * Parser.JointOrientationMatrix;
								}

								FbxAMatrix	LocalFbxMatrix = GlobalFbxParentMatrix.Inverse() * GlobalFbxMatrix;
								FTransform LocalTransform = GetConvertedTransform(Node, LocalFbxMatrix);
								JointNode->SetCustomLocalTransform(&NodeContainer, LocalTransform);
							}
							else
							{
								FTransform GlobalTransform = GetConvertedTransform(Node, GlobalFbxMatrix);
								//No parent, set the same matrix has the global
								JointNode->SetCustomLocalTransform(&NodeContainer, GlobalTransform);
							}
						}

						FString JointNodeName = Parser.GetFbxHelper()->GetFbxObjectName(Node, true);
						JointNode->SetDisplayLabel(JointNodeName);
					};

				const bool bIsRootNode = Node == SDKScene->GetRootNode();

				UInterchangeSceneNode* UnrealNode;

				bool bIsJoint = false;
				bool bIsNodeContainJointAttribute = false;
				
				const FString NodeName = Parser.GetFbxHelper()->GetFbxObjectName(Node);
				const FString NodeUniqueID = Parser.GetFbxHelper()->GetFbxNodeHierarchyName(Node);
				const FString ParentNodeUid = UnrealParentNode ? UnrealParentNode->GetUniqueID() : TEXT("");

				const bool bNodeUnderCommonRootJoint = IsNodeUnderCommonJointRootNode(Node, CommonJointRootNodes);

				const int32 AttributeCount = Node->GetNodeAttributeCount();

				// We need to know in advance if we have to create a UInterchangeSceneNode or UInterchangeJointNode.
				for (int32 AttributeIndex = 0; AttributeIndex < AttributeCount; ++AttributeIndex)
				{
					FbxNodeAttribute* NodeAttribute = Node->GetNodeAttributeByIndex(AttributeIndex);

					if (!NodeAttribute)
					{
						continue;
					}

					if (NodeAttribute->GetAttributeType() == FbxNodeAttribute::eSkeleton)
					{
						// All eSkeleton/bones are converted to an Interchange Joint node
						bIsJoint = true;
						bIsNodeContainJointAttribute = true;
					}
					else
					{
						if (NodeAttribute->GetAttributeType() == FbxNodeAttribute::eNull && bNodeUnderCommonRootJoint)
						{
							// Any eNull under an eSkeleton or equivalent is considered an eSkeleton too
							bIsJoint = true;
							bIsNodeContainJointAttribute = true;
						}
					}

					if (bIsJoint)
					{
						break;
					}
				}

				if (!bIsJoint) //Can happen when AttributeCount is 0.
				{
					if (ForceJointNodes.Contains(Node) || (!bIsRootNode && bNodeUnderCommonRootJoint))
					{
						// Finally, any node that is forced as a bone or that is under an eSkeleton or equivalent (except the RootNode) is considered non-native bones.
						bIsJoint = true;
						bIsNodeContainJointAttribute = false;
					}
				}

				if (bIsJoint)
				{
					UnrealNode = CreateUnrealSceneNodeInternal<UInterchangeJointNode>(NodeContainer, NodeName, NodeUniqueID, ParentNodeUid);
					ApplySkeletonAttribute(CastChecked<UInterchangeJointNode>(UnrealNode), bIsNodeContainJointAttribute);
				}
				else
				{
					UnrealNode = CreateUnrealSceneNodeInternal<UInterchangeSceneNode>(NodeContainer, NodeName, NodeUniqueID, ParentNodeUid);
				}

				if (!ensure(UnrealNode))
				{
					return nullptr;
				}

				if (bIsRootNode)
				{
					UnrealNode->SetCustomIsSceneRoot(true);
				}

				UnrealNode->SetCustomActorVisibility(Node->GetVisibility());

				return UnrealNode;
			}

			template<typename T>
			T* FFbxScene::CreateUnrealSceneNodeInternal(UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeName, const FString& NodeUID, const FString& ParentNodeUID)
			{
				static_assert(TIsDerivedFrom<T, UInterchangeSceneNode>::IsDerived, "T must derive from UInterchangeSceneNode");

				T* TransformNode = NewObject<T>(&NodeContainer, NAME_None);
				if (!ensure(TransformNode))
				{
					UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
					Message->Text = LOCTEXT("NodeAllocationError", "Unable to allocate a node when importing FBX.");
					return nullptr;
				}
				NodeContainer.SetupNode(TransformNode, NodeUID, NodeName, EInterchangeNodeContainerType::TranslatedScene, ParentNodeUID);

				return TransformNode;
			}
			
			bool FFbxScene::IsValidBindPose(FbxScene* SDKScene, FbxNode* RootJoint)
			{
				int32 PoseCount = SDKScene->GetPoseCount();
				if (PoseCount == 0)
				{
					SDKScene->GetFbxManager()->CreateMissingBindPoses(SDKScene);
					PoseCount = SDKScene->GetPoseCount();
				}
				
				TArray<FbxNode*> NodeArray;
				RecursiveHelper::RecursiveFillChildrenFbxNode(RootJoint, NodeArray);

				for (int32 PoseIndex = 0; PoseIndex < PoseCount; PoseIndex++)
				{
					FbxPose* CurrentPose = SDKScene->GetPose(PoseIndex);

					// current pose is bind pose, 
					if (CurrentPose && CurrentPose->IsBindPose())
					{
						// IsValidBindPose doesn't work reliably
						// It checks all the parent chain(regardless root given), and if the parent doesn't have correct bind pose, it fails
						// It causes more false positive issues than the real issue we have to worry about
						// If you'd like to try this, set CHECK_VALID_BIND_POSE to 1, and try the error message
						// when Autodesk fixes this bug, then we might be able to re-open this
						FString PoseName = CurrentPose->GetName();
						// all error report status
						FbxStatus Status;

						// it does not make any difference of checking with different node
						for(FbxNode* Current : NodeArray)
						{
							FString CurrentName = Current->GetName();
							FbxArray<FbxNode*> pMissingAncestors, pMissingDeformers, pMissingDeformersAncestors, pWrongMatrices;

							if (CurrentPose->IsValidBindPoseVerbose(Current, pMissingAncestors, pMissingDeformers, pMissingDeformersAncestors, pWrongMatrices, 0.0001, &Status))
							{
								return true;
							}
							else
							{
								// first try to fix up
								// add missing ancestors
								for (int i = 0; i < pMissingAncestors.GetCount(); i++)
								{
									FbxAMatrix mat = pMissingAncestors.GetAt(i)->EvaluateGlobalTransform(FBXSDK_TIME_ZERO);
									CurrentPose->Add(pMissingAncestors.GetAt(i), mat);
								}

								pMissingAncestors.Clear();
								pMissingDeformers.Clear();
								pMissingDeformersAncestors.Clear();
								pWrongMatrices.Clear();

								// check it again
								if (CurrentPose->IsValidBindPose(Current))
								{
									return true;
								}
								else
								{
									// first try to find parent who is null group and see if you can try test it again
									FbxNode* ParentNode = Current->GetParent();
									while (ParentNode)
									{
										FbxNodeAttribute* Attr = ParentNode->GetNodeAttribute();
										if (Attr && Attr->GetAttributeType() == FbxNodeAttribute::eNull)
										{
											// found it 
											break;
										}

										// find next parent
										ParentNode = ParentNode->GetParent();
									}

									if (ParentNode && CurrentPose->IsValidBindPose(ParentNode))
									{
										return true;
									}
								}
							}
						}
					}
				}
				return false;
			}


			void FFbxScene::AddHierarchy(FbxScene* SDKScene, UInterchangeBaseNodeContainer& NodeContainer, TMap<FString, TSharedPtr<FPayloadContextBase, ESPMode::ThreadSafe>>& PayloadContexts)
			{
				FbxNode* RootNode = SDKScene->GetRootNode();

				//Some fbx file have node without attribute that are link in cluster,
				//We must consider those node has joint
				TArray<FbxNode*> ForceJointNodes = FindForceJointNode(SDKScene);

				//Cache the common root joint
				FillCommonJointRootNode(SDKScene, ForceJointNodes);

				FFbxJointMeshBindPoseGenerator FbxJointMeshBindPoseGenerator(SDKScene, Parser, Parser.GetUseFbxClustersForBindPoseMatrices());

				bool bBadBindPoseMessageDisplay = false;
				AddHierarchyRecursively(nullptr, RootNode, SDKScene, NodeContainer, PayloadContexts, ForceJointNodes, bBadBindPoseMessageDisplay, FbxJointMeshBindPoseGenerator);

				int32 NodeCount = SDKScene->GetNodeCount();
				for (int32 NodeIndex = 0; NodeIndex < NodeCount; ++NodeIndex)
				{
					if (FbxNode* Node = SDKScene->GetNode(NodeIndex))
					{
						if (Node != RootNode)
						{
							if (Node->GetParent() == nullptr)
							{
								AddHierarchyRecursively(nullptr, Node, SDKScene, NodeContainer, PayloadContexts, ForceJointNodes, bBadBindPoseMessageDisplay, FbxJointMeshBindPoseGenerator);
							}
						}
					}
				}
			}

			void FFbxScene::AddRigidAnimation(FbxNode* Node
				, const FbxScene* SDKScene
				, const int32& AnimationIndex
				, const UInterchangeSceneNode* UnrealNode
				, UInterchangeBaseNodeContainer& NodeContainer
				, TMap<FString, TSharedPtr<FPayloadContextBase, ESPMode::ThreadSafe>>& PayloadContexts) const
			{
				FbxAnimCurveNode* TranlsationCurveNode = nullptr;
				FbxAnimCurveNode* RotationCurveNode = nullptr;
				FbxAnimCurveNode* ScaleCurveNode = nullptr;

				FbxAnimStack* AnimationStack = (FbxAnimStack*)SDKScene->GetSrcObject<FbxAnimStack>(AnimationIndex);

				FbxProperty Property = Node->GetFirstProperty();
				while (Property.IsValid())
				{
					EFbxType PropertyType = Property.GetPropertyDataType().GetType();

					if (FFbxAnimation::IsFbxPropertyTypeSupported(PropertyType))
					{
						FbxAnimCurveNode* CurveNode = Property.GetCurveNode(AnimationStack);

						//only translation/rotation/scale is supported
						if (CurveNode && CurveNode->IsAnimated())
						{
							//(currently FBXSDK_CURVENODE_TRANSFORM is not supported for Curve based animations)

							const char* CurveNodeName = CurveNode->GetName(); //which lets us know the component that we are animating:
							if (std::strcmp(CurveNodeName, FBXSDK_CURVENODE_TRANSLATION) == 0)
							{
								TranlsationCurveNode = CurveNode;
							}
							else if (std::strcmp(CurveNodeName, FBXSDK_CURVENODE_ROTATION) == 0)
							{
								RotationCurveNode = CurveNode;
							}
							else if (std::strcmp(CurveNodeName, FBXSDK_CURVENODE_SCALING) == 0)
							{
								ScaleCurveNode = CurveNode;
							}
						}
					}
					Property = Node->GetNextProperty(Property);
				}

				//
				constexpr int32 TranslationChannel = 0x0001 | 0x0002 | 0x0004;
				constexpr int32 RotationChannel = 0x0008 | 0x0010 | 0x0020;
				constexpr int32 ScaleChannel = 0x0040 | 0x0080 | 0x0100;

				int32 UsedChannels = 0;
				if (TranlsationCurveNode)
				{
					UsedChannels |= TranslationChannel;
				}
				if (RotationCurveNode)
				{
					UsedChannels |= RotationChannel;
				}
				if (ScaleCurveNode)
				{
					UsedChannels |= ScaleChannel;
				}

				if (UsedChannels)
				{
					TOptional<FString> PayloadKey;

					FFbxAnimation::AddRigidTransformAnimation(Parser, Node, AnimationIndex, TranlsationCurveNode, RotationCurveNode, ScaleCurveNode, PayloadContexts, PayloadKey);

					if (PayloadKey.IsSet())
					{
						UInterchangeTransformAnimationTrackNode* TransformAnimTrackNode = NewObject< UInterchangeTransformAnimationTrackNode >(&NodeContainer);

						const FString TransformAnimTrackNodeName = FString::Printf(TEXT("%s"), *UnrealNode->GetDisplayLabel());
						const FString TransformAnimTrackNodeUid = TEXT("\\AnimationTrack\\") + TransformAnimTrackNodeName;

						NodeContainer.SetupNode(TransformAnimTrackNode, TransformAnimTrackNodeUid, TransformAnimTrackNodeName, EInterchangeNodeContainerType::TranslatedAsset);
						TransformAnimTrackNode->SetCustomActorDependencyUid(*UnrealNode->GetUniqueID());
						TransformAnimTrackNode->SetCustomAnimationPayloadKey(PayloadKey.GetValue(), EInterchangeAnimationPayLoadType::CURVE);
						TransformAnimTrackNode->SetCustomUsedChannels(UsedChannels);

						ProcessCustomAttributes(Parser, Node, TransformAnimTrackNode);
					}
				}
			}

			FbxNode* FFbxScene::Internal_GetRootSkeleton(FbxScene* SDKScene, FbxNode* Link, bool bIsCreatorBlender)
			{
				FbxNode* RootBone = Link;

				// get Unreal skeleton root
				// mesh and dummy are used as bone if they are in the skeleton hierarchy
				while (RootBone && RootBone->GetParent())
				{
					bool bIsBlenderArmatureBone = false;
					if (bIsCreatorBlender)
					{
						//Hack to support armature dummy node from blender
						//Users do not want the null attribute node named armature which is the parent of the real root bone in blender fbx file
						//This is a hack since if a rigid mesh group root node is named "armature" it will be skip
						const FString RootBoneParentName(RootBone->GetParent()->GetName());
						FbxNode* GrandFather = RootBone->GetParent()->GetParent();
						bIsBlenderArmatureBone = (GrandFather == nullptr || GrandFather == SDKScene->GetRootNode()) && (RootBoneParentName.Compare(TEXT("armature"), ESearchCase::IgnoreCase) == 0);
					}

					FbxNodeAttribute* Attr = RootBone->GetParent()->GetNodeAttribute();
					if (Attr &&
						(Attr->GetAttributeType() == FbxNodeAttribute::eMesh ||
							(Attr->GetAttributeType() == FbxNodeAttribute::eNull && !bIsBlenderArmatureBone) ||
							Attr->GetAttributeType() == FbxNodeAttribute::eSkeleton) &&
						RootBone->GetParent() != SDKScene->GetRootNode())
					{
						// in some case, skeletal mesh can be ancestor of bones
						// this avoids this situation
						if (Attr->GetAttributeType() == FbxNodeAttribute::eMesh)
						{
							FbxMesh* Mesh = (FbxMesh*)Attr;
							if (Mesh->GetDeformerCount(FbxDeformer::eSkin) > 0)
							{
								break;
							}
						}

						RootBone = RootBone->GetParent();
					}
					else
					{
						break;
					}
				}

				return RootBone;
			}

			void FFbxScene::FillCommonJointRootNode(FbxScene* SDKScene, const TArray<FbxNode*>& ForceJointNodes)
			{
				//Process the ForceJointNodes and any skeleton joint node
				int32 NodeCount = SDKScene->GetNodeCount();
				for (int32 NodeIndex = 0; NodeIndex < NodeCount; ++NodeIndex)
				{
					if (FbxNode* Node = SDKScene->GetNode(NodeIndex))
					{
						bool bProcessNode = ForceJointNodes.Contains(Node);
						if (!bProcessNode)
						{
							const int32 AttributeCount = Node->GetNodeAttributeCount();
							for (int32 AttributeIndex = 0; AttributeIndex < AttributeCount; ++AttributeIndex)
							{
								if (Node->GetNodeAttributeByIndex(AttributeIndex)->GetAttributeType() == FbxNodeAttribute::eSkeleton)
								{
									bProcessNode = true;
									break;
								}
							}
						}
						if (bProcessNode)
						{
							if (FbxNode* Root = Internal_GetRootSkeleton(SDKScene, Node, Parser.IsCreatorBlender()))
							{
								if (!CommonJointRootNodes.Contains(Root))
								{
									CommonJointRootNodes.Add(Root, FFbxScene::FRootJointInfo(IsValidBindPose(SDKScene, Root)));
								}
							}
						}
					}
				}
			}

			TArray<FbxNode*> FFbxScene::FindForceJointNode(FbxScene* SDKScene)
			{
				TArray<FbxNode*> ForceJointNodes;

				int32 GeometryCount = SDKScene->GetGeometryCount();
				for (int32 GeometryIndex = 0; GeometryIndex < GeometryCount; ++GeometryIndex)
				{
					FbxGeometry* Geometry = SDKScene->GetGeometry(GeometryIndex);
					if (Geometry->GetAttributeType() != FbxNodeAttribute::eMesh)
					{
						continue;
					}
					const FbxMesh* Mesh = static_cast<FbxMesh*>(Geometry);
					if (!Mesh)
					{
						continue;
					}
					const int32 SkinDeformerCount = Mesh->GetDeformerCount(FbxDeformer::eSkin);
					for (int32 DeformerIndex = 0; DeformerIndex < SkinDeformerCount; DeformerIndex++)
					{
						FbxSkin* Skin = (FbxSkin*)Mesh->GetDeformer(DeformerIndex, FbxDeformer::eSkin);
						if (!ensure(Skin))
						{
							continue;
						}
						const int32 ClusterCount = Skin->GetClusterCount();
						for (int32 ClusterIndex = 0; ClusterIndex < ClusterCount; ClusterIndex++)
						{
							FbxCluster* Cluster = Skin->GetCluster(ClusterIndex);
							// When Maya plug-in exports rigid binding, it will generate "CompensationCluster" for each ancestor links.
							// FBX writes these "CompensationCluster" out. The CompensationCluster also has weight 1 for vertices.
							// Unreal importer should skip these clusters.
							if (!Cluster || (FCStringAnsi::Strcmp(Cluster->GetUserDataID(), "Maya_ClusterHint") == 0 && FCStringAnsi::Strcmp(Cluster->GetUserData(), "CompensationCluster") == 0))
							{
								continue;
							}
							ForceJointNodes.AddUnique(Cluster->GetLink());
						}
					}
				}

				return ForceJointNodes;
			}

			void FFbxScene::AddAnimationRecursively(FbxNode* Node
				, FbxScene* SDKScene
				, UInterchangeBaseNodeContainer& NodeContainer
				, TMap<FString, TSharedPtr<FPayloadContextBase, ESPMode::ThreadSafe>>& PayloadContexts
				, UInterchangeSkeletalAnimationTrackNode* SkeletalAnimationTrackNode, bool SkeletalAnimationAddedToContainer
				, const FString& RootSceneNodeUid, const TSet<FString>& SkeletonRootNodeUids
				, const int32& AnimationIndex
				, const TArray<FbxNode*>& ForceJointNodes)
			{
				FString NodeUniqueID = Parser.GetFbxHelper()->GetFbxNodeHierarchyName(Node);
				const bool bIsRootNode = Node == SDKScene->GetRootNode();
				if (UInterchangeSceneNode* UnrealNode = const_cast<UInterchangeSceneNode*>(Cast< UInterchangeSceneNode >(NodeContainer.GetNode(NodeUniqueID))))
				{
					bool HasSkeletonAttribute = false;
					auto ApplySkeletonAttribute = [this, &SDKScene, &HasSkeletonAttribute, &SkeletonRootNodeUids, &NodeUniqueID, &SkeletalAnimationTrackNode, &UnrealNode, &AnimationIndex, &NodeContainer, &Node]()
					{
						HasSkeletonAttribute = true;
						if (SkeletonRootNodeUids.Contains(NodeUniqueID))
						{
							FbxAnimStack* CurrentAnimationStack = (FbxAnimStack*)SDKScene->GetSrcObject<FbxAnimStack>(AnimationIndex);
							FString TrackNodeUid = CreateTrackNodeUid(UnrealNode->GetUniqueID(), AnimationIndex);
							NodeContainer.BreakableIterateNodesOfType<UInterchangeSkeletalAnimationTrackNode>([&TrackNodeUid, &SkeletalAnimationTrackNode](const FString& NodeUid, UInterchangeSkeletalAnimationTrackNode* Node)
							{
								if (NodeUid == TrackNodeUid)
								{
									SkeletalAnimationTrackNode = Node;
									return true;
								}
								return false;
							});

							if(!SkeletalAnimationTrackNode)
							{
								SkeletalAnimationTrackNode = NewObject< UInterchangeSkeletalAnimationTrackNode >(&NodeContainer);
								//In this specific instance, the processing of AnimationSequence and their existance are dependent on the "up-coming" nodes 
								// (aka the root skeleton node should create the SkeletalAnimationTrackNode, but it really only should exist if at least 1 of the joints of the Skeleton has animation.)
								// (this could potentially be done in a neater way by passing down the stack the SkeleotnNodeUid instead of a potentially 'fake' UInterchangeSkeletalAnimationTrackNode.)
								SkeletalAnimationTrackNode->InitializeNode(TrackNodeUid, Parser.GetFbxHelper()->GetFbxObjectName(CurrentAnimationStack), EInterchangeNodeContainerType::TranslatedAsset);
							}

							double FrameRate = FbxTime::GetFrameRate(SDKScene->GetGlobalSettings().GetTimeMode());
							SkeletalAnimationTrackNode->SetCustomAnimationSampleRate(FrameRate);

							SkeletalAnimationTrackNode->SetCustomSkeletonNodeUid(UnrealNode->GetUniqueID());

							//Calculate AnimationTime:

							//Node animated time interval (Animated time for all concern bones under the skeletalmesh root bone)
							FbxTimeSpan AnimatedInterval(FBXSDK_TIME_INFINITE, FBXSDK_TIME_MINUS_INFINITE);
							Node->GetAnimationInterval(AnimatedInterval, CurrentAnimationStack);
							SkeletalAnimationTrackNode->SetCustomAnimationStartTime(AnimatedInterval.GetStart().GetSecondDouble());
							SkeletalAnimationTrackNode->SetCustomAnimationStopTime(AnimatedInterval.GetStop().GetSecondDouble());

							//Animation stack time interval (Source timeline interval)
							// Note - can be less or more the bones animationlength, it can be configure when exporting by manipulating the timeline in the dcc (Maya have a range cursor on the timeline)
							FbxTimeSpan AnimStackInterval(FBXSDK_TIME_INFINITE, FBXSDK_TIME_MINUS_INFINITE);
							AnimStackInterval = CurrentAnimationStack->GetLocalTimeSpan();

							//AnimStackInterval represent the animation time line setup in the DCC
							SkeletalAnimationTrackNode->SetCustomSourceTimelineAnimationStartTime(AnimStackInterval.GetStart().GetSecondDouble());
							SkeletalAnimationTrackNode->SetCustomSourceTimelineAnimationStopTime(AnimStackInterval.GetStop().GetSecondDouble());
							return true;
						}

						return false;
					};

					bool bIsNodeContainJointAttribute = false;

					int32 AttributeCount = Node->GetNodeAttributeCount();

					bool bNewSkeletalAnimationStarted = false;

					for (int32 AttributeIndex = 0; AttributeIndex < AttributeCount && !HasSkeletonAttribute; ++AttributeIndex)
					{
						FbxNodeAttribute* NodeAttribute = Node->GetNodeAttributeByIndex(AttributeIndex);
						switch (NodeAttribute->GetAttributeType())
						{
						case FbxNodeAttribute::eNull:
							if (!IsNodeUnderCommonJointRootNode(Node, CommonJointRootNodes))
							{
								//eNull node not under any joint are not joint
								break;
							}
						case FbxNodeAttribute::eSkeleton:
							bIsNodeContainJointAttribute = true;
							bNewSkeletalAnimationStarted = ApplySkeletonAttribute() || bNewSkeletalAnimationStarted;
							break;
						default:
							break;
						}
					}

					if (!bIsNodeContainJointAttribute)
					{
						//Make sure to treat the node like a joint if it's in the ForcejointNodes array
						if (ForceJointNodes.Contains(Node))
						{
							bNewSkeletalAnimationStarted = ApplySkeletonAttribute() || bNewSkeletalAnimationStarted;
						}
						else if (!bIsRootNode && IsNodeUnderCommonJointRootNode(Node, CommonJointRootNodes))
						{
							bNewSkeletalAnimationStarted = ApplySkeletonAttribute() || bNewSkeletalAnimationStarted;
						}
					}

					if (bNewSkeletalAnimationStarted)
					{
						ProcessCustomAttributes(Parser, Node, SkeletalAnimationTrackNode);
					}

					if (!HasSkeletonAttribute)
					{
						//in case the joint node "hierarchy finished" then the SkeletalAnimationTrackNode should be reset:
						//as on the next occurrence of a Joint node a New skeleton will start:
						SkeletalAnimationTrackNode = nullptr;

						SkeletalAnimationAddedToContainer = false;
					}
					else if (SkeletalAnimationTrackNode)
					{
						//Scene node transform can be animated, add the transform animation payload key.
						if (FFbxAnimation::AddSkeletalTransformAnimation(NodeContainer, SDKScene, Parser, Node, UnrealNode, PayloadContexts, SkeletalAnimationTrackNode, AnimationIndex, bNewSkeletalAnimationStarted)
							&& !SkeletalAnimationAddedToContainer)
						{
							SkeletalAnimationAddedToContainer = true;
							NodeContainer.AddNode(SkeletalAnimationTrackNode);
						}
					}

					AddRigidAnimation(Node, SDKScene, AnimationIndex, UnrealNode, NodeContainer, PayloadContexts);
				}

				const int32 ChildCount = Node->GetChildCount();
				for (int32 ChildIndex = 0; ChildIndex < ChildCount; ++ChildIndex)
				{
					FbxNode* ChildNode = Node->GetChild(ChildIndex);
					AddAnimationRecursively(ChildNode, SDKScene, NodeContainer, PayloadContexts, SkeletalAnimationTrackNode, SkeletalAnimationAddedToContainer, RootSceneNodeUid, SkeletonRootNodeUids, AnimationIndex, ForceJointNodes);
				}
			}

			void FFbxScene::AddAnimation(FbxScene* SDKScene, UInterchangeBaseNodeContainer& NodeContainer, TMap<FString, TSharedPtr<FPayloadContextBase, ESPMode::ThreadSafe>>& PayloadContexts)
			{
				FbxNode* RootNode = SDKScene->GetRootNode();
				FString RootSceneNodeUniqueID = Parser.GetFbxHelper()->GetFbxNodeHierarchyName(RootNode);

				//Some fbx file have node without attribute that are link in cluster,
				//We must consider those node has joint
				TArray<FbxNode*> ForceJointNodes = FindForceJointNode(SDKScene);

				//acquire Skeletal Node Uids from Meshes (via the skeletondependencies: )
				TSet<FString> SkeletonRootNodeUids;
				NodeContainer.IterateNodesOfType<UInterchangeMeshNode>([&](const FString& NodeUid, UInterchangeMeshNode* MeshNode)
					{
						//Find the root joint for this MeshGeometry
						FString JointNodeUid;
						FString ParentNodeUid;
						MeshNode->GetSkeletonDependency(0, JointNodeUid);
						ParentNodeUid = JointNodeUid;
						
						while (!JointNodeUid.Equals(UInterchangeBaseNode::InvalidNodeUid()))
						{
							if (const UInterchangeSceneNode* Node = Cast< UInterchangeSceneNode >(NodeContainer.GetNode(ParentNodeUid)))
							{
								if (Node->IsA(UInterchangeJointNode::StaticClass()))
								{
									JointNodeUid = ParentNodeUid;
									ParentNodeUid = Node->GetParentUid();
								}
								else
								{
									break;
								}
							}
							else
							{
								break;
							}
						}

						if (!JointNodeUid.Equals(UInterchangeBaseNode::InvalidNodeUid()))
						{
							SkeletonRootNodeUids.Add(JointNodeUid);
						}
					});
				
				//In case we import animation only and there is no meshes
				if (SkeletonRootNodeUids.Num() == 0)
				{
					NodeContainer.IterateNodesOfType<UInterchangeSceneNode>([&](const FString& NodeUid, UInterchangeSceneNode* SceneNode)
						{
							if (SceneNode->IsA(UInterchangeJointNode::StaticClass()))
							{
								//Find the root joint for this MeshGeometry
								FString JointNodeUid = NodeUid;
								FString ParentNodeUid = SceneNode->GetParentUid();

								while (!JointNodeUid.Equals(UInterchangeBaseNode::InvalidNodeUid()))
								{
									if (const UInterchangeSceneNode* Node = Cast< UInterchangeSceneNode >(NodeContainer.GetNode(ParentNodeUid)))
									{
										if (Node->IsA(UInterchangeJointNode::StaticClass()))
										{
											JointNodeUid = ParentNodeUid;
											ParentNodeUid = Node->GetParentUid();
										}
										else
										{
											break;
										}
									}
									else
									{
										break;
									}
								}

								if (!JointNodeUid.Equals(UInterchangeBaseNode::InvalidNodeUid()))
								{
									SkeletonRootNodeUids.Add(JointNodeUid);
								}
							}
						});
				}

				int32 NumAnimations = SDKScene->GetSrcObjectCount<FbxAnimStack>();


				for (int32 AnimationIndex = 0; AnimationIndex < NumAnimations; AnimationIndex++)
				{
					AddAnimationRecursively(RootNode, SDKScene, NodeContainer, PayloadContexts, nullptr, false, RootSceneNodeUniqueID, SkeletonRootNodeUids, AnimationIndex, ForceJointNodes);
					int32 NodeCount = SDKScene->GetNodeCount();
					for (int32 NodeIndex = 0; NodeIndex < NodeCount; ++NodeIndex)
					{
						if (FbxNode* Node = SDKScene->GetNode(NodeIndex))
						{
							if (Node != RootNode)
							{
								if (Node->GetParent() == nullptr)
								{
									AddAnimationRecursively(Node, SDKScene, NodeContainer, PayloadContexts, nullptr, false, RootSceneNodeUniqueID, SkeletonRootNodeUids, AnimationIndex, ForceJointNodes);
								}
							}
						}
					}
				}

				TArray<FString> TransformAnimTrackNodeUids;
				NodeContainer.IterateNodesOfType<UInterchangeAnimationTrackNode>([&](const FString& NodeUid, UInterchangeAnimationTrackNode* TransformAnimationTrackNode)
					{
						TransformAnimTrackNodeUids.Add(NodeUid);
					});

				//Only one Track Set Node per fbx file:
				if (TransformAnimTrackNodeUids.Num() > 0)
				{
					UInterchangeAnimationTrackSetNode* TrackSetNode = NewObject< UInterchangeAnimationTrackSetNode >(&NodeContainer);

					double FrameRate = FbxTime::GetFrameRate(SDKScene->GetGlobalSettings().GetTimeMode());
					TrackSetNode->SetCustomFrameRate(FrameRate);

					const FString AnimTrackSetNodeUid = TEXT("\\Animation\\") + FString(RootNode->GetName());
					const FString AnimTrackSetNodeDisplayLabel = FString(RootNode->GetName()) + TEXT("_TrackSetNode");

					NodeContainer.SetupNode(TrackSetNode, AnimTrackSetNodeUid, AnimTrackSetNodeDisplayLabel, EInterchangeNodeContainerType::TranslatedAsset);

					for (const FString& TransformAnimTrackNodeUid : TransformAnimTrackNodeUids)
					{
						TrackSetNode->AddCustomAnimationTrackUid(TransformAnimTrackNodeUid);
					}
				}
			}

			void FFbxScene::AddMorphTargetAnimations(FbxScene* SDKScene, UInterchangeBaseNodeContainer& NodeContainer, TMap<FString, TSharedPtr<FPayloadContextBase, ESPMode::ThreadSafe>>& PayloadContexts, const TArray<FMorphTargetAnimationBuildingData>& MorphTargetAnimationsBuildingData)
			{
				//Group the Morph Target animations based on SkeletonNodeUid and AnimationIndex
				TMap<const UInterchangeSceneNode*, TMap<int32, TArray<FMorphTargetAnimationBuildingData>>> MorphTargetAnimationsBuildingDataGrouped;

				TMap<FString, FString> EvaluatedJoints;
				for (const FMorphTargetAnimationBuildingData& MorphTargetAnimationBuildingData : MorphTargetAnimationsBuildingData)
				{
					if (MorphTargetAnimationBuildingData.StartTime == MorphTargetAnimationBuildingData.StopTime)
					{
						//in case the interval is 0 skip the MorphTargetAnimation.
						continue;
					}
					
					TSet<FString> SkeletonUids;
					
					if (MorphTargetAnimationBuildingData.InterchangeMeshNode->IsSkinnedMesh())
					{
						//Find the root joint(s) for this MeshGeometry
						TArray<FString> SkeletonDependencies;
						MorphTargetAnimationBuildingData.InterchangeMeshNode->GetSkeletonDependencies(SkeletonDependencies);
						for (const FString& SkeletonDependency : SkeletonDependencies)
						{
							FString JointNodeUid = SkeletonDependency;
							FString& RootJointNodeForJoint = EvaluatedJoints.FindOrAdd(JointNodeUid);
							if (!RootJointNodeForJoint.IsEmpty())
							{
								SkeletonUids.Add(RootJointNodeForJoint);
							}
							else
							{
								FString ParentNodeUid = SkeletonDependency;
								while (!JointNodeUid.Equals(UInterchangeBaseNode::InvalidNodeUid()))
								{
									if (const UInterchangeSceneNode* Node = Cast< UInterchangeSceneNode >(NodeContainer.GetNode(ParentNodeUid)))
									{
										if (Node->IsA(UInterchangeJointNode::StaticClass()))
										{
											JointNodeUid = ParentNodeUid;
											ParentNodeUid = Node->GetParentUid();
										}
										else
										{
											break;
										}
									}
									else
									{
										break;
									}
								}

								if (!JointNodeUid.Equals(UInterchangeBaseNode::InvalidNodeUid()))
								{
									RootJointNodeForJoint = JointNodeUid;
									SkeletonUids.Add(JointNodeUid);
								}
							}
						}
					}
					else
					{
						//Find MeshInstances: where CustomAssetInstanceUid == MeshNode->GetUniqueID
						// For every occurance create a morphtarget entry with given MeshNode->GetUniqueID 
						NodeContainer.IterateNodesOfType<UInterchangeSceneNode>([&](const FString& NodeUid, UInterchangeSceneNode* SceneNode)
							{
								FString AssetInstanceUid;
								if (SceneNode->GetCustomAssetInstanceUid(AssetInstanceUid) && AssetInstanceUid == MorphTargetAnimationBuildingData.InterchangeMeshNode->GetUniqueID())
								{
									SkeletonUids.Add(SceneNode->GetUniqueID());
								}
							});
						
						if (SkeletonUids.Num() == 0)
						{
							//If it is not skinned and does not have an instantation, then it is presumed to get used on the RootNode level.
							FbxNode* RootNode = SDKScene->GetRootNode();
							SkeletonUids.Add(Parser.GetFbxHelper()->GetFbxNodeHierarchyName(RootNode));
						}						
					}

					for (const FString& SkeletonUid : SkeletonUids)
					{
						if (const UInterchangeSceneNode* SkeletonNode = Cast< UInterchangeSceneNode >(NodeContainer.GetNode(SkeletonUid)))
						{
							//For the given skeleton:
							TMap<int32, TArray<FMorphTargetAnimationBuildingData>>& MorphTargetAnimationPerAnimationIndex = MorphTargetAnimationsBuildingDataGrouped.FindOrAdd(SkeletonNode);
							//For the given skeleton and animationindex:
							TArray<FMorphTargetAnimationBuildingData>& MorphTargetAnimations = MorphTargetAnimationPerAnimationIndex.FindOrAdd(MorphTargetAnimationBuildingData.AnimationIndex);
							MorphTargetAnimations.Add(MorphTargetAnimationBuildingData);
						}
					}
				}
				
				for (const TPair<const UInterchangeSceneNode*, TMap<int32, TArray<FMorphTargetAnimationBuildingData>>>& MorphTargetAnimationBuildingDataGrouped : MorphTargetAnimationsBuildingDataGrouped)
				{
					const UInterchangeSceneNode* SkeletonNode = MorphTargetAnimationBuildingDataGrouped.Key;
					FString SkeletonDisplayLabel = SkeletonNode->GetDisplayLabel();

					for (const TPair<int32, TArray<FMorphTargetAnimationBuildingData>>& MorphTargetAnimationsBuildingDataPerSkeleton : MorphTargetAnimationBuildingDataGrouped.Value)
					{
						int32 AnimationIndex = MorphTargetAnimationsBuildingDataPerSkeleton.Key;

						FbxAnimStack* CurrentAnimationStack = (FbxAnimStack*)SDKScene->GetSrcObject<FbxAnimStack>(AnimationIndex);
						FbxTimeSpan TimeSpan = CurrentAnimationStack->GetLocalTimeSpan();

						UInterchangeSkeletalAnimationTrackNode* SkeletalAnimationTrackNode = nullptr;
						FString TrackNodeUid = CreateTrackNodeUid(SkeletonNode->GetUniqueID(), AnimationIndex);
						NodeContainer.BreakableIterateNodesOfType<UInterchangeSkeletalAnimationTrackNode>([&TrackNodeUid, &SkeletalAnimationTrackNode](const FString& NodeUid, UInterchangeSkeletalAnimationTrackNode* Node)
						{
							if (NodeUid == TrackNodeUid)
							{
								SkeletalAnimationTrackNode = Node;
								return true;
							}
							return false;
						});

						if(!SkeletalAnimationTrackNode)
						{
							SkeletalAnimationTrackNode = NewObject< UInterchangeSkeletalAnimationTrackNode >(&NodeContainer);
							NodeContainer.SetupNode(SkeletalAnimationTrackNode, TrackNodeUid, Parser.GetFbxHelper()->GetFbxObjectName(CurrentAnimationStack), EInterchangeNodeContainerType::TranslatedAsset);

							double FrameRate = FbxTime::GetFrameRate(SDKScene->GetGlobalSettings().GetTimeMode());
							SkeletalAnimationTrackNode->SetCustomAnimationSampleRate(FrameRate);

							SkeletalAnimationTrackNode->SetCustomSkeletonNodeUid(SkeletonNode->GetUniqueID());

							SkeletalAnimationTrackNode->SetCustomAnimationStartTime(TimeSpan.GetStart().GetSecondDouble());
							SkeletalAnimationTrackNode->SetCustomAnimationStopTime(TimeSpan.GetStop().GetSecondDouble());
						}

						ProcessCustomAttributes(Parser, CurrentAnimationStack, SkeletalAnimationTrackNode);
						
						for (const FMorphTargetAnimationBuildingData& MorphTargetAnimationBuildingDataPerSkeletonPerAnimationIndex : MorphTargetAnimationsBuildingDataPerSkeleton.Value)
						{
							UE::Interchange::Private::FFbxAnimation::AddMorphTargetCurvesAnimation(SDKScene, Parser, SkeletalAnimationTrackNode, PayloadContexts, MorphTargetAnimationBuildingDataPerSkeletonPerAnimationIndex);
						}
					}
				}
			}
		} //ns Private
	} //ns Interchange
}//ns UE

#undef LOCTEXT_NAMESPACE
