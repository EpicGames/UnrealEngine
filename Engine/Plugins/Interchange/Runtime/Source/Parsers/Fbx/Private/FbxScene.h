// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FbxInclude.h"

/** Forward declarations */
namespace UE::Interchange::Private
{
	class FPayloadContextBase;
	struct FFbxJointMeshBindPoseGenerator;
}
class UInterchangeBaseNodeContainer;
class UInterchangeMeshLODContainerNode;
class UInterchangeSceneNode;
class UInterchangeSkeletalAnimationTrackNode;

namespace UE
{
	namespace Interchange
	{
		namespace Private
		{
			class FFbxParser;
			struct FMorphTargetAnimationBuildingData;

			class FFbxScene
			{
			public:
				explicit FFbxScene(FFbxParser& InParser)
					: Parser(InParser)
				{}

				void AddHierarchy(FbxScene* SDKScene, UInterchangeBaseNodeContainer& NodeContainer, TMap<FString, TSharedPtr<FPayloadContextBase, ESPMode::ThreadSafe>>& PayloadContexts);
				void AddAnimation(FbxScene* SDKScene, UInterchangeBaseNodeContainer& NodeContainer, TMap<FString, TSharedPtr<FPayloadContextBase, ESPMode::ThreadSafe>>& PayloadContexts);
				void AddMorphTargetAnimations(FbxScene* SDKScene, UInterchangeBaseNodeContainer& NodeContainer, TMap<FString, TSharedPtr<FPayloadContextBase, ESPMode::ThreadSafe>>& PayloadContexts, const TArray<FMorphTargetAnimationBuildingData>& MorphTargetAnimationsBuildingData);
				
				struct FRootJointInfo
				{
					FRootJointInfo(bool InValue) : bValidBindPose(InValue) {}
					const bool bValidBindPose;
				};

			protected:

				void CreateMeshNodeReference(UInterchangeSceneNode* UnrealSceneNode, FbxNodeAttribute* NodeAttribute, UInterchangeBaseNodeContainer& NodeContainer, const FTransform& GeometricTransform, const FTransform& PivotNodeTransform);
				void CreateCameraNodeReference(UInterchangeSceneNode* UnrealSceneNode, FbxNodeAttribute* NodeAttribute, UInterchangeBaseNodeContainer& NodeContainer);
				void CreateLightNodeReference(UInterchangeSceneNode* UnrealSceneNode, FbxNodeAttribute* NodeAttribute, UInterchangeBaseNodeContainer& NodeContainer);

				void AddHierarchyRecursively(UInterchangeSceneNode* UnrealParentNode
					, FbxNode* Node
					, FbxScene* SDKScene
					, UInterchangeBaseNodeContainer& NodeContainer
					, TMap<FString, TSharedPtr<FPayloadContextBase, ESPMode::ThreadSafe>>& PayloadContexts
					, const TArray<FbxNode*>& ForceJointNodes
					, bool& bBadBindPoseMessageDisplay
					, FFbxJointMeshBindPoseGenerator& FbxJointMeshBindPoseGenerator);

				void AddAnimationRecursively(FbxNode* Node
					, FbxScene* SDKScene
					, UInterchangeBaseNodeContainer& NodeContainer
					, TMap<FString, TSharedPtr<FPayloadContextBase, ESPMode::ThreadSafe>>& PayloadContexts
					, UInterchangeSkeletalAnimationTrackNode* SkeletalAnimationTrackNode, bool SkeletalAnimationAddedToContainer
					, const FString& RootSceneNodeUid, const TSet<FString>& SkeletonRootNodeUids
					, const int32& AnimationIndex
					, const TArray<FbxNode*>& ForceJointNodes);


				UInterchangeSceneNode* CreateUnrealSceneNode(UInterchangeSceneNode* UnrealParentNode
					, FbxNode* Node
					, FbxScene* SDKScene
					, UInterchangeBaseNodeContainer& NodeContainer
					, const TArray<FbxNode*>& ForceJointNodes
					, bool& bBadBindPoseMessageDisplay
					, FFbxJointMeshBindPoseGenerator& FbxJointMeshBindPoseGenerator);

			private:

				template<typename T>
				T* CreateUnrealSceneNodeInternal(UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeName, const FString& NodeUID, const FString& ParentNodeUID);

				void AddRigidAnimation(FbxNode* Node
					, const FbxScene* SDKScene
					, const int32& AnimationIndex
					, const UInterchangeSceneNode* UnrealNode
					, UInterchangeBaseNodeContainer& NodeContainer
					, TMap<FString, TSharedPtr<FPayloadContextBase, ESPMode::ThreadSafe>>& PayloadContexts) const;
				
				void FillCommonJointRootNode(FbxScene* SDKScene, const TArray<FbxNode*>& ForceJointNodes);

				void ParseMesh(UInterchangeBaseNodeContainer& NodeContainer
					, UInterchangeSceneNode* UnrealNode
					, FbxNode* Node
					, FbxNodeAttribute* NodeAttribute);

				void ParseMeshLODs(UInterchangeMeshLODContainerNode& LODContainerNode
					, int32 LODIndex
					, FbxNode* Node
					, FbxScene* SDKScene
					, const FbxAMatrix& InvTransformMatrix /*Global Inverse Transform Matrix Of LODContainer*/);

				static FbxNode* Internal_GetRootSkeleton(FbxScene* SDKScene, FbxNode* Link, bool IsBlenderArmatureBone);
				static TArray<FbxNode*> FindForceJointNode(FbxScene* SDKScene);
				static bool IsValidBindPose(FbxScene* SDKScene, FbxNode* RootJoint);

				TMap<FbxNode*, FRootJointInfo> CommonJointRootNodes;
				FFbxParser& Parser;
			};
		}//ns Private
	}//ns Interchange
}//ns UE
