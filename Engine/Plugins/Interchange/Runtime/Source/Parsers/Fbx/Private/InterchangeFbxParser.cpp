// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeFbxParser.h"

#include "CoreMinimal.h"
#include "FbxAPI.h"
#include "Ufbx/UfbxParser.h"
#include "HAL/IConsoleManager.h"
#include "InterchangeTextureNode.h"
#include "Misc/ScopeLock.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"
#include "InterchangeHelper.h"
#include "InterchangeCommonAnimationPayload.h"

#define LOCTEXT_NAMESPACE "InterchangeFbxParser"

namespace UE::Interchange
{

	FInterchangeFbxParser::FInterchangeFbxParser()
	{
		ResultsContainer.Reset(NewObject<UInterchangeResultsContainer>(GetTransientPackage()));
		FbxParserPrivate = MakeUnique<Private::FFbxParser>(ResultsContainer.Get());
	}
	FInterchangeFbxParser::~FInterchangeFbxParser()
	{
		ReleaseResources();
	}

	void FInterchangeFbxParser::ReleaseResources()
	{
		ResultsContainer = nullptr;
		FbxParserPrivate = nullptr;
	}

	const TCHAR* FInterchangeFbxParser::GetName()
	{
		return FbxParserPrivate.IsValid() ? FbxParserPrivate->GetName() : TEXT("UNSET");
	}

	bool FInterchangeFbxParser::IsThreadSafe()
	{
		return FbxParserPrivate.IsValid() ? FbxParserPrivate->IsThreadSafe() : false;
	}

	void FInterchangeFbxParser::Reset(bool bInUseUfbxParser)
	{
		ResultPayloads.Reset();
		FbxParserPrivate->Reset();

		// recreate parser when switched between fbxsdk/ufbx 
		bUseUfbxParser = bInUseUfbxParser;
		if (bUseUfbxParser)
		{
			FbxParserPrivate = MakeUnique<Private::FUfbxParser>(ResultsContainer.Get());
		}
		else
		{
			FbxParserPrivate = MakeUnique<Private::FFbxParser>(ResultsContainer.Get());
		}
	}

	void FInterchangeFbxParser::SetResultContainer(UInterchangeResultsContainer* Result)
	{
		InternalResultsContainer = Result;
		FbxParserPrivate->SetResultContainer(Result);
	}

	void FInterchangeFbxParser::SetConvertSettings(const bool InbConvertScene, const bool InbForceFrontXAxis, const bool InbConvertSceneUnit, const bool InbKeepFbxNamespace, const bool InbConsiderClusterBeforePoseForMeshBindPose)
	{
		GetFbxParser().SetConvertSettings(InbConvertScene, InbForceFrontXAxis, InbConvertSceneUnit, InbKeepFbxNamespace, InbConsiderClusterBeforePoseForMeshBindPose);
	}

	void FInterchangeFbxParser::LoadFbxFile(const FString& Filename, const FString& ResultFolder)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeFbxParser::LoadFbxFile)

		check(GetFbxParserPtr());
		SourceFilename = Filename;
		ResultsContainer->Empty();

		//Since we are not in main thread we cannot use TStrongPtr, so we will add the object to the root and remove it when we are done
		UInterchangeBaseNodeContainer* Container = NewObject<UInterchangeBaseNodeContainer>(GetTransientPackage(), NAME_None);

		if (!ensure(Container != nullptr))
		{
			if (UInterchangeResultError_Generic* Error = AddMessage<UInterchangeResultError_Generic>())
			{
				Error->SourceAssetName = SourceFilename;
				Error->Text = LOCTEXT("CantAllocate", "Cannot allocate base node container to add FBX scene data.");
			}
			return;
		}

		if (!GetFbxParser().LoadFbxFile(Filename, *Container))
		{
			if (UInterchangeResultError_Generic* Error = AddMessage<UInterchangeResultError_Generic>())
			{
				Error->SourceAssetName = SourceFilename;
				Error->Text = LOCTEXT("CantLoadFbxFile", "Cannot load the FBX file.");
			}
			return;
		}

		ResultFilepath = ResultFolder + TEXT("/SceneDescription.itc");

		Container->AddToRoot();
		GetFbxParser().FillContainerWithFbxScene(*Container);
		Container->SaveToFile(ResultFilepath);
		Container->RemoveFromRoot();
	}

	void FInterchangeFbxParser::LoadFbxFile(const FString& Filename, UInterchangeBaseNodeContainer& BaseNodecontainer)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeFbxParser::LoadFbxFile)

		SourceFilename = Filename;
		if (!ensure(GetFbxParserPtr()))
		{
			if (UInterchangeResultError_Generic* Error = AddMessage<UInterchangeResultError_Generic>())
			{
				Error->SourceAssetName = SourceFilename;
				Error->Text = LOCTEXT("CantLoadFbxFile_ParserInvalid", "FInterchangeFbxParser::LoadFbxFile: Cannot load the FBX file. The internal fbx parser is invalid.");
			}
			return;
		}
		
		if (!GetFbxParser().LoadFbxFile(Filename, BaseNodecontainer))
		{
			if (UInterchangeResultError_Generic* Error = AddMessage<UInterchangeResultError_Generic>())
			{
				Error->SourceAssetName = SourceFilename;
				Error->Text = LOCTEXT("CantLoadFbxFile_ParserError", "FInterchangeFbxParser::LoadFbxFile: Cannot load the FBX file. There was an error when parsing the file.");
			}
			return;
		}
		GetFbxParser().FillContainerWithFbxScene(BaseNodecontainer);
	}

	void FInterchangeFbxParser::FetchPayload(const FString& PayloadKey, const FString& ResultFolder)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeFbxParser::FetchPayload)

		ResultsContainer->Empty();
		if (!ensure(GetFbxParserPtr()))
		{
			if (UInterchangeResultError_Generic* Error = AddMessage<UInterchangeResultError_Generic>())
			{
				Error->SourceAssetName = PayloadKey;
				Error->Text = LOCTEXT("CantFetchPayload_ParserInvalid", "FInterchangeFbxParser::FetchPayload: Cannot fetch the payload. The internal fbx parser is invalid.");
			}
			return;
		}
		
		FString PayloadFilepathCopy;
		{
			FScopeLock Lock(&ResultPayloadsCriticalSection);
			FString& PayloadFilepath = ResultPayloads.FindOrAdd(PayloadKey);
			//To avoid file path with too many character, we hash the payloadKey so we have a deterministic length for the file path.
			FString PayloadKeyHash = Private::HashString(PayloadKey);
			PayloadFilepath = ResultFolder + TEXT("/") + PayloadKeyHash + FString::FromInt(UniqueIdCounter.IncrementExchange()) + TEXT(".payload");

			//Copy the map filename key because we are multithreaded and the TMap can be reallocated
			PayloadFilepathCopy = PayloadFilepath;
		}
		if (!GetFbxParser().FetchPayloadData(PayloadKey, PayloadFilepathCopy))
		{
			if (UInterchangeResultError_Generic* Error = AddMessage<UInterchangeResultError_Generic>())
			{
				Error->SourceAssetName = SourceFilename;
				Error->Text = LOCTEXT("CantFetchPayload", "Cannot fetch FBX payload data.");
			}
			return;
		}
	}

	FString FInterchangeFbxParser::FetchMeshPayload(const FString& PayloadKey, const FTransform& MeshGlobalTransform, const FString& ResultFolder)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeFbxParser::FetchMeshPayload)

		ResultsContainer->Empty();
		FString PayloadFilepathCopy;
		FString ResultPayloadUniqueId = PayloadKey + MeshGlobalTransform.ToString();

		Private::IFbxParser* FbxParserPtr = GetFbxParserPtr();
		if (!ensure(FbxParserPtr))
		{
			if (UInterchangeResultError_Generic* Error = AddMessage<UInterchangeResultError_Generic>())
			{
				Error->SourceAssetName = PayloadKey;
				Error->Text = LOCTEXT("CantFetchMeshPayload_ParserInvalid", "FInterchangeFbxParser::FetchMeshPayload: Cannot fetch the mesh payload. The internal fbx parser is invalid.");
			}
			return ResultPayloadUniqueId;
		}
		
		//If we already have extract this mesh, no need to extract again
		if (ResultPayloads.Contains(ResultPayloadUniqueId))
		{
			return ResultPayloadUniqueId;
		}

		{
			FScopeLock Lock(&ResultPayloadsCriticalSection);
			FString& PayloadFilepath = ResultPayloads.FindOrAdd(ResultPayloadUniqueId);
			//To avoid file path with too many character, we hash the payloadKey so we have a deterministic length for the file path.
			FString PayloadKeyHash = Private::HashString(PayloadKey);
			PayloadFilepath = ResultFolder + TEXT("/") + PayloadKeyHash + FString::FromInt(UniqueIdCounter.IncrementExchange()) + TEXT(".payload");

			//Copy the map filename key because we are multithreaded and the TMap can be reallocated
			PayloadFilepathCopy = PayloadFilepath;
		}

		if (!FbxParserPtr->FetchMeshPayloadData(PayloadKey, MeshGlobalTransform, PayloadFilepathCopy))
		{
			if (UInterchangeResultError_Generic* Error = AddMessage<UInterchangeResultError_Generic>())
			{
				Error->SourceAssetName = SourceFilename;
				Error->Text = LOCTEXT("CantFetchPayload", "Cannot fetch FBX payload data.");
			}
		}
		return ResultPayloadUniqueId;
	}

#if WITH_ENGINE
	bool FInterchangeFbxParser::FetchMeshPayload(const FString& PayloadKey, const FTransform& MeshGlobalTransform, FMeshPayloadData& OutMeshPayloadData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeFbxParser::FetchMeshPayload);

		if (!GetFbxParser().FetchMeshPayloadData(PayloadKey, MeshGlobalTransform, OutMeshPayloadData))
		{
			if (UInterchangeResultError_Generic* Error = AddMessage<UInterchangeResultError_Generic>())
			{
				Error->SourceAssetName = SourceFilename;
				Error->Text = LOCTEXT("CantFetchPayload", "Cannot fetch FBX payload data.");
			}
			return false;
		}
		return true;
	}

	bool FInterchangeFbxParser::FetchTexturePayload(const FString& PayloadKey, TOptional<TArray64<uint8>>& OutTexturePayloadData)
	{
		return GetFbxParser().FetchTexturePayload(PayloadKey, OutTexturePayloadData);
	}
#endif

	TArray<FString> FInterchangeFbxParser::GetJsonLoadMessages() const
	{
		TArray<FString> JsonResults;
		for (UInterchangeResult* Result : GetResultContainer()->GetResults())
		{
			JsonResults.Add(Result->ToJson());
		}

		return JsonResults;
	}

	UInterchangeResultsContainer* FInterchangeFbxParser::GetResultContainer() const
	{
		if (InternalResultsContainer)
		{
			return InternalResultsContainer;
		}
		ensure(ResultsContainer);
		return ResultsContainer.Get();
	}

	Private::IFbxParser* FInterchangeFbxParser::GetFbxParserPtr() const
	{
		return FbxParserPrivate.Get();
	}

	Private::IFbxParser& FInterchangeFbxParser::GetFbxParser() 
	{
		return *GetFbxParserPtr();
	}

	void FInterchangeFbxParser::FetchAnimationBakeTransformPayloads(const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries, const FString& ResultFolder)
	{
		GetFbxParser().FetchAnimationBakeTransformPayload(PayloadQueries, ResultFolder, &ResultPayloadsCriticalSection, UniqueIdCounter, ResultPayloads);
	}

	//Used by DispatchWorker
	TMap<FString, FString> FInterchangeFbxParser::FetchAnimationBakeTransformPayloads(const FString& PayloadQueriesJsonString, const FString& ResultFolder)
	{
		TMap<FString, FString> HashToFilePaths;

		TArray<UE::Interchange::FAnimationPayloadQuery> PayloadQueries;
		UE::Interchange::FAnimationPayloadQuery::FromJson(PayloadQueriesJsonString, PayloadQueries);

		GetFbxParser().FetchAnimationBakeTransformPayload(PayloadQueries, ResultFolder, &ResultPayloadsCriticalSection, UniqueIdCounter, ResultPayloads);

		//Acquire FilePaths:
		for (const UE::Interchange::FAnimationPayloadQuery& PayloadQuery : PayloadQueries)
		{
			HashToFilePaths.Add(PayloadQuery.GetHashString(), GetResultPayloadFilepath(PayloadQuery.GetHashString()));
		}

		return HashToFilePaths;
	}
}//ns UE::Interchange

#undef LOCTEXT_NAMESPACE
