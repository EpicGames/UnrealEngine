// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeFbxParserModule.h"

#include "InterchangeFbxParserLog.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"



#define LOCTEXT_NAMESPACE "FInterchangeFbxParserModule"

DEFINE_LOG_CATEGORY(LogInterchangeFbxParser);

FInterchangeFbxParserModule& FInterchangeFbxParserModule::Get()
{
	return FModuleManager::LoadModuleChecked< FInterchangeFbxParserModule >(INTERCHANGEFBXPARSER_MODULE_NAME);
}

bool FInterchangeFbxParserModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded(INTERCHANGEFBXPARSER_MODULE_NAME);
}

void FInterchangeFbxParserModule::StartupModule()
{
}

IMPLEMENT_MODULE(FInterchangeFbxParserModule, InterchangeFbxParser);

#undef LOCTEXT_NAMESPACE // "InterchangeFbxParserModule"

