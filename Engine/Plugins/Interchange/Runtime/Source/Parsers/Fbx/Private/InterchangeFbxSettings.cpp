// Copyright Epic Games, Inc.All Rights Reserved.

#include "InterchangeFbxSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeFbxSettings)

UInterchangeFbxSettings::UInterchangeFbxSettings()
{
	if(HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
	{
		PredefinedPropertyTracks =
		{
			{ TEXT("bHidden"), {EInterchangePropertyTracks::ActorHiddenInGame} },
			{ TEXT("bAutoActivate"), {EInterchangePropertyTracks::AutoActivate} },

			{ TEXT("Intensity"), { EInterchangePropertyTracks::LightIntensity}	},
			{ TEXT("Color"), {EInterchangePropertyTracks::LightColor} },
			{ TEXT("bUseTemperature"), {EInterchangePropertyTracks::LightUseTemperature} },
			{ TEXT("IntensityUnits"), {EInterchangePropertyTracks::LightIntensityUnits} },

			{ TEXT("AspectRatio"), {EInterchangePropertyTracks::CameraAspectRatio} },
			{ TEXT("AspectRatioAxisConstraint"), {EInterchangePropertyTracks::CameraAspectRatioAxisConstraint} },
			{ TEXT("bAutoCalculateOrthoPlanes"), {EInterchangePropertyTracks::CameraAutoCalculateOrthoPlanes} },
			{ TEXT("CurrentAperture"), {EInterchangePropertyTracks::CameraCurrentAperture} },
			{ TEXT("FieldOfView"), {EInterchangePropertyTracks::CameraFieldOfView} },
			{ TEXT("FilmAspectRatio"), {EInterchangePropertyTracks::CameraFilmbackSensorAspectRatio} },
			{ TEXT("FilmHeight"), {EInterchangePropertyTracks::CameraFilmbackSensorHeight} },
			{ TEXT("FilmWidth"), {EInterchangePropertyTracks::CameraFilmbackSensorWidth} },
			{ TEXT("FocalLength"), {EInterchangePropertyTracks::CameraCurrentFocalLength} },
			{ TEXT("OrthoFarClipPlane"), {EInterchangePropertyTracks::CameraOrthoFarClipPlane} },
			{ TEXT("OrthoNearClipPlane"), {EInterchangePropertyTracks::CameraOrthoNearClipPlane} },
			{ TEXT("FocusDistance"), {EInterchangePropertyTracks::CameraFocusSettingsManualFocusDistance} },
		};
	}
}

EInterchangePropertyTracks UInterchangeFbxSettings::GetPropertyTrack(const FString& PropertyName) const
{
	EInterchangePropertyTracks Result = EInterchangePropertyTracks::None;
	if(const EInterchangePropertyTracks* Property = PredefinedPropertyTracks.Find(PropertyName))
	{
		Result = *Property;
	}
	else if(Property = CustomPropertyTracks.Find(PropertyName); Property != nullptr)
	{
		Result = *Property;
	}

	return Result;
}
