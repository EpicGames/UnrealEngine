// Copyright Epic Games, Inc. All Rights Reserved.

#include "UfbxAnimation.h"
#include "UfbxParser.h"
#include "UfbxScene.h"
#include "UfbxConvert.h"

#include "InterchangeCommonAnimationPayload.h"
#include "InterchangeMeshNode.h"
#include "UfbxMesh.h"
#include "Algo/AnyOf.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#define LOCTEXT_NAMESPACE "InterchangeFbxParser"

namespace UE::Interchange::Private
{

	namespace AnimUtils
	{

		template<typename ValueList>
		auto GetAnimationKeyValue(const ValueList& Keys, const bool& IsConstant, const int32 TransformIndex, const double& Frequency) -> decltype(Keys[0].value)
		{
			// #ufbx_todo: investigate key start time handling, e.g. TransformIndex-FMath::CeilToInt(Keys[0].time*Frequency - UE_KINDA_SMALL_NUMBER)
			int32 KeyIndex = IsConstant ? 0
			: FMath::Clamp<int32>(TransformIndex, 0, Keys.count-1);

			return Keys[KeyIndex].value;
		};

		inline FVector ConvertValue(const ufbx_vec3& V)
		{
			return Convert::ConvertVec3(V);
		}

		inline FQuat ConvertValue(const ufbx_quat& V)
		{
			return Convert::ConvertQuat(V);
		}

		// Get interval where the values are actually animated
		template<typename ValueList, typename Value>
		void UpdateAnimationInterval(TInterval<double>& Interval, const ValueList& Keys, const bool& IsConstant, const Value& Reference)
		{
			if (!IsConstant 
				|| (Keys.count > 0 
					&& !ConvertValue(Keys[0].value).Equals(
						ConvertValue(Reference), UE_SMALL_NUMBER)))
			{
				if (Keys.count > 1)
				{
					Interval.Include(Keys[0].time);
					Interval.Include(Keys[Keys.count-1].time);
				}
			}
		}

		bool ComputeTangent(const ufbx_tangent& Tangent, const float ScaleValue, float& OutTangent, float& OutTangentWeight)
		{
			if (FMath::IsNearlyZero(Tangent.dx))
			{
				return false;
			}

			// ufbx computes tangent dx/dy as
			// Tangent.dx = Weight*Delta
			// Tangent.dy = Tangent.dx * Slope
			// also, see UE::Curves::WeightedEvalForTwoKeys for cubic curve evaluation

			const float Slope = (Tangent.dy / Tangent.dx) * ScaleValue;
			OutTangent = Slope;
			OutTangentWeight = FMath::Sqrt(FMath::Square(Tangent.dx) + FMath::Square(Tangent.dx*Slope));
			return true;
		};

		// inspired by UE::Interchange::Private::ImportCurve - Engine/Plugins/Interchange/Runtime/Source/Parsers/Fbx/Private/FbxAnimation.cpp
		void ConvertCurve(const ufbx_anim_curve& SourceFloatCurves, const float ScaleValue, TArray<FInterchangeCurveKey>& DestinationFloatCurve)
		{
			int32 KeyCount = SourceFloatCurves.keyframes.count;
			for (int32 KeyIndex = 0; KeyIndex < KeyCount; ++KeyIndex)
			{
				
				const ufbx_keyframe& Key = SourceFloatCurves.keyframes[KeyIndex];
				
				const float KeyTimeValue = static_cast<float>(Key.time);
				const float Value = Key.value * ScaleValue;
				FInterchangeCurveKey& InterchangeCurveKey = DestinationFloatCurve.AddDefaulted_GetRef();
				InterchangeCurveKey.Time = KeyTimeValue;
				InterchangeCurveKey.Value = Value;

				EInterchangeCurveInterpMode NewInterpMode = EInterchangeCurveInterpMode::Linear;

				switch (Key.interpolation)
				{
					// #ufbx_todo: distinguish two const interps?
				case UFBX_INTERPOLATION_CONSTANT_NEXT:
				case UFBX_INTERPOLATION_CONSTANT_PREV:
					NewInterpMode = EInterchangeCurveInterpMode::Constant;
					break;
				case UFBX_INTERPOLATION_LINEAR:
					NewInterpMode = EInterchangeCurveInterpMode::Linear;
					break;
				case UFBX_INTERPOLATION_CUBIC:
					{
						NewInterpMode = EInterchangeCurveInterpMode::Cubic;
						EInterchangeCurveTangentMode NewTangentMode = EInterchangeCurveTangentMode::User;

						float RightTangent = 0.0f;
						float LeftTangent = 0.0f;
						float RightTangentWeight = 0.0f;
						float LeftTangentWeight = 0.0f;

						ComputeTangent(Key.left, ScaleValue, LeftTangent, LeftTangentWeight);
						
						ComputeTangent(Key.right, ScaleValue, RightTangent, RightTangentWeight);

						// #ufbx_todo: possible to optimize curve by checking value, tangents and tangent weights for equality, in case cubic curve segment is actually constant/linear
						InterchangeCurveKey.TangentMode = NewTangentMode;
						InterchangeCurveKey.TangentWeightMode = EInterchangeCurveTangentWeightMode::WeightedNone;

						InterchangeCurveKey.ArriveTangent = LeftTangent;
						InterchangeCurveKey.LeaveTangent = RightTangent;
						InterchangeCurveKey.ArriveTangentWeight = LeftTangentWeight;
						InterchangeCurveKey.LeaveTangentWeight = RightTangentWeight;
					}
					break;
				default: ;
				}

				InterchangeCurveKey.InterpMode = NewInterpMode;
			}
		}

		FString CreateTrackNodeUid(const FString& JointUid, const int32 AnimationIndex)
		{
			FString TrackNodeUid = TEXT("\\SkeletalAnimation\\") + JointUid + TEXT("\\") + FString::FromInt(AnimationIndex);
			return TrackNodeUid;
		}

		template<typename T>
		void ConvertStepCurveKeyframes(const ufbx_anim_curve& Curve, TArray<float>& OutTimes, TOptional<TArray<T>>& OutValues)
		{
			const int32 Count = Curve.keyframes.count;
			OutTimes.Reserve(Count);
			TArray<T>& Values = OutValues.Emplace();
			Values.Reserve(Count);

			for (const ufbx_keyframe& Keyframe : Curve.keyframes)
			{
				OutTimes.Add(static_cast<float>(Keyframe.time));
				Values.Add(static_cast<T>(Keyframe.value));
			}
		}

	}

	void FUfbxAnimation::FetchSkinnedAnimation(ufbx_scene* Scene, const UE::Interchange::FAnimationPayloadQuery& PayloadQuery, const FSkeletalAnimationPayloadContext& SkeletalAnimationPayload, FAnimationPayloadData& PayloadData)
	{
		ufbx_shared_ptr<ufbx_baked_anim> BakedAnim = SkeletalAnimationPayload.BakedAnim;
		const ufbx_baked_node& BakedNode = BakedAnim->nodes[SkeletalAnimationPayload.BakedNodeIndex];

		const double BakeFrequency = PayloadQuery.TimeDescription.BakeFrequency;

		PayloadData.BakeFrequency = BakeFrequency;
		PayloadData.RangeStartTime = PayloadQuery.TimeDescription.RangeStartSecond;
		PayloadData.RangeEndTime = PayloadQuery.TimeDescription.RangeStopSecond;

		const double AnimationDuration = PayloadQuery.TimeDescription.RangeStopSecond - PayloadQuery.TimeDescription.RangeStartSecond;
		int32 TotalKeyCount = FMath::RoundToInt32(AnimationDuration*BakeFrequency) + 1;
		PayloadData.Transforms.SetNum(TotalKeyCount);

		const double FrameDuration = 1.0 / PayloadData.BakeFrequency;
		for (int TransformIndex = 0; TransformIndex < TotalKeyCount; ++TransformIndex)
		{
			ufbx_transform Transform = ufbx_evaluate_transform_flags(
				Scene->anim_stacks[SkeletalAnimationPayload.AnimStackIndex]->anim, 
				Scene->nodes[BakedNode.typed_id], PayloadQuery.TimeDescription.RangeStartSecond + FrameDuration * TransformIndex,
				0
				| UFBX_TRANSFORM_FLAG_IGNORE_SCALE_HELPER 
				| UFBX_TRANSFORM_FLAG_IGNORE_COMPONENTWISE_SCALE
				| UFBX_TRANSFORM_FLAG_EXPLICIT_INCLUDES
				| UFBX_TRANSFORM_FLAG_INCLUDE_TRANSLATION
				| UFBX_TRANSFORM_FLAG_INCLUDE_ROTATION
				| UFBX_TRANSFORM_FLAG_INCLUDE_SCALE
			);
			PayloadData.Transforms[TransformIndex] = Convert::ConvertTransform(Transform);
		}
	}

	bool FUfbxAnimation::FetchMorphTargetAnimation(const FUfbxParser& Parser, const FMorphAnimationPayloadContext& Animation, TArray<FInterchangeCurve>& OutCurves)
	{
		if (Animation.Curve)
		{
			// #ufbx_todo: UE::Interchange::Private::FAnimationPayloadContext::InternalFetchMorphTargetCurvePayload uses 0.01f scale
			AnimUtils::ConvertCurve(*Animation.Curve, 0.01f, OutCurves.AddDefaulted_GetRef().Keys);
			return true;
		}
		
		return false;
	}

	void ReduceConstantKeys(const ufbx_anim_curve& Curve, TArray<FInterchangeCurveKey>& Keys, double Tolerance)
	{
		if (Keys.IsEmpty())
		{
			return;
		}

		// might use 1 instead of 0 if want to leave some keys even in all-constant curve?
		constexpr int32 FirstKeyIndex = 0;
		for (int32 KeyIndex = Keys.Num() - 1; KeyIndex >= FirstKeyIndex;)
		{
			const double CurrentValue = Keys[KeyIndex].Value;
			FDoubleInterval ToleranceInterval(CurrentValue - Tolerance, CurrentValue + Tolerance);

			// find all consecutive constant keys
			int32 ConstantLeftIndex = KeyIndex;
			int32 ConstantRightIndex = KeyIndex;
			while (KeyIndex >= FirstKeyIndex)
			{
				const FInterchangeCurveKey& Key = Keys[KeyIndex];

				const double KeyValue = Key.Value;
				const bool bValueInBand = ToleranceInterval.Contains(KeyValue);

				const bool bIsEffectivelyConstant = bValueInBand && (
					Key.InterpMode == EInterchangeCurveInterpMode::Constant
					// Linear within epsilon range is constant
					|| Key.InterpMode == EInterchangeCurveInterpMode::Linear
					// Cubic but both tangents are about zero
					|| (Key.InterpMode == EInterchangeCurveInterpMode::Cubic
						&& FMath::IsNearlyZero(Key.LeaveTangent, Tolerance)
						&& FMath::IsNearlyZero(Key.ArriveTangent, Tolerance))
					);

				// also, might check constantness by evaluating curve between keys at desired fbx to see that values are somewhat constant
				// instead of just testing tangent for constantness
				// ufbx_evaluate_curve(&Curve, Key.Time, Key.Value);

				if (bIsEffectivelyConstant)
				{
					// Constant: mark for deletion, step left
					ConstantLeftIndex = KeyIndex;
					--KeyIndex;
				}
				else
				{
					--KeyIndex;
					break;
				}
			}
			
			if (ConstantLeftIndex != ConstantRightIndex)
			{
				int32 ConstantKeysNum = ConstantRightIndex - ConstantLeftIndex + 1;
				if (ConstantKeysNum < Keys.Num())
				{
					// Remove constant interval except one(left-most) key
					Keys.RemoveAt(ConstantLeftIndex + 1, ConstantKeysNum - 1);
				}
				else
				{
					// All keys are constant
					// maybe, should check if curve is distinct from the default anim value to keep something here?
					Keys.Empty();
					return;
				}
			}
		}
	}

	bool FUfbxAnimation::FetchRigidAnimation(const FUfbxParser& Parser, const FRigidAnimationPayloadContext& Animation,
		TArray<FInterchangeCurve>& OutCurves)
	{
		auto AddCurve = [&OutCurves](const ufbx_anim_prop* Prop, int32 CurveIndex, double Scale=1.0)
		{
			const ufbx_anim_curve* Curve =  Prop ? Prop->anim_value->curves[CurveIndex] : nullptr;
			if (Curve)
			{
				FInterchangeCurve& OutCurve = OutCurves.AddDefaulted_GetRef();
				TArray<FInterchangeCurveKey>& Keys = OutCurve.Keys;
				AnimUtils::ConvertCurve(*Curve, Scale, Keys);

				// ConvertPivotAnimationRecursive in FBX Parser does key reduction, trying repeat it 
				ReduceConstantKeys(*Curve, Keys, 0.009);
			}
			else
			{
				OutCurves.AddDefaulted();
			}
		};

		OutCurves.Reserve(9);

		// #ufbx_todo: FAnimationPayloadContext::InternalFetchCurveNodePayloadToFile does some pivot handling, see ResetPivotsPrePostRotationsAndSetRotationOrder

		// ufbx doesn't convert transform curve data to desired coord space(or unit scale), so need to handle node transform curve mapping here

		ufbx_real TranslationScaling = 1;

		int32 AxisMap[3] = {0, 1, 2};
		double AxisSign[3] = {1.0, 1.0, 1.0};
		int32 AxisRotationMap[3] = {0, 1, 2};
		double AxisRotationSign[3] = {1.0, 1.0, 1.0};

		const ufbx_node* Node = Animation.Node;

		if (Node && Node->has_adjust_transform && (Parser.Scene->settings.axes.up == UFBX_COORDINATE_AXIS_POSITIVE_Y))
		{
			// Nodes that are just below the RootNode(artificial) have adjust_transform and have they YZ swapped
			// When FBX was exported as Yup
			AxisMap[1] = 2;
			AxisMap[2] = 1;
		}
		else
		{
			// Other nodes just need to fix handedness by mirroring on Y axis
			AxisSign[1] = -1.0f;
		}

		// Rotations need to be inverted due to handedness change
		AxisRotationSign[1] = -1.0f;
		AxisRotationSign[2] = -1.0f;

		if (Node)
		{
			TranslationScaling = Node->adjust_pre_scale;
		}

		AddCurve(Animation.TranslationProp, AxisMap[0], AxisSign[0]*TranslationScaling);
		AddCurve(Animation.TranslationProp, AxisMap[1], AxisSign[1]*TranslationScaling);
		AddCurve(Animation.TranslationProp, AxisMap[2], AxisSign[2]*TranslationScaling);

		AddCurve(Animation.RotationProp, AxisRotationMap[0], AxisRotationSign[0]);
		AddCurve(Animation.RotationProp, AxisRotationMap[1], AxisRotationSign[1]);
		AddCurve(Animation.RotationProp, AxisRotationMap[2], AxisRotationSign[2]);
		AddCurve(Animation.ScalingProp, AxisMap[0], 1);
		AddCurve(Animation.ScalingProp, AxisMap[1], 1);
		AddCurve(Animation.ScalingProp, AxisMap[2], 1);

		return true;
	}

	bool FUfbxAnimation::FetchPropertyAnimationCurves(const FUfbxParser& Parser, const FPropertyAnimationPayloadContext& Animation, TArray<FInterchangeCurve>& OutCurves)
	{
		for (const ufbx_anim_curve* Curve : Animation.AnimValue->curves)
		{
			if (Curve)
			{
				AnimUtils::ConvertCurve(*Curve, 1.f, OutCurves.AddDefaulted_GetRef().Keys);
			}
		}
		return true;
	}

	bool FUfbxAnimation::FetchPropertyAnimationStepCurves(const FUfbxParser& Parser, const FPropertyAnimationPayloadContext& Animation, TArray<FInterchangeStepCurve>& OutCurves)
	{
		for (const ufbx_anim_curve* Curve : Animation.AnimValue->curves)
		{
			if (Curve)
			{
				

				switch (Animation.Prop->type)
				{
				case UFBX_PROP_BOOLEAN:
					{
						FInterchangeStepCurve& DestinationCurve = OutCurves.AddDefaulted_GetRef();
						AnimUtils::ConvertStepCurveKeyframes(*Curve, DestinationCurve.KeyTimes, DestinationCurve.BooleanKeyValues);
					}
					break;
				case UFBX_PROP_INTEGER:
					{
						FInterchangeStepCurve& DestinationCurve = OutCurves.AddDefaulted_GetRef();
						AnimUtils::ConvertStepCurveKeyframes(*Curve, DestinationCurve.KeyTimes, DestinationCurve.IntegerKeyValues);
					}
					break;
				case UFBX_PROP_NUMBER:
					{
						FInterchangeStepCurve& DestinationCurve = OutCurves.AddDefaulted_GetRef();
						AnimUtils::ConvertStepCurveKeyframes(*Curve, DestinationCurve.KeyTimes, DestinationCurve.FloatKeyValues);
					}
					break;
				}

			}
		}
		return true;
	}

	void FUfbxAnimation::AddAnimation(FUfbxParser& Parser, FUfbxScene& Scene, const FUfbxMesh& Meshes, UInterchangeBaseNodeContainer& NodeContainer)
	{
		const TMap<const ufbx_blend_shape*, const ufbx_mesh*> MeshPerBlendShape = Meshes.GetBlendShapes();
		double FrameRate = Parser.Scene->settings.frames_per_second;

		int32 AnimStackCount = Parser.Scene->anim_stacks.count;
		for (int32 AnimStackIndex = 0; AnimStackIndex < AnimStackCount; ++AnimStackIndex)
		{
			ufbx_anim_stack* Stack = Parser.Scene->anim_stacks[AnimStackIndex];

			// keep track nodes for current AnimStackIndex only (not all TrackNodes!)
			TMap<FString, UInterchangeSkeletalAnimationTrackNode*> TrackNodePerSkeletonRoot;

			ufbx_bake_opts BakeOptions = {};
			BakeOptions.maximum_sample_rate = FrameRate;
			BakeOptions.minimum_sample_rate = FrameRate;
			BakeOptions.resample_rate = FrameRate;

			ufbx_error ErrorResult;

			ufbx_shared_ptr<ufbx_baked_anim> BakedAnim(ufbx_bake_anim(Parser.Scene, Stack->anim, &BakeOptions, &ErrorResult));
			if (!BakedAnim)
			{
				UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();

				TArray<char> ErrorBuf;
				ErrorBuf.SetNum(ErrorResult.info_length+64);
				ufbx_format_error(ErrorBuf.GetData(), ErrorBuf.Num(), &ErrorResult);

				Message->Text = FText::Format(LOCTEXT("AnimBakeError", "UFBX couldn't bake animation: '{0}'."), FText::FromString(UTF8_TO_TCHAR(ErrorBuf.GetData())));
				continue;
			}

			TConstArrayView<ufbx_baked_node> BakedNodes(BakedAnim->nodes.data, BakedAnim->nodes.count);

			TConstArrayView<ufbx_node*> FbxNodes(Parser.Scene->nodes.data, Parser.Scene->nodes.count);

			FString DisplayString = Convert::ToUnrealString(Stack->name);

			auto GetTrackNode = [&](FString SkeletonRootNodeUid, TInterval<double> AnimInterval=TInterval<double>())
			{
				UInterchangeSkeletalAnimationTrackNode*& SkeletalAnimationTrackNode = TrackNodePerSkeletonRoot.FindOrAdd(SkeletonRootNodeUid);
				if (!SkeletalAnimationTrackNode)
				{
					FString TrackNodeUid = AnimUtils::CreateTrackNodeUid(SkeletonRootNodeUid, AnimStackIndex);

					SkeletalAnimationTrackNode = NewObject< UInterchangeSkeletalAnimationTrackNode >(&NodeContainer);
					SkeletalAnimationTrackNode->InitializeNode(TrackNodeUid, DisplayString, EInterchangeNodeContainerType::TranslatedAsset);
					SkeletalAnimationTrackNode->AddBooleanAttribute(TEXT("RenameLikeLegacyFbx"), true);
					SkeletalAnimationTrackNode->SetCustomAnimationSampleRate(Parser.Scene->settings.frames_per_second);

					SkeletalAnimationTrackNode->SetCustomSkeletonNodeUid(SkeletonRootNodeUid);

					SkeletalAnimationTrackNode->SetCustomSourceTimelineAnimationStartTime(Stack->time_begin);
					SkeletalAnimationTrackNode->SetCustomSourceTimelineAnimationStopTime(Stack->time_end);
					NodeContainer.AddNode(SkeletalAnimationTrackNode);
				}

				double CustomAnimationStartTime;
				double CustomAnimationStopTime;
				if (SkeletalAnimationTrackNode->GetCustomAnimationStartTime(CustomAnimationStartTime))
				{
					AnimInterval.Include(CustomAnimationStartTime);
				}
				if (SkeletalAnimationTrackNode->GetCustomAnimationStopTime(CustomAnimationStopTime))
				{
					AnimInterval.Include(CustomAnimationStopTime);
				}

				if (AnimInterval.IsValid())
				{
					SkeletalAnimationTrackNode->SetCustomAnimationStartTime(AnimInterval.Min);
					SkeletalAnimationTrackNode->SetCustomAnimationStopTime(AnimInterval.Max);
				}

				return SkeletalAnimationTrackNode;
			};

			for (int32 BakedNodeIndex = 0; BakedNodeIndex < BakedNodes.Num(); ++BakedNodeIndex)
			{
				const ufbx_baked_node& BakedNode = BakedNodes[BakedNodeIndex];

				if (!ensureMsgf(FbxNodes.IsValidIndex(BakedNode.typed_id), TEXT("Unexpected type_id encountered in ufbx baked anim nodes")))
				{
					continue;
				}

				ufbx_node* Node = FbxNodes[BakedNode.typed_id];
				if (Node && (Node->bone || Scene.IsInSkeleton(*Node)))
				{
					FString NodeUid = Parser.GetNodeUid(*Node);

					TInterval<double> Interval;
					AnimUtils::UpdateAnimationInterval(Interval, BakedNode.translation_keys, BakedNode.constant_translation, Node->local_transform.translation);
					AnimUtils::UpdateAnimationInterval(Interval, BakedNode.rotation_keys, BakedNode.constant_rotation, Node->local_transform.rotation);
					AnimUtils::UpdateAnimationInterval(Interval, BakedNode.scale_keys, BakedNode.constant_scale, Node->local_transform.scale);

					if (Interval.IsValid())
					{
						UInterchangeSkeletalAnimationTrackNode* SkeletalAnimationTrackNode = GetTrackNode(Scene.GetSkeletonRoot(Node), Interval);

						// #ufbx_todo: Need check animated parents - whole subtree has to be animated too?
						// see UE::Interchange::Private::FFbxAnimation::AddSkeletalTransformAnimation
						// or maybe there's an option in ufbx baking that already does that?

						FString PayLoadKey = NodeUid + TEXT("_") + FString::FromInt(AnimStackIndex) + TEXT("_SkeletalAnimationPayloadKey");

						Parser.PayloadContexts.Add(PayLoadKey, FSkeletalAnimationPayloadContext{NodeUid, AnimStackIndex, BakedNodeIndex, BakedAnim});

						SkeletalAnimationTrackNode->SetAnimationPayloadKeyForSceneNodeUid(NodeUid, PayLoadKey, EInterchangeAnimationPayLoadType::BAKED);
					}
				}
			}

			// Collect curves for other animated properties - morph, ..
			for (int32 LayerIndex = 0; LayerIndex < Stack->layers.count; ++LayerIndex)
			{
				ufbx_anim_layer* Layer = Stack->layers[LayerIndex];

				TMap<const ufbx_node*, FRigidAnimationPayloadContext> TransformationPayloadContexts;

				for (int32 AnimPropIndex = 0; AnimPropIndex < Layer->anim_props.count; ++AnimPropIndex)
				{
					const ufbx_anim_prop& Prop = Layer->anim_props[AnimPropIndex];

					// Morph animation is DeformPercent
					if (FCStringAnsi::Strcmp(UFBX_DeformPercent, Prop.prop_name.data)==0)
					{
						if (ufbx_blend_channel* Channel = ufbx_as_blend_channel(Prop.element))
						{
							ufbx_blend_shape* TargetShape = Channel->target_shape;

							ufbx_anim_curve* MorphCurve = Prop.anim_value->curves[0];

							Convert::FMeshNameAndUid MorphTargetNameId(Parser, *TargetShape);

							// from UE::Interchange::Private::FFbxAnimation::AddMorphTargetCurvesAnimation
							// #ufbx_todo: do we need different PayloadKey for same curve used to animate different meshes?
							// (MorphTargetAnimationBuildingData.InterchangeMeshNode ? MorphTargetAnimationBuildingData.InterchangeMeshNode->GetUniqueID() : FString()) //Same shape can be animated on different mesh node
							// If not - maybe we could just create key based on curve's global(within scene) index - ufbx_scene::anim_curves?

							FString PayloadKey = MorphTargetNameId.UniqueID
								+ TEXT("\\") + FString::FromInt(AnimStackIndex)
								+ TEXT("\\") + FString::FromInt(LayerIndex)
								+ TEXT("\\") + FString::FromInt(AnimPropIndex)
								+ TEXT("_CurveAnimationPayloadKey");

							const ufbx_mesh*const* MeshFound = MeshPerBlendShape.Find(TargetShape);

							const ufbx_mesh* Mesh = *MeshFound;
							TSet<UInterchangeSkeletalAnimationTrackNode*> Tracks;

							const UInterchangeMeshNode*const* MeshNodeFound = Meshes.MeshToMeshNode.Find(Mesh);
							if (MeshNodeFound)
							{
								const UInterchangeMeshNode* MeshNode = *MeshNodeFound;

								if (MeshNode->IsSkinnedMesh())
								{
									for (ufbx_skin_deformer* Deformer : Mesh->skin_deformers)
									{
										for (ufbx_skin_cluster* Cluster : Deformer->clusters)
										{
											Tracks.Add(GetTrackNode(Scene.GetSkeletonRoot(Cluster->bone_node)));
										}
									}
								}
								else
								{
									//Find MeshInstances: where CustomAssetInstanceUid == MeshNode->GetUniqueID
									// For every occurence create a Track node MeshNode->GetUniqueID
									// #ufbx_todo: maybe better to keep map of instances per mesh, instead of iterating...
									TSet<UInterchangeSceneNode*> SkeletonNodes;
									NodeContainer.IterateNodesOfType<UInterchangeSceneNode>([&](const FString& NodeUid, UInterchangeSceneNode* SceneNode)
										{
											FString AssetInstanceUid;
											if (SceneNode->GetCustomAssetInstanceUid(AssetInstanceUid) && AssetInstanceUid == MeshNode->GetUniqueID())
											{
												
												SkeletonNodes.Add(SceneNode);
											}
										});
									// Make track nodes separately to prevent modifying NodeContainer while iterating it
									for (UInterchangeSceneNode* SceneNode : SkeletonNodes)
									{
										Tracks.Add(GetTrackNode(SceneNode->GetUniqueID()));
									}
								}
							}

							for (UInterchangeSkeletalAnimationTrackNode* Track : Tracks)
							{
								Track->SetAnimationPayloadKeyForMorphTargetNodeUid(MorphTargetNameId.UniqueID, PayloadKey, EInterchangeAnimationPayLoadType::MORPHTARGETCURVE);
							}

							FMorphAnimationPayloadContext MorphPayloadContext;
							MorphPayloadContext.Curve = MorphCurve;

							for (const ufbx_blend_keyframe& Keyframe : Channel->keyframes)
							{
								MorphPayloadContext.InbetweenFullWeights.Add(Keyframe.target_weight);
								MorphPayloadContext.InbetweenCurveNames.Add(Parser.GetElementName(Keyframe.shape->element));
							}

							Parser.PayloadContexts.Add(PayloadKey, MorphPayloadContext);
						}
					}

					const ufbx_node* Node = ufbx_as_node(Prop.element);
					// #ufbx_todo: ignore bones? Btw, why bones are not here?
					if (Node && !Node->bone)
					{
						if (FCStringAnsi::Strcmp(UFBX_Lcl_Translation, Prop.prop_name.data)==0)
						{
							FRigidAnimationPayloadContext& RigidAnimationPayloadContext = TransformationPayloadContexts.FindOrAdd(Node);
							RigidAnimationPayloadContext.Node = Node;
							RigidAnimationPayloadContext.TranslationProp = &Prop;
						}
						if (FCStringAnsi::Strcmp(UFBX_Lcl_Rotation, Prop.prop_name.data)==0)
						{
							FRigidAnimationPayloadContext& RigidAnimationPayloadContext = TransformationPayloadContexts.FindOrAdd(Node);
							RigidAnimationPayloadContext.Node = Node;
							TransformationPayloadContexts.FindOrAdd(Node).RotationProp = &Prop;
						}
						if (FCStringAnsi::Strcmp(UFBX_Lcl_Scaling, Prop.prop_name.data)==0)
						{
							FRigidAnimationPayloadContext& RigidAnimationPayloadContext = TransformationPayloadContexts.FindOrAdd(Node);
							RigidAnimationPayloadContext.Node = Node;
							TransformationPayloadContexts.FindOrAdd(Node).ScalingProp = &Prop;
						}
					}
				}

				for (const TPair<const ufbx_node*, FRigidAnimationPayloadContext>& Pair  : TransformationPayloadContexts)
				{
					const ufbx_node* Node = Pair.Key;
					const FRigidAnimationPayloadContext& RigidAnimationPayloadContext = Pair.Value;

					// inspired by UE::Interchange::Private::FFbxAnimation::AddRigidTransformAnimation

					constexpr int32 TranslationChannel = 0x0001 | 0x0002 | 0x0004;
					constexpr int32 RotationChannel = 0x0008 | 0x0010 | 0x0020;
					constexpr int32 ScaleChannel = 0x0040 | 0x0080 | 0x0100;

					int32 UsedChannels = 0;
					if (RigidAnimationPayloadContext.TranslationProp && Algo::AnyOf(MakeConstArrayView(RigidAnimationPayloadContext.TranslationProp->anim_value->curves)))
					{
						UsedChannels |= TranslationChannel;
					}
					if (RigidAnimationPayloadContext.RotationProp && Algo::AnyOf(MakeConstArrayView(RigidAnimationPayloadContext.RotationProp->anim_value->curves)))
					{
						UsedChannels |= RotationChannel;
					}
					if (RigidAnimationPayloadContext.ScalingProp && Algo::AnyOf(MakeConstArrayView(RigidAnimationPayloadContext.ScalingProp->anim_value->curves)))
					{
						UsedChannels |= ScaleChannel;
					}

					if (UsedChannels != 0)
					{
						if (UInterchangeSceneNode** Found = Parser.ElementIdToSceneNode.Find(Node->element_id))
						{
							UInterchangeSceneNode* SceneNode = *Found; 

							const FString PayloadKeySuffix = FString::Printf(TEXT("_RigidAnimationPayloadKey_%d"), AnimStackIndex);
							const FString PayloadKey = Parser.GetNodeUid(*Node) + PayloadKeySuffix;
							if (ensure(!Parser.PayloadContexts.Contains(PayloadKey)))
							{
								Parser.PayloadContexts.Add(PayloadKey, RigidAnimationPayloadContext);

								UInterchangeTransformAnimationTrackNode* TransformAnimTrackNode = NewObject< UInterchangeTransformAnimationTrackNode >(&NodeContainer);

								const FString TransformAnimTrackNodeName = FString::Printf(TEXT("%s"), *SceneNode->GetDisplayLabel());
								const FString TransformAnimTrackNodeUid = TEXT("\\AnimationTrack\\") + TransformAnimTrackNodeName;

								NodeContainer.SetupNode(TransformAnimTrackNode, TransformAnimTrackNodeUid, TransformAnimTrackNodeName, EInterchangeNodeContainerType::TranslatedAsset);
								TransformAnimTrackNode->SetCustomActorDependencyUid(*SceneNode->GetUniqueID());
								TransformAnimTrackNode->SetCustomAnimationPayloadKey(PayloadKey, EInterchangeAnimationPayLoadType::CURVE);
								TransformAnimTrackNode->SetCustomUsedChannels(UsedChannels);
							}

							// #ufbx_todo: UE::Interchange::Private::ProcessCustomAttributes
							// ProcessCustomAttributes(Parser, Node, TransformAnimTrackNode);
						}
					}

				}
			}
		}

		TArray<FString> AnimTrackNodeUids;
		NodeContainer.IterateNodesOfType<UInterchangeAnimationTrackNode>([&](const FString& NodeUid, UInterchangeAnimationTrackNode* TransformAnimationTrackNode)
			{
				AnimTrackNodeUids.Add(NodeUid);
			});

		//Only one Track Set Node per fbx file:
		if (AnimTrackNodeUids.Num() > 0)
		{
			FString Name;
			if (UInterchangeSceneNode** RootSceneNodeFound = Parser.ElementIdToSceneNode.Find(Parser.Scene->root_node->element_id))
			{
				UInterchangeSceneNode* RootSceneNode = *RootSceneNodeFound;
				Name = RootSceneNode->GetName();
			}

			UInterchangeAnimationTrackSetNode* TrackSetNode = NewObject< UInterchangeAnimationTrackSetNode >(&NodeContainer);

			TrackSetNode->SetCustomFrameRate(FrameRate);

			const FString AnimTrackSetNodeUid = TEXT("\\Animation\\") + Name;
			const FString AnimTrackSetNodeDisplayLabel = Name + TEXT("_TrackSetNode");

			NodeContainer.SetupNode(TrackSetNode, AnimTrackSetNodeUid, AnimTrackSetNodeDisplayLabel, EInterchangeNodeContainerType::TranslatedAsset);

			for (const FString& AimTrackNodeUid : AnimTrackNodeUids)
			{
				TrackSetNode->AddCustomAnimationTrackUid(AimTrackNodeUid);
			}
		}
	}

	bool FUfbxAnimation::AddNodeAttributeCurvesAnimation(FUfbxParser& Parser
		, const FString NodeUid
		, const ufbx_prop& Prop
		, const ufbx_anim_value& AnimValue
		, TOptional<FString>& OutPayloadKey
		, TOptional<bool>& OutIsStepCurve
		, FString& OutCurveName)
	{
		FString PropertyName = Convert::ToUnrealString(Prop.name);
		FString PayLoadKey = NodeUid + PropertyName + TEXT("_AnimationPayloadKey");
		if (Parser.PayloadContexts.Contains(PayLoadKey))
		{
			// Guard against multiple Props with the same name
			PayLoadKey = NodeUid + PropertyName + FString::FromInt(AnimValue.element_id) + TEXT("_AnimationPayloadKey");
		}
		if (!ensure(!Parser.PayloadContexts.Contains(PayLoadKey)))
		{
			return false;
		}

		bool bIsStepCurve;
		switch (Prop.type)
		{
		case UFBX_PROP_NUMBER:
		case UFBX_PROP_VECTOR:
		case UFBX_PROP_COLOR:
		case UFBX_PROP_COLOR_WITH_ALPHA:
		case UFBX_PROP_TRANSLATION:
		case UFBX_PROP_ROTATION:
		case UFBX_PROP_SCALING:
		case UFBX_PROP_DISTANCE:
		case UFBX_PROP_COMPOUND:
			bIsStepCurve = false;
			break;
		default:
			bIsStepCurve = true;
		};

		bool bHasCurve = false;
		//Only curves with Constant interpolation on all keys are deemed as StepCurves.
		for (const ufbx_anim_curve* Curve: AnimValue.curves)
		{
			if (Curve)
			{
				bHasCurve = true;
				for (const ufbx_keyframe& Keyframe : Curve->keyframes)
				{
					if (Keyframe.interpolation != UFBX_INTERPOLATION_CONSTANT_PREV 
						&& Keyframe.interpolation != UFBX_INTERPOLATION_CONSTANT_NEXT)
					{
						bIsStepCurve = false;
					}
				}
			}
		}

		if (!bHasCurve)
		{
			// Property even though has an anim_value but still not animated
			return false;
		}

		OutIsStepCurve = bIsStepCurve;
		Parser.PayloadContexts.Add(PayLoadKey, FPropertyAnimationPayloadContext{
			.Prop = &Prop,
			.AnimValue = &AnimValue,
			.bIsStepAnimation = bIsStepCurve
		});
		OutPayloadKey = PayLoadKey;
		OutCurveName = Convert::ToUnrealString(AnimValue.name);

		return true;
	}
}

#undef LOCTEXT_NAMESPACE
