// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UfbxParser.h"

namespace UE::Interchange
{
	struct FAnimationPayloadQuery;
	struct FAnimationPayloadData;
}

struct FInterchangeCurve;
struct FInterchangeStepCurve;

namespace UE::Interchange::Private
{
	struct FSkeletalAnimationPayloadContext;
	class FUfbxScene;
	class FUfbxMesh;

	class FUfbxAnimation
	{
	public:
		static void FetchSkinnedAnimation(ufbx_scene* Scene, const UE::Interchange::FAnimationPayloadQuery& PayloadQuery, const FSkeletalAnimationPayloadContext& SkeletalAnimationPayload, FAnimationPayloadData& PayloadData);
		static bool FetchMorphTargetAnimation(const FUfbxParser& Parser, const FMorphAnimationPayloadContext& Animation, TArray<FInterchangeCurve>& OutCurves);
		static bool FetchRigidAnimation(const FUfbxParser& Parser, const FRigidAnimationPayloadContext& Animation, TArray<FInterchangeCurve>& OutCurves);

		static bool FetchPropertyAnimationCurves(const FUfbxParser& Parser, const FPropertyAnimationPayloadContext& Animation, TArray<FInterchangeCurve>& OutCurves);
		static bool FetchPropertyAnimationStepCurves(const FUfbxParser& Parser, const FPropertyAnimationPayloadContext& Animation, TArray<FInterchangeStepCurve>& OutCurves);

		static void AddAnimation(FUfbxParser& Parser, FUfbxScene& Scene, const FUfbxMesh& Meshes, UInterchangeBaseNodeContainer& NodeContainer);

		static bool AddNodeAttributeCurvesAnimation(FUfbxParser& Parser
			, const FString NodeUid
			, const ufbx_prop& Prop
			, const ufbx_anim_value& AnimValue
			, TOptional<FString>& OutPayloadKey
			, TOptional<bool>& OutIsStepCurve
			, FString& OutCurveName);

	};
}
