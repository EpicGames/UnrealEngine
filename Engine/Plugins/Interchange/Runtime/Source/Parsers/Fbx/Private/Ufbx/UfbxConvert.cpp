// Copyright Epic Games, Inc. All Rights Reserved.

#include "UfbxConvert.h"
#include "UfbxParser.h"

#include "InterchangeSceneNode.h"


namespace UE::Interchange::Private
{

Convert::FMeshNameAndUid::FMeshNameAndUid(FUfbxParser& Parser, const ufbx_element& Mesh)
	: Label(Parser.GetMeshLabel(Mesh))
	, UniqueID(Parser.GetMeshUid(Mesh))
{
}

}
