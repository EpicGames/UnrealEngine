// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include <ufbx.h>

class UInterchangeSceneNode;

namespace UE::Interchange::Private
{

class FUfbxParser;

namespace Convert
{
	inline FString ToUnrealString(const ufbx_string& Src)
	{
		return UTF8_TO_TCHAR(Src.data);
	}

	struct FMeshNameAndUid
	{
		FMeshNameAndUid(FUfbxParser& Parser, const ufbx_element& Mesh);

		FMeshNameAndUid(FUfbxParser& Parser, const ufbx_mesh& Mesh) : FMeshNameAndUid(Parser, Mesh.element) {}
		FMeshNameAndUid(FUfbxParser& Parser, const ufbx_blend_shape& Shape) : FMeshNameAndUid(Parser, Shape.element) {}

		FString Label;
		FString UniqueID;
	};

	inline FVector2d ConvertVec2(const ufbx_vec2& V)
	{
		return FVector2d(V.x, V.y);
	}

	inline FVector2d ConvertUV(const ufbx_vec2& V)
	{
		return FVector2d(V.x, 1.0-V.y);
	}

	inline FVector ConvertVec3(const ufbx_vec3& V)
	{
		return FVector(V.x, V.y, V.z);
	}

	inline FVector4d ConvertVec4(const ufbx_vec4& V)
	{
		return FVector4d(V.x, V.y, V.z, V.w);
	}

	inline FQuat ConvertQuat(const ufbx_quat& Q)
	{
		return FQuat(Q.x, Q.y, Q.z, Q.w);
	}

	inline FLinearColor ConvertColor(const ufbx_vec4& Color) 
	{
		// FBX colors are sRGB
		FVector4d ConvertedColor = Convert::ConvertVec4(Color
			).ComponentMax(FVector4d::Zero()
			).ComponentMin(FVector4d::One())
			*255.f;
		return FLinearColor(FColor{uint8(ConvertedColor.X), uint8(ConvertedColor.Y), uint8(ConvertedColor.Z), uint8(ConvertedColor.W)});
	}

	inline FTransform ConvertTransform(const ufbx_transform& InTransform)
	{
		FTransform Result;

		Result.SetTranslation(ConvertVec3(InTransform.translation));
		Result.SetScale3D(ConvertVec3(InTransform.scale));
		Result.SetRotation(ConvertQuat(InTransform.rotation));

		return Result;
	}

	inline FMatrix ConvertMatrix(const ufbx_matrix& Matrix)
	{
		return FMatrix(
			ConvertVec3(Matrix.cols[0]),
			ConvertVec3(Matrix.cols[1]),
			ConvertVec3(Matrix.cols[2]),
			ConvertVec3(Matrix.cols[3])
		);
	}
};

}
