// Copyright Epic Games, Inc. All Rights Reserved.

#include "UfbxMaterial.h"
#include "UfbxParser.h"

#include "UfbxConvert.h"

#include "FbxMaterial.h"

#include "InterchangeMaterialDefinitions.h"
#include "InterchangeMaterialInstanceNode.h"
#include "InterchangeTexture2DNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "Fbx/InterchangeFbxMessages.h"

#include "Misc/Paths.h"

#if WITH_ENGINE
#include "Texture/InterchangeImageWrapperTranslator.h"
#endif

#define GET_VALLUE_FROM_FACTOR_MAP( FactorMap ) FactorMap.has_value ? FactorMap.value_real : 1.f;

#define LOCTEXT_NAMESPACE "InterchangeUfbxMaterial"

namespace UE::Interchange::Private
{

	namespace Inputs
	{
		///PostFixes
		namespace PostFix
		{
			const FString Color_RGB = TEXT("_RGB");
			const FString Color_A = TEXT("_A");

			const FString TexCoord = TEXT("_TexCoord");

			const FString OffsetX = TEXT("_Offset_X");
			const FString OffsetY = TEXT("_Offset_Y");
			const FString ScaleX = TEXT("_Scale_X");
			const FString ScaleY = TEXT("_Scale_Y");

			const FString OffsetScale = TEXT("_OffsetScale");

			const FString Rotation = TEXT("_Rotation");

			const FString TilingMethod = TEXT("_TilingMethod");
		}
	}

	class  FMaterialMapConverter
	{
	public:
		explicit FMaterialMapConverter(FUfbxParser& InParser, UInterchangeBaseNodeContainer& InNodeContainer, UInterchangeBaseNode& InMaterialInstance)
			: Parser(InParser)
			, NodeContainer(InNodeContainer)
			, MaterialInstance(InMaterialInstance)
		{
		}

		static FString FindTextureFile(const FUfbxParser& Parser, const ufbx_texture& Texture)
		{
			// #ufbx_todo: maybe just for each of "filename" always check absolute/relative? Just need to decide on the order...

			FString TextureFilepath = Convert::ToUnrealString(Texture.absolute_filename);
			FPaths::NormalizeFilename(TextureFilepath);
			if(FPaths::FileExists(TextureFilepath))
			{
				return TextureFilepath;
			}

			FString FileBasePath = FPaths::GetPath(Parser.SourceFilename);

			FString RelativeToFileBasePath = FileBasePath / FPaths::GetCleanFilename(TextureFilepath);
			FPaths::NormalizeFilename(RelativeToFileBasePath);
			if (FPaths::FileExists(RelativeToFileBasePath))
			{
				return RelativeToFileBasePath;
			}

			FString RelativeToFBXTextureFilepath = FileBasePath / Convert::ToUnrealString(Texture.relative_filename);
			FPaths::NormalizeFilename(RelativeToFBXTextureFilepath);
			if( FPaths::FileExists(RelativeToFBXTextureFilepath) )
			{
				return RelativeToFBXTextureFilepath;
			}

			FString FilenameToFBXTextureFilepath = FileBasePath / Convert::ToUnrealString(Texture.filename);
			FPaths::NormalizeFilename(FilenameToFBXTextureFilepath);
			if( FPaths::FileExists(FilenameToFBXTextureFilepath) )
			{
				return FilenameToFBXTextureFilepath;
			}

			// Some fbx files do not store the actual absolute filename as absolute and it is actually relative.  Try to get it relative to the FBX file we are importing
			FString AbsoluteAsRelativeToFBXTextureFilepath = FileBasePath / Convert::ToUnrealString(Texture.absolute_filename);
			FPaths::NormalizeFilename(AbsoluteAsRelativeToFBXTextureFilepath );
			if( FPaths::FileExists(AbsoluteAsRelativeToFBXTextureFilepath ) )
			{
				return AbsoluteAsRelativeToFBXTextureFilepath ;
			}

			if (TextureFilepath.IsEmpty())
			{
				// Make a name so it could be used as a key or in a message
				return FString::Printf(TEXT("Texture_%d"), Texture.element_id);
			}
			return TextureFilepath;
		}

		const UInterchangeTexture2DNode* CreateTexture2DNode(const ufbx_texture& Texture)
		{
			return CreateTexture2DNode(Parser, NodeContainer, Texture, &MaterialInstance);
		}

		static const UInterchangeTexture2DNode* CreateTexture2DNode(FUfbxParser& Parser, UInterchangeBaseNodeContainer& NodeContainer, const ufbx_texture& Texture, UInterchangeBaseNode* InMaterialInstance=nullptr)
		{
			if (Texture.type != UFBX_TEXTURE_FILE)
			{
				return nullptr;
			}
			const FString TextureFilename = FindTextureFile(Parser, Texture);

			if (TextureFilename.IsEmpty() || !FPaths::FileExists(TextureFilename))
			{
				// Texture might be embedded
				if (Texture.content.data && Texture.content.size > 0)
				{
#if WITH_ENGINE
					Parser.PayloadContexts.Add(TextureFilename, FPayloadContext(FPayloadContext::Element, Texture.element_id));
#endif
				}
				else 
				{
					if (!GIsAutomationTesting)
					{
						UInterchangeResultTextureDisplay_TextureFileDoNotExist* Message = Parser.AddMessage<UInterchangeResultTextureDisplay_TextureFileDoNotExist>();
						Message->TextureName = TextureFilename;
						Message->MaterialName = InMaterialInstance ? InMaterialInstance->GetDisplayLabel() : FString();
					}
					return nullptr;
				}
			}

			FString TextureName;
			FString TextureNodeID;
			ensure(UInterchangeTextureNode::ExtractNodeUidAndNameFromFilePath(TextureFilename, TextureNodeID, TextureName));

			if (const UInterchangeTexture2DNode* TextureNode = Cast<const UInterchangeTexture2DNode>(NodeContainer.GetNode(TextureNodeID)))
			{
				return TextureNode;
			}

			UInterchangeTexture2DNode* NewTextureNode = UInterchangeTexture2DNode::Create(&NodeContainer, TextureNodeID, TextureName);

			//All texture translator expect a file as the payload key
			NewTextureNode->SetPayLoadKey(TextureFilename);

			return NewTextureNode;
		}

		FUfbxParser& Parser;
		UInterchangeBaseNodeContainer& NodeContainer;
		UInterchangeBaseNode& MaterialInstance;
	};

	UInterchangeMaterialInstanceNode* CreateMaterialInstanceNode(UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeUID, const FString& NodeLabel)
	{
		UInterchangeMaterialInstanceNode* MaterialInstanceNode = NewObject<UInterchangeMaterialInstanceNode>(&NodeContainer);
		NodeContainer.SetupNode(MaterialInstanceNode, NodeUID, NodeLabel, EInterchangeNodeContainerType::TranslatedAsset);

		return MaterialInstanceNode;
	}

	void FUfbxMaterial::AddAllTextures(UInterchangeBaseNodeContainer& NodeContainer)
	{
		for (const ufbx_texture* Texture : Parser.Scene->textures)
		{
			FMaterialMapConverter::CreateTexture2DNode(Parser, NodeContainer, *Texture);
		}
	}

	void FUfbxMaterial::AddMaterials(UInterchangeBaseNodeContainer& NodeContainer)
	{
		using namespace UE::Interchange::Materials;

		for (ufbx_material* Material : Parser.Scene->materials)
		{
			if (!Material)
			{
				continue;
			}

			FString MaterialLabel = Parser.GetMaterialLabel(*Material);
			
			FString MaterialUid = Parser.GetMaterialUid(*Material);
			UInterchangeMaterialInstanceNode* MaterialInstanceNode = CreateMaterialInstanceNode(NodeContainer, MaterialUid, MaterialLabel);
			if (MaterialInstanceNode == nullptr)
			{
				FFormatNamedArguments Args
				{
					{ TEXT("MaterialName"), FText::FromString(MaterialLabel) }
				};
				UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
				Message->Text = FText::Format(LOCTEXT("CannotCreateUfbxMaterial", "Cannot create uFBX material '{MaterialName}'."), Args);
				continue;
			}

			// WithUVTransform
			bool bHasUVTransform = false;

			FMaterialMapConverter MaterialMapConverter(Parser, NodeContainer, *MaterialInstanceNode);
			
			auto SetColorFromMap = [&MaterialInstanceNode](FName InputName, const ufbx_material_map& Map, float Factor, TFunctionRef<bool(const FLinearColor&)> Filter = [](const FLinearColor&){return true;}) -> bool
			{
				if (Map.value_components >=3)
				{
					const FString InputAttributeKey = InputName.ToString();

					FLinearColor PropertyValue = (Map.value_components == 4) 
					? FLinearColor(Convert::ConvertVec4(Map.value_vec4) * Factor)
					: FLinearColor(Convert::ConvertVec3(Map.value_vec3) * Factor);
					if (Filter(PropertyValue))
					{
						return MaterialInstanceNode->AddVectorParameterValue(InputAttributeKey, PropertyValue);
					}
				}

				return false;
			};

			auto SetScalarFromMap = [&MaterialInstanceNode](FName InputName, const ufbx_material_map& Map, float Factor) -> bool
			{
				const FString InputAttributeKey = InputName.ToString();
				float PropertyValue(Map.value_real);
				return MaterialInstanceNode->AddScalarParameterValue(InputAttributeKey, PropertyValue);
			};

			auto SetTextureFromMap = [&MaterialInstanceNode, &MaterialMapConverter, &bHasUVTransform](FName InputName, const ufbx_material_map& Map, float Factor) -> const UInterchangeTexture2DNode*
			{
				const ufbx_texture* Texture = Map.texture_enabled ? Map.texture : nullptr;
				if (!Texture)
				{
					return nullptr;	
				}

				const UInterchangeTexture2DNode* TextureNode = MaterialMapConverter.CreateTexture2DNode(*Texture);

				if (!TextureNode)
				{
					return nullptr;
				}

				const FString TextureMapInput = InputName.ToString() + TEXT("Map");
				MaterialInstanceNode->AddTextureParameterValue(TextureMapInput, TextureNode->GetUniqueID());

				// Using MapWeight=1.0 to have same behavior as FBX SDK/legacy parser
				MaterialInstanceNode->AddScalarParameterValue(TextureMapInput + TEXT("Weight"), 1.0);

				if (Texture->has_uv_transform)
				{
					MaterialInstanceNode->AddScalarParameterValue(TextureMapInput + TEXT("TilingU"), Texture->uv_transform.scale.x);
					MaterialInstanceNode->AddScalarParameterValue(TextureMapInput + TEXT("TilingV"), Texture->uv_transform.scale.y);
				}

				return TextureNode;
			};

			bool bHasInput = false;

			// Diffuse
			if (Material->fbx.diffuse_color.has_value)
			{
				FName InputName = Phong::Parameters::DiffuseColor;
				const float Factor = GET_VALLUE_FROM_FACTOR_MAP(Material->fbx.diffuse_factor);
				const ufbx_material_map& Map = Material->fbx.diffuse_color;

				bHasInput |= SetTextureFromMap(InputName, Map, Factor) ? true : SetColorFromMap(InputName, Map, Factor);
			}

			// Ambient
			if (Material->fbx.ambient_color.has_value)
			{
				FName InputName = Phong::Parameters::AmbientColor;
				const float Factor = GET_VALLUE_FROM_FACTOR_MAP(Material->fbx.ambient_factor);
				const ufbx_material_map& Map = Material->fbx.ambient_color;

				bHasInput |= SetTextureFromMap(InputName, Map, Factor) ? true : SetColorFromMap(InputName, Map, Factor);
			}
			
			// Emissive
			if (Material->fbx.emission_color.has_value)
			{
				const FName InputName = Phong::Parameters::EmissiveColor;
				const float Factor = GET_VALLUE_FROM_FACTOR_MAP(Material->fbx.emission_factor);
				const ufbx_material_map& Map = Material->fbx.emission_color;

				bHasInput |= SetTextureFromMap(InputName, Map, Factor) ? true 
				: Convert::ConvertVec3(Material->fbx.emission_color.value_vec3).IsNearlyZero() ? false 
				: SetColorFromMap(InputName, Map, Factor);
			}

			bool bIsPhong = false;

			// Specular
			if (Material->fbx.specular_color.has_value)
			{
				const FName InputName = Phong::Parameters::SpecularColor;
				const float Factor = GET_VALLUE_FROM_FACTOR_MAP(Material->fbx.specular_factor);
				const ufbx_material_map& Map = Material->fbx.specular_color;

				if (bool bHasSpecular = SetTextureFromMap(InputName, Map, Factor) ? true : SetColorFromMap(InputName, Map, Factor))
				{
					bHasInput = true;
					bIsPhong = true;
				}
			}

			// Shininess
			if (Material->fbx.specular_exponent.has_value)
			{
				const FName InputName = Phong::Parameters::Shininess;
				const float Shininess = Material->fbx.specular_exponent.value_real;
				const ufbx_material_map& Map = Material->fbx.specular_exponent;

				if (SetTextureFromMap(InputName, Map, 1.f))
				{
					bIsPhong |= MaterialInstanceNode->AddStaticSwitchParameterValue(TEXT("bHasShininessMap"), true);
				}
				else
				{
					bIsPhong |= SetScalarFromMap(InputName, Map, Shininess);
				}

				bHasInput |= bIsPhong;
			}

			FString BumpMaterial;

			// Normal
			if (Material->fbx.normal_map.has_value)
			{
				if (SetTextureFromMap(Phong::Parameters::Normal, Material->fbx.normal_map, 1.f))
				{
					bHasInput = true;
				}
				else if (SetTextureFromMap(Phong::Parameters::Normal, Material->fbx.bump, 1.f))
				{
					BumpMaterial = TEXT("Bump");
					bHasInput = true;
				}
			}

			MaterialInstanceNode->AddStaticSwitchParameterValue(TEXT("bIsPhong"), bIsPhong);


			// Opacity

			// A Legacy Material might be opaque but with fbx.transparency_color - still should be treated as opaque
			bool bOpaque = false;
			if (Material->shader_type == UFBX_SHADER_FBX_PHONG
				|| Material->shader_type == UFBX_SHADER_FBX_LAMBERT)
			{
				for (const ufbx_prop& Prop : Material->props.props)
				{
					if (Convert::ToUnrealString(Prop.name) == TEXT("Opacity"))
					{
						bOpaque = Prop.value_real >= 1.f;
					}
				}
			}

			// Connect only if transparency is either a texture or different from 0.f
			// Check fbx.transparency first...
			if (Material->fbx.transparency_color.has_value)
			{
				FName InputName = Phong::Parameters::Opacity;
				const ufbx_material_map& ColorMap = Material->fbx.transparency_color;
				const float Factor = GET_VALLUE_FROM_FACTOR_MAP(Material->fbx.transparency_factor);

				if (ColorMap.texture_enabled && ColorMap.texture)
				{
					FString HasOpacity{ TEXT("bHasOpacity") };
					EBlendMode BlendMode = EBlendMode::BLEND_Translucent;

					// When material is opaque but has texture this means it has opacity map
					if (bOpaque && ColorMap.value_int == 0)
					{
						InputName = Phong::Parameters::OpacityMask;
						BlendMode = EBlendMode::BLEND_Masked;
						HasOpacity += TEXT("Mask");
					}

					HasOpacity += TEXT("Map");
					MaterialInstanceNode->SetCustomBlendMode(BlendMode);

					if (const UInterchangeTexture2DNode* Texture2DNode = SetTextureFromMap(InputName, ColorMap, 1.0f))
					{
						bHasInput = true;

						// When alpha is present in the image use it as opacity, instead of RGB
						if (Parser.DoesTextureHaveAlpha(*Texture2DNode->GetPayLoadKey(), *ColorMap.texture))
						{
							MaterialInstanceNode->AddScalarParameterValue(TEXT("OpacitySourceChannel"), 1);
						}
					}
				}
				else
				{
					if (!bOpaque)
					{
						const float TransparencyScalar = Factor * ColorMap.value_real;
						const float OpacityScalar = 1.f - FMath::Clamp(TransparencyScalar, 0.f, 1.f);
						if (OpacityScalar < 1.f)
						{
							MaterialInstanceNode->SetCustomBlendMode(EBlendMode::BLEND_Translucent);
							bHasInput |= MaterialInstanceNode->AddScalarParameterValue(InputName.ToString(), OpacityScalar);
						}
					}
				}
			}

			FString ShadingModel = bIsPhong ? TEXT("Phong") : TEXT("Lambert");
			FString ParentMaterial{ TEXT("FBXLegacy") + ShadingModel + TEXT("SurfaceMaterial") + BumpMaterial };
			MaterialInstanceNode->SetCustomParent(TEXT("/InterchangeAssets/Materials/") + ParentMaterial + TEXT('.') + ParentMaterial);

			// If no valid property found, create a material anyway
			TArray<FString> InputNames;
			if (!bHasInput)
			{
				FLinearColor BaseColor;
				BaseColor.R = 0.7f;
				BaseColor.G = 0.7f;
				BaseColor.B = 0.7f;

				const FString InputValueKey = Phong::Parameters::DiffuseColor.ToString();
				MaterialInstanceNode->AddVectorParameterValue(InputValueKey, BaseColor);
			}

			// #ufbx_todo: ProcessCustomAttributes(Parser, SurfaceMaterial, MaterialInstanceNode);

			for (const ufbx_prop& Prop : Material->props.props)
			{
				if (Prop.flags & UFBX_PROP_FLAG_USER_DEFINED)
				{
					Parser.ConvertProperty(Prop, MaterialInstanceNode);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
