// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UInterchangeBaseNodeContainer;

namespace UE::Interchange::Private
{
	class FUfbxParser;

	class FUfbxMaterial
	{
	public:
		explicit FUfbxMaterial(FUfbxParser& Parser)
			: Parser(Parser)
		{
		}

		void AddMaterials(UInterchangeBaseNodeContainer& NodeContainer);
		void AddAllTextures(UInterchangeBaseNodeContainer& NodeContainer);

	private:
		FUfbxParser& Parser;
	};
}
