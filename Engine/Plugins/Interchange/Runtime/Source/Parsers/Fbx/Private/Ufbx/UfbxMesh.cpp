// Copyright Epic Games, Inc. All Rights Reserved.

#include "UfbxMesh.h"

#include "UfbxConvert.h"
#include "UfbxParser.h"

#include "InterchangeShaderGraphNode.h"
#include "InterchangeMeshNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#ifdef WITH_MESH_DESCRIPTION_BUILDER
#include "MeshDescriptionBuilder.h"
#endif 

#include "Algo/NoneOf.h"
#include "SkeletalMeshAttributes.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "VectorUtil.h"
#include "Fbx/InterchangeFbxMessages.h"

#define LOCTEXT_NAMESPACE "InterchangeFbxParser"

namespace UE::Interchange::Private
{
	namespace MeshUtils
	{
		ufbx_material* GetMeshMaterial(ufbx_mesh& Mesh, int MaterialIndex)
		{
			// ufbx_mesh has material_parts even when it has no materials
			TConstArrayView<ufbx_material*> FbxMaterials(Mesh.materials.data, Mesh.materials.count);
			return FbxMaterials.IsValidIndex(MaterialIndex) ? FbxMaterials[MaterialIndex] : nullptr;
		}
	}

	class FFaceCollector
	{
	public:
#ifdef WITH_MESH_DESCRIPTION_BUILDER
		FFaceCollector(FMeshDescriptionBuilder& InBuilder, ufbx_mesh& InMesh, const int32 UvCount)
			: Builder(InBuilder)
			, Mesh(InMesh)
		{
			VertexMapping.Reserve(Mesh.num_vertices);
			TriIndicesBuffer.SetNumUninitialized(Mesh.max_face_triangles * 3);

			UVIndexMap.SetNum(UvCount);
			for (int32 UvLayerIndex = 0; UvLayerIndex < UvCount; ++UvLayerIndex)
			{
				ufbx_uv_set& UvSet = Mesh.uv_sets[UvLayerIndex];
				TArray<FUVID>& UvIndexMap = UVIndexMap[UvLayerIndex];

				Builder.ReserveNewUVs(UvSet.vertex_uv.values.count, UvLayerIndex);
				UvIndexMap.Reserve(UvSet.vertex_uv.values.count);

				for (const ufbx_vec2& Uv : UvSet.vertex_uv.values)
				{
					FUVID UvId = Builder.AppendUV(Convert::ConvertUV(Uv), UvLayerIndex);
					UvIndexMap.Add(UvId);
				}
			}
		}
#endif

		int32 TriangulateFace(const ufbx_face& Face)
		{
			return ufbx_triangulate_face(TriIndicesBuffer.GetData(), TriIndicesBuffer.Num(), &Mesh, Face);
		}

		FVector ConvertVertexNormal(uint32_t TriangulatedIndex) const
		{
			return Convert::ConvertVec3(ufbx_get_vertex_vec3(&Mesh.vertex_normal, TriangulatedIndex));
		}

		FLinearColor ConvertVertexColor(uint32_t TriangulatedIndex) const
		{
			return Convert::ConvertColor(ufbx_get_vertex_vec4(&Mesh.vertex_color, TriangulatedIndex));
		}

		FVector ConvertVertexTangent(uint32_t TriangulatedIndex) const
		{
			return Convert::ConvertVec3(ufbx_get_vertex_vec3(&Mesh.vertex_tangent, TriangulatedIndex));
		}

		FVector ConvertVertexBiTangent(uint32_t TriangulatedIndex) const
		{
			return Convert::ConvertVec3(ufbx_get_vertex_vec3(&Mesh.vertex_bitangent, TriangulatedIndex));
		}

		void AddFace(FPolygonGroupID GroupID, const ufbx_face& Face, uint32 FaceIndex)
		{
			int32 FaceTriCount = TriangulateFace(Face);

			const uint32* TriangulatedIndices = TriIndicesBuffer.GetData();
			for (int32 TriangleIndex = 0; TriangleIndex < FaceTriCount; ++TriangleIndex, TriangulatedIndices+=3)
			{
				FVertexInstanceID Triangle[3];

				uint32 SrcVertexIndices[3];
				for (int32 Vertex = 0; Vertex < 3; ++Vertex)
				{
					uint32_t TriangulatedIndex = TriangulatedIndices[Vertex];
					SrcVertexIndices[Vertex] = Mesh.vertex_position.indices[TriangulatedIndex];
				}

				// Check degenerated
				if (SrcVertexIndices[0] == SrcVertexIndices[1] || SrcVertexIndices[1] == SrcVertexIndices[2] || SrcVertexIndices[2] == SrcVertexIndices[0])
				{
					NumDegenerateTriangles ++;
					continue;
				}
				
#ifdef WITH_MESH_DESCRIPTION_BUILDER
				for (int32 Vertex = 0; Vertex < 3; ++Vertex)
				{
					FVertexInstanceID VertexInstanceID = Builder.AppendInstance(VertexMapping[SrcVertexIndices[Vertex]]);
					Triangle[Vertex] = VertexInstanceID;

					const uint32_t TriangulatedIndex = TriangulatedIndices[Vertex];
					if (Mesh.vertex_normal.exists)
					{
						FVector Normal = ConvertVertexNormal(TriangulatedIndex);

						if (Mesh.vertex_tangent.exists && Mesh.vertex_bitangent.exists)
						{
							FVector Tangent = ConvertVertexTangent(TriangulatedIndex);
							FVector BiTangent = ConvertVertexBiTangent(TriangulatedIndex);

							Builder.SetInstanceTangentSpace(VertexInstanceID, Normal, Tangent,
							                                        UE::Geometry::VectorUtil::BitangentSign(Normal, Tangent, BiTangent)
							);
						}
						else
						{
							Builder.SetInstanceNormal(VertexInstanceID, Normal);
						}
					}

					if (Mesh.vertex_color.exists)
					{
						FLinearColor Color(ConvertVertexColor(TriangulatedIndex));
						Builder.SetInstanceColor(VertexInstanceID, FVector4f(Color.R, Color.G, Color.B, Color.A));
					}
				}

				const FTriangleID TriangleId = Builder.AppendTriangle(Triangle[0], Triangle[1], Triangle[2], GroupID);
				PolygonToFaceIndex.Add(FaceIndex);

				const int32 UVCount = UVIndexMap.Num();
				for (int32 UvLayerIndex = 0; UvLayerIndex < UVCount; ++UvLayerIndex)
				{
					const ufbx_uv_set& UvSet = Mesh.uv_sets[UvLayerIndex];
					const TArray<FUVID>& UvIndexMap = UVIndexMap[UvLayerIndex];

					Builder.AppendUVTriangle(TriangleId, 
						UvIndexMap[UvSet.vertex_uv.indices[TriangulatedIndices[0]]],
						UvIndexMap[UvSet.vertex_uv.indices[TriangulatedIndices[1]]],
						UvIndexMap[UvSet.vertex_uv.indices[TriangulatedIndices[2]]],
						UvLayerIndex);
				}
#endif
			}
		};

		// Map ufbx vertices index to FMeshDescription's FVertexID
		TArray<FVertexID> VertexMapping;

		// Maps FPolygonID->GetValue() for triangle to it's ufbx face index in ufbx
		TArray<FPolygonID> PolygonToFaceIndex;

#ifdef WITH_MESH_DESCRIPTION_BUILDER
		FMeshDescriptionBuilder& Builder;
#endif
		const ufbx_mesh& Mesh;

		// Temp buffer to keep triangulated face indices 
		TArray<uint32_t> TriIndicesBuffer;

		// Each array corresponds to processed ufbx_uv_set from Mesh.uv_sets
		// Maps position(index) of the uv in ufbx_uv_set to created UV in MeshDescription
		TArray<TArray<FUVID>> UVIndexMap;

		uint32 NumDegenerateTriangles = 0;
	};
	
	void FUfbxMesh::AddAllMeshes(UInterchangeBaseNodeContainer& NodeContainer)
	{
		for (ufbx_mesh* Mesh : Parser.Scene->meshes)
		{
			Convert::FMeshNameAndUid MeshNameAndUid(Parser, *Mesh);

			if (Cast<UInterchangeMeshNode>(NodeContainer.GetNode(MeshNameAndUid.UniqueID)))
			{
				continue;
			}

			UInterchangeMeshNode* MeshNode = CreateMeshNode(NodeContainer, MeshNameAndUid.Label, MeshNameAndUid.UniqueID);

			MeshToMeshNode.Add(Mesh, MeshNode);

			if (Mesh->skin_deformers.count > 0)
			{
				TArray<FString> JointNodeUniqueIDs;
				for (ufbx_skin_deformer* Deformer : Mesh->skin_deformers)
				{
					JointNodeUniqueIDs.Reserve(JointNodeUniqueIDs.Num()+Deformer->clusters.count);
					for (ufbx_skin_cluster* Cluster : Deformer->clusters)
					{
						if (Cluster->bone_node)
						{
							FString Uid = Parser.GetNodeUid(*Cluster->bone_node);
							if (!JointNodeUniqueIDs.Contains(Uid))
							{
								JointNodeUniqueIDs.Add(Uid);
								MeshNode->SetSkeletonDependencyUid(Uid);
							}
						}
					}	
				}

				if (!JointNodeUniqueIDs.IsEmpty())
				{
					MeshNode->SetSkinnedMesh(true);
				}
			}

			FBox MeshBoundingBox;
			for (int VertexIndex = 0; VertexIndex < Mesh->num_vertices; ++VertexIndex)
			{
				FVector Pos = Convert::ConvertVec3(Mesh->vertex_position.values.data[VertexIndex]);
				MeshBoundingBox += Pos;
			}

			bool bMeshHasVertexNormal = Mesh->vertex_normal.exists;
			bool bMeshHasVertexBinormal = Mesh->vertex_bitangent.exists && Mesh->vertex_tangent.exists;
			bool bMeshHasVertexTangent = bMeshHasVertexBinormal;
			bool bMeshHasSmoothGroup = Mesh->face_smoothing.count > 0 || Mesh->edge_smoothing.count > 0;
			bool bMeshHasVertexColor = Mesh->vertex_color.exists;
			int32 MeshUVCount = Mesh->uv_sets.count;
			
			// Why are these needed?
			MeshNode->SetCustomVertexCount(Mesh->num_vertices);
			MeshNode->SetCustomPolygonCount(Mesh->num_triangles);
			MeshNode->SetCustomBoundingBox(MeshBoundingBox);
			MeshNode->SetCustomHasVertexNormal(bMeshHasVertexNormal);
			MeshNode->SetCustomHasVertexBinormal(bMeshHasVertexBinormal);
			MeshNode->SetCustomHasVertexTangent(bMeshHasVertexTangent);
			MeshNode->SetCustomHasSmoothGroup(bMeshHasSmoothGroup);
			MeshNode->SetCustomHasVertexColor(bMeshHasVertexColor);
			MeshNode->SetCustomUVCount(MeshUVCount);

			// As long as there are materials then ufbx_mesh::material_parts indexing corresponds to ufbx_mesh::materials
			for (int32 MaterialIndex = 0; MaterialIndex < Mesh->materials.count; ++MaterialIndex)
			{
				const ufbx_mesh_part& MeshPart = Mesh->material_parts[MaterialIndex];
				// Skipping creating slot for empty material_parts as well as skipping creation of a polygon group below
				if (MeshPart.num_faces > 0)
				{
					ufbx_material* Material =  Mesh->materials[MaterialIndex];
					MeshNode->SetSlotMaterialDependencyUid(Parser.GetMaterialSlotName(Material).ToString(), Parser.GetMaterialUid(*Material));
				}
			}

			for (ufbx_node* Instance : Mesh->instances)
			{
				if (UInterchangeSceneNode** Found = Parser.ElementIdToSceneNode.Find(Instance->element_id))
				{
					(*Found)->SetCustomAssetInstanceUid(MeshNameAndUid.UniqueID);
				}
			}

			FString PayLoadKey = MeshNameAndUid.UniqueID;

			if (!Parser.PayloadContexts.Contains(PayLoadKey))
			{
				Parser.PayloadContexts.Add(PayLoadKey, FPayloadContext(MeshNode->IsSkinnedMesh() ? FPayloadContext::SkinnedMesh : FPayloadContext::Element, Mesh->element_id));
			}

			MeshNode->SetPayLoadKey(PayLoadKey, MeshNode->IsSkinnedMesh() ? EInterchangeMeshPayLoadType::SKELETAL : EInterchangeMeshPayLoadType::STATIC);

			for (ufbx_blend_deformer* Deformer : Mesh->blend_deformers)
			{
				const FString MorphTargetName = Parser.GetElementName(Deformer->element);
				for (int32 ChannelIndex = 0; ChannelIndex < Deformer->channels.count; ++ChannelIndex)
				{
					const ufbx_blend_channel* Channel = Deformer->channels[ChannelIndex];

					const int32 CurrentChannelMorphTargetCount = Channel->keyframes.count;

					FString ChannelName = Parser.GetElementName(Channel->element);
					// Maya adds the name of the MorphTarget and an underscore to the front of the channel name, so remove it
					if (ChannelName.StartsWith(MorphTargetName) && (ChannelName.Compare(MorphTargetName, ESearchCase::IgnoreCase) != 0))
					{
						ChannelName.RightInline(ChannelName.Len() - (MorphTargetName.Len() + 1), EAllowShrinking::No);
					}

					for (int32 ChannelMorphTargetIndex = 0; ChannelMorphTargetIndex < Channel->keyframes.count; ++ChannelMorphTargetIndex)
					{
						const ufbx_blend_keyframe& Keyframe = Channel->keyframes[ChannelMorphTargetIndex];
						const ufbx_blend_shape* Shape = Keyframe.shape;

						FString ShapeName = CurrentChannelMorphTargetCount > 1
							? Parser.GetElementName(Shape->element)
							// Maya concatenates the number of the shape to the end of its name, so instead use the name of the channel
							: ChannelName;

						if (ShapeName.IsEmpty())
						{
							ShapeName = TEXT("MorphTarget_") + FString::FromInt(ChannelIndex) + TEXT("_") + FString::FromInt(ChannelMorphTargetIndex);
						}

						Convert::FMeshNameAndUid MorphTargetNameId(Parser, *Shape);
						const UInterchangeMeshNode* ExistingMorphTargetNode = Cast<const UInterchangeMeshNode>(NodeContainer.GetNode(MorphTargetNameId.UniqueID));
						if (ExistingMorphTargetNode)
						{
							int32 UniqueId = 1;
							const FString NameClash = "_ncl_";
							const FString UniqueIDBase = MorphTargetNameId.UniqueID;
							while (ExistingMorphTargetNode)
							{
								MorphTargetNameId.UniqueID = UniqueIDBase + NameClash + FString::FromInt(UniqueId++);
								ExistingMorphTargetNode = Cast<const UInterchangeMeshNode>(NodeContainer.GetNode(MorphTargetNameId.UniqueID));
							}
						}

						if (!ExistingMorphTargetNode)
						{
							UInterchangeMeshNode* MorphTargetNode = CreateMeshNode(NodeContainer, MorphTargetNameId.Label, MorphTargetNameId.UniqueID);
							const bool bIsMorphTarget = true;
							MorphTargetNode->SetMorphTarget(bIsMorphTarget);
							MorphTargetNode->SetMorphTargetName(ShapeName);

							//Create a Mesh node dependency, so the mesh node can retrieve is associate morph target
							MeshNode->SetMorphTargetDependencyUid(MorphTargetNameId.UniqueID);
							const FString MorphTargetPayLoadKey = MorphTargetNameId.UniqueID;
							if (!Parser.PayloadContexts.Contains(MorphTargetPayLoadKey))
							{
								Parser.PayloadContexts.Add(MorphTargetPayLoadKey, FMorph{Mesh->typed_id, Shape->typed_id});
							}
							MorphTargetNode->SetPayLoadKey(MorphTargetPayLoadKey, EInterchangeMeshPayLoadType::MORPHTARGET);
						}
						if (!SkeletonRootPerBlendShape.Contains(Shape))
						{
							SkeletonRootPerBlendShape.Add(Shape, Mesh);	
						}
					}
				}
			}
		}
	}

	UInterchangeMeshNode* FUfbxMesh::CreateMeshNode(UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeDisplayLabel,
	                                                const FString& NodeUniqueID)
	{
		UInterchangeMeshNode* MeshNode = NewObject<UInterchangeMeshNode>(&NodeContainer, NAME_None);
		if (!MeshNode)
		{
			UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
			Message->InterchangeKey = NodeUniqueID;
			Message->Text = LOCTEXT("NodeAllocationError", "Mesh node allocation failed when importing FBX.");
			return nullptr;
		}
		NodeContainer.SetupNode(MeshNode, FString(NodeUniqueID), NodeDisplayLabel, EInterchangeNodeContainerType::TranslatedAsset);
		MeshNode->AddBooleanAttribute(TEXT("RenameLikeLegacyFbx"), true);
		return MeshNode;
	}

	bool FUfbxMesh::FetchBlendShape(FUfbxParser& Parser, FMeshDescription& MeshDescription, const ufbx_mesh& Mesh, const ufbx_blend_shape& BlendShape, const FTransform& MeshGlobalTransform)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FUfbxMesh::FetchBlendShape);

		// #ufbx_todo: normals

		FStaticMeshAttributes Attributes(MeshDescription);
		Attributes.Register();

		MeshDescription.SuspendVertexIndexing();

		int32 VertexCount = Mesh.num_vertices;

		TArray<ufbx_vec3> Vertices(Mesh.vertices.data, Mesh.vertices.count);
		ufbx_add_blend_shape_vertex_offsets(&BlendShape, Vertices.GetData(), VertexCount, 1.0f);

		TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
		int32 VertexOffset = MeshDescription.Vertices().Num();

		// The below code expects Num() to be equivalent to GetArraySize(), i.e. that all added elements are appended, not inserted into existing gaps
		check(VertexOffset == MeshDescription.Vertices().GetArraySize());

		MeshDescription.ReserveNewVertices(VertexCount);
		for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
		{
			int32 RealVertexIndex = VertexOffset + VertexIndex;
			const FVector VertexPosition = Convert::ConvertVec3(Vertices[VertexIndex]);
			FVertexID AddedVertexId = MeshDescription.CreateVertex();
			if (AddedVertexId.GetValue() != RealVertexIndex)
			{
				UInterchangeResultMeshError_Generic* Message = Parser.AddMessage<UInterchangeResultMeshError_Generic>();
				Message->Text = LOCTEXT("CantCreateVertex", "Cannot create valid vertex for the mesh '{MeshName}'.");
				return false;
			}
			VertexPositions[AddedVertexId] = FVector3f(VertexPosition);
		}
		MeshDescription.ResumeVertexIndexing();

		FStaticMeshOperations::ApplyTransform(MeshDescription, MeshGlobalTransform, true);

		return true;
	}

	namespace UfbxMeshInernal
	{
		
	bool FetchMesh(FUfbxParser& Parser, FMeshDescription& MeshDescription, ufbx_element* Element, const FTransform& MeshGlobalTransform, TFunctionRef<void (FFaceCollector& FaceCollector)> HandleMesh=[](FFaceCollector&){})
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FUfbxMesh::FetchMesh);

		ufbx_mesh& Mesh = *ufbx_as_mesh(Element);

		int32 VertexCount = Mesh.num_vertices;

		int32 UvCount = Mesh.uv_sets.count;

		if (UvCount > MAX_MESH_TEXTURE_COORDS_MD)
		{
			UInterchangeResultMeshWarning_TooManyUVs* Message = Parser.AddMessage<UInterchangeResultMeshWarning_TooManyUVs>();
			Message->ExcessUVs = UvCount - MAX_MESH_TEXTURE_COORDS_MD;

			UvCount = MAX_MESH_TEXTURE_COORDS_MD;
		}

		// FSkeletalMeshAttributes MeshAttributes(MeshDescription); for skeletal
		FStaticMeshAttributes Attributes(MeshDescription);
		// #ufbx_todo: why bKeepExistingAttribute>
		constexpr bool bKeepExistingAttribute = true;
		Attributes.Register(bKeepExistingAttribute);

		// #ufbx_todo Revisit how FMeshDescription is filled up
		bool bAppliedSmoothingFromData = false;
#ifdef WITH_MESH_DESCRIPTION_BUILDER
		FMeshDescriptionBuilder Builder;
		Builder.SetMeshDescription(&MeshDescription);
		Builder.SuspendMeshDescriptionIndexing();

		Builder.EnablePolyGroups();

		// #ufbx_todo: had to set at least one UV otherwise FStaticMeshOperations::ComputeMikktTangents
		//  same was also done on FBX SDK parser 
		Builder.SetNumUVLayers(FMath::Max(1, UvCount));

		Builder.ReserveNewVertices(VertexCount);

		FFaceCollector FaceCollector(Builder, Mesh, UvCount);

		for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
		{
			ufbx_vec3 Position = Mesh.vertex_position.values[VertexIndex];
			// #ufbx_todo: FbxPosition = TotalMatrix.MultT(FbxPosition);
			// also find out what is that Matrix about
			const FVector VertexPosition = Convert::ConvertVec3(Position);
			const FVertexID VertexID = Builder.AppendVertex(VertexPosition);
			FaceCollector.VertexMapping.Add(VertexID);
		}

		// make single mesh, without considering distinct materials  assigned to faces
		constexpr bool bMakeMeshGroupsPerMaterial = true;

		if (bMakeMeshGroupsPerMaterial)
		{

			for (int32 MaterialIndex = 0; MaterialIndex < Mesh.material_parts.count; ++MaterialIndex)
			{
				// #ufbx_todo: maybe should specify slot name as just part's index? Why use materials name? Expect that tests verify this.
				const ufbx_mesh_part& MeshPart = Mesh.material_parts[MaterialIndex];
				if (MeshPart.num_faces > 0)
				{
					const ufbx_material* Material = MeshUtils::GetMeshMaterial(Mesh, MaterialIndex);
					const FPolygonGroupID GroupID = Builder.AppendPolygonGroup(Parser.GetMaterialSlotName(Material));

					const int32 FaceCount = MeshPart.num_faces;
					for (int32 FaceIndex = 0; FaceIndex < FaceCount; ++FaceIndex)
					{
						const uint32 UfbxFaceIndex = MeshPart.face_indices.data[FaceIndex];
						FaceCollector.AddFace(GroupID, Mesh.faces[UfbxFaceIndex], UfbxFaceIndex);
					}
				}
			}
		}
		else
		{
			FPolygonGroupID GroupID = Builder.AppendPolygonGroup();
			const int32 FaceCount = Mesh.num_faces;
			for (int32 FaceIndex = 0; FaceIndex < FaceCount; ++FaceIndex)
			{
				FaceCollector.AddFace(GroupID, Mesh.faces[FaceIndex], (uint32)FaceIndex);
			}
		}

		if (FaceCollector.NumDegenerateTriangles)
		{
			Parser.AddMessage<UInterchangeResultMeshWarning_Generic>()->Text = FText::Format(LOCTEXT("PrimitiveFoundDegenerateTriangles", "Mesh '{0}' has {1} degenerate triangles"), FText::FromString(Parser.GetMeshLabel(Mesh.element)), FaceCollector.NumDegenerateTriangles);;
		}

		// Provide edge smoothing
		HandleMesh(FaceCollector);

		Builder.ResumeMeshDescriptionIndexing();

		// Apply edge hardness from ufbx smoothing data if available.
		// ByEdge: direct 1:1 mapping from ufbx edge flags to MeshDescription edges via vertex-pair lookup.
		// ByPolygon (face_smoothing bool): edge is hard when ALL faces have face_smoothing==false. This handles the common all-zero case correctly. 
		// But:
		// #ufbx_todo: face_smoothing is bool in ufbx and this is wrong since in source data in fbx it is group BIT MASK(32 bit for 32 groups, why 3ds Max has 32 group limit btw) 
		// and ufbx converts it to bool. So different smoothing groups both appear true and get soft edges here 
		// Correct fix requires changing ufbx.c and storing the raw int32 bitmask. 
		if (Mesh.edge_smoothing.count > 0)
		{
			bAppliedSmoothingFromData = true;
			TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();

			for (int32 EdgeIdx = 0; EdgeIdx < (int32)Mesh.edge_smoothing.count; ++EdgeIdx)
			{
				const ufbx_edge& Edge = Mesh.edges[EdgeIdx];
				const FVertexID VertexA = FaceCollector.VertexMapping[Mesh.vertex_position.indices[Edge.a]];
				const FVertexID VertexB = FaceCollector.VertexMapping[Mesh.vertex_position.indices[Edge.b]];
				FEdgeID EdgeID = MeshDescription.GetVertexPairEdge(VertexA, VertexB);
				if (EdgeID == INDEX_NONE)
				{
					EdgeID = MeshDescription.CreateEdge(VertexA, VertexB);
				}
				// ufbx softness to UE hardness
				EdgeHardnesses[EdgeID] = !Mesh.edge_smoothing.data[EdgeIdx];
			}
		}
		else if (Mesh.face_smoothing_groups.count > 0)
		{
			// Raw int32 ByPolygon bitmask smoothing groups. Two faces share a smooth edge iff
			// (group_a & group_b) != 0. 
			// All-zero groups are handled naturally. Zero means polygon doesn't belong to any group. Or doesn't share a group with another polygon. (0 & 0) == 0 -> hard.
			// Exception: skinned mesh with all-zero groups uses normals-based detection (FBX SDK parity - see FMeshDescriptionImporter::FillMeshDescriptionFromFbxMesh).
			bool bSkinnedAllZero = (Mesh.skin_deformers.count > 0);
			if (bSkinnedAllZero)
			{
				for (uint32 FaceSmoothingMask: Mesh.face_smoothing_groups)
				{
					if (FaceSmoothingMask)
					{
						bSkinnedAllZero = false;
						break;
					}
				}
			}

			if (!bSkinnedAllZero)
			{
				bAppliedSmoothingFromData = true;
				TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();

				// Determine edge hardnesses
				for (const FEdgeID EdgeID : MeshDescription.Edges().GetElementIDs())
				{
					TArray<FPolygonID, TInlineAllocator<2>> ConnectedPolygons;
					MeshDescription.GetEdgeConnectedPolygons(EdgeID, ConnectedPolygons);

					uint32 GroupA = 0;
					uint32 GroupB = 0;
					if (ConnectedPolygons.Num() > 1)
					{
						GroupA = Mesh.face_smoothing_groups.data[FaceCollector.PolygonToFaceIndex[ConnectedPolygons[0].GetValue()]];
						GroupB = Mesh.face_smoothing_groups.data[FaceCollector.PolygonToFaceIndex[ConnectedPolygons[1].GetValue()]];
					}
					EdgeHardnesses[EdgeID] = (GroupA & GroupB) == 0;
				}
			}
		}
#endif

		if (!bAppliedSmoothingFromData)
		{
			FStaticMeshOperations::DetermineEdgeHardnessesFromVertexInstanceNormals(MeshDescription);
		}

		FStaticMeshOperations::ApplyTransform(MeshDescription, MeshGlobalTransform, true);
		return true;
	}
	}

	bool FUfbxMesh::FetchMesh(FUfbxParser& Parser, FMeshDescription& MeshDescription, ufbx_element* Element, const FTransform& MeshGlobalTransform)
	{
		return UfbxMeshInernal::FetchMesh(Parser, MeshDescription, Element, MeshGlobalTransform);
	}

	bool FUfbxMesh::FetchSkinnedMesh(FUfbxParser& Parser, FMeshDescription& MeshDescription, ufbx_element* Element, const FTransform& MeshGlobalTransform, TArray<FString>& OutJointUniqueNames)
	{

		auto HandleMesh = [&Parser, &MeshDescription, &OutJointUniqueNames, Element](FFaceCollector& FaceCollector)
		{

			TRACE_CPUPROFILER_EVENT_SCOPE(Interchange_ImportSkin);
			FSkeletalMeshAttributes SkeletalMeshAttributes(MeshDescription);
			SkeletalMeshAttributes.Register(true);

			using namespace UE::AnimationCore;
			TMap<FVertexID, TArray<FBoneWeight>> RawBoneWeights;
							
			//Add the influence data in the skeletalmesh description
			FSkinWeightsVertexAttributesRef VertexSkinWeights = SkeletalMeshAttributes.GetVertexSkinWeights();

			const ufbx_mesh& Mesh = *ufbx_as_mesh(Element);

			const ufbx_skin_deformer& Skin = *Mesh.skin_deformers[0];

			const int32 ClusterCount = Skin.clusters.count;
			TArray<const ufbx_node*> SortedJoints;
			SortedJoints.Reserve(ClusterCount);
			OutJointUniqueNames.Reserve(ClusterCount);
			
			for (int32 VertexIndex = 0; VertexIndex < Skin.vertices.count; ++VertexIndex)
			{
				const ufbx_skin_vertex& SkinVertex  = Skin.vertices[VertexIndex];
				const FVertexID VertexID = FaceCollector.VertexMapping[VertexIndex];

				// #ufbx_todo: reuse array
				TArray<FBoneWeight> BoneWeights;

				for (uint32 WeightIndex = 0; WeightIndex < SkinVertex.num_weights; ++WeightIndex)
				{
					const ufbx_skin_weight SkinWeight = Skin.weights[SkinVertex.weight_begin+WeightIndex];

					const ufbx_node* BoneNode = Skin.clusters[SkinWeight.cluster_index]->bone_node;
					double Weight = SkinWeight.weight;

					int32 BoneIndex = -1;
					if (BoneNode && !SortedJoints.Find(BoneNode, BoneIndex))
					{
						BoneIndex = SortedJoints.Add(BoneNode);
						FString JointNodeUniqueID = Parser.GetBoneNodeName(*BoneNode);
						OutJointUniqueNames.Add(JointNodeUniqueID);
					}

					BoneWeights.Add(FBoneWeight(BoneIndex, (float)Weight));
				}

				VertexSkinWeights.Set(VertexID, BoneWeights);
			}
		};

		if (!UfbxMeshInernal::FetchMesh(Parser, MeshDescription, Element, MeshGlobalTransform, HandleMesh))
		{
			return false;
		}


		return true;
	}
}

#undef LOCTEXT_NAMESPACE
