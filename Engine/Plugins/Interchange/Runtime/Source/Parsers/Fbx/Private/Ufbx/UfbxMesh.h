// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FMeshDescription;

class UInterchangeBaseNodeContainer;
class UInterchangeMeshNode;

struct ufbx_element;typedef ufbx_element ufbx_element;
struct ufbx_mesh;typedef ufbx_mesh ufbx_mesh;
struct ufbx_blend_shape;typedef ufbx_blend_shape ufbx_blend_shape;

namespace UE::Interchange::Private
{

	class FUfbxParser;

	class FUfbxMesh
	{
	public:
		explicit FUfbxMesh(FUfbxParser& InParser)
			: Parser(InParser)
		{}

		void AddAllMeshes(UInterchangeBaseNodeContainer& NodeContainer);

		static bool FetchMesh(FUfbxParser& Parser, FMeshDescription& MeshDescription, ufbx_element* Element, const FTransform& MeshGlobalTransform);
		static bool FetchSkinnedMesh(FUfbxParser& Parser, FMeshDescription& MeshDescription, ufbx_element* Element, const FTransform& MeshGlobalTransform, TArray<FString>& OutJointUniqueNames);
		static bool FetchBlendShape(FUfbxParser& Parser, FMeshDescription& MeshDescription, const ufbx_mesh& Mesh, const ufbx_blend_shape& BlendShape, const FTransform& MeshGlobalTransform);

		const TMap<const ufbx_blend_shape*, const ufbx_mesh*>& GetBlendShapes() const
		{
			return SkeletonRootPerBlendShape;
		}

		TMap<const ufbx_mesh*, UInterchangeMeshNode*> MeshToMeshNode;

	private:
		UInterchangeMeshNode* CreateMeshNode(UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeDisplayLabel, const FString& NodeUniqueID);

		FUfbxParser& Parser;

		TMap<const ufbx_blend_shape*, const ufbx_mesh*> SkeletonRootPerBlendShape;
	};

}