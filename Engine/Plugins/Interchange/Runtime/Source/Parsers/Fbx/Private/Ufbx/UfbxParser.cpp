// Copyright Epic Games, Inc. All Rights Reserved.

#include "UfbxParser.h"

#include "UfbxAnimation.h"
#include "UfbxConvert.h"
#include "UfbxMesh.h"
#include "UfbxMaterial.h"
#include "UfbxScene.h"

#include "Nodes/InterchangeSourceNode.h"

#if WITH_ENGINE
#include "Mesh/InterchangeMeshPayload.h"
#endif

#include "IImageWrapperModule.h"
#include "InterchangeCameraNode.h"
#include "Fbx/InterchangeFbxMessages.h"
#include "InterchangeHelper.h"
#include "InterchangeCommonAnimationPayload.h"
#include "InterchangeLightNode.h"

#include "MeshDescription.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Serialization/MemoryWriter.h"

#include "ufbx.c"
#include "Modules/ModuleManager.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"


#define LOCTEXT_NAMESPACE "InterchangeFbxParser"

namespace UE::Interchange::Private
{

FUfbxParser::FUfbxParser(TWeakObjectPtr<UInterchangeResultsContainer> InResultsContainer)
	: ResultsContainer(InResultsContainer)
	, ElementNames(*this)
	, NodeNames(*this)
	, BoneNames(*this)
	, MeshNames(*this)
	, CameraNames(*this)
	, LightNames(*this)
	, MaterialNames(*this)
	{
		NodeNames.DefaultName = TEXT("Node");
		NodeNames.bUseNCL = false;
		NodeNames.bUseNodeRules = true;

		BoneNames.DefaultName = TEXT("Node");
		BoneNames.bUseNCL = false;
		BoneNames.bIsJoint = true;

		MeshNames.DefaultName = TEXT("Mesh");

		CameraNames.DefaultName = TEXT("Camera");
		CameraNames.bUseNCL = false;

		LightNames.DefaultName =  TEXT("Light");
		LightNames.bUseNCL = false;

		MaterialNames.DefaultName = TEXT("Material");

	}

FUfbxParser::~FUfbxParser()
{
	Reset();
}

void FUfbxParser::SetResultContainer(UInterchangeResultsContainer* Result)
{
	ResultsContainer = Result;
}

bool FUfbxParser::LoadFbxFile(const FString& Filename, UInterchangeBaseNodeContainer& NodeContainer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUfbxParser::FillContainerWithFbxScene)

	SourceFilename = Filename;

	ufbx_load_opts LoadOpts = { 0 };
	LoadOpts.evaluate_skinning = false;
	LoadOpts.load_external_files = true;
	// this option allows to override root transform with provided LoadOpts.root_transform
	LoadOpts.use_root_transform = false;

	// #ufbx_todo: LoadOpts.allow_nodes_out_of_root= true; // might just to this(to reparent all out-of-root nodes under root_node) -
	// LoadOpts.allow_empty_faces = true;

	// #ufbx_todo: progress reporting. In case someday we want to report detailed progress from Worker/Parser
	// LoadOpts.progress_cb

	// #ufbx_todo: handle geometry transform - UInterchangeSceneNode::SetCustomGeometricTransform
	// LoadOpts.geometry_transform_handling

	// #ufbx_todo:
	// LoadOpts.inherit_mode_handling

	// #ufbx_todo:
	// LoadOpts.pivot_handling

	LoadOpts.space_conversion = UFBX_SPACE_CONVERSION_ADJUST_TRANSFORMS;


	if (bForceFrontXAxis)
	{
		// When X for front 
		LoadOpts.target_axes = ufbx_coordinate_axes{
	UFBX_COORDINATE_AXIS_NEGATIVE_Y, UFBX_COORDINATE_AXIS_POSITIVE_Z, UFBX_COORDINATE_AXIS_POSITIVE_X,
		};
	}
	else	
	{
		LoadOpts.target_axes = ufbx_axes_left_handed_z_up;
	}
	LoadOpts.handedness_conversion_axis = UFBX_MIRROR_AXIS_Y;

	LoadOpts.target_unit_meters = 0.01; // Centimeters
	LoadOpts.reverse_winding = true;

	LoadOpts.target_camera_axes.front = UFBX_COORDINATE_AXIS_NEGATIVE_X;
	LoadOpts.target_camera_axes.up = UFBX_COORDINATE_AXIS_POSITIVE_Z;
	LoadOpts.target_camera_axes.right = UFBX_COORDINATE_AXIS_NEGATIVE_Y;

	LoadOpts.target_light_axes.front = UFBX_COORDINATE_AXIS_NEGATIVE_X;
	LoadOpts.target_light_axes.up = UFBX_COORDINATE_AXIS_NEGATIVE_Z;
	LoadOpts.target_light_axes.right = UFBX_COORDINATE_AXIS_POSITIVE_Y;

	ufbx_error Error;
	Scene = ufbx_load_file(TCHAR_TO_UTF8(*Filename), &LoadOpts, &Error);

	if (!Scene)
	{
		FFormatNamedArguments FilenameText
		{
			{ TEXT("Filename"), FText::FromString(Filename) }
		};
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->Text = FText::Format(LOCTEXT("CannotOpenFBXFile", "Cannot open FBX file '{Filename}'."), FilenameText);
		return false;
	}
	// #ufbx_todo: read scene converter to UE's

	UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(&NodeContainer);
	
	SourceNode->SetCustomSourceFrameRateNumerator(Scene->settings.frames_per_second);
	SourceNode->SetCustomSourceFrameRateDenominator(1.0);

	
	SourceNode->SetExtraInformation(TEXT("File Version"), Convert::ToUnrealString(Scene->metadata.original_application.version));
	SourceNode->SetExtraInformation(TEXT("File Creator"), Convert::ToUnrealString(Scene->metadata.creator));

	// FBX SDK extrapolates unit name(e.g. "centimeters") from numerical scaling value that is contained in the fbx
	SourceNode->SetExtraInformation(TEXT("File Units"),  FString::Printf(TEXT("UniteMeters: %.3f"), Scene->settings.original_unit_meters));

	FString AxisDirection;

	// Negative axes are odd
	int32 Negative = Scene->settings.original_axis_up % 2;
	switch (Scene->settings.original_axis_up / 2)
	{
	case UFBX_COORDINATE_AXIS_POSITIVE_X:
		AxisDirection += TEXT("X");
		break;
	case UFBX_COORDINATE_AXIS_POSITIVE_Y:
		AxisDirection += TEXT("Y");
		break;
	case UFBX_COORDINATE_AXIS_POSITIVE_Z:
		AxisDirection += TEXT("Z");
		break;
	case UFBX_COORDINATE_AXIS_UNKNOWN:
		break;
	default: ;
	}

	AxisDirection += Negative ? TEXT("-DOWN") : TEXT("-UP");

	SourceNode->SetExtraInformation(TEXT("File Axis Direction"), AxisDirection);
	SourceNode->SetExtraInformation(TEXT("File Frame Rate"), FString::Printf(TEXT("%.2f"), Scene->settings.frames_per_second));


	FString ApplicationVendor = Convert::ToUnrealString(Scene->metadata.latest_application.vendor);
	if (!ApplicationVendor.IsEmpty())
	{
		SourceNode->SetExtraInformation(FSourceNodeExtraInfoStaticData::GetApplicationVendorExtraInfoKey(), ApplicationVendor);
	}
	FString ApplicationName = Convert::ToUnrealString(Scene->metadata.latest_application.name);
	if (!ApplicationName.IsEmpty())
	{
		SourceNode->SetExtraInformation(FSourceNodeExtraInfoStaticData::GetApplicationNameExtraInfoKey(), ApplicationName);
	}
	FString ApplicationVersion = Convert::ToUnrealString(Scene->metadata.latest_application.version);
	if (!ApplicationVersion.IsEmpty())
	{
		SourceNode->SetExtraInformation(FSourceNodeExtraInfoStaticData::GetApplicationVersionExtraInfoKey(), ApplicationVersion);
	}

	// Fbx legacy has a special way to bake the skeletal mesh that do not fit the interchange standard
	// The interchange skeletal mesh factory will read this to use the proper bake transform so it match legacy behavior.
	// This fix the issue with blender armature bone skip
	SourceNode->SetCustomAlignSkeletalMeshInScene(true);

	//Fbx legacy does not allow Scene Root Nodes to be part of the skeletons (to be joints).
	SourceNode->SetCustomAllowSceneRootAsJoint(false);

	// #ufbx_todo: FileDetails

	return true;
}

void FUfbxParser::FillContainerWithFbxScene(UInterchangeBaseNodeContainer& NodeContainer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUfbxParser::FillContainerWithFbxScene)

	FUfbxScene SceneConverter(*this);
	SceneConverter.InitHierarchy(NodeContainer);

	// #ufbx_todo: CleanupFbxData();
	FUfbxMaterial UfbxMaterial(*this);
	UfbxMaterial.AddMaterials(NodeContainer);
	UfbxMaterial.AddAllTextures(NodeContainer);

	FUfbxMesh Meshes(*this);
	// Get meshes first, nodes reference them later in ProcessNodes
	Meshes.AddAllMeshes(NodeContainer);

	SceneConverter.ProcessNodes(NodeContainer);
	FUfbxAnimation::AddAnimation(*this, SceneConverter, Meshes, NodeContainer);
}

bool FUfbxParser::FetchPayloadData(const FString& PayloadKey, const FString& PayloadFilepath)
{
	// #ufbx_todo: defence - see UE::Interchange::Private::FFbxParser::FetchPayloadData

	const FPayloadContext* PayloadContextFound = PayloadContexts.Find(PayloadKey);
	if (!PayloadContextFound)
	{
		return false;
	}

	const FPayloadContext& PayloadContext = *PayloadContextFound;
	switch (PayloadContext.Kind)
	{
	case FPayloadContext::AnimationMorph:
		{
			FMorphAnimationPayloadContext& Animation = const_cast<FMorphAnimationPayloadContext&>(PayloadContexts.GetMorphAnimation(PayloadContext));

			TArray<FInterchangeCurve> InterchangeCurves;
			if (FUfbxAnimation::FetchMorphTargetAnimation(*this, Animation, InterchangeCurves))
			{
				TArray64<uint8> Buffer;
				FMemoryWriter64 Ar(Buffer);
				Ar << InterchangeCurves;
				Ar << Animation.InbetweenCurveNames;
				Ar << Animation.InbetweenFullWeights;
				FFileHelper::SaveArrayToFile(Buffer, *PayloadFilepath);
				return true;
			}
		}
		break;
	case FPayloadContext::AnimationRigid:
		{
			const FRigidAnimationPayloadContext& Animation = PayloadContexts.GetRigidAnimation(PayloadContext);

			TArray<FInterchangeCurve> InterchangeCurves;

			if (FUfbxAnimation::FetchRigidAnimation(*this, Animation, InterchangeCurves))
			{
				TArray64<uint8> Buffer;
				FMemoryWriter64 Ar(Buffer);
				Ar << InterchangeCurves;
				FFileHelper::SaveArrayToFile(Buffer, *PayloadFilepath);
				return true;
			}
		}
		break;
	case FPayloadContext::AnimationProperty:
		{
			const FPropertyAnimationPayloadContext& Animation = PayloadContexts.GetPropertyAnimation(PayloadContext);

			if (Animation.bIsStepAnimation)
			{
				TArray<FInterchangeStepCurve> InterchangeStepCurves;

				if (FUfbxAnimation::FetchPropertyAnimationStepCurves(*this, Animation, InterchangeStepCurves))
				{
					TArray64<uint8> Buffer;
					FMemoryWriter64 Ar(Buffer);
					Ar << InterchangeStepCurves;
					FFileHelper::SaveArrayToFile(Buffer, *PayloadFilepath);
					return true;
				}
			}
			else
			{
				TArray<FInterchangeCurve> InterchangeCurves;

				if (FUfbxAnimation::FetchPropertyAnimationCurves(*this, Animation, InterchangeCurves))
				{
					TArray64<uint8> Buffer;
					FMemoryWriter64 Ar(Buffer);
					Ar << InterchangeCurves;
					FFileHelper::SaveArrayToFile(Buffer, *PayloadFilepath);
					return true;
				}
				
			}
		}
		break;
	}
	
	return false;
}

bool FUfbxParser::FetchMeshPayloadData(const FString& PayloadKey, const FTransform& MeshGlobalTransform, const FString& PayloadFilepath)
{
	return false;	
}

void FUfbxParser::Reset()
{
	NodeToUid.Reset();

	ElementNames.Reset();
	NodeNames.Reset();
	BoneNames.Reset();
	MeshNames.Reset();
	CameraNames.Reset();
	LightNames.Reset();
	MaterialNames.Reset();

	ElementIdToSceneNode.Reset();

	PayloadContexts.Reset();

	ufbx_free_scene(Scene);
	Scene = nullptr;
}

FString FUfbxParser::GetElementNameRaw(const ufbx_element& Element) const
{
	return Convert::ToUnrealString(Element.name);
}

FString FUfbxParser::GetElementName(const ufbx_element& Element) const
{
	return ElementNames.GetOrMake(Element, [&]
		{
			return GetElementNameRaw(Element);
		});
}

FString FUfbxParser::GetNodeName(const ufbx_node& Node) const
{
	return NodeNames.GetSafe(Node.element);
}

FString FUfbxParser::GetNodeUid(const ufbx_node& Node) const
{
	if (const FString* Found = NodeToUid.Find(&Node))
	{
		return *Found;
	}

	FString NodeUid = GetNodeName(Node);
	// As we don't want the name to have the namespace removed,
	// so name for the node wasn't sanitized(to keep namespace name within it)
	// we manually generate the name here:
	UE::Interchange::SanitizeName(NodeUid);

	// Compose NodeUid as node "path" - unique node names connected with dot
	// #ufbx_todo: check that element_id is enough
	// reference FBX parser's uses UE::Interchange::Private::FFbxHelper::GetFbxNodeHierarchyName to create NodeUid
	if (Node.parent)
	{
		const FString* ParentNodeUid = NodeToUid.Find(Node.parent);
		if (ensureMsgf(ParentNodeUid, TEXT("Expected to have child node Uid generated after parent was processed first")))
		{
			NodeUid = *ParentNodeUid + TEXT(".") + NodeUid;
		}
	}

	NodeToUid.Add(&Node, NodeUid);
	return NodeUid;
}

FString FUfbxParser::GetNodeLabel(const ufbx_node& Node) const
{
	FString Label(GetNodeName(Node));
	NodeNames.ManageNamespace(Label);
	UE::Interchange::SanitizeName(Label);
	return Label;
}

FString FUfbxParser::GetBoneNodeName(const ufbx_node& Node) const
{
	return BoneNames.GetOrMake(Node.element, [&]()
		{
			return GetElementNameRaw(Node.element);
		});
}

FString FUfbxParser::GetMeshUid(const ufbx_element& Mesh) const
{
	return MeshNames.GetOrMake(Mesh, [&]()
		{	return GetElementNameRaw(Mesh); });
}

FString FUfbxParser::GetMeshLabel(const ufbx_element& Mesh) const
{
	return GetMeshUid(Mesh);
}

FString FUfbxParser::GetMaterialName(const ufbx_material& Material) const
{
	return MaterialNames.GetOrMake(Material.element, [&]
		{
			return GetElementNameRaw(Material.element);
		});
}

FString FUfbxParser::GetMaterialUid(const ufbx_material& Material) const
{
	return  TEXT("\\Material\\") + GetMaterialName(Material);
}

FString FUfbxParser::GetMaterialLabel(const ufbx_material& Material) const
{
	return GetMaterialName(Material);
}

FString FUfbxParser::GetCameraName(const ufbx_camera& Camera) const
{
	return CameraNames.GetOrMake(Camera.element, [&]
		{
			FString CameraName = GetElementNameRaw(Camera.element);

			if (CameraName.IsEmpty())
			{
				FString DefaultNamePrefix(UInterchangePhysicalCameraNode::StaticAssetTypeName());
				CameraName = DefaultNamePrefix + TEXT("_") + FString::FromInt(Camera.typed_id);
			}
			return CameraName;
		});
}

FString FUfbxParser::GetLightName(const ufbx_light& Light) const
{
	return LightNames.GetOrMake(Light.element, [&]
		{
			FString LightName = GetElementNameRaw(Light.element);

			if (LightName.IsEmpty())
			{
				FString DefaultNamePrefix(UInterchangeBaseLightNode::StaticAssetTypeName());
				LightName = DefaultNamePrefix + TEXT("_") + FString::FromInt(Light.typed_id);
			}
			return LightName;
		});
}

void FUfbxParser::ConvertProperty(const ufbx_prop& Prop, UInterchangeBaseNode* SceneNode, const TOptional<FString>& PayloadKey)
{
	if (Prop.flags & UFBX_PROP_FLAG_USER_DEFINED)
	{
		FString PropertyName = Convert::ToUnrealString(Prop.name);

		switch (Prop.type)
		{
		case UFBX_PROP_UNKNOWN:
			break;
		case UFBX_PROP_BOOLEAN:
			{
				const bool PropertyValue = Prop.value_int != 0;
				UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(SceneNode, PropertyName, PropertyValue, PayloadKey);
			}
			break;
		case UFBX_PROP_INTEGER:
			{
				const int32 PropertyValue = Prop.value_int;
				UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(SceneNode, PropertyName, PropertyValue, PayloadKey);
			}
			break;
		case UFBX_PROP_NUMBER:
			{
				const float PropertyValue = Prop.value_real;
				UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(SceneNode, PropertyName, PropertyValue, PayloadKey);
			}
			break;
		case UFBX_PROP_VECTOR:
			{
				const FVector4d PropertyValue = Convert::ConvertVec4(Prop.value_vec4);
				UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(SceneNode, PropertyName, PropertyValue, PayloadKey);
			}
			break;
		case UFBX_PROP_COLOR:
			{
				const FVector PropertyValue = FVector(Convert::ConvertColor(Prop.value_vec4));
				UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(SceneNode, PropertyName, PropertyValue, PayloadKey);
			}
			break;
		case UFBX_PROP_COLOR_WITH_ALPHA:
			{
				const FLinearColor PropertyValue = Convert::ConvertColor(Prop.value_vec4);
				UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(SceneNode, PropertyName, PropertyValue, PayloadKey);
			}
			break;
		case UFBX_PROP_STRING:
			{
				const FString PropertyValue = Convert::ToUnrealString(Prop.value_str);
				UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(SceneNode, PropertyName, PropertyValue, PayloadKey);
			}
			break;
		case UFBX_PROP_DATE_TIME:
			{
				const FDateTime PropertyValue(Prop.value_int);
				UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(SceneNode, PropertyName, PropertyValue, PayloadKey);
			}
			break;
		// #ufbx_todo: support other useful custom attributes among these types
		case UFBX_PROP_TRANSLATION:
			break;
		case UFBX_PROP_ROTATION:
			break;
		case UFBX_PROP_SCALING:
			break;
		case UFBX_PROP_DISTANCE:
			break;
		case UFBX_PROP_COMPOUND:
			break;
		case UFBX_PROP_BLOB:
			break;
		case UFBX_PROP_REFERENCE:
			break;
		case UFBX_PROP_TYPE_FORCE_32BIT:
			break;
		}
	}
}

#if WITH_ENGINE
bool FUfbxParser::FetchMeshPayloadData(const FString& PayloadKey, const FTransform& MeshGlobalTransform, FMeshPayloadData& OutMeshPayloadData)
{
	FMeshDescription& MeshDescription = OutMeshPayloadData.MeshDescription;

	if (FPayloadContext* Found = PayloadContexts.Find(PayloadKey))
	{
		switch (Found->Kind)
		{
		case FPayloadContext::Element:
			{
				ufbx_element* Element = Scene->elements[Found->Index];

				constexpr bool bDebugSpecificElement = false;

				if (bDebugSpecificElement)
				{
					uint32_t SpecificElement = 2096;
					if (Element->element_id != SpecificElement)
					{
						return false;
					}
				}

				if (Element->type == UFBX_ELEMENT_MESH)
				{
					return FUfbxMesh::FetchMesh(*this, MeshDescription, Element, MeshGlobalTransform);
				}
				else
				{
					ensure(false); // #ufbx_todo: implement
				}
			}
			break;
		case FPayloadContext::SkinnedMesh:
			{
				ufbx_element* Element = Scene->elements[Found->Index];
				
				return FUfbxMesh::FetchSkinnedMesh(*this, MeshDescription, Element, MeshGlobalTransform, OutMeshPayloadData.JointNames);
			}
		case FPayloadContext::Morph:
			{
				if (const FMorph* MorphPayload = PayloadContexts.GetMorph(Found))
				{
					return FUfbxMesh::FetchBlendShape(*this, MeshDescription, 
					   *Scene->meshes[MorphPayload->MeshElement],
					   *Scene->blend_shapes[MorphPayload->BlendShapeElement], 
					   MeshGlobalTransform);
				}
			}
			break;
		default: ;
		}
	}
	else
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->Text = LOCTEXT("CannotRetrievePayload", "Cannot retrieve payload; payload key doesn't have any context.");
	}

	// need this?
	MeshDescription.Empty();
	return false;
}

bool FUfbxParser::FetchTexturePayload(const FString& PayloadKey, TOptional<TArray64<uint8>>& OutTexturePayloadData)
{
	if (FPayloadContext* Found = PayloadContexts.Find(PayloadKey))
	{
		if (!ensure(Found->Kind == FPayloadContext::Element))
		{
			return false;
		}

		ufbx_element* Element = Scene->elements[Found->Index];

		if (Element->type == UFBX_ELEMENT_TEXTURE)
		{
			const ufbx_texture* Texture = ufbx_as_texture(Element);
			if (!ensure(Texture))
			{
				return false;
			}
			if (Texture->content.data && (Texture->content.size > 0))
			{
				OutTexturePayloadData = TArray64<uint8>(static_cast<const uint8*>(Texture->content.data), Texture->content.size);
				return true;
			}
			return false;
		}

		AddMessage<UInterchangeResultError_Generic>()->Text = LOCTEXT("CannotRetrieveTexturePayloadNonTextureKey", "Cannot retrieve payload; FUfbxParser::FetchTexturePayload was called with non-texture payload key.");
	}

	return false;
}

#endif

bool FUfbxParser::DoesTextureHaveAlpha(const FString& TextureFilename, const ufbx_texture& Texture)
{
	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));

	TArray64<uint8> ImageData;
	TConstArrayView64<uint8> ImageDataView;

	if (Texture.content.data && (Texture.content.size > 0))
	{
		// embedded texture
		ImageDataView = TConstArrayView64<uint8>(static_cast<const uint8*>(Texture.content.data), Texture.content.size);
	}
	else if (FFileHelper::LoadFileToArray(ImageData, *TextureFilename))
	{
		ImageDataView = ImageData;
	}
	else
	{
		return false;
	}

	switch (ImageWrapperModule.DetectImageFormat(ImageDataView.GetData(), ImageDataView.Num()))
	{
		case EImageFormat::Invalid:
		// Early exit for Jpeg, which doesn't support alpha
		case EImageFormat::JPEG:
		case EImageFormat::GrayscaleJPEG:
		case EImageFormat::UEJPEG:
		case EImageFormat::GrayscaleUEJPEG:
			return false;
	};

	FImage LoadedImage;
	if (ImageWrapperModule.DecompressImage(ImageDataView.GetData(), ImageDataView.Num(), LoadedImage))
	{
		return FImageCore::DetectAlphaChannel(LoadedImage);
	}

	return false;
}

bool FUfbxParser::FetchAnimationBakeTransformPayload(const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries,
                                                     const FString& ResultFolder, FCriticalSection* ResultPayloadsCriticalSection, TAtomic<int64>& UniqueIdCounter,
                                                     TMap<FString, FString>& ResultPayloads)
{
	// #ufbx_todo: resolve concurrent access to ResultPayloads; or, maybe replace ResultPayloads with TQueue? Not clear why it's a map and why it's tested that keys already exist there...

	for (const FAnimationPayloadQuery& PayloadQuery : PayloadQueries)
	{
		FPayloadContext* PayloadContext = PayloadContexts.Find(PayloadQuery.PayloadKey.UniqueId);
		if (ensure(PayloadContext && (PayloadContext->Kind == FPayloadContext::AnimationSkinned)))
		{
			const FSkeletalAnimationPayloadContext& SkeletalAnimationPayloadContext = PayloadContexts.GetAnimation(*PayloadContext);
			FAnimationPayloadData PayloadData = FAnimationPayloadData(SkeletalAnimationPayloadContext.NodeUid, PayloadQuery.PayloadKey);
			FUfbxAnimation::FetchSkinnedAnimation(Scene, PayloadQuery, SkeletalAnimationPayloadContext, PayloadData);

			FString QueryHashString = PayloadQuery.GetHashString();

			// #ufbx_todo: check, FBX SDK parser somehow checks that ResultPayloads already has QueryHashString(i.e. same query already processed) Why is this?
			FString PayloadFilepathCopy;
			{
				FScopeLock Lock(ResultPayloadsCriticalSection);
				if (ResultPayloads.Contains(QueryHashString))
				{
					continue;
				}

				FString& PayloadFilepath = ResultPayloads.FindOrAdd(QueryHashString);
				//To avoid file path with too many character, we hash the payloadKey so we have a deterministic length for the file path.
				PayloadFilepath = ResultFolder + TEXT("/") + QueryHashString + FString::FromInt(UniqueIdCounter.IncrementExchange()) + TEXT(".payload");

				//Copy the map filename key because we are multithreaded and the TMap can be reallocated
				PayloadFilepathCopy = PayloadFilepath;

			}
			FString PayloadFilepath = PayloadFilepathCopy;

			TArray64<uint8> Buffer;
			FMemoryWriter64 Ar(Buffer);
			PayloadData.SerializeBaked(Ar);
			FFileHelper::SaveArrayToFile(Buffer, *PayloadFilepath);
		}
	}

	return false;
}

}

#undef LOCTEXT_NAMESPACE

