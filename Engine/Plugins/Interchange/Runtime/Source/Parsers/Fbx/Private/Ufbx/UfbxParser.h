// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeResultsContainer.h"
#include "InterchangeSceneNode.h"

#define UFBX_NO_SKINNING_EVALUATION
// also might do #define UFBX_MINIMAL and enable what's needed
#include <ufbx.h>

// #ufbx_todo: move IFbxParser to separate file and include IFbxParser.h instead
#include "FbxAPI.h"
#include "InterchangeHelper.h"

#include "MeshDescription.h"

namespace UE::Interchange::Private
{
	struct FPayloadContext
	{
		enum EKind
		{
			Element,
			SkinnedMesh,
			Morph,
			AnimationSkinned,
			AnimationMorph,
			AnimationRigid,
			AnimationProperty
		};

		FPayloadContext() = delete;
		FPayloadContext(const EKind InKind, const uint32 InIndex)
			: Kind(InKind)
			, Index(InIndex)
		{
		}

		EKind Kind;
		uint32 Index;
	};

	struct FMorph
	{
		uint32 MeshElement;
		uint32 BlendShapeElement;
	};

	struct FSkeletalAnimationPayloadContext
	{
		FString NodeUid;
		int32 AnimStackIndex;
		int32 BakedNodeIndex;
		ufbx_shared_ptr<ufbx_baked_anim> BakedAnim;
	};

	struct FMorphAnimationPayloadContext
	{
		const ufbx_anim_curve* Curve;

		// #ufbx_todo:
		//In between blend shape animation support
		//The count of the curve names should match the in between full weights
		TArray<FString> InbetweenCurveNames;
		TArray<float> InbetweenFullWeights;
	};

	struct FRigidAnimationPayloadContext
	{
		// Transform animation curves array in the order Interchange accepts it - Tx, Ty, Tz, Rx, Ry, Rz, Sx, Sy, Sz
		const ufbx_node* Node = nullptr;
		const ufbx_anim_prop* TranslationProp = nullptr;
		const ufbx_anim_prop* RotationProp = nullptr;
		const ufbx_anim_prop* ScalingProp = nullptr;
	};

	struct FPropertyAnimationPayloadContext
	{
		const ufbx_prop* Prop;
		const ufbx_anim_value* AnimValue;
		bool bIsStepAnimation;
	};

	class FUfbxParser: public IFbxParser, public FNoncopyable
	{
	public:
		explicit FUfbxParser(TWeakObjectPtr<UInterchangeResultsContainer> InResultsContainer);
		virtual ~FUfbxParser() override;

		virtual const TCHAR* GetName() override { return TEXT("uFBX"); }

		virtual bool IsThreadSafe() override { return true; }

		virtual void SetResultContainer(UInterchangeResultsContainer* Result) override;

		virtual void SetConvertSettings(const bool InbConvertScene, const bool InbForceFrontXAxis, const bool InbConvertSceneUnit, const bool InbKeepFbxNamespace, const bool InbConsiderClusterBeforePoseForMeshBindPose)  override
		{
			bForceFrontXAxis = InbForceFrontXAxis;
			bKeepFbxNamespace = InbKeepFbxNamespace;
			bConsiderClusterBeforePoseForMeshBindPose = InbConsiderClusterBeforePoseForMeshBindPose;
		}

		virtual bool LoadFbxFile(const FString& Filename, UInterchangeBaseNodeContainer& NodeContainer) override;

		/* Extract the fbx data from the sdk into our node container */
		virtual void FillContainerWithFbxScene(UInterchangeBaseNodeContainer& NodeContainer) override;

		/* Extract the fbx data from the sdk into our node container */
		virtual bool FetchPayloadData(const FString& PayloadKey, const FString& PayloadFilepath) override;

		/* Extract the fbx mesh data from the sdk into our node container */
		virtual bool FetchMeshPayloadData(const FString& PayloadKey, const FTransform& MeshGlobalTransform, const FString& PayloadFilepath) override;
#if WITH_ENGINE
		virtual bool FetchMeshPayloadData(const FString& PayloadKey, const FTransform& MeshGlobalTransform, FMeshPayloadData& OutMeshPayloadData) override;
		virtual bool FetchTexturePayload(const FString& PayloadKey, TOptional<TArray64<uint8>>& OutTexturePayloadData) override;
#endif

		/* Extract the fbx data from the sdk into our node container
		* @Param PayloadQueries - Will be grouped based on their TimeDescription Hashes (so that we acquire the same timings in one iteration, avoiding cache rebuilds)
		*/
		virtual bool FetchAnimationBakeTransformPayload(const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries, const FString& ResultFolder, FCriticalSection* ResultPayloadsCriticalSection, TAtomic<int64>& UniqueIdCounter, TMap<FString, FString>& ResultPayloads/*PayloadUniqueID to FilePath*/) override;

		virtual void Reset() override;

		FString SourceFilename;
		TWeakObjectPtr<UInterchangeResultsContainer> ResultsContainer;

		// #ufbx_todo: Messages functionality is a copy of FFbxParser's, maybe share
		/**
		 * This function is used to add the given message object directly into the results for this operation.
		 */
		template <typename T>
		T* AddMessage() const
		{
			check(ResultsContainer.IsValid());
			T* Item = ResultsContainer->Add<T>();
			Item->SourceAssetName = SourceFilename;
			return Item;
		}


		void AddMessage(UInterchangeResult* Item) const
		{
			check(ResultsContainer.IsValid());
			ResultsContainer->Add(Item);
			Item->SourceAssetName = SourceFilename;
		}

		/**
		 * Get sanitized element name, not unique
		 * @param Element 
		 * @param bIsJoint 
		 * @return 
		 */
		FString GetElementNameRaw(const ufbx_element& Element) const;

		/**
		 * Returns name build for element making sure it's unique for each element
		 * - name based on sanitized name of the element so it can be used in any place when sanitized string is needed
		 * @param Element *
		 * @return 
		 */
		FString GetElementName(const ufbx_element& Element) const;

		/**
		 * Get/Make "Name" of the node, unique(taking into account sanitized form) bu tnot sanitized(to keep namespace which contains colon), used to base Uid on, and set as a label(except bone nodes - they have different label, see @ref GetBoneNodeName)
		 * @param Node 
		 * @return 
		 */
		FString GetNodeName(const ufbx_node& Node) const;
		FString GetNodeLabel(const ufbx_node& Node) const;
		FString GetNodeUid(const ufbx_node& Node) const;

		// Set name for the joint node, same name to be passed to UE::Interchange::FMeshPayloadData::JointNames when fetching the skinned mesh payload
		// sanitized to replace underscore with dash
		FString GetBoneNodeName(const ufbx_node& Node) const;

		FString GetMeshUid(const ufbx_element& Node) const;
		FString GetMeshLabel(const ufbx_element& Mesh) const;

		FString GetMaterialName(const ufbx_material& Material) const;
		FString GetMaterialUid(const ufbx_material& Material) const;
		FString GetMaterialLabel(const ufbx_material& Material) const;

		FName GetMaterialSlotName(const ufbx_material* Material) const
		{
			return Material ? FName(*GetMaterialName(*Material)) : NAME_None;
		}

		FString GetCameraName(const ufbx_camera& Camera) const;
		FString GetLightName(const ufbx_light& Light) const;

		static bool DoesTextureHaveAlpha(const FString& TextureFilename, const ufbx_texture& Texture);

		static void ConvertProperty(const ufbx_prop& Prop, UInterchangeBaseNode* SceneNode, const TOptional<FString>& PayloadKey = TOptional<FString>());

		ufbx_scene* Scene = nullptr;

		struct FPayloadContexts
		{
			void Reset()
			{
				PayloadContexts.Reset();

				MorphPayloads.Reset();

				SkeletalAnimations.Reset();
				MorphAnimations.Reset();
				RigidAnimations.Reset();
				PropertyAnimations.Reset();
			}

			void Add(const FString& PayloadKey, const FPayloadContext& Payload)
			{
				PayloadContexts.Add(PayloadKey, Payload);
			}

			void Add(const FString& MorphTargetPayLoadKey, const FMorph& Morph)
			{
				int32 MorphId = MorphPayloads.Add(Morph);
				PayloadContexts.Add(MorphTargetPayLoadKey, FPayloadContext(FPayloadContext::Morph, MorphId));
			}

			void Add(const FString& PayloadKey, const FSkeletalAnimationPayloadContext& Payload)
			{
				int32 PayloadId = SkeletalAnimations.Add(Payload);
				PayloadContexts.Add(PayloadKey, FPayloadContext(FPayloadContext::AnimationSkinned, PayloadId));
			}

			void Add(const FString& PayloadKey, const FMorphAnimationPayloadContext& Payload)
			{
				int32 PayloadId = MorphAnimations.Add(Payload);
				PayloadContexts.Add(PayloadKey, FPayloadContext(FPayloadContext::AnimationMorph, PayloadId));
			}

			void Add(const FString& PayloadKey, const FRigidAnimationPayloadContext& Payload)
			{
				int32 PayloadId = RigidAnimations.Add(Payload);
				PayloadContexts.Add(PayloadKey, FPayloadContext(FPayloadContext::AnimationRigid, PayloadId));
			}

			void Add(const FString& PayloadKey, const FPropertyAnimationPayloadContext& Payload)
			{
				int32 PayloadId = PropertyAnimations.Add(Payload);
				PayloadContexts.Add(PayloadKey, FPayloadContext(FPayloadContext::AnimationProperty, PayloadId));
			}

			bool Contains(const FString& PayloadKey)
			{
				return PayloadContexts.Contains(PayloadKey);
			}

			FPayloadContext* Find(const FString& PayloadKey)
			{
				return PayloadContexts.Find(PayloadKey);
			}

			FMorph* GetMorph(const FPayloadContext* PayloadPtr)
			{
				if (!ensure(PayloadPtr && PayloadPtr->Kind==FPayloadContext::Morph))
				{
					return nullptr;
				}

				return &MorphPayloads[PayloadPtr->Index];
			}

			const FSkeletalAnimationPayloadContext& GetAnimation(const FPayloadContext& Payload)
			{
				return SkeletalAnimations[Payload.Index];
			}

			const FMorphAnimationPayloadContext& GetMorphAnimation(const FPayloadContext& Payload)
			{
				return MorphAnimations[Payload.Index];
			}

			const FRigidAnimationPayloadContext& GetRigidAnimation(const FPayloadContext& Payload)
			{
				return RigidAnimations[Payload.Index];
			}

			const FPropertyAnimationPayloadContext& GetPropertyAnimation(const FPayloadContext& Payload)
			{
				return PropertyAnimations[Payload.Index];
			}

			TMap<FString, FPayloadContext> PayloadContexts;

			TArray<FMorph> MorphPayloads;

			TArray<FSkeletalAnimationPayloadContext> SkeletalAnimations;
			TArray<FMorphAnimationPayloadContext> MorphAnimations;
			TArray<FRigidAnimationPayloadContext> RigidAnimations;
			TArray<FPropertyAnimationPayloadContext> PropertyAnimations;
		};

		TMap<uint32_t, UInterchangeSceneNode*> ElementIdToSceneNode;
		FPayloadContexts PayloadContexts;

		/**
		 * Maintain map of elements unique names
		 */
		struct FUniqueNames
		{
			FUniqueNames(const FUfbxParser& InParser)
				: Parser(InParser)
				, bUseNCL(true)
				, bUseNodeRules(false)
				, bIsJoint(false)
			{
			}

			/**
			 * Ensures that name is unique modulo sanitation. Although stored name may not be sanitized(Node name need to keep namespace which contains colon)
			 * still to pass Uid(based on Node name) to Interchange that Uid should dbe sanitized.
			 * @param Element 
			 * @param BaseName 
			 * @param bOverrideUseNCL whether to use '_ncl_' suffix when making unique name, or just add a number
			 * @return 
			 */
			FString MakeUniqueName(const ufbx_element& Element, const FString& BaseName) const 
			{

				bool bOverrideUseNCL = BaseName.IsEmpty();

				// Fix base name, non-node elements might need to have namespace removed
				const FString FixedBaseName = [&]
					{
						FString Result = BaseName.IsEmpty() ? DefaultName : BaseName;

						if (!bUseNodeRules)
						{
							ManageNamespace(Result);
						}
						return Result;
					}();

				// Start with base name.
				FString Name = FixedBaseName;
				FString SanitizedName = Name;
				UE::Interchange::SanitizeName(SanitizedName, bIsJoint);

				const FString SanitizedNameInitial = SanitizedName;

				if (!bUseNodeRules)
				{
					Name = SanitizedName;
				}

				for (int32 Count = 0;;++Count)
				{
					if (!NamesUsed.Contains(SanitizedName))
					{
						break;
					}

					FString Suffix = (bUseNCL || bOverrideUseNCL ? TEXT("_ncl_") : TEXT("")) + FString::FromInt(Count+1);
					SanitizedName = SanitizedNameInitial + Suffix;

					if (bUseNodeRules)
					{
						Name = FixedBaseName + Suffix;
					}
					else
					{
						Name = SanitizedName;
					}
				}
				NamesUsed.Add(SanitizedName);

				ElementToName.Add(&Element, Name);
				return Name;
			}

			void ManageNamespace(FString& ObjectName) const
			{
				if (Parser.bKeepFbxNamespace)
				{
					if (ObjectName.Contains(TEXT(":")))
					{
						ObjectName.ReplaceInline(TEXT(":"), TEXT("_"));
					}
				}
				else
				{
					// Remove namespaces
					int32 LastNamespaceTokenIndex = INDEX_NONE;
					if (ObjectName.FindLastChar(TEXT(':'), LastNamespaceTokenIndex))
					{
						//+1 to remove the ':' character we found
						ObjectName.RightChopInline(LastNamespaceTokenIndex + 1, EAllowShrinking::Yes);
					}
				}
			}

			const FString* FindName(const ufbx_element& Element) const
			{
				return ElementToName.Find(&Element);
			}

			FString GetOrMake(const ufbx_element& Element, const TFunctionRef<FString()>& GetBaseName) const
			{
				if (const FString* Found = ElementToName.Find(&Element))
				{
					return *Found;
				}
				return MakeUniqueName(Element, GetBaseName());
			}

			FString GetSafe(const ufbx_element& Element) const
			{
				if (const FString* Found = FindName(Element))
				{
					return *Found;
				}
				// Shouldn't happen!
				ensure(false);
				return TEXT("<not found>");
			}

			void Reset()
			{
				NamesUsed.Reset();
				ElementToName.Reset();
			}

			// Unique sanitized name already in use
			mutable TSet<FString> NamesUsed;

			// Name (unique) registered for each element, maybe non-sanitized(e.g. for Nodes)! But having it sanitized should still produce unique
			mutable TMap<const ufbx_element*, FString> ElementToName;

			const FUfbxParser& Parser;
			// use "_ncl_" suffix to create uniques, followed by number, or just ""
			bool bUseNCL;
			// Keeping namespace name for nodes and generating unique names without removing namespace 
			bool bUseNodeRules;
			// Name to use when source base name is empty
			FString DefaultName;
			// Joins have different rules for name sanitation
			bool bIsJoint;
		};

		bool bForceFrontXAxis = false;
		bool bKeepFbxNamespace = false;
		bool bConsiderClusterBeforePoseForMeshBindPose = false;

		// Unique names for elements by type, to base Label and Uid on
		FUniqueNames ElementNames;
		FUniqueNames NodeNames;
		FUniqueNames BoneNames;
		FUniqueNames MeshNames;
		FUniqueNames CameraNames;
		FUniqueNames LightNames;
		FUniqueNames MaterialNames;

		// Cache of node Uids
		mutable TMap<const ufbx_node*, FString> NodeToUid;
	};
}
