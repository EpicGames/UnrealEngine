// Copyright Epic Games, Inc. All Rights Reserved.

#include "UfbxScene.h"

#include "UfbxConvert.h"
#include "UfbxParser.h"
#include "UfbxAnimation.h"

#include "InterchangeFbxSettings.h"

#include "InterchangeMeshNode.h"
#include "InterchangeCameraNode.h"
#include "InterchangeJointNode.h"
#include "InterchangeLightNode.h"
#include "InterchangeMeshLODContainerNode.h"
#include "Math/UnitConversion.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"

#define LOCTEXT_NAMESPACE "InterchangeFbxParser"

namespace UE::Interchange::Private
{
	namespace SceneUtils
	{
		// UE::Interchange::Private::CreateTrackNodeUid
		FString CreateTrackNodeUid(const FString& JointUid, const int32 AnimationIndex)
		{
			FString TrackNodeUid = TEXT("\\SkeletalAnimation\\") + JointUid + TEXT("\\") + FString::FromInt(AnimationIndex);
			return TrackNodeUid;
		}

		template<typename T>
		T* CreateTransformNode(FUfbxParser& Parser, UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeDisplayLabel, const FString& NodeUID, const FTransform& LocalTransform, const FString& ParentNodeUID)
		{
			static_assert(TIsDerivedFrom<T, UInterchangeSceneNode>::IsDerived, "T must derive from UInterchangeSceneNode");

			T* TransformNode = NewObject<T>(&NodeContainer, NAME_None);
			if (!ensure(TransformNode))
			{
				UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
				Message->Text = LOCTEXT("TransformNodeAllocationError", "Unable to allocate a node when importing FBX.");
				return nullptr;
			}
			NodeContainer.SetupNode(TransformNode, NodeUID, NodeDisplayLabel, EInterchangeNodeContainerType::TranslatedScene, ParentNodeUID);
			TransformNode->SetCustomLocalTransform(&NodeContainer, LocalTransform);


			return TransformNode;
		}

		// Inspired by UE::Interchange::Private::FFbxScene::FindCommonJointRootNode, to match how FbxParser creates skeleton
		// This function makes root of the skeleton to be a node just above topmost bone(node participating in skinning)
		// Not sure why this is done like this in FBX SDK 
		const ufbx_node* FindSkeletonRoot(const FUfbxParser& Parser, const ufbx_node* Bone)
		{
			while (Bone && Bone->parent)
			{
				ufbx_node* ParentNode = Bone->parent;

				bool bIsBlenderArmatureBone = false;
				ufbx_scene& Scene = *Parser.Scene;
				if (Scene.metadata.exporter == UFBX_EXPORTER_BLENDER_ASCII ||
					Scene.metadata.exporter == UFBX_EXPORTER_BLENDER_BINARY)
				{
					//Hack to support armature dummy node from blender
					//Users do not want the null attribute node named armature which is the parent of the real root bone in blender fbx file
					//This is a hack since if a rigid mesh group root node is named "armature" it will be skip
					const FString ParentName = Parser.GetElementNameRaw(ParentNode->element);

					const ufbx_node* GrandFather = ParentNode->parent;
					
					bIsBlenderArmatureBone = (GrandFather == nullptr || GrandFather == Scene.root_node) && (ParentName.Compare(TEXT("armature"), ESearchCase::IgnoreCase) == 0);
				}
				// Don't drop Armature bone with multiple nodes under armature since this will remove extra bones from the skeleton
				const bool bIgnoreBlenderArmatureBone = ParentNode->children.count > 1;

				if (ParentNode->parent &&
					((ParentNode->attrib_type == UFBX_ELEMENT_EMPTY && (!bIsBlenderArmatureBone||bIgnoreBlenderArmatureBone)) || 
						ParentNode->attrib_type == UFBX_ELEMENT_MESH || 
						ParentNode->attrib_type == UFBX_ELEMENT_BONE)
					&& ParentNode != Scene.root_node)
				{
					if (ParentNode->mesh && ParentNode->mesh->skin_deformers.count > 0)
					{
						break;
					}
					Bone = ParentNode;
				}
				else
				{
					break;
				}
			}
			return Bone;
		}

		bool IsPropTypeSupported(const ufbx_prop_type& PropType)
		{
			// Also see FUfbxParser::ConvertProperty
			switch (PropType)
			{
			case UFBX_PROP_BOOLEAN:
			case UFBX_PROP_INTEGER:
			case UFBX_PROP_NUMBER:
			case UFBX_PROP_VECTOR:
			case UFBX_PROP_COLOR:
			case UFBX_PROP_COLOR_WITH_ALPHA:
			case UFBX_PROP_STRING:
			case UFBX_PROP_DATE_TIME:
				return true;
			// #ufbx_todo: support other useful custom attributes among these types
			case UFBX_PROP_TRANSLATION:
			case UFBX_PROP_ROTATION:
			case UFBX_PROP_SCALING:
			case UFBX_PROP_DISTANCE:
			case UFBX_PROP_COMPOUND:
			case UFBX_PROP_BLOB:
			case UFBX_PROP_REFERENCE:
			case UFBX_PROP_TYPE_FORCE_32BIT:
			default: ;
			}
			return false;
		}

		FMatrix EvaluateNodeTransformGlobal(const FUfbxParser& Parser, const ufbx_node& Node, const double Time=0)
		{
			FMatrix LocalTransform = Convert::ConvertTransform(ufbx_evaluate_transform(Parser.Scene->anim, &Node, Time)).ToMatrixWithScale();

			if (Node.parent)
			{
				FMatrix ParentTransform = EvaluateNodeTransformGlobal(Parser, *Node.parent, Time);
				return LocalTransform * ParentTransform;
			}

			return LocalTransform;
		}
	}

	FMatrix FUfbxScene::GetBindPoseMatrix(const ufbx_node& InNode)
	{
		if (const FMatrix* Found = BindPose.Find(&InNode))
		{
			return *Found;
		}
		return Convert::ConvertMatrix(InNode.node_to_world);
	}

	bool FUfbxScene::IsInSkeleton(const ufbx_node& Node)
	{
		if (CommonJointRootNodes.IsEmpty())
		{
			return false;
		}

		for (const ufbx_node* Parent = &Node; Parent; Parent = Parent->parent)
		{
			if (CommonJointRootNodes.Contains(Parent))
			{
				return true;
			}
		}
		return false;

	};

	void FUfbxScene::InitHierarchyRecursively(TArray<UnrealNodeInfo>& OutNodeInfos, const ufbx_node& Node, const FString& ParentUid)
	{
		FString Label(Parser.GetNodeLabel(Node));
		FString UniqueID(Parser.GetNodeUid(Node));

		OutNodeInfos.Emplace(&Node, Label, UniqueID, ParentUid);

		if (Node.attrib_type == UFBX_ELEMENT_LOD_GROUP)
		{
			//In case LOD Group, do not parse children into sceneNodes.
			//As they should be parsed into MeshLODContainerNodes.
			return;
		}

		for (ufbx_node* Child : Node.children)
		{
			InitHierarchyRecursively(OutNodeInfos, *Child, UniqueID);
		}
	};

	void FUfbxScene::ParseMeshLODs(UInterchangeMeshLODContainerNode& LODContainerNode,
		const ufbx_node& Node,
		int32 LODIndex,
		const FMatrix& InvTransformMatrix /*Global Inverse Transform Matrix Of LODContainer*/)
	{
		//#interchange_LODRefactor_Note: Seems the expectation is that the LODIndex is driven by the Child placement (index) on the eLODGroup.
		//									(Not the node names.)

		if (Node.mesh)
		{
			FString MeshUniqueID = Parser.GetMeshUid(Node.mesh->element);

			FTransform RelativeTransform = FTransform(Convert::ConvertMatrix(Node.node_to_world) * InvTransformMatrix);

			const FTransform GeometryTransform = Convert::ConvertTransform(Node.geometry_transform);

			FTransform CombinedTransform = GeometryTransform * RelativeTransform;
			LODContainerNode.AddMeshForLODIndex(LODIndex, MeshUniqueID, CombinedTransform);
		}

		for (ufbx_node* Child : Node.children)
		{
			if (Child)
			{
				ParseMeshLODs(LODContainerNode, *Child, LODIndex, InvTransformMatrix);
			}
		}
	}

	void FUfbxScene::ProcessNode(UInterchangeBaseNodeContainer& NodeContainer, const ufbx_node& InNode)
	{
		UInterchangeSceneNode* SceneNode;
		{
			UInterchangeSceneNode** SceneNodeFound = Parser.ElementIdToSceneNode.Find(InNode.element_id);
			if (!SceneNodeFound)
			{
				return;
			}

			SceneNode = *SceneNodeFound;

			if (!ensure(IsValid(SceneNode)))
			{
				return;
			}
		}

		const bool bIsRootNode = &InNode == Parser.Scene->root_node;
		if (bIsRootNode)
		{
			SceneNode->SetCustomIsSceneRoot(true);
		}

		SceneNode->SetCustomActorVisibility(InNode.visible);

		const FTransform GeometryTransform = Convert::ConvertTransform(InNode.geometry_transform);
		if (!GeometryTransform.Equals(FTransform::Identity))
		{
			SceneNode->SetCustomGeometricTransform(GeometryTransform);
		}

		if (InNode.mesh)
		{
			FString MeshUniqueID = Parser.GetMeshUid(InNode.mesh->element);
			
			if (const UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(NodeContainer.GetNode(MeshUniqueID)))
			{
				SceneNode->SetCustomAssetInstanceUid(MeshNode->GetUniqueID());
			}

			// Add override materials
			int32 MaterialCount = InNode.materials.count;
			TMap<ufbx_material*, int32> UniqueSlotNames;
			UniqueSlotNames.Reserve(MaterialCount);
			for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex)
			{
				if (ufbx_material* SurfaceMaterial = InNode.materials[MaterialIndex])
				{
					FString MaterialUid = Parser.GetMaterialUid(*SurfaceMaterial);

					int32& SlotMaterialCount = UniqueSlotNames.FindOrAdd(SurfaceMaterial);

					FString MaterialSlotName = Parser.GetMaterialSlotName(SurfaceMaterial).ToString();
					if (SlotMaterialCount > 0)
					{
						MaterialSlotName += TEXT("_Section") + FString::FromInt(SlotMaterialCount);
					}
					SceneNode->SetSlotMaterialDependencyUid(MaterialSlotName, MaterialUid);
					SlotMaterialCount++;
				}
			}
		}
		if (UInterchangeJointNode* JointNode = Cast<UInterchangeJointNode>(SceneNode))
		{
			FString JointNodeName = Parser.GetBoneNodeName(InNode);
			JointNode->SetDisplayLabel(JointNodeName);

			//Get the bind pose transform for this joint
			{
				// #ufbx_todo: probably, we don't actually need Global transform, as FBX Parser does since FBX SDK returns only evaluate globally, but source is local
				// so might refactor this carefully to locally evaluated transforms
				FMatrix GlobalBindPoseJointMatrix = GetBindPoseMatrix(InNode);

				FTransform GlobalBindPoseJointTransform(GlobalBindPoseJointMatrix);

				TMap<FString, FMatrix> MeshIdToGlobalBindPoseReferenceMap;
				if (const TMap<FString, FMatrix>* Found = JointToMeshIdToGlobalBindPoseReferenceMap.Find(&InNode))
				{
					MeshIdToGlobalBindPoseReferenceMap = *Found;
				}

				JointNode->SetMeshUIDToGlobalBindPoseReferenceMap(MeshIdToGlobalBindPoseReferenceMap);

				if (const TMap<FString, FMatrix>* Found = JointToMeshIdToGlobalBindPoseJointMap.Find(&InNode))
				{
					for (const TPair<FString, FMatrix>& MeshIdAndBindPoseJointMatrix : *Found)
					{
						FString AttributeKey = TEXT("JointBindPosePerMesh_") + MeshIdAndBindPoseJointMatrix.Key;
						JointNode->RegisterAttribute<FMatrix>(FAttributeKey(AttributeKey), MeshIdAndBindPoseJointMatrix.Value);
					}
				}

				if (InNode.parent)
				{
					FMatrix GlobalFbxParentMatrix = GetBindPoseMatrix(*InNode.parent);

					FMatrix LocalFbxMatrix = GlobalBindPoseJointMatrix * GlobalFbxParentMatrix.Inverse();
					FTransform LocalBindPoseJointTransform(LocalFbxMatrix);

					JointNode->SetBindPoseLocalTransform(&NodeContainer, LocalBindPoseJointTransform);
				}
				else
				{
					JointNode->SetBindPoseLocalTransform(&NodeContainer, GlobalBindPoseJointTransform);
				}
			}

			//Get time Zero transform for this joint
			{
				FMatrix GlobalFbxMatrix = SceneUtils::EvaluateNodeTransformGlobal(Parser, InNode, 0);
				JointNode->SetGlobalMatrixForT0Rebinding(GlobalFbxMatrix);

				FTransform GlobalTransform(GlobalFbxMatrix);

				if (InNode.parent)
				{
					FMatrix GlobalFbxParentMatrix = SceneUtils::EvaluateNodeTransformGlobal(Parser, *InNode.parent, 0);

					FMatrix LocalFbxMatrix = GlobalFbxMatrix * GlobalFbxParentMatrix.Inverse();
					FTransform LocalTransform(LocalFbxMatrix);

					JointNode->SetTimeZeroLocalTransform(&NodeContainer, LocalTransform);
				}
				else
				{
					JointNode->SetTimeZeroLocalTransform(&NodeContainer, GlobalTransform);
				}
			}

			//Set Default transforms
			{
				FMatrix GlobalFbxMatrix = Convert::ConvertMatrix(InNode.node_to_world);

				if (InNode.parent)
				{
					FMatrix GlobalFbxParentMatrix = Convert::ConvertMatrix(InNode.parent->node_to_world);

					FMatrix	LocalFbxMatrix = GlobalFbxMatrix * GlobalFbxParentMatrix.Inverse();
					FTransform LocalTransform(LocalFbxMatrix);

					JointNode->SetCustomLocalTransform(&NodeContainer, LocalTransform);
				}
				else
				{
					FTransform GlobalTransform(GlobalFbxMatrix);
					//No parent, set the same matrix has the global
					JointNode->SetCustomLocalTransform(&NodeContainer, GlobalTransform);
				}
			}
		}

		if (InNode.camera)
		{
			const ufbx_camera& Camera = *InNode.camera;

			FString CameraName = Parser.GetCameraName(Camera);
			FString CameraLabel = CameraName;

			switch (Camera.projection_mode)
			{
			case UFBX_PROJECTION_MODE_PERSPECTIVE:
			{
				FString CameraUid = FString(TEXT("\\")) + UInterchangePhysicalCameraNode::StaticAssetTypeName() + TEXT("\\") + CameraName;

				const UInterchangePhysicalCameraNode* CameraNode = Cast<const UInterchangePhysicalCameraNode>(NodeContainer.GetNode(CameraUid));
				if (!CameraNode)
				{
					UInterchangePhysicalCameraNode* NewCameraNode = NewObject<UInterchangePhysicalCameraNode>(&NodeContainer, NAME_None);
					if (!ensure(NewCameraNode))
					{
						UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
						Message->Text = LOCTEXT("CannotAllocateNode", "Cannot allocate a node when importing FBX.");
						return;
					}

					NodeContainer.SetupNode(NewCameraNode, CameraUid, CameraName, EInterchangeNodeContainerType::TranslatedAsset);

					// ufbx computes fov tan for every aperture mode for we can recalculate focal length
					float ApertureSize = FUnitConversion::Convert(Camera.aperture_size_inch.x, EUnit::Inches, EUnit::Millimeters);
					float FocalLength = 0.5f * ApertureSize / FMath::Max(Camera.field_of_view_tan.x, UE_SMALL_NUMBER);

					NewCameraNode->SetCustomFocalLength(FocalLength); //Both FBX and UE have their focal length in mm
					NewCameraNode->SetCustomSensorHeight(FUnitConversion::Convert(Camera.aperture_size_inch.y, EUnit::Inches, EUnit::Millimeters));
					NewCameraNode->SetCustomSensorWidth(FUnitConversion::Convert(Camera.aperture_size_inch.x, EUnit::Inches, EUnit::Millimeters));

					CameraNode = NewCameraNode;
				}

				SceneNode->SetCustomAssetInstanceUid(CameraUid);
			}
				break;
			case UFBX_PROJECTION_MODE_ORTHOGRAPHIC:
			{
				FString CameraUid = FString(TEXT("\\")) + UInterchangeStandardCameraNode::StaticAssetTypeName() + TEXT("\\") + CameraName;

				const UInterchangeStandardCameraNode* CameraNode = Cast<const UInterchangeStandardCameraNode>(NodeContainer.GetNode(CameraUid));
				if (!CameraNode)
				{
					UInterchangeStandardCameraNode* NewCameraNode = NewObject<UInterchangeStandardCameraNode>(&NodeContainer, NAME_None);
					if (!ensure(NewCameraNode))
					{
						UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
						Message->Text = LOCTEXT("CannotAllocateNode", "Cannot allocate a node when importing FBX.");
						return;
					}

					NodeContainer.SetupNode(NewCameraNode, CameraUid, CameraName, EInterchangeNodeContainerType::TranslatedAsset);

					NewCameraNode->SetCustomProjectionMode(EInterchangeCameraProjectionType::Orthographic);

					NewCameraNode->SetCustomWidth(Camera.orthographic_size.x);
					NewCameraNode->SetCustomNearClipPlane(Camera.near_plane);
					NewCameraNode->SetCustomFarClipPlane(Camera.far_plane);
					NewCameraNode->SetCustomAspectRatio(Camera.aspect_ratio);

					CameraNode = NewCameraNode;
				}

				SceneNode->SetCustomAssetInstanceUid(CameraUid);
			}
				break;
			};
		}

		if (InNode.light)
		{
			const ufbx_light& Light = *InNode.light;

			FString LightName = Parser.GetLightName(Light);
			FString LightUid = FString(TEXT("\\")) + UInterchangeBaseLightNode::StaticAssetTypeName() + TEXT("\\") + LightName;
			const UInterchangeBaseLightNode* LightNode = Cast<const UInterchangeBaseLightNode>(NodeContainer.GetNode(LightUid));

			if (!LightNode)
			{
				UClass* LightClass;

				switch(Light.type)
				{
				case UFBX_LIGHT_POINT:
				case UFBX_LIGHT_VOLUME:
					LightClass = UInterchangePointLightNode::StaticClass();
					break;
				case UFBX_LIGHT_DIRECTIONAL:
					LightClass = UInterchangeDirectionalLightNode::StaticClass();
					break;
				case UFBX_LIGHT_SPOT:
					LightClass = UInterchangeSpotLightNode::StaticClass();
					break;
				case UFBX_LIGHT_AREA:
					LightClass = UInterchangeRectLightNode::StaticClass();
					break;
				default:
					LightClass = UInterchangePointLightNode::StaticClass();
					break;
				}

				UInterchangeBaseLightNode* BaseLightNode = NewObject<UInterchangeBaseLightNode>(&NodeContainer, LightClass, NAME_None);
				if (!ensure(BaseLightNode))
				{
					UInterchangeResultError_Generic* Message = Parser.AddMessage<UInterchangeResultError_Generic>();
					Message->Text = LOCTEXT("CannotAllocateNode", "Cannot allocate a node when importing FBX.");
					return;
				}
				else
				{
					FString LightLabel = LightName;
				
					NodeContainer.SetupNode(BaseLightNode, LightUid, LightLabel, EInterchangeNodeContainerType::TranslatedAsset);

					// ufbx divides intensity by 100, re-multiplied here to match FBX SDK parser
					BaseLightNode->SetCustomIntensity(static_cast<float>(Light.intensity) * 100.f);

					const ufbx_vec4 Color = { {{Light.color.x, Light.color.y, Light.color.z, 1.0}} };
					BaseLightNode->SetCustomLightColor(Convert::ConvertColor(Color));

					if (UInterchangeLightNode* InterchangeLightNode = Cast<UInterchangeLightNode>(BaseLightNode))
					{
						float CustomAttenuationRadius = Light.decay == UFBX_LIGHT_DECAY_NONE ? FBXSDK_FLOAT_MAX : Light.decay_start;
						if (CustomAttenuationRadius > 0)
						{
							InterchangeLightNode->SetCustomAttenuationRadius(CustomAttenuationRadius);
						}
					}

					if (Light.type == UFBX_LIGHT_SPOT )
					{
						if (UInterchangeSpotLightNode* SpotLightNode = Cast<UInterchangeSpotLightNode>(BaseLightNode))
						{
							SpotLightNode->SetCustomInnerConeAngle(Light.inner_angle);
							SpotLightNode->SetCustomOuterConeAngle(Light.outer_angle);
						}
					}

				}
			}
			SceneNode->SetCustomAssetInstanceUid(LightUid);
		}

		if (InNode.attrib_type == UFBX_ELEMENT_LOD_GROUP)
		{	
			FString LODContainerUid = UInterchangeMeshLODContainerNode::MakeMeshLODContainerUid(SceneNode->GetUniqueID());

			UInterchangeMeshLODContainerNode* LODContainerNode = NewObject<UInterchangeMeshLODContainerNode>(&NodeContainer);
			if (!ensure(LODContainerNode))
			{
				return;
			}

			NodeContainer.SetupNode(
				LODContainerNode,
				LODContainerUid,
				SceneNode->GetDisplayLabel(),
				EInterchangeNodeContainerType::TranslatedAsset
			);

			SceneNode->SetCustomAssetInstanceUid(LODContainerUid);

			FMatrix GlobalMatrixInverse = Convert::ConvertMatrix(InNode.node_to_world).Inverse();

			int32 LODIndex = 0;
			for (ufbx_node* Child : InNode.children)
			{
				if (Child)
				{
					ParseMeshLODs(*LODContainerNode, *Child, LODIndex, GlobalMatrixInverse);
				}
				
				LODIndex++;
			}

		}

		for (const ufbx_prop& Prop : InNode.props.props)
		{
			if (SceneUtils::IsPropTypeSupported(Prop.type) &&
				Prop.flags & UFBX_PROP_FLAG_USER_DEFINED)
			{
				TOptional<FString> PayloadKey;
				if (Prop.flags & UFBX_PROP_FLAG_ANIMATED)
				{
					for (const ufbx_connection& Connection : InNode.element.connections_dst)
					{
						// Property and its connection have same name string pointer in ufbx
						if (Connection.dst_prop.data == Prop.name.data)
						{
							if (ufbx_anim_value* AnimValue = ufbx_as_anim_value(Connection.src))
							{
								FString CurveName;
								TOptional<bool> bIsStepCurve;
								if (FUfbxAnimation::AddNodeAttributeCurvesAnimation(Parser, SceneNode->GetUniqueID(), Prop, *AnimValue, PayloadKey, bIsStepCurve, CurveName))
								{
									EInterchangeAnimationPayLoadType PayloadType = bIsStepCurve.GetValue() ? EInterchangeAnimationPayLoadType::STEPCURVE : EInterchangeAnimationPayLoadType::CURVE;

									const UInterchangeFbxSettings* InterchangeFbxSettings = GetDefault<UInterchangeFbxSettings>();
									if (EInterchangePropertyTracks PropertyTrack = InterchangeFbxSettings->GetPropertyTrack(CurveName); PropertyTrack != EInterchangePropertyTracks::None)
									{
										UInterchangeAnimationTrackNode* AnimTrackNode = NewObject< UInterchangeAnimationTrackNode >(&NodeContainer);
										const FString AnimTrackNodeName = SceneNode->GetDisplayLabel() + CurveName;
										const FString AnimTrackNodeUid = TEXT("\\AnimationTrack\\") + AnimTrackNodeName;

										NodeContainer.SetupNode(AnimTrackNode, AnimTrackNodeUid, AnimTrackNodeName, EInterchangeNodeContainerType::TranslatedAsset);

										AnimTrackNode->SetCustomActorDependencyUid(SceneNode->GetUniqueID());
										AnimTrackNode->SetCustomAnimationPayloadKey(PayloadKey.GetValue(), PayloadType);
										AnimTrackNode->SetCustomPropertyTrack(PropertyTrack);
									}

									SceneNode->SetAnimationCurveTypeForCurveName(CurveName, PayloadType);
								}
							}
							break;
						}
					}
				}
				Parser.ConvertProperty(Prop, SceneNode, PayloadKey);
			}
		}

		for (const ufbx_element* Attrib : InNode.all_attribs)
		{
			for (const ufbx_prop& Prop : Attrib->props.props)
			{
				if (SceneUtils::IsPropTypeSupported(Prop.type) && Prop.flags & UFBX_PROP_FLAG_ANIMATED)
				{
					for (const ufbx_connection& Connection : Attrib->connections_dst)
					{
						// Property and its connection have same name string pointer in ufbx
						if (Connection.dst_prop.data == Prop.name.data)
						{
							if (ufbx_anim_value* AnimValue = ufbx_as_anim_value(Connection.src))
							{
								FString CurveName;
								TOptional<bool> bIsStepCurve;
								TOptional<FString> PayloadKey;
								if (FUfbxAnimation::AddNodeAttributeCurvesAnimation(Parser, SceneNode->GetUniqueID(), Prop, *AnimValue, PayloadKey, bIsStepCurve, CurveName))
								{
									EInterchangeAnimationPayLoadType PayloadType = bIsStepCurve.GetValue() ? EInterchangeAnimationPayLoadType::STEPCURVE : EInterchangeAnimationPayLoadType::CURVE;

									const UInterchangeFbxSettings* InterchangeFbxSettings = GetDefault<UInterchangeFbxSettings>();
									if (EInterchangePropertyTracks PropertyTrack = InterchangeFbxSettings->GetPropertyTrack(CurveName); PropertyTrack != EInterchangePropertyTracks::None)
									{
										UInterchangeAnimationTrackNode* AnimTrackNode = NewObject< UInterchangeAnimationTrackNode >(&NodeContainer);
										const FString AnimTrackNodeName = SceneNode->GetDisplayLabel() + CurveName;
										const FString AnimTrackNodeUid = TEXT("\\AnimationTrack\\") + AnimTrackNodeName;

										NodeContainer.SetupNode(AnimTrackNode, AnimTrackNodeUid, AnimTrackNodeName, EInterchangeNodeContainerType::TranslatedAsset);

										AnimTrackNode->SetCustomActorDependencyUid(SceneNode->GetUniqueID());
										AnimTrackNode->SetCustomAnimationPayloadKey(PayloadKey.GetValue(), PayloadType);
										AnimTrackNode->SetCustomPropertyTrack(PropertyTrack);
									}

									SceneNode->SetAnimationCurveTypeForCurveName(CurveName, PayloadType);
								}
							}
						}
					}
				}
			}
		}
	}

	void FUfbxScene::ProcessNodes(UInterchangeBaseNodeContainer& NodeContainer)
	{
		for (ufbx_node* Node : Parser.Scene->nodes)
		{
			ProcessNode(NodeContainer, *Node);
		}
	}

	FUfbxScene::FUfbxScene(FUfbxParser& InParser): Parser(InParser)
	{
		// Build bind pose for skinned mesh.

		// Normally correct Bind Pose would be fully defined in fbx - 
		//   a Pose object for each bone, with Type: "BindPose"
		// that would identify bone position for mesh at the time of skin association
		// But BindPose might be missing.
		// Still, clusters(an object identifying skinning per bone per mesh)
		// contain bone bind position
		// But clusters will lack position for bones that are part of skeleton structure but
		// not affecting mesh. Those will use just initial pose stored in the file.

		TMap<const ufbx_node*, TMap<FString, FMatrix>> BoneToMeshIdToBindToWorldMap;
		TMap<const ufbx_node*, TMap<FString, FMatrix>> BoneToMeshIdToGeometryToBone;

		// Collect bind_to_world for each bone
		for (ufbx_mesh* Mesh : Parser.Scene->meshes)
		{
			// #ufbx_todo: fix getting existing mesh element Uid using ufbx_mesh param instead of element
			FString MeshUid = Parser.GetMeshUid(Mesh->element);
			for (ufbx_skin_deformer* Deformer : Mesh->skin_deformers)
			{
				for (ufbx_skin_cluster* Cluster : Deformer->clusters)
				{
					
					TMap<FString, FMatrix>& MeshIdToGlobalBindPoseJointMap = BoneToMeshIdToBindToWorldMap.FindOrAdd(Cluster->bone_node);

					const FMatrix BindMatrix = Convert::ConvertMatrix(Cluster->bind_to_world);
					MeshIdToGlobalBindPoseJointMap.Add(MeshUid, BindMatrix);

					const FMatrix GeometryToBone = Convert::ConvertMatrix(Cluster->geometry_to_bone);
					BoneToMeshIdToGeometryToBone.FindOrAdd(Cluster->bone_node).Add(MeshUid, GeometryToBone);
				}
			}
		}

		// Find root bone of each skeleton
		for (const ufbx_node* Node : Parser.Scene->nodes)
		{
			if (Node->bone || BoneToMeshIdToGeometryToBone.Contains(Node))
			{
				if (const ufbx_node* SkeletonRoot = SceneUtils::FindSkeletonRoot(Parser, Node))
				{
					CommonJointRootNodes.FindOrAdd(SkeletonRoot);
				}
			}
		}

		// Record which node has bind pose defined somehow - in a skin cluster or specified bind pose
		TSet<const ufbx_node*> bHasPose;

		// Build BindPose from skin clusters
		for (const ufbx_node* Node : Parser.Scene->nodes)
		{
			if (TMap<FString, FMatrix>* Found = BoneToMeshIdToBindToWorldMap.Find(Node))
			{
				FString MeshUid;
				for (const TTuple<FString, FMatrix>& P : *Found)
				{
					const FString& NewMeshUid = P.Key;
					const FMatrix& NewMatrix = P.Value;
					if (bHasPose.Contains(Node))
					{
						FMatrix BindPoseMatrix = BindPose[Node];
						if (!BindPoseMatrix.Equals(NewMatrix, UE_KINDA_SMALL_NUMBER))
						{
							// Normally, each bone is used in one cluster per mesh. But Fbx supports multiple skins on each mesh,
							// this means there could be different clusters for single bone and with different bind matrices
							UInterchangeResultWarning_Generic* Message = Parser.AddMessage<UInterchangeResultWarning_Generic>();
							Message->Text = FText::Format(LOCTEXT("FFbxJointMeshBindPoseGenerator_ConflictingBindPoseMatrices", 
								"Imported SkeletalMesh and/or Skeleton might be incorrect.\nConflicting Bind Pose Matrices found for Joint [{0}].\n   Currently using Geometry [{1}] for Bind Pose Matrix [{2}].\n   However Geometry [{3}] stores a differing Bind Pose Matrix [{4}]."), 
								FText::FromString(Parser.GetElementNameRaw(Node->element)), 
								FText::FromString(MeshUid), FText::FromString(BindPoseMatrix.ToString()),
								FText::FromString(NewMeshUid), FText::FromString(NewMatrix.ToString()));
						}
					}
					else
					{
						MeshUid = NewMeshUid;
						bHasPose.Add(Node);
						BindPose.Add(Node, NewMatrix);
					}
				}
			}
		}

		// Fill bind pose for bones that are not found in clusters from bind_pose found on node
		bool bClusterVsPoseMissmatch = false;
		for (ufbx_node* Node : Parser.Scene->nodes)
		{
			if (Node->bind_pose && Node->bind_pose->is_bind_pose)
			{
				ufbx_bone_pose* Pose = ufbx_get_bone_pose(Node->bind_pose, Node);
				if (Pose)
				{
					const FMatrix BindPoseMatrix = Convert::ConvertMatrix(Pose->bone_to_world);
					
					if (!bHasPose.Contains(Node) || !Parser.bConsiderClusterBeforePoseForMeshBindPose)
					{
						BindPose.Add(Node, BindPoseMatrix);
						bHasPose.Add(Node);
					}
					else
					{
						// Check matrices previously taken from cluster
						if (const FMatrix* Found = BindPose.Find(Node))
						{
							if (!BindPoseMatrix.Equals(*Found, KINDA_SMALL_NUMBER))
							{
								bClusterVsPoseMissmatch = true;
							}
						}
						
					}
				}
			}
		}

		if (bClusterVsPoseMissmatch)
		{
			UInterchangeResultWarning_Generic* Message = Parser.AddMessage<UInterchangeResultWarning_Generic>();
			Message->Text = LOCTEXT("FFbxJointMeshBindPoseGenerator_NonMatchingBindPoses", "BindPose Matrix generation acquired 2 different Matrices from FbxCluster vs FbxPose.\n If import is incorrect, consider using a different setting for FBX option 'bConsiderClusterBeforePoseForMeshBindPose'.");
		}

		// Set BindPose For nodes not found in bind pose or clusters
		// This is needed in the first place for rigid animation which is converted to skinned.
		// Since rigid animation has no bind pose or skin clusters to get bone position from.
		for (const ufbx_node* Node : Parser.Scene->nodes)
		{
			if (!bHasPose.Contains(Node))
			{
				// #ufbx_todo: Using first frame of animation instead of Node->node_to_world to match legacy behavior but node_to_world might 
				// be more convenient to use all the time since this is the 'default' position and we are discarding it otherwise.
				if (Parser.Scene->anim)
				{
					FMatrix GlobalFbxMatrix = SceneUtils::EvaluateNodeTransformGlobal(Parser, *Node, 0);
					BindPose.Add(Node, GlobalFbxMatrix);
				}
				else
				{
					BindPose.Add(Node, Convert::ConvertMatrix(Node->node_to_world));
				}
			}
		}

		// Bind pose can be detached from skeleton root(this happens when root bone in not in bind pose/clusters)
		// then we need to rebase it (assume global transforms of nodes as local to this root)
		for (TTuple<const ufbx_node*, FMatrix>& P : BindPose)
		{
			const ufbx_node* Node = P.Key;

			// Check node ancestors having defined pose when this node has the pose
			if (bHasPose.Contains(Node))
			{
				for(const ufbx_node* Parent = Node->parent; Parent && IsInSkeleton(*Parent); Parent = Parent->parent)
				{
					if (BindPose.Contains(Parent) && !bHasPose.Contains(Parent) && CommonJointRootNodes.Contains(Parent))
					{
						FMatrix BindPoseMatrix = BindPose[Parent];

						// Rebase node under node without bind pose
						// When bone has no bind pose (transform defined in cluster)
						// then we treat children bind poses that are defined under it as relative to this
						// parent's. Usually they are all global.
						P.Value = P.Value * BindPoseMatrix;
						break;
					}
				}
			}
		}

		// Finally, Collect bind pose matrices for reskin
		for (ufbx_mesh* Mesh : Parser.Scene->meshes)
		{
			// #ufbx_todo: fix getting existing mesh element Uid using ufbx_mesh param instead of element
			FString MeshUid = Parser.GetMeshUid(Mesh->element);
			for (const ufbx_skin_deformer* Deformer : Mesh->skin_deformers)
			{
					
				for (const ufbx_skin_cluster* Cluster : Deformer->clusters)
				{
					const FMatrix GeometryToBone = Convert::ConvertMatrix(Cluster->geometry_to_bone);
					const FMatrix BindToWorld = Convert::ConvertMatrix(Cluster->bind_to_world);

					JointToMeshIdToGlobalBindPoseReferenceMap.FindOrAdd(Cluster->bone_node).Add(MeshUid, GeometryToBone*BindToWorld);
				}
			}

			for (ufbx_skin_deformer* Deformer : Mesh->skin_deformers)
			{
				for (ufbx_skin_cluster* Cluster : Deformer->clusters)
				{
					TMap<FString, FMatrix>& MeshIdToGlobalBindPoseJointMap = JointToMeshIdToGlobalBindPoseJointMap.FindOrAdd(Cluster->bone_node);

					const FMatrix BindMatrix = Convert::ConvertMatrix(Cluster->bind_to_world);
					MeshIdToGlobalBindPoseJointMap.Add(MeshUid, BindMatrix);
				}
			}
		}
	}

	void FUfbxScene::InitHierarchy(UInterchangeBaseNodeContainer& NodeContainer)
	{
		ufbx_node* RootNode = Parser.Scene->root_node;
		Parser.ElementIdToSceneNode.Reserve(Parser.Scene->nodes.count);

		// Make unique names for nodes
		for (const ufbx_node* Node : Parser.Scene->nodes)
		{
			FString BaseName = Parser.GetElementNameRaw(Node->element);


			if (Node->is_root)
			{
				// ufbx doesn't forcefully initialize root node name, so in case it's empty - set it
				BaseName = TEXT("RootNode");
			}

			Parser.NodeNames.MakeUniqueName(Node->element, BaseName);
		}

		TArray<UnrealNodeInfo> UnrealNodeInfos;
		UnrealNodeInfos.Reserve(Parser.Scene->nodes.count);

		InitHierarchyRecursively(UnrealNodeInfos, *RootNode, TEXT(""));

		for (const ufbx_node* Node : Parser.Scene->nodes)
		{
			if (!ensureMsgf(Node, TEXT("Not expected ufbx scene nodes to contain a nullptr")))
			{
				continue;
			}


			if (!Node->parent && (Node->element_id != RootNode->element_id)) 
			{
				InitHierarchyRecursively(UnrealNodeInfos, *Node, TEXT(""));
			}
		}

		//Create SceneNodes
		for (const UnrealNodeInfo& Info : UnrealNodeInfos)
		{
			UInterchangeSceneNode* SceneNode;

			const ufbx_node* Node = Info.Node;
			FTransform Transform = Convert::ConvertTransform(Node->local_transform);

			if (Node->camera)
			{
				Transform.SetScale3D(FVector::OneVector);
			}

			if (Node->bone || IsInSkeleton(*Node))
			{
				SceneNode = SceneUtils::CreateTransformNode<UInterchangeJointNode>(Parser, NodeContainer,
					Info.Label, Info.UniqueId, Transform, Info.ParentUid);
			}
			else
			{
				SceneNode = SceneUtils::CreateTransformNode<UInterchangeSceneNode>(Parser, NodeContainer,
					Info.Label, Info.UniqueId, Transform, Info.ParentUid);
			}

			if (ensure(SceneNode))
			{
				Parser.ElementIdToSceneNode.Add(Node->element_id, SceneNode);
			}
		}
	}

	FString FUfbxScene::GetSkeletonRoot(ufbx_node* Node)
	{
		FString* SkeletonRootFound = SkeletonRootPerBone.Find(Node);
		return SkeletonRootFound ? *SkeletonRootFound : Parser.GetNodeUid(*SceneUtils::FindSkeletonRoot(Parser, Node));
	}
}

#undef LOCTEXT_NAMESPACE
