// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UInterchangeBaseNodeContainer;
class UInterchangeMeshLODContainerNode;
class UInterchangeSceneNode;

struct ufbx_node;typedef ufbx_node ufbx_node;
struct ufbx_mesh;typedef ufbx_mesh ufbx_mesh;
struct ufbx_blend_shape;typedef ufbx_blend_shape ufbx_blend_shape;

namespace UE::Interchange::Private
{
	class FUfbxParser;

	class FUfbxScene
	{
	public:
		explicit FUfbxScene(FUfbxParser& InParser);

		void InitHierarchy(UInterchangeBaseNodeContainer& NodeContainer);
		void ProcessNodes(UInterchangeBaseNodeContainer& NodeContainer);

		FString GetSkeletonRoot(ufbx_node* Node);
		bool IsInSkeleton(const ufbx_node& Node);

	private:
		void ProcessNode(UInterchangeBaseNodeContainer& NodeContainer, const ufbx_node& Node);
		FMatrix GetBindPoseMatrix(const ufbx_node& InNode);

		void ParseMeshLODs(UInterchangeMeshLODContainerNode& LODContainerNode,
			const ufbx_node& InNode,
			int32 LODIndex,
			const FMatrix& InvTransformMatrix /*Global Inverse Transform Matrix Of LODContainer*/);

		struct UnrealNodeInfo
		{
			const ufbx_node* Node;
			const FString Label;
			const FString UniqueId;
			const FString ParentUid;

			UnrealNodeInfo(const ufbx_node* InNode, const FString& InLabel, const FString& InUniqueId, const FString& InParentUid)
				: Node(InNode), Label(InLabel), UniqueId(InUniqueId), ParentUid(InParentUid)
			{
			}
		};

		void InitHierarchyRecursively(TArray<UnrealNodeInfo>& OutNodeInfos, const ufbx_node& Node, const FString& ParentUid);

		FUfbxParser& Parser;

		TSet<const ufbx_node*> CommonJointRootNodes;
		TMap<const ufbx_node*, FString> SkeletonRootPerBone;

		TMap<const ufbx_node*, FMatrix> BindPose;
		TMap<const ufbx_node*, TMap<FString, FMatrix>> JointToMeshIdToGlobalBindPoseReferenceMap;
		TMap<const ufbx_node*, TMap<FString, FMatrix>> JointToMeshIdToGlobalBindPoseJointMap;
	};

}
