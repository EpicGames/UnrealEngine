// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#if WITH_ENGINE
#include "Mesh/InterchangeMeshPayload.h"
#endif
#include "Misc/ScopeLock.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "InterchangeResultsContainer.h"
#include "UObject/StrongObjectPtr.h"

namespace UE
{
	namespace Interchange
	{
		struct FAnimationPayloadQuery;
		namespace Private
		{
			class IFbxParser;
			class FFbxParser;
			class FUfbxParser;
		}

		class FInterchangeFbxParser
		{
		public:
			INTERCHANGEFBXPARSER_API FInterchangeFbxParser();
			INTERCHANGEFBXPARSER_API ~FInterchangeFbxParser();

			INTERCHANGEFBXPARSER_API void ReleaseResources();

			INTERCHANGEFBXPARSER_API const TCHAR* GetName();
			
			INTERCHANGEFBXPARSER_API bool IsThreadSafe();

			INTERCHANGEFBXPARSER_API void Reset(bool bInUseUfbxParser = false);

			INTERCHANGEFBXPARSER_API void SetResultContainer(UInterchangeResultsContainer* Result);

			INTERCHANGEFBXPARSER_API void SetConvertSettings(const bool InbConvertScene, const bool InbForceFrontXAxis, const bool InbConvertSceneUnit, const bool InbKeepFbxNamespace, const bool InbConsiderClusterBeforePoseForMeshBindPose);
			/**
			 * Parse a file support by the fbx sdk. It just extract all the fbx node and create a FBaseNodeContainer and dump it in a json file inside the ResultFolder
			 * @param - Filename is the file that the fbx sdk will read (.fbx or .obj)
			 * @param - ResultFolder is the folder where we must put any result file
			 */
			INTERCHANGEFBXPARSER_API void LoadFbxFile(const FString& Filename, const FString& ResultFolder);

			/**
			 * Parse a file support by the fbx sdk. It just extract all the fbx node and create a FBaseNodeContainer and dump it in a json file inside the ResultFolder
			 * @param - Filename is the file that the fbx sdk will read (.fbx or .obj)
			 * @param - BaseNodecontainer is the container of the scene graph
			 */
			INTERCHANGEFBXPARSER_API void LoadFbxFile(const FString& Filename, UInterchangeBaseNodeContainer& BaseNodecontainer);

			/**
			 * Extract payload data from the fbx, the key tell the translator what payload the client ask
			 * @param - PayloadKey is the key that describe the payload data to extract from the fbx file
			 * @param - ResultFolder is the folder where we must put any result file
			 */
			INTERCHANGEFBXPARSER_API void FetchPayload(const FString& PayloadKey, const FString& ResultFolder);

			/**
			 * Extract mesh payload data from the fbx, the key tell the translator what payload the client ask
			 * @param - PayloadKey is the key that describe the payload data to extract from the fbx file
			 * @param - MeshGlobalTransform is the transform we want to apply to the mesh vertex
			 * @param - ResultFolder is the folder where we must put any result file
			 * @return - Return the 'ResultPayloads' key unique id. We cannot use only the payload key because the mesh global transform can be different.
			 */
			INTERCHANGEFBXPARSER_API FString FetchMeshPayload(const FString& PayloadKey, const FTransform& MeshGlobalTransform, const FString& ResultFolder);

#if WITH_ENGINE
			/**
			 * Extract mesh payload data from the fbx, the key tell the translator what payload the client ask
			 * @param - PayloadKey is the key that describe the payload data to extract from the fbx file
			 * @param - MeshGlobalTransform is the transform we want to apply to the mesh vertex
			 * @param - OutMeshPayloadData structure receiving the data
			 */
			INTERCHANGEFBXPARSER_API bool FetchMeshPayload(const FString& PayloadKey, const FTransform& MeshGlobalTransform, FMeshPayloadData& OutMeshPayloadData);

			/**
			 * Retrieve image file contents for embedded textures
			 * @param PayloadKey 
			 * @param OutTexturePayloadData 
			 * @return 
			 */
			INTERCHANGEFBXPARSER_API bool FetchTexturePayload(const FString& PayloadKey, TOptional<TArray64<uint8>>& OutTexturePayloadData);
#endif

			/**
			 * Extract bake transform animation payload data from the fbx
			 */
			INTERCHANGEFBXPARSER_API void FetchAnimationBakeTransformPayloads(const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries, const FString& ResultFolder);
			INTERCHANGEFBXPARSER_API TMap<FString, FString> FetchAnimationBakeTransformPayloads(const FString& PayloadQueriesJsonString, const FString& ResultFolder);

			FString GetResultFilepath() const { return ResultFilepath; }
			FString GetResultPayloadFilepath(const FString& PayloadKey) const
			{
				FScopeLock Lock(&ResultPayloadsCriticalSection);
				if (const FString* PayloadPtr = ResultPayloads.Find(PayloadKey))
				{
					return *PayloadPtr;
				}
				return FString();
			}

			/**
			 * Transform the results container into an array of Json strings
			 */
			INTERCHANGEFBXPARSER_API TArray<FString> GetJsonLoadMessages() const;

			template <typename T>
			T* AddMessage()
			{
				if (UInterchangeResultsContainer* ResultContainer = GetResultContainer())
				{
					return ResultContainer->Add<T>();
				}
				return nullptr;
			}

		private:
			INTERCHANGEFBXPARSER_API UInterchangeResultsContainer* GetResultContainer() const;

			Private::IFbxParser* GetFbxParserPtr() const;
			Private::IFbxParser& GetFbxParser();

			TObjectPtr<UInterchangeResultsContainer> InternalResultsContainer = nullptr;
			TStrongObjectPtr<UInterchangeResultsContainer> ResultsContainer = nullptr;
			FString SourceFilename;
			FString ResultFilepath;
			mutable FCriticalSection ResultPayloadsCriticalSection;
			TMap<FString, FString> ResultPayloads;
			bool bUseUfbxParser = false;
			TUniquePtr<UE::Interchange::Private::IFbxParser> FbxParserPrivate;
			TAtomic<int64> UniqueIdCounter = 0;
		};
	} // ns Interchange
}//ns UE
