// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Modules/ModuleInterface.h"

#define INTERCHANGEFBXPARSER_MODULE_NAME TEXT("InterchangeFbxParser")

/**
 * This module allows out-of-process Interchange translators in case a third-party SDK is not thread-safe.
 */
class FInterchangeFbxParserModule : public IModuleInterface
{
public:
	static FInterchangeFbxParserModule& Get();
	static bool IsAvailable();

private:
	virtual void StartupModule() override;
};
