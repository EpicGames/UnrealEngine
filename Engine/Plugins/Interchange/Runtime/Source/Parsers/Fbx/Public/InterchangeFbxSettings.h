// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InterchangeAnimationDefinitions.h"
#include "InterchangeAnimationTrackSetNode.h"


#include "Engine/DeveloperSettings.h"

#include "InterchangeFbxSettings.generated.h"

#define UE_API INTERCHANGEFBXPARSER_API

UCLASS(MinimalAPI, config = Interchange, meta = (DisplayName = "FBX Settings"))
class UInterchangeFbxSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	UE_API UInterchangeFbxSettings();

	/** Search for a predefined property track, if the property has been found it returns it, otherwise we search for a custom property track*/
	UE_API EInterchangePropertyTracks GetPropertyTrack(const FString& PropertyName) const;

	UPROPERTY(EditAnywhere, config, Category = "FBX | Property Tracks")
	TMap<FString, EInterchangePropertyTracks > CustomPropertyTracks;

private:
	TMap<FString, EInterchangePropertyTracks > PredefinedPropertyTracks;
};

#undef UE_API
