// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;

namespace UnrealBuildTool.Rules
{
	public class GLTFCore : ModuleRules
	{
		public GLTFCore(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddAll(
				"Core",
				"CoreUObject"
			);

			PrivateDependencyModuleNames.AddAll(
				"Draco",
				"Engine",
				"MeshDescription",
				"StaticMeshDescription",
				"Json",
				"RenderCore",
				"HTTP",
				"InterchangeCore"
			);
		}
	}
}
