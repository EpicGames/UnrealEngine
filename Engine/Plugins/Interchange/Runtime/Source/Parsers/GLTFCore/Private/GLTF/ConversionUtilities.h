// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "GLTFAnimation.h"
#include "GLTFMaterial.h"
#include "GLTFMesh.h"
#include "GLTFTexture.h"
#include "Math/Quat.h"

namespace GLTF
{
	// math conversion

	inline FVector ConvertVec3(const FVector& Vec)
	{
		// glTF uses a right-handed coordinate system, with Y up.
		// Unreal uses a left-handed coordinate system, with Z up.
		return {Vec.X, Vec.Z, Vec.Y};
	}

	inline FVector4 ConvertTangent(const FVector4& Tangent)
	{
		// glTF stores tangent as Vec4, with W component indicating handedness of tangent basis.

		FVector4 Result(ConvertVec3(FVector(Tangent.X, Tangent.Y, Tangent.Z)), Tangent.W);
		return Result;
	}

	template<typename T>
	UE::Math::TQuat<T> ConvertQuat(const UE::Math::TQuat<T>& Quat)
	{
		// glTF uses a right-handed coordinate system, with Y up.
		// Unreal uses a left-handed coordinate system, with Z up.
		// Quat = (qX, qY, qZ, qW) = (sin(angle/2) * aX, sin(angle/2) * aY, sin(angle/2) * aZ, cons(angle/2))
		// where (aX, aY, aZ) - rotation axis, angle - rotation angle
		// Y swapped with Z between these coordinate systems
		// also, as handedness is changed rotation is inversed - hence negation
		// therefore QuatUE = (-qX, -qZ, -qY, qw)

		UE::Math::TQuat<T> Result(-Quat.X, -Quat.Z, -Quat.Y, Quat.W);
		// Not checking if quaternion is normalized
		// e.g. some sources use non-unit Quats for rotation tangents
		return Result;
	}

	inline FMatrix ConvertMat(const FMatrix& Matrix)
	{
		// glTF stores matrix elements in column major order
		// Unreal's FMatrix is row major

		FMatrix Result;
		for (int32 Row = 0; Row < 4; ++Row)
		{
			for (int32 Col = 0; Col < 4; ++Col)
			{
				Result.M[Row][Col] = Matrix.M[Col][Row];
			}
		}
		return Result;
	}

	// enum conversion

	inline FMaterial::EAlphaMode AlphaModeFromString(const FString& Str)
	{
		// case sensitive comparison
		if (FPlatformString::Strcmp(*Str, TEXT("OPAQUE")) == 0)
		{
			return FMaterial::EAlphaMode::Opaque;
		}
		if (FPlatformString::Strcmp(*Str, TEXT("MASK")) == 0)
		{
			return FMaterial::EAlphaMode::Mask;
		}
		if (FPlatformString::Strcmp(*Str, TEXT("BLEND")) == 0)
		{
			return FMaterial::EAlphaMode::Blend;
		}
		ensure(false);
		return FMaterial::EAlphaMode::Opaque;
	}

	inline FAnimation::EPath AnimationPathFromString(const FString& Str)
	{
		// case sensitive comparison
		if (FPlatformString::Strcmp(*Str, TEXT("rotation")) == 0)
		{
			return FAnimation::EPath::Rotation;
		}
		if (FPlatformString::Strcmp(*Str, TEXT("scale")) == 0)
		{
			return FAnimation::EPath::Scale;
		}
		if (FPlatformString::Strcmp(*Str, TEXT("translation")) == 0)
		{
			return FAnimation::EPath::Translation;
		}
		if (FPlatformString::Strcmp(*Str, TEXT("weights")) == 0)
		{
			return FAnimation::EPath::Weights;
		}
		ensure(false);
		return FAnimation::EPath::Rotation;
	}

	inline FPrimitive::EMode PrimitiveModeFromNumber(uint32 Num)
	{
		static const TArray<FPrimitive::EMode> SafetyCheck({
		    FPrimitive::EMode::Points,
		    FPrimitive::EMode::LineLoop,
		    FPrimitive::EMode::Lines,
		    FPrimitive::EMode::LineStrip,
		    FPrimitive::EMode::Triangles,
		    FPrimitive::EMode::TriangleStrip,
		    FPrimitive::EMode::TriangleFan,
		});

		if (!ensure(SafetyCheck.Find(static_cast<FPrimitive::EMode>(Num)) != INDEX_NONE))
		{
			return FPrimitive::EMode::Unknown;
		}

		return static_cast<FPrimitive::EMode>(Num);
	}

	inline FImage::EFormat ImageFormatFromFilename(const FString& Filename)
	{
		// attempt to guess from filename extension
		if (Filename.EndsWith(TEXT(".png")))
		{
			return FImage::EFormat::PNG;
		}
		else if (Filename.EndsWith(TEXT(".jpg")) || Filename.EndsWith(TEXT(".jpeg")))
		{
			return FImage::EFormat::JPEG;
		}

		return FImage::EFormat::Unknown;
	}

	inline FImage::EFormat ImageFormatFromMimeType(const FString& Str)
	{
		// case sensitive comparison
		if (FPlatformString::Strcmp(*Str, TEXT("image/jpeg")) == 0 || FPlatformString::Strcmp(*Str, TEXT("image/jpg")) == 0)
		{
			return FImage::EFormat::JPEG;
		}
		if (FPlatformString::Strcmp(*Str, TEXT("image/png")) == 0)
		{
			return FImage::EFormat::PNG;
		}

		return FImage::EFormat::Unknown;
	}

	inline FSampler::EFilter FilterFromNumber(uint16 Num)
	{
		static const TArray<FSampler::EFilter> SafetyCheck({FSampler::EFilter::Nearest, FSampler::EFilter::Linear,
		                                                    FSampler::EFilter::NearestMipmapNearest, FSampler::EFilter::LinearMipmapNearest,
		                                                    FSampler::EFilter::NearestMipmapLinear, FSampler::EFilter::LinearMipmapLinear});
		if (!ensure(SafetyCheck.Find(static_cast<FSampler::EFilter>(Num)) != INDEX_NONE))
		{
			return FSampler::EFilter::Linear;
		}

		return static_cast<FSampler::EFilter>(Num);
	}

	inline FSampler::EWrap WrapModeFromNumber(uint16 Num)
	{
		static const TArray<FSampler::EWrap> SafetyCheck({FSampler::EWrap::ClampToEdge, FSampler::EWrap::MirroredRepeat, FSampler::EWrap::Repeat});
		if (!ensure(SafetyCheck.Find(static_cast<FSampler::EWrap>(Num)) != INDEX_NONE))
		{
			return FSampler::EWrap::Repeat;
		}

		return static_cast<FSampler::EWrap>(Num);
	}

	inline FAccessor::EType AccessorTypeFromString(const FString& Str)
	{
		// case sensitive comparison
		if (FPlatformString::Strcmp(*Str, TEXT("SCALAR")) == 0)
		{
			return FAccessor::EType::Scalar;
		}
		if (FPlatformString::Strcmp(*Str, TEXT("VEC2")) == 0)
		{
			return FAccessor::EType::Vec2;
		}
		if (FPlatformString::Strcmp(*Str, TEXT("VEC3")) == 0)
		{
			return FAccessor::EType::Vec3;
		}
		if (FPlatformString::Strcmp(*Str, TEXT("VEC4")) == 0)
		{
			return FAccessor::EType::Vec4;
		}
		if (FPlatformString::Strcmp(*Str, TEXT("MAT2")) == 0)
		{
			return FAccessor::EType::Mat2;
		}
		if (FPlatformString::Strcmp(*Str, TEXT("MAT3")) == 0)
		{
			return FAccessor::EType::Mat3;
		}
		if (FPlatformString::Strcmp(*Str, TEXT("MAT4")) == 0)
		{
			return FAccessor::EType::Mat4;
		}
		return FAccessor::EType::Unknown;
	}

	inline FAccessor::EComponentType ComponentTypeFromNumber(uint16 Num)
	{
		switch (Num)
		{
			case 5120:
				return FAccessor::EComponentType::S8;
			case 5121:
				return FAccessor::EComponentType::U8;
			case 5122:
				return FAccessor::EComponentType::S16;
			case 5123:
				return FAccessor::EComponentType::U16;
			case 5125:
				return FAccessor::EComponentType::U32;
			case 5126:
				return FAccessor::EComponentType::F32;
			default:
				return FAccessor::EComponentType::None;
		}
	}

}  // namespace GLTF
