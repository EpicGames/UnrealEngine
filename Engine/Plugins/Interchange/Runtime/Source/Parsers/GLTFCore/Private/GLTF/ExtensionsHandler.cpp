// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensionsHandler.h"

#include "GLTF/JsonUtilities.h"
#include "GLTFAsset.h"
#include "GLTFNode.h"
#include "MaterialUtilities.h"
#include "GenericPlatform/GenericPlatformHttp.h"
#include "Misc/Paths.h"

#if USE_DRACO_LIBRARY
#include "draco/mesh/mesh.h"
#include "draco/core/decoder_buffer.h"
#include "draco/compression/decode.h"
#endif


#define LOCTEXT_NAMESPACE "InterchangeGLTFExtensionHandler"

namespace GLTF
{
#if USE_DRACO_LIBRARY
	namespace DracoHelpers
	{
		template <typename T>
		void AcquireIndicesFromDracoMesh(draco::Mesh* Mesh, T* BinaryDataPtr)
		{
			for (size_t FaceIndex = 0; FaceIndex < Mesh->num_faces(); FaceIndex++)
			{
				const draco::Mesh::Face& Face = Mesh->face(draco::FaceIndex(FaceIndex));
				BinaryDataPtr[FaceIndex * 3 + 0] = Face[0].value();
				BinaryDataPtr[FaceIndex * 3 + 1] = Face[1].value();
				BinaryDataPtr[FaceIndex * 3 + 2] = Face[2].value();
			}
		}

		bool AcquireIndicesFromDracoMesh(draco::Mesh* Mesh, const FAccessor::EComponentType ComponentType, uint8* BinaryDataPtr)
		{
			switch (ComponentType)
			{
			case FAccessor::EComponentType::U8:
				AcquireIndicesFromDracoMesh<uint8>(Mesh, BinaryDataPtr);
				break;
			case FAccessor::EComponentType::U16:
				AcquireIndicesFromDracoMesh<uint16>(Mesh, reinterpret_cast<uint16*>(BinaryDataPtr));
				break;
			case FAccessor::EComponentType::U32:
				AcquireIndicesFromDracoMesh<uint32>(Mesh, reinterpret_cast<uint32*>(BinaryDataPtr));
				break;
			case FAccessor::EComponentType::S8:
			case FAccessor::EComponentType::S16:
			case FAccessor::EComponentType::F32:
			default:
				return false;
			}

			return true;
		}

		template <typename T>
		bool AcquireDataFromDracoAttribute(
			const draco::PointAttribute* Attribute,
			uint32 VertexCount,
			uint32 Stride,
			uint32 NumberOfComponents,
			uint8* BinaryDataPtr)
		{
			size_t ByteOffset = 0;
			TArray<T> Values;
			Values.Reserve(NumberOfComponents);
			Values.SetNumUninitialized(NumberOfComponents);

			for (size_t VertexIndex = 0; VertexIndex < VertexCount; VertexIndex++)
			{
				const draco::AttributeValueIndex MappedIndex = Attribute->mapped_index(draco::PointIndex(VertexIndex));
				if (!Attribute->ConvertValue<T>(MappedIndex, NumberOfComponents, Values.GetData()))
				{
					return false;
				}

				std::memcpy(BinaryDataPtr + ByteOffset, Values.GetData(), Stride);
				ByteOffset += Stride;
			}

			return true;
		}

		bool AcquireDataFromDracoAttribute(
			const FAccessor::EComponentType ComponentType,
			const draco::PointAttribute* Attribute,
			uint32 VertexCount,
			uint32 Stride,
			uint32 NumberOfComponents,
			uint8* BinaryDataPtr)
		{
			switch (ComponentType)
			{
			case FAccessor::EComponentType::S8:
				return AcquireDataFromDracoAttribute<int8>(Attribute, VertexCount, Stride, NumberOfComponents, BinaryDataPtr);
			case FAccessor::EComponentType::U8:
				return AcquireDataFromDracoAttribute<uint8>(Attribute, VertexCount, Stride, NumberOfComponents, BinaryDataPtr);
			case FAccessor::EComponentType::S16:
				return AcquireDataFromDracoAttribute<int16>(Attribute, VertexCount, Stride, NumberOfComponents, BinaryDataPtr);
			case FAccessor::EComponentType::U16:
				return AcquireDataFromDracoAttribute<uint16>(Attribute, VertexCount, Stride, NumberOfComponents, BinaryDataPtr);
			case FAccessor::EComponentType::U32:
				return AcquireDataFromDracoAttribute<uint32>(Attribute, VertexCount, Stride, NumberOfComponents, BinaryDataPtr);
			case FAccessor::EComponentType::F32:
				return AcquireDataFromDracoAttribute<float>(Attribute, VertexCount, Stride, NumberOfComponents, BinaryDataPtr);
			default:
				return false;
			}
		}
	}
#endif

	namespace
	{
		static const TArray<FString> MaterialsExtensions = { GLTF::ToString(GLTF::EExtension::KHR_MaterialsVariants) };

		TSharedPtr<FJsonObject> GetExtensions(const FJsonObject& Object)
		{
			if (!Object.HasTypedField<EJson::Object>(TEXT("extensions")))
			{
				return nullptr;
			}

			return Object.GetObjectField(TEXT("extensions"));
		}
	}

	FExtensionsHandler::FExtensionsHandler(TArray<FLogMessage>& InMessages)
	    : Messages(InMessages)
	    , Asset(nullptr)
	{
	}

	void FExtensionsHandler::SetupLightExtensions(const FJsonObject& Object, const FString& InResourcesPath) const
	{
		static const TArray<EExtension> Extensions = {
			EExtension::KHR_LightsPunctual,
			EExtension::KHR_Lights,
			EExtension::EXT_LightsIES
		};
		TArray<FString> ExtensionsStringified;
		for (size_t ExtensionIndex = 0; ExtensionIndex < Extensions.Num(); ExtensionIndex++)
		{
			ExtensionsStringified.Add(GLTF::ToString(Extensions[ExtensionIndex]));
		}

		const FJsonObject& ExtensionsObj = *Object.GetObjectField(TEXT("extensions"));
		for (int32 Index = 0; Index < Extensions.Num(); ++Index)
		{
			const FString ExtensionName = ExtensionsStringified[Index];
			if (!ExtensionsObj.HasTypedField<EJson::Object>(ExtensionName))
				continue;

			const FJsonObject& ExtObj = *ExtensionsObj.GetObjectField(ExtensionName);

			const EExtension Extension = Extensions[Index];
			
			switch (Extension)
			{
				case EExtension::KHR_Lights:
				case EExtension::KHR_LightsPunctual:
					{
						const FJsonObject& LightsObj = ExtObj;

						Asset->ProcessedExtensions.Add(EExtension::KHR_LightsPunctual);

						uint32 LightCount = ArraySize(LightsObj, TEXT("lights"));
						if (LightCount > 0)
						{
							Asset->Lights.Reserve(LightCount);
							for (const TSharedPtr<FJsonValue>& Value : LightsObj.GetArrayField(TEXT("lights")))
							{
								SetupLightPunctual(*Value->AsObject());
							}
						}
					}
					break;
				case EExtension::EXT_LightsIES:
					{
						const FJsonObject& LightsIESObj = ExtObj;

						Asset->ProcessedExtensions.Add(EExtension::EXT_LightsIES);

						uint32 LightIESCount = ArraySize(LightsIESObj, TEXT("lights"));
						if (LightIESCount > 0)
						{
							Asset->LightsIES.Reserve(LightIESCount);
							
							for (const TSharedPtr<FJsonValue>& Value : LightsIESObj.GetArrayField(TEXT("lights")))
							{
								const FJsonObject& ObjectLightIES = *Value->AsObject();
								
								FLightIES& LightIES = Asset->LightsIES.Emplace_GetRef();
								LightIES.Index = Asset->LightsIES.Num() - 1;
								
								LightIES.URI = GetString(ObjectLightIES, TEXT("uri"));
								if (LightIES.URI.Len() > 0)
								{
									LightIES.URI = FGenericPlatformHttp::UrlDecode(LightIES.URI);
									LightIES.FilePath = InResourcesPath / LightIES.URI;
								}

								LightIES.BufferViewIndex = GetIndex(ObjectLightIES, TEXT("bufferView"));
								if (Asset->BufferViews.IsValidIndex(LightIES.BufferViewIndex))
								{
									const FBufferView& BufferView = Asset->BufferViews[LightIES.BufferViewIndex];
									LightIES.DataByteLength = BufferView.ByteLength;
									LightIES.Data = static_cast<const uint8*>(BufferView.DataAt(0));
								}

								LightIES.MimeType = GetString(ObjectLightIES, TEXT("mimeType"));
								LightIES.Name = GetString(ObjectLightIES, TEXT("name"));

								if (LightIES.Name.Len() == 0 && LightIES.URI.Len() > 0)
								{
									LightIES.Name = FPaths::GetBaseFilename(LightIES.URI);
								}
							}
						}
					}
					break;
				default:
					break;
			}
		}

		CheckExtensions(Object, ExtensionsStringified);
	}

	void FExtensionsHandler::SetupAssetExtensions(const FJsonObject& Object, const FString& InResourcesPath) const
	{
		const TSharedPtr<FJsonObject>& ExtensionsObj = GetExtensions(Object);

		if (!ExtensionsObj.IsValid())
		{
			return;
		}

		// lights
		SetupLightExtensions(Object, InResourcesPath);

		// variants

		if (ExtensionsObj->HasTypedField<EJson::Object>(GLTF::ToString(GLTF::EExtension::KHR_MaterialsVariants)))
		{
			Asset->ProcessedExtensions.Add(EExtension::KHR_MaterialsVariants);

			TSharedPtr<FJsonObject> VariantsObj = ExtensionsObj->GetObjectField(GLTF::ToString(GLTF::EExtension::KHR_MaterialsVariants));
			uint32 VariantsCount = ArraySize(*VariantsObj, TEXT("variants"));
			if (VariantsCount > 0)
			{
				Asset->Variants.Reserve(VariantsCount);
				for (const TSharedPtr<FJsonValue>& Value : VariantsObj->GetArrayField(TEXT("variants")))
				{
					const TSharedPtr<FJsonObject>& NameObj = Value->AsObject();
					const FString Name = NameObj->GetStringField(TEXT("name"));
					Asset->Variants.Add(Name);
				}
			}
		}
		
		TArray<FString> SupportedExtensions;
		SupportedExtensions.Append(MaterialsExtensions);
		SupportedExtensions.Add(GLTF::ToString(GLTF::EExtension::KHR_Lights));
		SupportedExtensions.Add(GLTF::ToString(GLTF::EExtension::KHR_LightsPunctual));
		SupportedExtensions.Add(GLTF::ToString(GLTF::EExtension::EXT_LightsIES));

		CheckExtensions(Object, SupportedExtensions);
	}

	void FExtensionsHandler::SetupMaterialExtensions(const FJsonObject& Object, FMaterial& Material) const
	{
		if (!Object.HasTypedField<EJson::Object>(TEXT("extensions")))
		{
			return;
		}

		static const TArray<EExtension> Extensions = { 
			EExtension::KHR_MaterialsPbrSpecularGlossiness,
			EExtension::KHR_MaterialsUnlit,
			EExtension::KHR_MaterialsClearCoat,
			EExtension::KHR_MaterialsTransmission,
			EExtension::KHR_MaterialsSheen,
			EExtension::KHR_MaterialsIOR,
			EExtension::KHR_MaterialsSpecular,
			EExtension::KHR_MaterialsEmissiveStrength,
			EExtension::KHR_MaterialsIridescence,
			EExtension::KHR_MaterialsAnisotropy,
			EExtension::KHR_MaterialsVolume,
			EExtension::MSFT_PackingOcclusionRoughnessMetallic,
			EExtension::MSFT_PackingNormalRoughnessMetallic 
		};
		TArray<FString> ExtensionsStringified;
		for (size_t ExtensionIndex = 0; ExtensionIndex < Extensions.Num(); ExtensionIndex++)
		{
			ExtensionsStringified.Add(GLTF::ToString(Extensions[ExtensionIndex]));
		}

		const FJsonObject& ExtensionsObj = *Object.GetObjectField(TEXT("extensions"));
		for (int32 Index = 0; Index < Extensions.Num(); ++Index)
		{
			const FString ExtensionName = ExtensionsStringified[Index];
			if (!ExtensionsObj.HasTypedField<EJson::Object>(ExtensionName))
				continue;

			const FJsonObject& ExtObj = *ExtensionsObj.GetObjectField(ExtensionName);

			const EExtension Extension = Extensions[Index];
			switch (Extension)
			{
				case EExtension::KHR_MaterialsPbrSpecularGlossiness:
				{
					const FJsonObject& PBR = ExtObj;
					GLTF::SetTextureMap(PBR, TEXT("diffuseTexture"), nullptr, Asset->Textures, Material.BaseColor, Messages);
					Material.BaseColorFactor = FVector4f(GetVec4(PBR, TEXT("diffuseFactor"), FVector4(1.0f, 1.0f, 1.0f, 1.0f)));

					GLTF::SetTextureMap(PBR, TEXT("specularGlossinessTexture"), nullptr, Asset->Textures, Material.SpecularGlossiness.Map, Messages);
					Material.SpecularGlossiness.SpecularFactor   = GetVec3(PBR, TEXT("specularFactor"), FVector(1.0f));
					Material.SpecularGlossiness.GlossinessFactor = GetScalar(PBR, TEXT("glossinessFactor"), 1.0f);

					Material.ShadingModel = FMaterial::EShadingModel::SpecularGlossiness;

					Asset->ProcessedExtensions.Add(EExtension::KHR_MaterialsPbrSpecularGlossiness);
				}
				break;
				case EExtension::KHR_MaterialsUnlit:
				{
					Material.bIsUnlitShadingModel = true;
					Asset->ProcessedExtensions.Add(EExtension::KHR_MaterialsUnlit);
				}
				break;
				case EExtension::KHR_MaterialsClearCoat:
				{
					const FJsonObject& ClearCoat = ExtObj;

					Material.bHasClearCoat = true;

					Material.ClearCoat.ClearCoatFactor = GetScalar(ClearCoat, TEXT("clearcoatFactor"), 0.0f);
					GLTF::SetTextureMap(ClearCoat, TEXT("clearcoatTexture"), nullptr, Asset->Textures, Material.ClearCoat.ClearCoatMap, Messages);

					Material.ClearCoat.Roughness = GetScalar(ClearCoat, TEXT("clearcoatRoughnessFactor"), 0.0f);
					GLTF::SetTextureMap(ClearCoat, TEXT("clearcoatRoughnessTexture"), nullptr, Asset->Textures, Material.ClearCoat.RoughnessMap, Messages);

					Material.ClearCoat.NormalMapUVScale = GLTF::SetTextureMap(ClearCoat, TEXT("clearcoatNormalTexture"), TEXT("scale"), Asset->Textures, Material.ClearCoat.NormalMap, Messages);

					Asset->ProcessedExtensions.Add(EExtension::KHR_MaterialsClearCoat);
				}
				break;
				case EExtension::KHR_MaterialsTransmission:
				{
					const FJsonObject& Transm = ExtObj;

					Material.bHasTransmission = true;

					Material.Transmission.TransmissionFactor = GetScalar(Transm, TEXT("transmissionFactor"), 0.0f);
					GLTF::SetTextureMap(Transm, TEXT("transmissionTexture"), nullptr, Asset->Textures, Material.Transmission.TransmissionMap, Messages);

					Asset->ProcessedExtensions.Add(EExtension::KHR_MaterialsTransmission);
				}
				break;
				case EExtension::KHR_MaterialsSheen:
				{
					const FJsonObject& Sheen = ExtObj;

					Material.bHasSheen = true;

					Material.Sheen.SheenColorFactor = GetVec3(Sheen, TEXT("sheenColorFactor"));
					GLTF::SetTextureMap(Sheen, TEXT("sheenColorTexture"), nullptr, Asset->Textures, Material.Sheen.SheenColorMap, Messages);

					Material.Sheen.SheenRoughnessFactor = GetScalar(Sheen, TEXT("sheenRoughnessFactor"));
					GLTF::SetTextureMap(Sheen, TEXT("sheenRoughnessTexture"), nullptr, Asset->Textures, Material.Sheen.SheenRoughnessMap, Messages);

					Asset->ProcessedExtensions.Add(EExtension::KHR_MaterialsSheen);
				}
				break;
				case EExtension::KHR_MaterialsIOR:
				{
					const FJsonObject& IOR = ExtObj;

					Material.bHasIOR = true;
					Material.IOR = GetScalar(IOR, TEXT("ior"), 1.0f);

					Asset->ProcessedExtensions.Add(EExtension::KHR_MaterialsIOR);
				}
				break;
				case EExtension::KHR_MaterialsSpecular:
				{
					const FJsonObject& Specular = ExtObj;

					Material.bHasSpecular = true;
					Material.Specular.SpecularFactor = GetScalar(Specular, TEXT("specularFactor"), 1.0f);
					Material.Specular.SpecularColorFactor = GetVec3(Specular, TEXT("specularColorFactor"), FVector::OneVector);
					GLTF::SetTextureMap(Specular, TEXT("specularTexture"), nullptr, Asset->Textures, Material.Specular.SpecularMap, Messages);
					GLTF::SetTextureMap(Specular, TEXT("specularColorTexture"), nullptr, Asset->Textures, Material.Specular.SpecularColorMap, Messages);

					Asset->ProcessedExtensions.Add(EExtension::KHR_MaterialsSpecular);
				}
				break;
				case EExtension::KHR_MaterialsEmissiveStrength:
				{
					const FJsonObject& Emissive = ExtObj;

					Material.bHasEmissiveStrength = true;
					Material.EmissiveStrength = GetScalar(Emissive, TEXT("emissiveStrength"), 1.0f);

					Asset->ProcessedExtensions.Add(EExtension::KHR_MaterialsEmissiveStrength);
				}
				break;
				case EExtension::MSFT_PackingOcclusionRoughnessMetallic:
				{
					const FJsonObject& Packing = ExtObj;
					if (GLTF::SetTextureMap(Packing, TEXT("occlusionRoughnessMetallicTexture"), nullptr, Asset->Textures, Material.Packing.Map, Messages))
					{
						Material.Packing.Flags = (int)GLTF::FMaterial::EPackingFlags::OcclusionRoughnessMetallic;
					}
					else if (GLTF::SetTextureMap(Packing, TEXT("roughnessMetallicOcclusionTexture"), nullptr, Asset->Textures, Material.Packing.Map, Messages))
					{
						Material.Packing.Flags = (int)GLTF::FMaterial::EPackingFlags::RoughnessMetallicOcclusion;
					}
					if (GLTF::SetTextureMap(Packing, TEXT("normalTexture"), nullptr, Asset->Textures, Material.Packing.NormalMap, Messages))
					{
						// can have extra packed two channel(RG) normal map
						Material.Packing.Flags |= (int)GLTF::FMaterial::EPackingFlags::NormalRG;
					}

					if (Material.Packing.Flags != (int)GLTF::FMaterial::EPackingFlags::None)
						Asset->ProcessedExtensions.Add(EExtension::MSFT_PackingOcclusionRoughnessMetallic);
				}
				break;
				case EExtension::MSFT_PackingNormalRoughnessMetallic:
				{
					const FJsonObject& Packing = ExtObj;
					if (GLTF::SetTextureMap(Packing, TEXT("normalRoughnessMetallicTexture"), nullptr, Asset->Textures, Material.Packing.Map, Messages))
					{
						Material.Packing.NormalMap = Material.Packing.Map;
						Material.Packing.Flags     = (int)GLTF::FMaterial::EPackingFlags::NormalRoughnessMetallic;
						Asset->ProcessedExtensions.Add(EExtension::MSFT_PackingNormalRoughnessMetallic);
					}
				}
				break;
				case EExtension::KHR_MaterialsIridescence:
				{
					const FJsonObject& Iridescence = ExtObj;

					Material.Iridescence.bHasIridescence = true;

					Material.Iridescence.Factor = GetScalar(Iridescence, TEXT("iridescenceFactor"), 0.0f);
					GLTF::SetTextureMap(Iridescence, TEXT("iridescenceTexture"), nullptr, Asset->Textures, Material.Iridescence.Texture, Messages);
					Material.Iridescence.IOR = GetScalar(Iridescence, TEXT("iridescenceIor"), 1.3f);

					Material.Iridescence.Thickness.Minimum = GetScalar(Iridescence, TEXT("iridescenceThicknessMinimum"), 100.0f);
					Material.Iridescence.Thickness.Maximum = GetScalar(Iridescence, TEXT("iridescenceThicknessMaximum"), 400.0f);
					GLTF::SetTextureMap(Iridescence, TEXT("iridescenceThicknessTexture"), nullptr, Asset->Textures, Material.Iridescence.Thickness.Texture, Messages);
					Asset->ProcessedExtensions.Add(EExtension::KHR_MaterialsIridescence);
				}
				break;
				case EExtension::KHR_MaterialsAnisotropy:
				{
					const FJsonObject& Anisotropy = ExtObj;

					Material.Anisotropy.bHasAnisotropy = true;

					Material.Anisotropy.Strength = GetScalar(Anisotropy, TEXT("anisotropyStrength"), 0.0f);
					Material.Anisotropy.Rotation = GetScalar(Anisotropy, TEXT("anisotropyRotation"), 0.0f);
					GLTF::SetTextureMap(Anisotropy, TEXT("anisotropyTexture"), nullptr, Asset->Textures, Material.Anisotropy.Texture, Messages);
					Asset->ProcessedExtensions.Add(EExtension::KHR_MaterialsAnisotropy);
				}
				break;
				case EExtension::KHR_MaterialsVolume:
				{
					const FJsonObject& Volume = ExtObj;
					Material.Volume.bHasVolume = true;

					Material.Volume.Thickness = GetScalar(Volume, TEXT("thicknessFactor"));
					Material.Volume.AttenuationDistance = GetScalar(Volume, TEXT("attenuationDistance"), 1.f);
					Material.Volume.AttenuationColor = GetVec3(Volume, TEXT("attenuationColor"), FVector::OneVector);
					GLTF::SetTextureMap(Volume, TEXT("thicknessTexture"), nullptr, Asset->Textures, Material.Volume.ThicknessTexture, Messages);
					Asset->ProcessedExtensions.Add(EExtension::KHR_MaterialsVolume);
				}
				break;
				default:
					if (!ensure(false))
					{
						Messages.Emplace(RuntimeWarningSeverity(), FText::Format(LOCTEXT("UnsupportedMaterialExtension", "Material.Extension not supported: {0}"), FText::FromString(ToString(Extension))));
					}
					break;
			}
		}

		CheckExtensions(Object, ExtensionsStringified);
	}

	void FExtensionsHandler::SetupBufferExtensions(const FJsonObject& Object, FBuffer& Buffer) const
	{
		static const TArray<FString> Extensions;
		CheckExtensions(Object, Extensions);
	}

	void FExtensionsHandler::SetupBufferViewExtensions(const FJsonObject& Object, FBufferView& BufferView) const
	{
		static const TArray<FString> Extensions;
		CheckExtensions(Object, Extensions);
	}

	void FExtensionsHandler::SetupAccessorExtensions(const FJsonObject& Object, FAccessor& Accessor) const
	{
		static const TArray<FString> Extensions;
		CheckExtensions(Object, Extensions);
	}

	void FExtensionsHandler::SetupPrimitiveExtensions(const FJsonObject& Object, FPrimitive& Primitive, uint32 PrimitiveIndex, const FString& MeshUniqueId) const
	{
		if (!Object.HasTypedField<EJson::Object>(TEXT("extensions")))
		{
			return;
		}

		static const TArray<EExtension> Extensions = {
			EExtension::KHR_MaterialsVariants,
			EExtension::KHR_DracoMeshCompression
		};
		TArray<FString> ExtensionsStringified;
		for (size_t ExtensionIndex = 0; ExtensionIndex < Extensions.Num(); ExtensionIndex++)
		{
			ExtensionsStringified.Add(GLTF::ToString(Extensions[ExtensionIndex]));
		}

		const FJsonObject& ExtensionsObj = *Object.GetObjectField(TEXT("extensions"));
		for (int32 Index = 0; Index < Extensions.Num(); ++Index)
		{
			const FString ExtensionName = ExtensionsStringified[Index];
			if (!ExtensionsObj.HasTypedField<EJson::Object>(ExtensionName))
				continue;

			const FJsonObject& ExtObj = *ExtensionsObj.GetObjectField(ExtensionName);

			const EExtension Extension = Extensions[Index];
			switch (Extension)
			{
				case EExtension::KHR_MaterialsVariants:
				{
					const TArray<TSharedPtr<FJsonValue>>& Mappings = ExtObj.GetArrayField(TEXT("mappings"));

					for (const TSharedPtr<FJsonValue>& Mapping : Mappings)
					{
						FVariantMapping& VariantMapping = Primitive.VariantMappings.Emplace_GetRef();

						const TSharedPtr<FJsonObject> MappingObj = Mapping->AsObject();
						VariantMapping.MaterialIndex = MappingObj->GetIntegerField(TEXT("material"));
						const TArray<TSharedPtr<FJsonValue>>& Variants = MappingObj->GetArrayField(TEXT("variants"));

						VariantMapping.VariantIndices.Reserve(Variants.Num());
						for (const TSharedPtr<FJsonValue>& Variant : Variants)
						{
							const int32 VariantIndex = static_cast<int32>(Variant->AsNumber());
							VariantMapping.VariantIndices.Add(VariantIndex);
						}
					}

					Asset->ProcessedExtensions.Add(EExtension::KHR_MaterialsVariants);
				}
				break;

#if USE_DRACO_LIBRARY
				case EExtension::KHR_DracoMeshCompression:
				{
					const int32 BufferViewIndex = GetIndex(ExtObj, TEXT("bufferView"));
					if (Asset->BufferViews.IsValidIndex(BufferViewIndex))
					{
						const FBufferView& BufferView = Asset->BufferViews[BufferViewIndex];

						const FJsonObject& AttributesObj = *ExtObj.GetObjectField(TEXT("attributes"));

						draco::Decoder* Decoder = new draco::Decoder();

						draco::DecoderBuffer DecoderBuffer;
						DecoderBuffer.Init((char*)BufferView.Buffer.Data + BufferView.ByteOffset, BufferView.ByteLength);

						const draco::EncodedGeometryType geom_type = draco::Decoder::GetEncodedGeometryType(&DecoderBuffer).value();

						//Point Cloud is not supported by glTF
						if (geom_type == draco::TRIANGULAR_MESH)
						{
							std::unique_ptr<draco::Mesh> Mesh = Decoder->DecodeMeshFromBuffer(&DecoderBuffer).value();

							//Acquire Indices:
							{
								uint32 AccessorIndex = Primitive.GetIndicesAccessorIndex();
								if (Asset->Accessors.IsValidIndex(AccessorIndex))
								{
									FAccessor& Accessor = Asset->CreateBuffersForAccessorIndex(AccessorIndex);

									//Acquire uncompressed Binary Data:
									if (!DracoHelpers::AcquireIndicesFromDracoMesh(Mesh.get(), Accessor.ComponentType, Accessor.BufferView.Buffer.Data))
									{
										Accessor.BufferView = FBufferView();
										Messages.Emplace(EMessageSeverity::Warning, FText::Format(LOCTEXT("DracoMeshIndexAcquisitionFailed", "Failed to acquire Indices from Draco Mesh, for PrimitiveIdx: {0}, in Mesh: {1}"), PrimitiveIndex, FText::FromString(MeshUniqueId)));
									}
								}
							}

							//Acquire Attributes:
							{
								auto ProcessDracoAttribute = [this, &Primitive, &Mesh, &AttributesObj, &PrimitiveIndex, &MeshUniqueId](EMeshAttributeType MeshAttributeType)
								{
									if (uint32 DracoAttributeId; GetIndex(AttributesObj, *ToString(MeshAttributeType), DracoAttributeId))
									{
										//In a draco compression description the Attributes hold the unique attribute Identifier for within the compressed data.
										//(Compared to the 'basic' Attributes which hold the indices of the actual Accessor from within the Accessors list.)

										const draco::PointAttribute* Attribute = Mesh->GetAttributeByUniqueId(DracoAttributeId);
										if (Attribute != nullptr)
										{
											uint32 AccessorIndex = Primitive.GetAttributeAccessorIndex(MeshAttributeType);
											if (Asset->Accessors.IsValidIndex(AccessorIndex))
											{
												FAccessor& Accessor = Asset->CreateBuffersForAccessorIndex(AccessorIndex);

												//Acquire uncompressed Binary Data:
												if (!DracoHelpers::AcquireDataFromDracoAttribute(Accessor.ComponentType, Attribute, Accessor.Count, Accessor.ByteStride, Accessor.NumberOfComponents, Accessor.BufferView.Buffer.Data))
												{
													//Clear out the BufferView so it is not used:
													Accessor.BufferView = FBufferView();
													Messages.Emplace(EMessageSeverity::Warning, FText::Format(LOCTEXT("DracoMeshAttributeAcquisitionFailed", "Failed to acquire {0} Attributes from Draco Mesh, for PrimitiveIdx: {1}, in Mesh: {2}"), FText::FromString(ToString(MeshAttributeType)), PrimitiveIndex, FText::FromString(MeshUniqueId)));
												}
											}
										}
									}
								};

								for (size_t AttirbuteType = 0; AttirbuteType < EMeshAttributeType::COUNT; AttirbuteType++)
								{
									ProcessDracoAttribute(EMeshAttributeType(AttirbuteType));
								}
							}
						}
					}
				}
				break;
#endif

				default:
					if (!ensure(false))
					{
						Messages.Emplace(RuntimeWarningSeverity(), FText::Format(LOCTEXT("UnsupportedPrimitiveExtension", "Primitive.Extension not supported: {0}"), FText::FromString(ToString(Extension))));
					}
				break;
			}
		}

		CheckExtensions(Object, ExtensionsStringified);
	}

	void FExtensionsHandler::SetupMeshExtensions(const FJsonObject& Object, FMesh& Mesh) const
	{
		static const TArray<FString> Extensions;
		CheckExtensions(Object, Extensions);
	}

	void FExtensionsHandler::SetupSceneExtensions(const FJsonObject& Object, FScene& Scene) const
	{
		static const TArray<FString> Extensions;
		CheckExtensions(Object, Extensions);
	}

	void FExtensionsHandler::SetupNodeExtensions(const FJsonObject& Object, FNode& Node) const
	{
		static const TArray<EExtension> Extensions = {
			EExtension::KHR_LightsPunctual,
			EExtension::KHR_Lights,
			EExtension::EXT_LightsIES
		};
		TArray<FString> ExtensionsStringified;
		for (size_t ExtensionIndex = 0; ExtensionIndex < Extensions.Num(); ExtensionIndex++)
		{
			ExtensionsStringified.Add(GLTF::ToString(Extensions[ExtensionIndex]));
		}

		if (Object.HasField(TEXT("extensions")))
		{
			const FJsonObject& ExtensionsObj = *Object.GetObjectField(TEXT("extensions"));
			for (int32 Index = 0; Index < Extensions.Num(); ++Index)
			{
				const FString ExtensionName = ExtensionsStringified[Index];
				if (!ExtensionsObj.HasTypedField<EJson::Object>(ExtensionName))
					continue;

				const FJsonObject& ExtObj = *ExtensionsObj.GetObjectField(ExtensionName);

				const EExtension Extension = Extensions[Index];

				switch (Extension)
				{
					case EExtension::KHR_Lights:
					case EExtension::KHR_LightsPunctual:
					{
						Node.LightIndex = GetIndex(ExtObj, TEXT("light"));
					}
					break;
					case EExtension::EXT_LightsIES:
					{
						const FJsonObject& IESNodeLight = ExtObj;

						Node.LightIES.Index = GetIndex(IESNodeLight, TEXT("light"));

						if (IESNodeLight.HasField(TEXT("multiplier")))
						{
							Node.LightIES.bHasIntensityMultiplier = true;
							Node.LightIES.IntensityMultipler = GetScalar(IESNodeLight, TEXT("multiplier"), 1.0f);
						}

						if (IESNodeLight.HasField(TEXT("color")))
						{
							Node.LightIES.bHasColor = true;
							Node.LightIES.Color = GetVec3(IESNodeLight, TEXT("color"), FVector(1.f));
						}

					}
					break;
					default:
						break;
				}
			}
		}

		CheckExtensions(Object, ExtensionsStringified);
	}

	void FExtensionsHandler::SetupCameraExtensions(const FJsonObject& Object, FCamera& Camera) const
	{
		static const TArray<FString> Extensions;
		CheckExtensions(Object, Extensions);
	}

	void FExtensionsHandler::SetupSkinExtensions(const FJsonObject& Object, FSkinInfo& Skin) const
	{
		static const TArray<FString> Extensions;
		CheckExtensions(Object, Extensions);
	}

	void FExtensionsHandler::SetupAnimationExtensions(const FJsonObject& Object, FAnimation& Animation) const
	{
		static const TArray<FString> Extensions;
		CheckExtensions(Object, Extensions);
	}

	void FExtensionsHandler::SetupImageExtensions(const FJsonObject& Object, FImage& Image) const
	{
		static const TArray<FString> Extensions;
		CheckExtensions(Object, Extensions);
	}

	void FExtensionsHandler::SetupSamplerExtensions(const FJsonObject& Object, FSampler& Sampler) const
	{
		static const TArray<FString> Extensions;
		CheckExtensions(Object, Extensions);
	}

	void FExtensionsHandler::SetupTextureExtensions(const FJsonObject& Object, FTexture& Texture) const
	{
		static const TArray<FString> Extensions;
		CheckExtensions(Object, Extensions);
	}

	void FExtensionsHandler::CheckExtensions(const FJsonObject& Object, const TArray<FString>& ExtensionsSupported) const
	{
		if (!Object.HasTypedField<EJson::Object>(TEXT("extensions")))
		{
			return;
		}

		const FJsonObject& ExtensionsObj = *Object.GetObjectField(TEXT("extensions"));
		for (const auto& StrValuePair : ExtensionsObj.Values)
		{
			if (ExtensionsSupported.Find(*StrValuePair.Key) == INDEX_NONE)
			{
				Messages.Emplace(RuntimeWarningSeverity(), FText::Format(LOCTEXT("UnsupportedExtension", "Extension is not supported: {0}"), FText::FromString(*StrValuePair.Key)));
			}
		}
	}

	void FExtensionsHandler::SetupLightPunctual(const FJsonObject& Object) const
	{
		const uint32 LightIndex = Asset->Lights.Num();
		const FNode* Found      = Asset->Nodes.FindByPredicate([LightIndex](const FNode& Node) { return LightIndex == Node.LightIndex; });

		FLight& Light = Asset->Lights.Emplace_GetRef(Found);

		Light.Name      = GetString(Object, TEXT("name"));
		Light.Color     = GetVec3(Object, TEXT("color"), FVector(1.f));
		Light.Intensity = GetScalar(Object, TEXT("intensity"), 1.f);
		Light.Range     = GetScalar(Object, TEXT("range"), Light.Range);

		const FString Type = GetString(Object, TEXT("type"));
		if (Type == TEXT("spot"))
		{
			Light.Type = FLight::EType::Spot;
			if (Object.HasTypedField<EJson::Object>(TEXT("spot")))
			{
				const TSharedPtr<FJsonObject>& SpotObj = Object.GetObjectField(TEXT("spot"));
				Light.Spot.InnerConeAngle              = GetScalar(*SpotObj, TEXT("innerConeAngle"), 0.f);
				Light.Spot.OuterConeAngle              = GetScalar(*SpotObj, TEXT("outerConeAngle"), Light.Spot.OuterConeAngle);
			}
		}
		else if (Type == TEXT("point"))
			Light.Type = FLight::EType::Point;
		else if (Type == TEXT("directional"))
			Light.Type = FLight::EType::Directional;
		else
			Messages.Emplace(RuntimeWarningSeverity(), FText::Format(LOCTEXT("UnspecifiedLightType", "Light has no type specified: {0}"), FText::FromString(Light.Name)));
	}

}  // namespace GLTF

#undef LOCTEXT_NAMESPACE