// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GLTFLogger.h"

class FJsonObject;

namespace GLTF
{
	struct FAsset;
	struct FMaterial;
	struct FBuffer;
	struct FBufferView;
	struct FAccessor;
	struct FPrimitive;
	struct FMesh;
	struct FScene;
	struct FNode;
	struct FSkinInfo;
	struct FCamera;
	struct FImage;
	struct FSampler;
	struct FTexture;
	struct FAnimation;

	class FExtensionsHandler
	{
	public:
		explicit FExtensionsHandler(TArray<FLogMessage>& InMessages);

		void SetAsset(GLTF::FAsset& InAsset);

		void SetupAssetExtensions(const FJsonObject& Object, const FString& InResourcesPath) const;
		void SetupBufferExtensions(const FJsonObject& Object, FBuffer& Buffer) const;
		void SetupBufferViewExtensions(const FJsonObject& Object, FBufferView& BufferView) const;
		void SetupAccessorExtensions(const FJsonObject& Object, FAccessor& Accessor) const;
		void SetupPrimitiveExtensions(const FJsonObject& Object, FPrimitive& Primitive, uint32 PrimitiveIndex, const FString& MeshUniqueId) const;
		void SetupMeshExtensions(const FJsonObject& Object, FMesh& Mesh) const;
		void SetupSceneExtensions(const FJsonObject& Object, FScene& Scene) const;
		void SetupNodeExtensions(const FJsonObject& Object, FNode& Node) const;
		void SetupCameraExtensions(const FJsonObject& Object, FCamera& Camera) const;
		void SetupSkinExtensions(const FJsonObject& Object, FSkinInfo& Skin) const;
		void SetupAnimationExtensions(const FJsonObject& Object, FAnimation& Animation) const;
		void SetupImageExtensions(const FJsonObject& Object, FImage& Image) const;
		void SetupSamplerExtensions(const FJsonObject& Object, FSampler& Sampler) const;
		void SetupTextureExtensions(const FJsonObject& Object, FTexture& Texture) const;
		void SetupMaterialExtensions(const FJsonObject& Object, FMaterial& Material) const;
		void SetupLightExtensions(const FJsonObject& Object, const FString& InResourcesPath) const;

		TArray<FLogMessage>& GetMessages()
		{
			return Messages;
		}

	private:
		void CheckExtensions(const FJsonObject& Object, const TArray<FString>& ExtensionsSupported) const;

		void SetupLightPunctual(const FJsonObject& Object) const;

	private:
		TArray<FLogMessage>& Messages;
		GLTF::FAsset*        Asset;
	};

	inline void FExtensionsHandler::SetAsset(GLTF::FAsset& InAsset)
	{
		Asset = &InAsset;
	}

}  // namespace GLTF
