// Copyright Epic Games, Inc. All Rights Reserved.

#include "GLTFAccessor.h"

#include "ConversionUtilities.h"

namespace GLTF
{
	FString ToString(const EMeshAttributeType& Type)
	{
		switch (Type)
		{
		case GLTF::POSITION:
			return TEXT("POSITION");
		case GLTF::NORMAL:
			return TEXT("NORMAL");
		case GLTF::TANGENT:
			return TEXT("TANGENT");
		case GLTF::TEXCOORD_0:
			return TEXT("TEXCOORD_0");
		case GLTF::TEXCOORD_1:
			return TEXT("TEXCOORD_1");
		case GLTF::TEXCOORD_2:
			return TEXT("TEXCOORD_2");
		case GLTF::TEXCOORD_3:
			return TEXT("TEXCOORD_3");
		case GLTF::TEXCOORD_4:
			return TEXT("TEXCOORD_4");
		case GLTF::TEXCOORD_5:
			return TEXT("TEXCOORD_5");
		case GLTF::TEXCOORD_6:
			return TEXT("TEXCOORD_6");
		case GLTF::TEXCOORD_7:
			return TEXT("TEXCOORD_7");
		case GLTF::COLOR_0:
			return TEXT("COLOR_0");
		case GLTF::JOINTS_0:
			return TEXT("JOINTS_0");
		case GLTF::WEIGHTS_0:
			return TEXT("WEIGHTS_0");
		default:
			break;
		}
		return TEXT("");
	}

	namespace
	{
		static const uint8 ComponentSize[] = { 0, 1, 1, 2, 2, 4, 4 };      // keep in sync with EComponentType
		static const uint8 ComponentsPerValue[] = { 0, 1, 2, 3, 4, 4, 9, 16 };  // keep in sync with EType

		uint32 GetNumberOfComponents(FAccessor::EType Type)
		{
			return ComponentsPerValue[(int)Type];
		}
		uint32 GetElementSize(FAccessor::EType Type, FAccessor::EComponentType ComponentType)
		{
			static_assert(
			    (int)FAccessor::EType::Unknown == 0 && ((int)FAccessor::EType::Count) == (sizeof(ComponentsPerValue) / sizeof(ComponentsPerValue[0])),
			    "EType doesn't match!");
			static_assert((int)FAccessor::EComponentType::None == 0 &&
			                  ((int)FAccessor::EComponentType::Count) == (sizeof(ComponentSize) / sizeof(ComponentSize[0])),
			              "EComponentType doesn't match!");

			return ComponentsPerValue[(int)Type] * ComponentSize[(int)ComponentType];
		}

		template <class ReturnType, uint32 Count>
		ReturnType GetNormalized(FAccessor::EComponentType ComponentType, const void* Pointer)
		{
			ReturnType Res;

			//U32 and F32 cannot be normalized.
			switch (ComponentType)
			{
			case GLTF::FAccessor::EComponentType::S8:
				{
					const int8* P = static_cast<const int8*>(Pointer);
					for (uint32 Index = 0; Index < Count; ++Index)
					{
						Res[Index] = FMath::Max((P[Index] / 127.0f), -1.0f);
					}
					break;
				}
			case GLTF::FAccessor::EComponentType::U8:
				{
					const uint8* P = static_cast<const uint8*>(Pointer);
					for (uint32 Index = 0; Index < Count; ++Index)
					{
						Res[Index] = P[Index] / 255.0f;
					}
					break;
				}
			case GLTF::FAccessor::EComponentType::S16:
				{
					const int16* P = static_cast<const int16*>(Pointer);
					for (uint32 Index = 0; Index < Count; ++Index)
					{
						Res[Index] = FMath::Max(P[Index] / 32767.0f, -1.0f);
					}
					break;
				}
			case GLTF::FAccessor::EComponentType::U16:
				{
					const uint16* P = static_cast<const uint16*>(Pointer);
					for (uint32 Index = 0; Index < Count; ++Index)
					{
						Res[Index] = P[Index] / 65535.0f;
					}
					break;
				}
			default:
				Res = ReturnType();
				ensure(false);
				break;
			}

			return Res;
		}

		float GetNormalizedFloat(FAccessor::EComponentType ComponentType, const void* Pointer)
		{
			switch (ComponentType)
			{
			case GLTF::FAccessor::EComponentType::S8:
				{
					const int8* P = static_cast<const int8*>(Pointer);
					return FMath::Max(P[0] / 127.0f, -1.0f);
				}
			case GLTF::FAccessor::EComponentType::U8:
				{
					const uint8* P = static_cast<const uint8*>(Pointer);
					return P[0] / 255.0f;
				}
			case GLTF::FAccessor::EComponentType::S16:
				{
					const int16* P = static_cast<const int16*>(Pointer);
					return FMath::Max(P[0] / 32767.0f, -1.0f);
				}
			case GLTF::FAccessor::EComponentType::U16:
				{
					const uint16* P = static_cast<const uint16*>(Pointer);
					return P[0] / 65535.0f;
				}
			default:
				ensure(false);
				break;
			}

			return 0.0f;
		}

		template <class ReturnType, uint32 Count>
		ReturnType GetNonNormalized(FAccessor::EComponentType ComponentType, const void* Pointer)
		{
			ReturnType Res;

			switch (ComponentType)
			{
			case GLTF::FAccessor::EComponentType::S8:
				{
					const int8* P = static_cast<const int8*>(Pointer);
					for (uint32 Index = 0; Index < Count; ++Index)
					{
						Res[Index] = P[Index];
					}
					break;
				}
			case GLTF::FAccessor::EComponentType::U8:
				{
					const uint8* P = static_cast<const uint8*>(Pointer);
					for (uint32 Index = 0; Index < Count; ++Index)
					{
						Res[Index] = P[Index];
					}
					break;
				}
			case GLTF::FAccessor::EComponentType::S16:
				{
					const int16* P = static_cast<const int16*>(Pointer);
					for (uint32 Index = 0; Index < Count; ++Index)
					{
						Res[Index] = P[Index];
					}
					break;
				}
			case GLTF::FAccessor::EComponentType::U16:
				{
					const uint16* P = static_cast<const uint16*>(Pointer);
					for (uint32 Index = 0; Index < Count; ++Index)
					{
						Res[Index] = P[Index];
					}
					break;
				}
			case GLTF::FAccessor::EComponentType::U32:
			{
				//This is unexpected, U32 is only supported on Indices,
				//and indices are acquired via GetUnsignedIntArray function.
				const uint32* P = static_cast<const uint32*>(Pointer);
				for (uint32 Index = 0; Index < Count; ++Index)
				{
					Res[Index] = P[Index];
				}
				break;
			}
			case FAccessor::EComponentType::F32:
				{
					const float* P = static_cast<const float*>(Pointer);
					for (uint32 Index = 0; Index < Count; ++Index)
					{
						Res[Index] = P[Index];
					}
					break;
				}
			default:
				Res = ReturnType();
				ensure(false);
				break;
			}

			return Res;
		}

		FMatrix GetMatrix(const void* Pointer)
		{
			// copy float mat4 directly from buffer
			const float* P = static_cast<const float*>(Pointer);

			FMatrix Matrix;
			for (int32 Row = 0; Row < 4; ++Row)
			{
				for (int32 Col = 0; Col < 4; ++Col)
				{
					// glTF stores matrix elements in column major order
					// Unreal's FMatrix is row major
					Matrix.M[Row][Col] = P[Col * 4 + Row];
				}
			}
			return Matrix;
		}

		template <typename DstT, typename SrcT>
		void Copy(DstT* Dst, const SrcT* Src, uint32 Count)
		{
			for (uint32 Index = 0; Index < Count; ++Index)
			{
				*(Dst++) = *(Src++);
			}
		}

		void CopyNormalizedFloat(float* Dst, const void* Src, FAccessor::EComponentType ComponentType, uint32 Count)
		{
			switch (ComponentType)
			{
			case GLTF::FAccessor::EComponentType::S8:
				{
					const int8* P = static_cast<const int8*>(Src);
					for (uint32 Index = 0; Index < Count; ++Index, P++)
					{
						float* VecDst = Dst++;
						*VecDst = FMath::Max((*P) / 127.0f, -1.0f);
					}
					break;
				}
			case GLTF::FAccessor::EComponentType::U8:
				{
					const uint8* P = static_cast<const uint8*>(Src);
					for (uint32 Index = 0; Index < Count; ++Index, P++)
					{
						float* VecDst = Dst++;
						*VecDst = (*P) / 255.0f;
					}
					break;
				}
			case GLTF::FAccessor::EComponentType::S16:
				{
					const int16* P = static_cast<const int16*>(Src);
					for (uint32 Index = 0; Index < Count; ++Index, P++)
					{
						float* VecDst = Dst++;
						*VecDst = FMath::Max((*P) / 32767.0f, -1.0f);
					}
					break;
				}
			case GLTF::FAccessor::EComponentType::U16:
				{
					const uint16* P = static_cast<const uint16*>(Src);
					for (uint32 Index = 0; Index < Count; ++Index, P++)
					{
						float* VecDst = Dst++;
						*VecDst = (*P) / 65535.0f;
					}
					break;
				}
			default:
				ensure(false);
				break;
			}
		}

		template<typename ItemType, uint32 ItemElementCount>
		void CopyWithoutConversion(const uint32 ByteStride, const FAccessor::EComponentType ComponentType, bool bNormalized, const uint32 Count, const FBufferView& BufferView, const uint64 ByteOffset, ItemType* Buffer)
		{
			if (!BufferView.IsValid())
			{
				//Copy can be called for when BufferView is not set, but Sparse is used.
				return;
			}

			// Stride equals item size => use simpler copy
			if ((ByteStride == 0) || ByteStride == sizeof(ItemType))
			{
				const void* Src = BufferView.DataAt((0 * ByteStride) + ByteOffset);
				if (ComponentType == FAccessor::EComponentType::F32)
				{
					memcpy(Buffer, Src, Count * sizeof(ItemType));
					return;
				}
			}

			if (bNormalized)
			{
				for (uint32 Index = 0; Index < Count; ++Index)
				{
					const void* Pointer = BufferView.DataAt((Index * ByteStride) + ByteOffset);
					Buffer[Index] = GetNormalized<ItemType, ItemElementCount>(ComponentType, Pointer);
				}
			}
			else
			{
				for (uint32 Index = 0; Index < Count; ++Index)
				{
					const void* Pointer = BufferView.DataAt((Index * ByteStride) + ByteOffset);
					Buffer[Index] = GetNonNormalized<ItemType, ItemElementCount>(ComponentType, Pointer);
				}
			}
		}

		// Copy data items that don't need conversion/expansion(i.e. Vec3 to Vec3, uint8 to uint8(not uint16)
		// but including normalized from fixed-point types
		template<typename ItemType, uint32 ItemElementCount>
		void CopyWithoutConversion(const FAccessor& Accessor, ItemType* Buffer)
		{
			if (!Accessor.IsValid() || !Accessor.BufferView.IsValid())
			{
				//Copy can be called for when BufferView is not set, but Sparse is used.
				return;
			}
			CopyWithoutConversion<ItemType, ItemElementCount>(Accessor.ByteStride, Accessor.ComponentType, Accessor.bNormalized, Accessor.Count, Accessor.BufferView, Accessor.ByteOffset, Buffer);
		}

		bool FillFloatArray(bool bNormalized, const FBufferView& BufferView, const uint64 ByteOffset, FAccessor::EComponentType ComponentType, const uint32 ByteStride, const uint32 Count, float* Buffer)
		{
			if (!bNormalized)
			{
				const uint8* Src = BufferView.DataAt((0 * ByteStride) + ByteOffset);
				switch (ComponentType)
				{
				case FAccessor::EComponentType::F32:
					memcpy(Buffer, Src, Count * sizeof(float));
					return true;
				default:
					break;
				}
			}
			else
			{
				// Stride equals item size => use simpler copy
				if ((ByteStride == 0) || ByteStride == sizeof(float))
				{
					const uint8* Src = BufferView.DataAt((0 * ByteStride) + ByteOffset);
					CopyNormalizedFloat(Buffer, Src, ComponentType, Count);
				}
				else
				{
					for (uint32 Index = 0; Index < Count; ++Index)
					{
						const uint8* Pointer = BufferView.DataAt((Index * ByteStride) + ByteOffset);
						Buffer[Index] = GetNormalizedFloat(ComponentType, Pointer);
					}
				}
				return true;
			}

			return false;
		}

		bool FillUnsignedIntArray(const FBufferView& BufferView, const uint64 ByteOffset, FAccessor::EComponentType ComponentType, const uint32 Count, uint32* Buffer)
		{
			const uint8* Src = BufferView.DataAt(ByteOffset);
			switch (ComponentType)
			{
				case FAccessor::EComponentType::U8:
					Copy(Buffer, reinterpret_cast<const uint8*>(Src), Count);
					return true;
				case FAccessor::EComponentType::U16:
					Copy(Buffer, reinterpret_cast<const uint16*>(Src), Count);
					return true;
				case FAccessor::EComponentType::U32:
					memcpy(Buffer, Src, Count * sizeof(uint32));
					return true;
				default:
					return false;
			}
		}

		TArray<uint32> AcquireUnsignedIntArray(const uint32 Count, const FBufferView& BufferView, const uint64 ByteOffset, FAccessor::EComponentType ComponentType)
		{
			TArray<uint32> ValueBuffer;

			ValueBuffer.SetNumUninitialized(Count, EAllowShrinking::No);

			FillUnsignedIntArray(BufferView, ByteOffset, ComponentType, Count, ValueBuffer.GetData());

			return ValueBuffer;
		}

		bool AcquireUnsignedInt(uint32 Index, const FBufferView& BufferView, const uint64 ByteOffset, const uint32 ByteStride, FAccessor::EComponentType ComponentType, uint32& Buffer)
		{
			Buffer = 0;

			const uint8* ValuePtr = BufferView.DataAt((Index * ByteStride) + ByteOffset);
			switch (ComponentType)
			{
				case FAccessor::EComponentType::U8:
					Buffer = *ValuePtr;
					return true;
				case FAccessor::EComponentType::U16:
					Buffer = *reinterpret_cast<const uint16*>(ValuePtr);
					return true;
				case FAccessor::EComponentType::U32:
					Buffer = *reinterpret_cast<const uint32*>(ValuePtr);
					return true;
				default:
					return false;
			}
		}

		bool AcquireUnsignedInt16x4(uint32 Index, const FBufferView& BufferView, const uint64 ByteOffset, const uint32 ByteStride, FAccessor::EComponentType ComponentType, uint16 Buffer[4])
		{
			Buffer[0] = Buffer[1] = Buffer[2] = Buffer[3] = 0;

			const uint8* ValuePtr = BufferView.DataAt((Index * ByteStride) + ByteOffset);
			switch (ComponentType)
			{
				case FAccessor::EComponentType::U8:
					for (int i = 0; i < 4; ++i)
					{
						Buffer[i] = ((uint8*)ValuePtr)[i];
					}
					return true;
				case FAccessor::EComponentType::U16:
					for (int i = 0; i < 4; ++i)
					{
						Buffer[i] = ((uint16*)ValuePtr)[i];
					}
					return true;
				default:
					return false;
			}
		}

		bool AcquireFloat(uint32 Index, const FBufferView& BufferView, const uint64 ByteOffset, const uint32 ByteStride, FAccessor::EComponentType ComponentType, float& Buffer)
		{
			Buffer = 0;

			const uint8* ValuePtr = BufferView.DataAt((Index * ByteStride) + ByteOffset);
			switch (ComponentType)
			{
				case FAccessor::EComponentType::F32:
					Buffer = *ValuePtr;
					return true;
				default:
					return false;
			}
		}

		bool AcquireVec2(uint32 Index, const FBufferView& BufferView, const uint64 ByteOffset, const uint32 ByteStride, FAccessor::EComponentType ComponentType, bool bNormalized, FVector2D& Buffer)
		{
			Buffer = FVector2D();

			const uint8* Pointer = BufferView.DataAt((Index * ByteStride) + ByteOffset);
			if (ComponentType == FAccessor::EComponentType::F32)
			{
				// copy float vec2 directly from buffer
				Buffer = FVector2D(*reinterpret_cast<const FVector2f*>(Pointer));
				return true;
			}
			if (bNormalized)
			{
				Buffer = FVector2D(GetNormalized<FVector2f, 2>(ComponentType, Pointer));
				return true;
			}

			return false;
		}

		bool AcquireVec3(uint32 Index, const FBufferView& BufferView, const uint64 ByteOffset, const uint32 ByteStride, FAccessor::EComponentType ComponentType, bool bNormalized, FVector& Buffer)
		{
			Buffer = FVector();

			const uint8* Pointer = BufferView.DataAt((Index * ByteStride) + ByteOffset);
			if (ComponentType == FAccessor::EComponentType::F32)
			{
				// copy float vec3 directly from buffer
				Buffer = FVector(*reinterpret_cast<const FVector3f*>(Pointer));
				return true;
			}
			if (bNormalized)
			{
				Buffer = FVector(GetNormalized<FVector3f, 3>(ComponentType, Pointer));
				return true;
			}

			return false;
		}

		bool AcquireVec4(uint32 Index, const FBufferView& BufferView, const uint64 ByteOffset, const uint32 ByteStride, FAccessor::EComponentType ComponentType, bool bNormalized, FVector4& Buffer)
		{
			Buffer = FVector4();

			const uint8* Pointer = BufferView.DataAt((Index * ByteStride) + ByteOffset);
			if (ComponentType == FAccessor::EComponentType::F32)
			{
				// copy float vec4 directly from buffer
				Buffer = FVector4(*reinterpret_cast<const FVector4f*>(Pointer));
				return true;
			}
			if (bNormalized)
			{
				Buffer = FVector4(GetNormalized<FVector4f, 4>(ComponentType, Pointer));
				return true;
			}

			return false;
		}

		bool AcquireMat4(uint32 Index, const FBufferView& BufferView, const uint64 ByteOffset, const uint32 ByteStride, FAccessor::EComponentType ComponentType, FMatrix& Buffer)
		{
			Buffer = FMatrix();

			if (ComponentType == FAccessor::EComponentType::F32)
			{
				const uint8* Pointer = BufferView.DataAt((Index * ByteStride) + ByteOffset);
				Buffer = GetMatrix(Pointer);
				return true;
			}

			return false;
		}
	}
	
	FAccessor::FSparse::FIndices::FIndices(uint32 InCount, FBufferView& InBufferView, uint64 InByteOffset, EComponentType InComponentType)
		: Count(InCount)
		, BufferView(InBufferView)
		, ByteOffset(InByteOffset)
		, ComponentType(InComponentType)
	{
	}

	void FAccessor::GetCoordArray(FVector3f* Buffer) const
	{
		GetVec3Array(Buffer);
		for (uint32 Index = 0; Index < Count; ++Index)
		{
			Buffer[Index] = (FVector3f)ConvertVec3((FVector)Buffer[Index]);
		}
	}

	void FAccessor::GetQuatArray(FVector4f* Buffer) const
	{
		GetVec4Array(Buffer);
		for (uint32 Index = 0; Index < Count; ++Index)
		{
			FVector4f& Value = Buffer[Index];
			FQuat4f Quat(Value[0], Value[1], Value[2], Value[3]);
			Quat = GLTF::ConvertQuat(Quat);
			Value.X = Quat.X;
			Value.Y = Quat.Y;
			Value.Z = Quat.Z;
			Value.W = Quat.W;
		}
	}

	bool FAccessor::CheckAccessorTypeForDataType(EMeshAttributeType MeshAttributeType, bool bMorphTargetProperty) const
	{
		switch (MeshAttributeType)
		{
			case GLTF::EMeshAttributeType::POSITION:
			case GLTF::EMeshAttributeType::NORMAL:
				return Type == FAccessor::EType::Vec3;
			case GLTF::EMeshAttributeType::TANGENT:
				return (!bMorphTargetProperty && Type == FAccessor::EType::Vec4) || (bMorphTargetProperty && Type == FAccessor::EType::Vec3);
			case GLTF::EMeshAttributeType::TEXCOORD_0:
			case GLTF::EMeshAttributeType::TEXCOORD_1:
			case GLTF::EMeshAttributeType::TEXCOORD_2:
			case GLTF::EMeshAttributeType::TEXCOORD_3:
			case GLTF::EMeshAttributeType::TEXCOORD_4:
			case GLTF::EMeshAttributeType::TEXCOORD_5:
			case GLTF::EMeshAttributeType::TEXCOORD_6:
			case GLTF::EMeshAttributeType::TEXCOORD_7:
				return Type == FAccessor::EType::Vec2;
			case GLTF::EMeshAttributeType::COLOR_0:
				return Type == FAccessor::EType::Vec3 || Type == FAccessor::EType::Vec4;
			case GLTF::EMeshAttributeType::JOINTS_0:
			case GLTF::EMeshAttributeType::WEIGHTS_0:
				return Type == FAccessor::EType::Vec4;
			default:
				ensure(false);
				break;
		}

		return false;
	}

	bool FAccessor::CheckNonQuantizedComponentTypeForDataType(EMeshAttributeType MeshAttributeType, bool bMorphTargetProperty) const
	{
		if (!bMorphTargetProperty)
		{
			//Non Morph Target:
			switch (MeshAttributeType)
			{
				case GLTF::EMeshAttributeType::POSITION:
				case GLTF::EMeshAttributeType::NORMAL:
				case GLTF::EMeshAttributeType::TANGENT:
					return !bNormalized && ComponentType == FAccessor::EComponentType::F32;
				case GLTF::EMeshAttributeType::TEXCOORD_0:
				case GLTF::EMeshAttributeType::TEXCOORD_1:
				case GLTF::EMeshAttributeType::TEXCOORD_2:
				case GLTF::EMeshAttributeType::TEXCOORD_3:
				case GLTF::EMeshAttributeType::TEXCOORD_4:
				case GLTF::EMeshAttributeType::TEXCOORD_5:
				case GLTF::EMeshAttributeType::TEXCOORD_6:
				case GLTF::EMeshAttributeType::TEXCOORD_7:
				case GLTF::EMeshAttributeType::COLOR_0:
					return (!bNormalized && ComponentType == FAccessor::EComponentType::F32) 
						|| (bNormalized && (ComponentType == FAccessor::EComponentType::U8 || ComponentType == FAccessor::EComponentType::U16));
				case GLTF::EMeshAttributeType::JOINTS_0:
					return (!bNormalized && (ComponentType == FAccessor::EComponentType::U8 || ComponentType == FAccessor::EComponentType::U16));
				case GLTF::EMeshAttributeType::WEIGHTS_0:
					return (!bNormalized && ComponentType == FAccessor::EComponentType::F32)
						|| (bNormalized && (ComponentType == FAccessor::EComponentType::U8 || ComponentType == FAccessor::EComponentType::U16));
				default:
					ensure(false);
					break;
			}
		}
		else
		{
			//Morph Target:
			switch (MeshAttributeType)
			{
				case GLTF::EMeshAttributeType::POSITION:
				case GLTF::EMeshAttributeType::NORMAL:
				case GLTF::EMeshAttributeType::TANGENT:
					return !bNormalized && ComponentType == FAccessor::EComponentType::F32;
				case GLTF::EMeshAttributeType::TEXCOORD_0:
				case GLTF::EMeshAttributeType::TEXCOORD_1:
				case GLTF::EMeshAttributeType::TEXCOORD_2:
				case GLTF::EMeshAttributeType::TEXCOORD_3:
				case GLTF::EMeshAttributeType::TEXCOORD_4:
				case GLTF::EMeshAttributeType::TEXCOORD_5:
				case GLTF::EMeshAttributeType::TEXCOORD_6:
				case GLTF::EMeshAttributeType::TEXCOORD_7:
				case GLTF::EMeshAttributeType::COLOR_0:
					return (!bNormalized && ComponentType == FAccessor::EComponentType::F32)
						|| (bNormalized && (ComponentType == FAccessor::EComponentType::S8 || ComponentType == FAccessor::EComponentType::S16 || ComponentType == FAccessor::EComponentType::U8 || ComponentType == FAccessor::EComponentType::U16));
				default:
					ensure(false);
					break;
			}
		}

		return false;
	}

	bool FAccessor::CheckQuantizedComponentTypeForDataType(EMeshAttributeType MeshAttributeType, bool bMorphTargetProperty) const
	{
		if (!bMorphTargetProperty)
		{
			//Non Morph Target:
			switch (MeshAttributeType)
			{
				case GLTF::EMeshAttributeType::POSITION:
					return ComponentType == FAccessor::EComponentType::S8 || ComponentType == FAccessor::EComponentType::U8 || ComponentType == FAccessor::EComponentType::S16 || ComponentType == FAccessor::EComponentType::U16;
				case GLTF::EMeshAttributeType::NORMAL:
				case GLTF::EMeshAttributeType::TANGENT:
					return bNormalized && (ComponentType == FAccessor::EComponentType::S8 || ComponentType == FAccessor::EComponentType::S16);
				case GLTF::EMeshAttributeType::TEXCOORD_0:
				case GLTF::EMeshAttributeType::TEXCOORD_1:
				case GLTF::EMeshAttributeType::TEXCOORD_2:
				case GLTF::EMeshAttributeType::TEXCOORD_3:
				case GLTF::EMeshAttributeType::TEXCOORD_4:
				case GLTF::EMeshAttributeType::TEXCOORD_5:
				case GLTF::EMeshAttributeType::TEXCOORD_6:
				case GLTF::EMeshAttributeType::TEXCOORD_7:
					return (ComponentType == FAccessor::EComponentType::S8 || ComponentType == FAccessor::EComponentType::S16)
						|| (!bNormalized && (ComponentType == FAccessor::EComponentType::U8 || ComponentType == FAccessor::EComponentType::U16));
				default:
					ensure(false);
					break;
			}
		}
		else
		{
			//Morph Target:
			switch (MeshAttributeType)
			{
				case GLTF::EMeshAttributeType::POSITION:
					return ComponentType == FAccessor::EComponentType::S8 || ComponentType == FAccessor::EComponentType::S16;
				case GLTF::EMeshAttributeType::NORMAL:
				case GLTF::EMeshAttributeType::TANGENT:
					return bNormalized && (ComponentType == FAccessor::EComponentType::S8 || ComponentType == FAccessor::EComponentType::S16);
				case GLTF::EMeshAttributeType::TEXCOORD_0:
				case GLTF::EMeshAttributeType::TEXCOORD_1:
				case GLTF::EMeshAttributeType::TEXCOORD_2:
				case GLTF::EMeshAttributeType::TEXCOORD_3:
				case GLTF::EMeshAttributeType::TEXCOORD_4:
				case GLTF::EMeshAttributeType::TEXCOORD_5:
				case GLTF::EMeshAttributeType::TEXCOORD_6:
				case GLTF::EMeshAttributeType::TEXCOORD_7:
					return !bNormalized && (ComponentType == FAccessor::EComponentType::S8 || ComponentType == FAccessor::EComponentType::S16);
				default:
					ensure(false);
					break;
			}
		}

		return false;
	}

	bool FAccessor::IsValidDataType(EMeshAttributeType MeshAttributeType, bool bMorphTargetProperty) const
	{
		//Check Accessor Type restrictions:
		if (!CheckAccessorTypeForDataType(MeshAttributeType, bMorphTargetProperty))
		{
			return false;
		}

		//Check Component Type restrictions (non-quantized) first:
		if (CheckNonQuantizedComponentTypeForDataType(MeshAttributeType, bMorphTargetProperty))
		{
			return true;
		}

		//Check quantized Component Type restrictions:
		if (bQuantized)
		{
			if (CheckQuantizedComponentTypeForDataType(MeshAttributeType, bMorphTargetProperty))
			{
				return true;
			}
		}

		return false;
	}

	//Sparse related helpers:
	void FAccessor::UpdateUnsignedIntWithSparse(uint32 Index, uint32& Data) const
	{
		if (Sparse.bHasSparse)
		{
			TArray<uint32> IndicesData = AcquireUnsignedIntArray(Sparse.Count, Sparse.Indices.BufferView, Sparse.Indices.ByteOffset, Sparse.Indices.ComponentType);
			int32 SparseArrayIndex = IndicesData.IndexOfByKey(Index);
			if (SparseArrayIndex != INDEX_NONE)
			{
				uint32 Buffer;
				if (AcquireUnsignedInt(Index, Sparse.Values.BufferView, Sparse.Values.ByteOffset, ByteStride, ComponentType, Buffer))
				{
					Data = Buffer;
				}
			}
		}
	}
	void FAccessor::UpdateUnsignedInt16x4WithSparse(uint32 Index, uint16 Data[4]) const
	{
		if (Sparse.bHasSparse)
		{
			TArray<uint32> IndicesData = AcquireUnsignedIntArray(Sparse.Count, Sparse.Indices.BufferView, Sparse.Indices.ByteOffset, Sparse.Indices.ComponentType);
			int32 SparseArrayIndex = IndicesData.IndexOfByKey(Index);
			if (SparseArrayIndex != INDEX_NONE)
			{
				uint16 Buffer[4];
				if (AcquireUnsignedInt16x4(Index, Sparse.Values.BufferView, Sparse.Values.ByteOffset, ByteStride, ComponentType, Buffer))
				{
					Data[0] = Buffer[0];
					Data[1] = Buffer[1];
					Data[2] = Buffer[2];
					Data[3] = Buffer[3];
				}
			}
		}
	}

	void FAccessor::UpdateFloatWithSparse(uint32 Index, float& Data) const
	{
		if (Sparse.bHasSparse)
		{
			TArray<uint32> IndicesData = AcquireUnsignedIntArray(Sparse.Count, Sparse.Indices.BufferView, Sparse.Indices.ByteOffset, Sparse.Indices.ComponentType);
			int32 SparseArrayIndex = IndicesData.IndexOfByKey(Index);
			if (SparseArrayIndex != INDEX_NONE)
			{
				float Buffer;
				if (AcquireFloat(Index, Sparse.Values.BufferView, Sparse.Values.ByteOffset, ByteStride, ComponentType, Buffer))
				{
					Data = Buffer;
				}
			}
		}
	}
	void FAccessor::UpdateVec2WithSparse(uint32 Index, FVector2D& Data) const
	{
		if (Sparse.bHasSparse)
		{
			TArray<uint32> IndicesData = AcquireUnsignedIntArray(Sparse.Count, Sparse.Indices.BufferView, Sparse.Indices.ByteOffset, Sparse.Indices.ComponentType);
			int32 SparseArrayIndex = IndicesData.IndexOfByKey(Index);
			if (SparseArrayIndex != INDEX_NONE)
			{
				FVector2D Buffer;
				if (AcquireVec2(Index, Sparse.Values.BufferView, Sparse.Values.ByteOffset, ByteStride, ComponentType, bNormalized, Buffer))
				{
					Data = Buffer;
				}
			}
		}
	}
	void FAccessor::UpdateVec3WithSparse(uint32 Index, FVector& Data) const
	{
		if (Sparse.bHasSparse)
		{
			TArray<uint32> IndicesData = AcquireUnsignedIntArray(Sparse.Count, Sparse.Indices.BufferView, Sparse.Indices.ByteOffset, Sparse.Indices.ComponentType);
			int32 SparseArrayIndex = IndicesData.IndexOfByKey(Index);
			if (SparseArrayIndex != INDEX_NONE)
			{
				FVector Buffer;
				if (AcquireVec3(Index, Sparse.Values.BufferView, Sparse.Values.ByteOffset, ByteStride, ComponentType, bNormalized, Buffer))
				{
					Data = Buffer;
				}
			}
		}
	}
	void FAccessor::UpdateVec4WithSparse(uint32 Index, FVector4& Data) const
	{
		if (Sparse.bHasSparse)
		{
			TArray<uint32> IndicesData = AcquireUnsignedIntArray(Sparse.Count, Sparse.Indices.BufferView, Sparse.Indices.ByteOffset, Sparse.Indices.ComponentType);
			int32 SparseArrayIndex = IndicesData.IndexOfByKey(Index);
			if (SparseArrayIndex != INDEX_NONE)
			{
				FVector4 Buffer;
				if (AcquireVec4(Index, Sparse.Values.BufferView, Sparse.Values.ByteOffset, ByteStride, ComponentType, bNormalized, Buffer))
				{
					Data = Buffer;
				}
			}
		}
	}

	void FAccessor::UpdateMat4WithSparse(uint32 Index, FMatrix& Data) const
	{
		if (Sparse.bHasSparse)
		{
			TArray<uint32> IndicesData = AcquireUnsignedIntArray(Sparse.Count, Sparse.Indices.BufferView, Sparse.Indices.ByteOffset, Sparse.Indices.ComponentType);
			int32 SparseArrayIndex = IndicesData.IndexOfByKey(Index);
			if (SparseArrayIndex != INDEX_NONE)
			{
				FMatrix Buffer;
				if (AcquireMat4(Index, Sparse.Values.BufferView, Sparse.Values.ByteOffset, ByteStride, ComponentType, Buffer))
				{
					Data = Buffer;
				}
			}
		}
	}

	void FAccessor::UpdateFloatArrayWithSparse(float* Data) const
	{
		if (Sparse.bHasSparse && Sparse.Count)
		{
			TArray<uint32> IndicesData = AcquireUnsignedIntArray(Sparse.Count, Sparse.Indices.BufferView, Sparse.Indices.ByteOffset, Sparse.Indices.ComponentType);

			TArray<float> ValueBuffer;
			ValueBuffer.SetNumUninitialized(Sparse.Count, EAllowShrinking::No);

			if (FillFloatArray(bNormalized, Sparse.Values.BufferView, Sparse.Values.ByteOffset, ComponentType, ByteStride, Sparse.Count, ValueBuffer.GetData()))
			{
				for (size_t Index = 0; Index < Sparse.Count; Index++)
				{
					const uint32 SparseIndex = IndicesData[Index];
					const float SparseValue = ValueBuffer[Index];

					if (SparseIndex < Count)
					{
						Data[SparseIndex] = SparseValue;
					}
				}
			}
		}
	}
	void FAccessor::UpdateUnsignedIntArrayWithSparse(uint32* Data) const
	{
		if (Sparse.bHasSparse && Sparse.Count)
		{
			TArray<uint32> IndicesData = AcquireUnsignedIntArray(Sparse.Count, Sparse.Indices.BufferView, Sparse.Indices.ByteOffset, Sparse.Indices.ComponentType);

			TArray<uint32> ValueBuffer;
			ValueBuffer.SetNumUninitialized(Sparse.Count, EAllowShrinking::No);

			if (FillUnsignedIntArray(Sparse.Values.BufferView, Sparse.Values.ByteOffset, ComponentType, Count, ValueBuffer.GetData()))
			{
				for (size_t Index = 0; Index < Sparse.Count; Index++)
				{
					const uint32 SparseIndex = IndicesData[Index];
					const float SparseValue = ValueBuffer[Index];

					if (SparseIndex < Count)
					{
						Data[SparseIndex] = SparseValue;
					}
				}
			}
		}
	}

	template<typename ItemType, uint32 ItemElementCount>
	void FAccessor::UpdateArrayWithSparse(ItemType* Data) const
	{
		if (Sparse.bHasSparse && Sparse.Count)
		{
			TArray<uint32> IndicesData = AcquireUnsignedIntArray(Sparse.Count, Sparse.Indices.BufferView, Sparse.Indices.ByteOffset, Sparse.Indices.ComponentType);

			TArray<ItemType> ValueBuffer;
			ValueBuffer.SetNumUninitialized(Sparse.Count, EAllowShrinking::No);

			CopyWithoutConversion<ItemType, ItemElementCount>(ByteStride, ComponentType, bNormalized, Sparse.Count, Sparse.Values.BufferView, Sparse.Values.ByteOffset, ValueBuffer.GetData());

			for (size_t Index = 0; Index < Sparse.Count; Index++)
			{
				const uint32 SparseIndex = IndicesData[Index];
				const ItemType SparseValue = ValueBuffer[Index];

				if (SparseIndex < Count)
				{
					Data[SparseIndex] = SparseValue;
				}
			}
		}
	}

	//
	FAccessor::FAccessor()
		: AccessorIndex(INDEX_NONE)
		, Count(0)
		, Type(EType::Unknown)
		, ComponentType(EComponentType::None)
		, bNormalized(false)
		, bQuantized(false)
		, Sparse()
		, BufferView()
		, ByteOffset(0)
		, NumberOfComponents(0)
		, ElementSize(0)
		, ByteStride(0)
	{
	}

	FAccessor::FAccessor(uint32 InAccessorIndex,
		FBufferView& InBufferView, uint64 InOffset, 
		uint32 InCount, EType InType, EComponentType InCompType, bool bInNormalized, 
		const FSparse& InSparse)
		: AccessorIndex(InAccessorIndex)
		, Count(InCount)
		, Type(InType)
		, ComponentType(InCompType)
		, bNormalized(bInNormalized)
		, bQuantized(false)
		, Sparse(InSparse)
	    , BufferView(InBufferView)
	    , ByteOffset(InOffset)
		, NumberOfComponents(GetNumberOfComponents(Type))
	    , ElementSize(GetElementSize(Type, ComponentType))
		// BufferView.ByteStride zero means elements are tightly packed
		, ByteStride(InBufferView.ByteStride ? InBufferView.ByteStride : ElementSize)
	{
	}

	FAccessor::FAccessor(
		uint32 InAccessorIndex,
		uint32 InCount, EType InType, EComponentType InCompType, bool bInNormalized, 
		const FSparse& InSparse)
		: AccessorIndex(InAccessorIndex)
		, Count(InCount)
		, Type(InType)
		, ComponentType(InCompType)
		, bNormalized(bInNormalized)
		, bQuantized(false)
		, Sparse(InSparse)
		, BufferView()
		, ByteOffset(0)
		, NumberOfComponents(GetNumberOfComponents(Type))
		, ElementSize(GetElementSize(Type, ComponentType))
		// BufferView.ByteStride zero means elements are tightly packed
		, ByteStride(ElementSize)
	{

	}

	bool FAccessor::IsValid() const
	{
		if (Sparse.bHasSparse)
		{
			if (Sparse.Count > Count)
			{
				//Sparse count must not be greater than the number of the base accessor.
				return false;
			}

			if (Sparse.Count > 0)
			{
				const uint32 IndicesElementSize = ComponentSize[(int)Sparse.Indices.ComponentType];
				if (!Sparse.Indices.BufferView.HasCapacityFor(Sparse.Count, Sparse.Indices.ByteOffset, IndicesElementSize, IndicesElementSize))
				{
					return false;
				}

				if (!Sparse.Values.BufferView.HasCapacityFor(Sparse.Count, Sparse.Values.ByteOffset, ElementSize, ElementSize))
				{
					return false;
				}
			}

			// Validate base buffer view if present
			if (BufferView.IsValid())
			{
				if (!BufferView.HasCapacityFor(Count, ByteOffset, ElementSize, ByteStride))
				{
					return false;
				}
			}

			return true;
		}
		
		return BufferView.HasCapacityFor(Count, ByteOffset, ElementSize, ByteStride);
	}

	FMD5Hash FAccessor::GetHash() const
	{
		if (!IsValid())
		{
			return FMD5Hash();
		}

		FMD5 MD5;

		// Contigous view into a buffer: Can hash all at once
		if (ByteStride == ElementSize)
		{
			MD5.Update(DataAt(0), Count * ElementSize);
		}
		// Shared buffer: Have to hash in strides
		else
		{
			for (uint32 Index = 0; Index < Count; ++Index)
			{
				MD5.Update(DataAt(Index), ElementSize);
			}
		}

		uint8 TypeInt = static_cast<uint8>(Type);
		uint8 ComponentInt = static_cast<uint8>(ComponentType);
		uint8 NormalizedInt = static_cast<uint8>(bNormalized);

		MD5.Update(&TypeInt, sizeof(TypeInt));
		MD5.Update(&ComponentInt, sizeof(ComponentInt));
		MD5.Update(&NormalizedInt, sizeof(NormalizedInt));

		FMD5Hash Hash;
		Hash.Set(MD5);
		return Hash;
	}

	uint32 FAccessor::GetUnsignedInt(uint32 Index) const
	{
		// should be Scalar, not bNormalized, unsigned integer (8, 16 or 32 bit)

		if (Index < Count)
		{
			if (Type == EType::Scalar && !bNormalized)
			{
				uint32 Buffer;
				if (!BufferView.IsValid() || AcquireUnsignedInt(Index, BufferView, ByteOffset, ByteStride, ComponentType, Buffer))
				{
					UpdateUnsignedIntWithSparse(Index, Buffer);
					return Buffer;
				}
			}
		}

		ensure(false);
		return 0;
	}

	void FAccessor::GetUnsignedInt16x4(uint32 Index, uint16 Values[4]) const
	{
		// should be Vec4, not bNormalized, unsigned integer (8 or 16 bit)

		if (Index < Count)
		{
			if (Type == EType::Vec4 && !bNormalized)
			{
				if (!BufferView.IsValid() || AcquireUnsignedInt16x4(Index, BufferView, ByteOffset, ByteStride, ComponentType, Values))
				{
					UpdateUnsignedInt16x4WithSparse(Index, Values);
					return;
				}
			}
		}
		ensure(false);
	}

	float FAccessor::GetFloat(uint32 Index) const
	{
		// should be Scalar float

		if (Index < Count)
		{
			if (Type == EType::Scalar && !bNormalized)
			{
				float Buffer;
				if (!BufferView.IsValid() || AcquireFloat(Index, BufferView, ByteOffset, ByteStride, ComponentType, Buffer))
				{
					UpdateFloatWithSparse(Index, Buffer);
					return Buffer;
				}
			}
		}

		ensure(false);
		return 0.f;
	}

	FVector2D FAccessor::GetVec2(uint32 Index) const
	{
		// Spec-defined attributes (TEXCOORD_0, TEXCOORD_1) use only these formats:
		// - F32
		// - U8 normalized
		// - U16 normalized
		// Custom attributes can use any CompType, so add support for those when needed.

		if (Index < Count)
		{
			if (Type == EType::Vec2)  // strict format match, unlike GPU shader fetch
			{
				FVector2D Buffer;
				if (!BufferView.IsValid() || AcquireVec2(Index, BufferView, ByteOffset, ByteStride, ComponentType, bNormalized, Buffer))
				{
					UpdateVec2WithSparse(Index, Buffer);
					return Buffer;
				}
			}
		}

		ensure(false);
		return FVector2D::ZeroVector;
	}

	FVector FAccessor::GetVec3(uint32 Index) const
	{
		// Spec-defined attributes (POSITION, NORMAL, COLOR_0) use only these formats:
		// - F32
		// - U8 normalized
		// - U16 normalized
		// Custom attributes can use any CompType, so add support for those when needed.

		if (Index < Count)
		{
			if (Type == EType::Vec3)  // strict format match, unlike GPU shader fetch
			{
				FVector Buffer;
				if (!BufferView.IsValid() || AcquireVec3(Index, BufferView, ByteOffset, ByteStride, ComponentType, bNormalized, Buffer))
				{
					UpdateVec3WithSparse(Index, Buffer);
					return Buffer;
				}
			}
		}

		ensure(false);
		return FVector::ZeroVector;
	}

	FVector4 FAccessor::GetVec4(uint32 Index) const
	{
		// Spec-defined attributes (TANGENT, COLOR_0) use only these formats:
		// - F32
		// - U8 normalized
		// - U16 normalized
		// Custom attributes can use any CompType, so add support for those when needed.

		if (Index < Count)
		{
			if (Type == EType::Vec4)  // strict format match, unlike GPU shader fetch
			{
				FVector4 Buffer;
				if (!BufferView.IsValid() || AcquireVec4(Index, BufferView, ByteOffset, ByteStride, ComponentType, bNormalized, Buffer))
				{
					UpdateVec4WithSparse(Index, Buffer);
					return Buffer;
				}
			}
		}

		ensure(false);
		return FVector4();
	}

	FMatrix FAccessor::GetMat4(uint32 Index) const
	{
		// Focus on F32 for now, add other types as needed.

		if (Index < Count)
		{
			if (Type == EType::Mat4)  // strict format match, unlike GPU shader fetch
			{
				FMatrix Buffer;
				if (!BufferView.IsValid() || AcquireMat4(Index, BufferView, ByteOffset, ByteStride, ComponentType, Buffer))
				{
					UpdateMat4WithSparse(Index, Buffer);
					return Buffer;
				}
			}
		}

		ensure(false);
		return FMatrix();
	}

	void FAccessor::GetUnsignedIntArray(uint32* Buffer) const
	{
		if (Type == EType::Scalar && !bNormalized)
		{
			if (!BufferView.IsValid() || FillUnsignedIntArray(BufferView, ByteOffset, ComponentType, Count, Buffer))
			{
				UpdateUnsignedIntArrayWithSparse(Buffer);
				return;
			}
		}

		ensure(false);
	}

	void FAccessor::GetFloatArray(float* Buffer) const
	{
		if (Type == EType::Scalar)
		{
			if (!BufferView.IsValid() || FillFloatArray(bNormalized, BufferView, ByteOffset, ComponentType, ByteStride, Count, Buffer))
			{
				UpdateFloatArrayWithSparse(Buffer);
				return;
			}
		}

		ensure(false);
	}

	void FAccessor::GetVec2Array(FVector2f* Buffer) const
	{
		if (!ensure(Type == EType::Vec2))
		{
			return;
		}
		CopyWithoutConversion<FVector2f, 2>(*this, Buffer);
		UpdateArrayWithSparse<FVector2f, 2>(Buffer);
	}

	void FAccessor::GetVec3Array(FVector3f* Buffer) const
	{
		if (!ensure(Type == EType::Vec3))
		{
			return;
		}
		CopyWithoutConversion<FVector3f, 3>(*this, Buffer);
		UpdateArrayWithSparse<FVector3f, 3>(Buffer);
	}

	void FAccessor::GetVec4Array(FVector4f* Buffer) const
	{
		if (!ensure(Type == EType::Vec4))
		{
			return;
		}
		CopyWithoutConversion<FVector4f, 4>(*this, Buffer);
		UpdateArrayWithSparse<FVector4f, 4>(Buffer);
	}

	void FAccessor::GetMat4Array(FMatrix44f* Buffer) const
	{
		if (Type == EType::Mat4 && ComponentType == EComponentType::F32)  // strict format match, unlike GPU shader fetch
		{
			for (uint32 Index = 0; Index < Count; ++Index)
			{
				const void* Pointer = DataAt(Index);
				Buffer[Index]       = FMatrix44f(GetMatrix(Pointer));
			}
			return;
		}

		ensure(false);
	}

	const uint8* FAccessor::DataAt(uint32 Index) const
	{
		const uint32 Offset = Index * ByteStride;
		return BufferView.DataAt(Offset + ByteOffset);
	}

	void FAccessor::GetUnsignedIntArray(TArray<uint32>& Buffer) const
	{
		if (IsValid())
		{
			Buffer.SetNumZeroed(Count, EAllowShrinking::No);
			GetUnsignedIntArray(Buffer.GetData());
		}
	}

	void FAccessor::GetFloatArray(TArray<float>& Buffer) const
	{
		if (IsValid())
		{
			Buffer.SetNumZeroed(Count, EAllowShrinking::No);
			GetFloatArray(Buffer.GetData());
		}
	}

	void FAccessor::GetVec2Array(TArray<FVector2f>& Buffer) const
	{
		if (IsValid())
		{
			Buffer.SetNumZeroed(Count, EAllowShrinking::No);
			GetVec2Array(Buffer.GetData());
		}
	}

	void FAccessor::GetVec3Array(TArray<FVector3f>& Buffer) const
	{
		if (IsValid())
		{
			Buffer.SetNumZeroed(Count, EAllowShrinking::No);
			GetVec3Array(Buffer.GetData());
		}
	}

	void FAccessor::GetCoordArray(TArray<FVector3f>& Buffer) const
	{
		if (IsValid())
		{
			Buffer.SetNumZeroed(Count, EAllowShrinking::No);
			GetCoordArray(Buffer.GetData());
		}
	}

	void FAccessor::GetVec4Array(TArray<FVector4f>& Buffer) const
	{
		if (IsValid())
		{
			Buffer.SetNumZeroed(Count, EAllowShrinking::No);
			GetVec4Array(Buffer.GetData());
		}
	}

	void FAccessor::GetQuatArray(TArray<FVector4f>& Buffer) const
	{
		if (IsValid())
		{
			Buffer.SetNumZeroed(Count, EAllowShrinking::No);
			GetQuatArray(Buffer.GetData());
		}
	}

	void FAccessor::GetMat4Array(TArray<FMatrix44f>& Buffer) const
	{
		if (IsValid())
		{
			Buffer.SetNumZeroed(Count, EAllowShrinking::No);
			GetMat4Array(Buffer.GetData());
		}
	}
}  // namespace GLTF
