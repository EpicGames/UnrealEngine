// Copyright Epic Games, Inc. All Rights Reserved.

#include "GLTFAsset.h"

#include "GLTFAnimation.h"
#include "Misc/Paths.h"
#include "GLTFMaterial.h"
#include "GLTFMesh.h"
#include "GLTFNode.h"
#include "GLTFTexture.h"
#include "InterchangeHelper.h"
#include "Misc/SecureHash.h"

namespace GLTF
{
	namespace
	{
		template <typename T>
		void GenerateNames(const FString& Prefix, TArray<T>& Objects)
		{
			for (int32 Index = 0; Index < Objects.Num(); ++Index)
			{
				T& Obj = Objects[Index];

				Obj.UniqueId = Prefix + FString::FromInt(Index);

				if (Obj.Name.IsEmpty())
				{
					Obj.Name = Obj.UniqueId;
				}

				UE::Interchange::SanitizeName(Obj.Name);
			}
		}

		//Make the Name FName compliant.
		//For Scenario where Asset.Name is '_x' (where x is a number) making FName to be none:
		//UE-174253
		template <typename T>
		void ValidateFNameCriteria(const FString& Prefix, TArray<T>& Objects)
		{
			for (int32 Index = 0; Index < Objects.Num(); ++Index)
			{
				T& Obj = Objects[Index];

				if (!Obj.Name.IsEmpty())
				{
					FName SanitizationTest(*Obj.Name);
					if (SanitizationTest.IsNone())
					{
						Obj.Name = Prefix + Obj.Name;
					}
				}
			}
		}
	}

	// Prefixes
	FString GLTFNodePrefixStaticHelper::GetScenePrefix(const FString& AssetName)
	{
		return AssetName + TEXT("_scene_");
	}

	FString GLTFNodePrefixStaticHelper::GetNodePrefix(const FString& AssetName)
	{
		return AssetName + TEXT("_node_");
	}

	FString GLTFNodePrefixStaticHelper::GetJointPrefix(const FString& AssetName)
	{
		return AssetName + TEXT("_joint_");
	}

	FString GLTFNodePrefixStaticHelper::GetTexturePrefix(const FString& AssetName)
	{
		return AssetName + TEXT("_texture_");
	}

	FString GLTFNodePrefixStaticHelper::GetCameraPrefix(const FString& AssetName)
	{
		return AssetName + TEXT("_camera_");
	}

	FString GLTFNodePrefixStaticHelper::GetLightPrefix(const FString& AssetName)
	{
		return AssetName + TEXT("_light_");
	}

	FString GLTFNodePrefixStaticHelper::GetMaterialPrefix(const FString& AssetName)
	{
		return AssetName + TEXT("_material_");
	}

	FString GLTFNodePrefixStaticHelper::GetSkinPrefix(const FString& AssetName)
	{
		return AssetName + TEXT("_skin_");
	}

	FString GLTFNodePrefixStaticHelper::GetAnimationPrefix(const FString& AssetName)
	{
		return AssetName + TEXT("_animation_");
	}

	FString GLTFNodePrefixStaticHelper::GetImagePrefix(const FString& AssetName)
	{
		return AssetName + TEXT("_image_");
	}

	FString GLTFNodePrefixStaticHelper::GetMeshPrefix(const FString& AssetName)
	{
		return AssetName + TEXT("_mesh_");
	}

	FString GLTFNodePrefixStaticHelper::GetIESPrefix(const FString& AssetName)
	{
		return AssetName + TEXT("_IES_");
	}

	const FMetadata::FExtraData* FMetadata::GetExtraData(const FString& Name) const
	{
		return Extras.FindByPredicate([&Name](const FMetadata::FExtraData& Data) { return Data.Name == Name; });
	}

	const TSet<GLTF::EExtension> FAsset::SupportedExtensions = { GLTF::EExtension::KHR_MaterialsPbrSpecularGlossiness,
																 GLTF::EExtension::KHR_MaterialsUnlit,
																 GLTF::EExtension::KHR_LightsPunctual,
																 GLTF::EExtension::KHR_MaterialsClearCoat,
																 GLTF::EExtension::KHR_MaterialsTransmission,
																 GLTF::EExtension::KHR_MaterialsSheen,
																 GLTF::EExtension::KHR_MaterialsVariants,
																 GLTF::EExtension::KHR_MaterialsSpecular,
																 GLTF::EExtension::KHR_MaterialsIOR };

	void FAsset::Clear(uint32 BinBufferKBytes, uint32 ExtraBinBufferKBytes)
	{
		Buffers.Empty();
		BufferViews.Empty();
		Accessors.Empty();
		Meshes.Empty();

		Scenes.Empty();
		Nodes.Empty();
		Cameras.Empty();
		Lights.Empty();
		Skins.Empty();

		Images.Empty();
		Samplers.Empty();
		Textures.Empty();
		Materials.Empty();

		ProcessedExtensions.Empty((int)EExtension::Count);
		ExtensionsRequired.Empty();
		Metadata.GeneratorName.Empty();
		Metadata.Extras.Empty();

		if (BinData.Num() > 0)
			BinData.Empty(BinBufferKBytes * 1024);
		if (ExtraBinData.Num() > 0)
			ExtraBinData.Empty(ExtraBinBufferKBytes * 1024);
	}

	void FAsset::GenerateNames()
	{
		{
			const FString ScenePrefix = GLTFNodePrefixStaticHelper::GetScenePrefix(Name);
			for (size_t SceneIndex = 0; SceneIndex < Scenes.Num(); SceneIndex++)
			{
				FScene& Scene = Scenes[SceneIndex];

				Scene.UniqueId = ScenePrefix + FString::FromInt(SceneIndex);

				if (Scene.Name.IsEmpty())
				{
					Scene.Name = Scene.UniqueId;
				}

				UE::Interchange::SanitizeName(Scene.Name);
			}
		}

		{
			//Nodes/Joints need to have (case ignored) unique naming,
			// as Skeletons don't accept (casen ignored) identical bone names
			//For that reason we are making sure here that the Node names are unique.

			TSet<FString> Names;

			const FString NodePrefix = GLTFNodePrefixStaticHelper::GetNodePrefix(Name);
			const FString JointPrefix = GLTFNodePrefixStaticHelper::GetJointPrefix(Name);
			for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); ++NodeIndex)
			{
				FNode& Node = Nodes[NodeIndex];

				bool bIsJoint = Node.Type == FNode::EType::Joint;
				Node.UniqueId = (bIsJoint ? JointPrefix : NodePrefix);

				Node.UniqueId = Node.UniqueId + FString::FromInt(NodeIndex);

				if (Node.Name.IsEmpty())
				{
					Node.Name = Node.UniqueId;
				}

				UE::Interchange::SanitizeName(Node.Name, bIsJoint);

				if (Names.Contains(Node.Name))
				{
					//to make the name unique we encode the UID into the name:
					Node.Name += TEXT("_") + FMD5::HashAnsiString(*Node.UniqueId);
				}

				Names.Add(Node.Name);
			}
		}

		{
			const FString TexPrefix = GLTFNodePrefixStaticHelper::GetTexturePrefix(Name);
			for (int32 TextureIndex = 0; TextureIndex < Textures.Num(); ++TextureIndex)
			{
				FTexture& Tex = Textures[TextureIndex];

				Tex.UniqueId = TexPrefix + FString::FromInt(TextureIndex);

				if (!Tex.Name.IsEmpty())
				{
					// keep base part of the name as it was in gltf
				}
				else if (!Tex.Source.Name.IsEmpty())
				{
					Tex.Name = Tex.Source.Name;
				}
				else if (!Tex.Source.URI.IsEmpty())
				{
					//in case texture is embedded:
					if (Tex.Source.URI.StartsWith(TEXT("data:")))
					{
						Tex.Name = Tex.UniqueId;
					}
					else
					{
						Tex.Name = FPaths::GetBaseFilename(Tex.Source.URI);
					}
				}
				else
				{
					Tex.Name = Tex.UniqueId;
				}

				UE::Interchange::SanitizeName(Tex.Name);
			}
		}

		{
			const FString CameraPrefix = GLTFNodePrefixStaticHelper::GetCameraPrefix(Name);
			for (int32 CameraIndex = 0; CameraIndex < Cameras.Num(); ++CameraIndex)
			{
				FCamera& Camera = Cameras[CameraIndex];

				Camera.UniqueId = CameraPrefix + FString::FromInt(CameraIndex);

				if (Camera.Name.IsEmpty())
				{
					if (!Camera.Node.Name.IsEmpty())
					{
						Camera.Name = Camera.Node.Name; // cant be empty
					}
					else
					{
						Camera.Name = Camera.UniqueId;
					}
				}

				UE::Interchange::SanitizeName(Camera.Name);
			}
		}

		{
			const FString LightPrefix = GLTFNodePrefixStaticHelper::GetLightPrefix(Name);
			for (int32 LightIndex = 0; LightIndex < Lights.Num(); ++LightIndex)
			{
				FLight& Light = Lights[LightIndex];

				Light.UniqueId = LightPrefix + FString::FromInt(LightIndex);

				if (Light.Name.IsEmpty())
				{
					if (Light.Node && !Light.Node->Name.IsEmpty())
					{
						Light.Name = Light.Node->Name;
					}
					else
					{
						Light.Name = Light.UniqueId;
					}
				}

				UE::Interchange::SanitizeName(Light.Name);
			}
		}

		// General pattern here is that we'll have a Prefix that will be a filename (like MyFile). 
		// The name (display label) and the UniqueId of an object has been separated.
		// UniqueId will consist of: [Prefix]+[_ElementType_]+[ElementIndex]
		// Name will be the provided GLTF object.name, if present,
		// if Name is not present the UniqueId will be used as the Name.
		GLTF::GenerateNames(GLTFNodePrefixStaticHelper::GetMaterialPrefix(Name), Materials);
		GLTF::GenerateNames(GLTFNodePrefixStaticHelper::GetSkinPrefix(Name), Skins);
		GLTF::GenerateNames(GLTFNodePrefixStaticHelper::GetAnimationPrefix(Name), Animations);
		GLTF::GenerateNames(GLTFNodePrefixStaticHelper::GetImagePrefix(Name), Images);
		GLTF::GenerateNames(GLTFNodePrefixStaticHelper::GetMeshPrefix(Name), Meshes);
		GLTF::GenerateNames(GLTFNodePrefixStaticHelper::GetIESPrefix(Name), LightsIES);

		// Validate FName criteria for Materials Textures Meshes Animations (assets)
		GLTF::ValidateFNameCriteria(TEXT("Material_"), Materials);
		GLTF::ValidateFNameCriteria(TEXT("Texture_"), Textures);
		GLTF::ValidateFNameCriteria(TEXT("Mesh_"), Meshes);
		GLTF::ValidateFNameCriteria(TEXT("Animation_"), Animations);
		GLTF::ValidateFNameCriteria(TEXT("TextureIES_"), LightsIES);

		//Generate MorphTarget Names:
		//MorphTargetNames have to be unique across meshes as well:
		TSet<FString> MorphTargetNamesUniquenessCheckAcrossMeshes;
		int32 MorphTargetCounterAcrossMeshes = 0;
		for (int32 MeshIndex = 0; MeshIndex < Meshes.Num(); MeshIndex++)
		{
			if (Meshes[MeshIndex].Primitives.Num() > 0)
			{
				TSet<FString> MorphTargetNamesUniquenessCheck; //per mesh

				for (const FString& MorphTargetName : Meshes[MeshIndex].MorphTargetNames)
				{
					MorphTargetNamesUniquenessCheckAcrossMeshes.Add(MorphTargetName);
					MorphTargetCounterAcrossMeshes++;

					//validate uniqueness of MorphTargetNames
					MorphTargetNamesUniquenessCheck.Add(MorphTargetName);
				}

				if (MorphTargetNamesUniquenessCheck.Num() != Meshes[MeshIndex].MorphTargetNames.Num())
				{
					Meshes[MeshIndex].MorphTargetNames.Empty();
				}
			}
		}
		bool bReBuildMorphTargetNames = MorphTargetNamesUniquenessCheckAcrossMeshes.Num() != MorphTargetCounterAcrossMeshes;

		for (int32 MeshIndex = 0; MeshIndex < Meshes.Num(); MeshIndex++)
		{
			if (Meshes[MeshIndex].Primitives.Num() > 0)
			{
				//at this point the NumberOfMorphTargets across primitives have been validated (they have to be equal)
				//Note: All primitives MUST have the same number of morph targets in the same order.
				
				if (bReBuildMorphTargetNames)
				{
					Meshes[MeshIndex].MorphTargetNames.Empty();
				}

				//check if number of morph targets and the number of morphtarget names:
				//NumberOfMorphTargetsPerPrimitive will return the first Primitive's MorphTarget's count,
				//		In case Primitives have varying MorphTarget counts then the ValidationCheck will report false.
				int32 NumberOfMorphTargets = Meshes[MeshIndex].NumberOfMorphTargetsPerPrimitive();
				if (NumberOfMorphTargets != Meshes[MeshIndex].MorphTargetNames.Num())
				{
					Meshes[MeshIndex].MorphTargetNames.Empty();
				}

				//if morph target names failed a criteria above, generate unique names:
				if (Meshes[MeshIndex].MorphTargetNames.IsEmpty())
				{
					for (int32 MorphTargetIndex = 0; MorphTargetIndex < NumberOfMorphTargets; MorphTargetIndex++)
					{
						Meshes[MeshIndex].MorphTargetNames.Add(Meshes[MeshIndex].UniqueId + TEXT("_") + LexToString(MorphTargetIndex) + TEXT("_MorphTarget"));
					}

					Meshes[MeshIndex].GenerateIsValidCache(false);
				}
			}
		}
	}

	void FAsset::GetRootNodes(TArray<int32>& NodeIndices)
	{
		TMap<int32, uint32> VisitCount;
		VisitCount.Reserve(Nodes.Num());
		for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); ++NodeIndex)
		{
			for (int32 ChildIndex : Nodes[NodeIndex].Children)
			{
				uint32* Found = VisitCount.Find(ChildIndex);
				if (Found)
				{
					(*Found)++;
				}
				else
				{
					VisitCount.Add(ChildIndex) = 1;
				}
			}
		}

		NodeIndices.Empty();
		for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); ++NodeIndex)
		{
			if (!VisitCount.Contains(NodeIndex))
				NodeIndices.Add(NodeIndex);
			else
				ensure(VisitCount[NodeIndex] == 1);
		}
	}

	FAsset::EValidationCheck FAsset::ValidationCheck() const
	{
		int32 Res = EValidationCheck::Valid;
		if (Meshes.FindByPredicate([](const FMesh& Mesh) { return !Mesh.IsValid(); }))
		{
			Res |= InvalidMeshPresent;
		}
		
		if (Nodes.FindByPredicate([this](const FNode& Node)
			{
				if (Node.MeshIndex != INDEX_NONE)
				{
					//check Mesh index validity:
					if (!Meshes.IsValidIndex(Node.MeshIndex))
					{
						return false;
					}

					//Check Morph Target Weights:
					//	Number of MorphTargets across primitives are validated in Mesh.IsValid.
					if (Node.MorphTargetWeights.Num() > 0 && (Node.MorphTargetWeights.Num() != Meshes[Node.MeshIndex].NumberOfMorphTargetsPerPrimitive()))
					{
						return false;
					}
				}

				return !Node.Transform.IsValid();
			}))
		{
			Res |= InvalidNodePresent;
		}

		// TODO: lots more validation

		// TODO: verify if image format is PNG or JPEG

		return static_cast<EValidationCheck>(Res);
	}

	const TCHAR* ToString(EExtension Extension)
	{
		switch (Extension)
		{
			case GLTF::EExtension::KHR_MaterialsPbrSpecularGlossiness:
				return TEXT("KHR_materials_pbrSpecularGlossiness");
			case GLTF::EExtension::KHR_MaterialsUnlit:
				return TEXT("KHR_materials_unlit");
			case GLTF::EExtension::KHR_MaterialsClearCoat:
				return TEXT("KHR_materials_clearcoat");
			case GLTF::EExtension::KHR_MaterialsTransmission:
				return TEXT("KHR_materials_transmission");
			case GLTF::EExtension::KHR_MaterialsSheen:
				return TEXT("KHR_materials_sheen");
			case GLTF::EExtension::KHR_MaterialsVariants:
				return TEXT("KHR_materials_variants");
			case GLTF::EExtension::KHR_MaterialsIOR:
				return TEXT("KHR_materials_ior");
			case GLTF::EExtension::KHR_MaterialsSpecular:
				return TEXT("KHR_materials_specular");
			case GLTF::EExtension::KHR_MaterialsEmissiveStrength:
				return TEXT("KHR_materials_emissive_strength");
			case GLTF::EExtension::KHR_MaterialsIridescence:
				return TEXT("KHR_materials_iridescence");
			case GLTF::EExtension::KHR_MaterialsAnisotropy:
				return TEXT("KHR_materials_anisotropy");
			case GLTF::EExtension::KHR_MaterialsVolume:
				return TEXT("KHR_materials_volume");
			case GLTF::EExtension::KHR_TextureTransform:
				return TEXT("KHR_texture_transform");
			case GLTF::EExtension::KHR_DracoMeshCompression:
				return TEXT("KHR_draco_mesh_compression");
			case GLTF::EExtension::KHR_LightsPunctual:
				return TEXT("KHR_lights_punctual");
			case GLTF::EExtension::KHR_Lights:
				return TEXT("KHR_lights");
			case GLTF::EExtension::EXT_LightsIES:
				return TEXT("EXT_lights_ies");
			case GLTF::EExtension::KHR_Blend:
				return TEXT("KHR_blend");
			case GLTF::EExtension::MSFT_TextureDDS:
				return TEXT("MSFT_texture_dds");
			case GLTF::EExtension::MSFT_PackingNormalRoughnessMetallic:
				return TEXT("MSFT_packing_normalRoughnessMetallic");
			case GLTF::EExtension::MSFT_PackingOcclusionRoughnessMetallic:
				return TEXT("MSFT_packing_occlusionRoughnessMetallic");
			case GLTF::EExtension::KHR_MeshQuantization:
				return TEXT("KHR_mesh_quantization");
			case GLTF::EExtension::Count:
				ensure(false);
			default:
				return TEXT("UnknwonExtension");
		}
	}

	FAccessor& FAsset::CreateBuffersForAccessorIndex(uint32 AccessorIndex)
	{
		if (!Accessors.IsValidIndex(AccessorIndex))
		{
			static FAccessor EmptyAccessor;
			return EmptyAccessor;
		}

		FAccessor& Accessor = Accessors[AccessorIndex];

		uint32 NumberOfComponents = Accessor.NumberOfComponents;
		uint32 Stride = Accessor.ByteStride;
		uint32 Count = Accessor.Count;
		uint64 BufferSize = Stride * Count;

		//Create Storage for Binary Data
		TArray64<uint8>& BinaryData = UncompressedDracoBinData.AddDefaulted_GetRef();
		BinaryData.Reserve(BufferSize);
		BinaryData.SetNumUninitialized(BufferSize);

		//Create Buffer
		FBuffer Buffer(BufferSize);
		Buffer.Data = BinaryData.GetData();

		//create UncompressedBufferView
		FBufferView UncompressedBufferView(Buffer, 0, BufferSize, Stride);

		//Set the new FBufferView on the FAccessor:
		Accessor.BufferView = UncompressedBufferView;

		return Accessor;
	}
}  // namespace GLTF
