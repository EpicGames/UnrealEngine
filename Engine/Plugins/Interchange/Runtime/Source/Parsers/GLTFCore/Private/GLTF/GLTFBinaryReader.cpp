// Copyright Epic Games, Inc. All Rights Reserved.

#include "GLTFBinaryReader.h"

#include "CoreGlobals.h"
#include "Internationalization/Internationalization.h"

namespace GLTF
{
	namespace
	{
		// round up to the nearest multiple of 4 (used for padding & vertex attribute alignment)
		inline uint32 Pad4(uint32 X)
		{
			return (X + 3) & ~3;
		}
		inline uint64 Pad4(uint64 X)
		{
			return (X + 3) & ~3;
		}

		bool SignatureMatches(uint32 Signature, const char* ExpectedSignature)
		{
			return Signature == *(const uint32*)ExpectedSignature;
		}

		bool IsHeaderValid(FArchive& Archive)
		{
			// Binary glTF files begin with a 12-byte header:
			// - magic bytes "glTF"
			// - format version
			// - size of this file

			const int64 FileSize = Archive.TotalSize();
			if (FileSize < 12)
			{
				return false;
			}

			uint32 Magic;
			Archive.SerializeInt(Magic, MAX_uint32);
			bool MagicOk = SignatureMatches(Magic, "glTF");

			uint32 Version;
			Archive.SerializeInt(Version, MAX_uint32);
			bool VersionOk = Version == 2;

			uint32 Size;
			Archive.SerializeInt(Size, MAX_uint32);
			bool SizeOk = Size == FileSize;

			return MagicOk && VersionOk && SizeOk;
		}

		bool ReadJSONChunk(FArchive& FileReader, const char* ExpectedChunkType, bool& OutHasMoreData, TArray<uint8>& OutData)
		{
			// Align to next 4-byte boundary before reading anything
			uint32 Offset        = FileReader.Tell();
			uint32 AlignedOffset = Pad4(Offset);
			if (Offset != AlignedOffset)
			{
				FileReader.Seek(AlignedOffset);
			}

			// Each chunk has the form [Size][Type][...Data...]
			uint32 ChunkType, ChunkDataSize;
			FileReader.SerializeInt(ChunkDataSize, MAX_uint32);
			FileReader.SerializeInt(ChunkType, MAX_uint32);

			constexpr uint32 ChunkHeaderSize = 8;
			const uint32     AvailableData   = FileReader.TotalSize() - (AlignedOffset + ChunkHeaderSize);

			// Is there room for another chunk after this one?
			OutHasMoreData = AvailableData - Pad4(ChunkDataSize) >= ChunkHeaderSize;

			// Is there room for this chunk's data? (should always be true)
			if (ChunkDataSize > AvailableData)
			{
				return false;
			}

			if (SignatureMatches(ChunkType, ExpectedChunkType))
			{
				// Read this chunk's data
				OutData.SetNumUninitialized(ChunkDataSize, EAllowShrinking::Yes);
				FileReader.Serialize(OutData.GetData(), ChunkDataSize);
				return true;
			}
			else
			{
				// Skip past this chunk's data
				FileReader.Seek(AlignedOffset + ChunkHeaderSize + ChunkDataSize);
				return false;
			}
		}

		bool ReadBinChunk(FArchive& FileReader, const char* ExpectedChunkType, bool& OutHasMoreData, TArray64<uint8>& OutData)
		{
			// Align to next 4-byte boundary before reading anything
			uint64 Offset = FileReader.Tell();
			uint64 AlignedOffset = Pad4(Offset);
			if (Offset != AlignedOffset)
			{
				FileReader.Seek(AlignedOffset);
			}

			// Each chunk has the form [Size][Type][...Data...]
			uint32 ChunkType, ChunkDataSize;
			FileReader << ChunkDataSize;
			FileReader << ChunkType;

			constexpr uint64 ChunkHeaderSize = 8;
			const uint64     AvailableData = FileReader.TotalSize() - (AlignedOffset + ChunkHeaderSize);

			// Is there room for another chunk after this one?
			OutHasMoreData = AvailableData - Pad4(ChunkDataSize) >= ChunkHeaderSize;

			// Is there room for this chunk's data? (should always be true)
			if (ChunkDataSize > AvailableData)
			{
				return false;
			}

			if (SignatureMatches(ChunkType, ExpectedChunkType))
			{
				// Read this chunk's data
				OutData.SetNumUninitialized(ChunkDataSize, EAllowShrinking::Yes);
				FileReader.Serialize(OutData.GetData(), ChunkDataSize);
				return true;
			}
			else
			{
				// Skip past this chunk's data
				FileReader.Seek(AlignedOffset + ChunkHeaderSize + ChunkDataSize);
				return false;
			}
		}
	}

	FBinaryFileReader::FBinaryFileReader()
	    : BinChunk(nullptr)
	{
	}

	bool FBinaryFileReader::ReadFile(FArchive& FileReader)
	{
		if (!ensure(BinChunk))
		{
			return false;
		}

		Messages.Empty();

		// Binary glTF files begin with a 12-byte header
		// followed by 1 chunk of JSON and (optionally) 1 chunk of binary data.

		if (!IsHeaderValid(FileReader))
		{
			Messages.Emplace(EMessageSeverity::Error, NSLOCTEXT("InterchangeGLTFBinaryReader", "InvalidHeader", "Invalid GLTF header!"));
			return false;
		}

		JsonChunk.Empty();
		BinChunk->Empty();

		bool HasMoreData;
		if (ReadJSONChunk(FileReader, "JSON", HasMoreData, JsonChunk))
		{
			// Get BIN chunk if present
			while (HasMoreData)
			{
				if (ReadBinChunk(FileReader, "BIN", HasMoreData, *BinChunk))
				{
					break;
				}
			}
		}
		return JsonChunk.Num() > 0;
	}

}  // namespace GLTF
