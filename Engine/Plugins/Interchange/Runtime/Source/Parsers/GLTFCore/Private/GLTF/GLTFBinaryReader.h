// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GLTFLogger.h"


struct FScriptContainerElement;

namespace GLTF
{
	class FBinaryFileReader : public FBaseLogger
	{
	public:
		FBinaryFileReader();

		bool ReadFile(FArchive& FileReader);

		void SetBuffer(TArray64<uint8>& InBuffer);

		const TArray<uint8>& GetJsonBuffer() const;

	private:
		TArray<uint8> JsonChunk;

		TArray64<uint8>* BinChunk;
	};

	inline void FBinaryFileReader::SetBuffer(TArray64<uint8>& InBuffer)
	{
		BinChunk = &InBuffer;
	}

	inline const TArray<uint8>& FBinaryFileReader::GetJsonBuffer() const
	{
		return JsonChunk;
	}
}
