// Copyright Epic Games, Inc. All Rights Reserved.

#include "GLTFMaterial.h"

namespace GLTF
{
	FMD5Hash FMaterial::GetHash() const
	{
		auto HashTextureMap = [](FMD5& MD5, const FTextureMap& Map)
		{
			FMD5Hash MapHash = Map.GetHash();
			if (MapHash.IsValid())
			{
				MD5.Update(MapHash.GetBytes(), MapHash.GetSize());
			}
		};

		FMD5 MD5;

		uint8 ShadingModelInt = static_cast<uint8>(ShadingModel);
		uint8 AlphaModeInt = static_cast<uint8>(AlphaMode);
		uint8 IsDoubleSized = static_cast<uint8>(bIsDoubleSided);
		uint8 IsUnlitShadingModel = static_cast<uint8>(bIsUnlitShadingModel);

		HashTextureMap(MD5, BaseColor);
		MD5.Update(reinterpret_cast<const uint8*>(&BaseColorFactor), sizeof(BaseColorFactor));
		MD5.Update(&ShadingModelInt, sizeof(ShadingModelInt));

		HashTextureMap(MD5, MetallicRoughness.Map);
		MD5.Update(reinterpret_cast<const uint8*>(&MetallicRoughness.MetallicFactor), sizeof(MetallicRoughness.MetallicFactor));
		MD5.Update(reinterpret_cast<const uint8*>(&MetallicRoughness.RoughnessFactor), sizeof(MetallicRoughness.RoughnessFactor));

		HashTextureMap(MD5, SpecularGlossiness.Map);
		MD5.Update(reinterpret_cast<const uint8*>(&SpecularGlossiness.GlossinessFactor), sizeof(SpecularGlossiness.GlossinessFactor));
		MD5.Update(reinterpret_cast<const uint8*>(&SpecularGlossiness.SpecularFactor), sizeof(SpecularGlossiness.SpecularFactor));

		HashTextureMap(MD5, Normal);
		MD5.Update(reinterpret_cast<const uint8*>(&NormalScale), sizeof(NormalScale));

		HashTextureMap(MD5, Occlusion);
		MD5.Update(reinterpret_cast<const uint8*>(&OcclusionStrength), sizeof(OcclusionStrength));

		HashTextureMap(MD5, Emissive);
		MD5.Update(reinterpret_cast<const uint8*>(&EmissiveFactor), sizeof(EmissiveFactor));

		MD5.Update(&IsDoubleSized, sizeof(IsDoubleSized));
		MD5.Update(&AlphaModeInt, sizeof(AlphaModeInt));
		MD5.Update(reinterpret_cast<const uint8*>(&AlphaCutoff), sizeof(AlphaCutoff));

		MD5.Update(reinterpret_cast<const uint8*>(&Packing.Flags), sizeof(Packing.Flags));
		HashTextureMap(MD5, Packing.Map);
		HashTextureMap(MD5, Packing.NormalMap);

		MD5.Update(&IsUnlitShadingModel, sizeof(IsUnlitShadingModel));

		FMD5Hash Hash;
		Hash.Set(MD5);
		return Hash;
	}
}; // namespace GLTF
