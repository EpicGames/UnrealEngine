// Copyright Epic Games, Inc. All Rights Reserved.

#include "GLTFMesh.h"

#include "ConversionUtilities.h"

namespace GLTF
{
	FAttributeAccessors::FAttributeAccessors(const TArray<TPair<EMeshAttributeType, FAccessor&>>& InAttributeAccessors, bool bInMorphTarget)
	{
		
		for (uint8 AttributeTypeCounter = 0; AttributeTypeCounter < EMeshAttributeType::COUNT; AttributeTypeCounter++)
		{
			EMeshAttributeType AttributeType = EMeshAttributeType(AttributeTypeCounter);

			if (InAttributeAccessors.Num() <= AttributeTypeCounter)
			{
				break;
			}

			if (InAttributeAccessors[AttributeType].Key == AttributeType)
			{
				AttributeAccessors.Emplace(AttributeType, InAttributeAccessors[AttributeType].Value);
			}
			else
			{
				//Expectation mismatch, update InAttributeAccessors array based on EMeshAttributeType.
				ensure(false);

				static FAccessor EmptyAccessor;

				AttributeAccessors.Emplace(AttributeType, EmptyAccessor);
			}
		}

		bMorphTarget = bInMorphTarget;
	}

	bool FAttributeAccessors::HasAttributeAccessor(const EMeshAttributeType& Type) const
	{
		if (AttributeAccessors.Num() > Type && AttributeAccessors[Type].Key == Type)
		{
			return AttributeAccessors[Type].Value.IsValid();
		}

		return false;
	}
	
	const FAccessor& FAttributeAccessors::GetAttributeAccessor(const EMeshAttributeType& Type) const
	{
		if (AttributeAccessors.Num() > Type && AttributeAccessors[Type].Key == Type)
		{
			return AttributeAccessors[Type].Value;
		}

		static FAccessor EmptyAccessor;
		return EmptyAccessor;
	}

	bool FAttributeAccessors::AreAttributeAccessorsValid(bool& bHasData, uint32 ExpectedElementCount) const
	{
		bHasData = false;

		for (const TPair<EMeshAttributeType, const FAccessor&> AccessorPair : AttributeAccessors)
		{
			if (AccessorPair.Value.IsValid())
			{
				bHasData = true;

				if (AccessorPair.Value.Count != ExpectedElementCount)
				{
					return false;
				}
				if (!AccessorPair.Value.IsValidDataType(AccessorPair.Key, bMorphTarget))
				{
					return false;
				}
			}
		}

		return true;
	}

	void FAttributeAccessors::HashAttributes(FMD5& MD5) const
	{
		auto HashAccessor = [](FMD5& MD5, const FAccessor& Accessor)
			{
				if (!Accessor.IsValid())
				{
					return;
				}

				FMD5Hash MD5Hash = Accessor.GetHash();
				if (MD5Hash.IsValid())
				{
					MD5.Update(MD5Hash.GetBytes(), MD5Hash.GetSize());
				}
			};

		for (const TPair<EMeshAttributeType, const FAccessor&> AccessorPair : AttributeAccessors)
		{
			HashAccessor(MD5, AccessorPair.Value);
		}
	}

	FMorphTarget::FMorphTarget(const TArray<TPair<EMeshAttributeType, FAccessor&>>& InAttributeAccessors)
		: FAttributeAccessors(InAttributeAccessors, true)
	{
	}

	bool FMorphTarget::IsValid(uint32 ExpectedElementCount) const
	{
		bool bHasData = false;
		if (!AreAttributeAccessorsValid(bHasData, ExpectedElementCount))
		{
			return false;
		}

		return bHasData;
	}

	FMD5Hash FMorphTarget::GetHash() const
	{
		FMD5 MD5;

		HashAttributes(MD5);
		
		FMD5Hash Hash;
		Hash.Set(MD5);

		return Hash;
	}

	void FMorphTarget::GetPositionDisplacements(TArray<FVector3f>& Buffer) const
	{
		GetAttributeAccessor(EMeshAttributeType::POSITION).GetCoordArray(Buffer);
	}
	int32 FMorphTarget::GetNumberOfPositionDisplacements() const
	{
		return GetAttributeAccessor(EMeshAttributeType::POSITION).Count;
	}

	void FMorphTarget::GetNormalDisplacements(TArray<FVector3f>& Buffer) const
	{
		GetAttributeAccessor(EMeshAttributeType::NORMAL).GetCoordArray(Buffer);
	}
	int32 FMorphTarget::GetNumberOfNormalDisplacements() const
	{
		return GetAttributeAccessor(EMeshAttributeType::NORMAL).Count;
	}

	void FMorphTarget::GetTangentDisplacements(TArray<FVector4f>& Buffer) const
	{
		const FAccessor& Tangent = GetAttributeAccessor(EMeshAttributeType::TANGENT);
		
		if (!Tangent.IsValid())
		{
			return;
		}

		const int32 N = Tangent.Count;
		Buffer.Reserve(N);
		for (int32 Index = 0; Index < N; ++Index)
		{
			const FVector TangentVector = Tangent.GetVec3(Index);
			Buffer.Emplace((FVector3f)GLTF::ConvertVec3(TangentVector), 0.f);
		}
	}
	int32 FMorphTarget::GetNumberOfTangentDisplacements() const
	{
		return GetAttributeAccessor(EMeshAttributeType::TANGENT).Count;
	}

	bool FMorphTarget::HasTexCoordDisplacements(uint32 Index) const
	{
		if (Index < SupportedTexCoordNum)
		{
			const uint32 TexcoordAttributeIndex = EMeshAttributeType::TEXCOORD_0 + Index;
			return HasAttributeAccessor(EMeshAttributeType(TexcoordAttributeIndex));
		}

		return false;
	}
	void FMorphTarget::GetTexCoordDisplacements(uint32 Index, TArray<FVector2f>& Buffer) const
	{
		if (Index < SupportedTexCoordNum)
		{
			const uint32 TexcoordAttributeIndex = EMeshAttributeType::TEXCOORD_0 + Index;
			return GetAttributeAccessor(EMeshAttributeType(TexcoordAttributeIndex)).GetVec2Array(Buffer);
		}
	}
	int32 FMorphTarget::GetNumberOfTexCoordDisplacements(uint32 Index) const
	{
		if (Index < SupportedTexCoordNum)
		{
			const uint32 TexcoordAttributeIndex = EMeshAttributeType::TEXCOORD_0 + Index;
			return GetAttributeAccessor(EMeshAttributeType(TexcoordAttributeIndex)).Count;
		}

		return 0;
	}

	void FMorphTarget::GetColorDeltas(TArray<FVector4f>& Buffer) const
	{
		const FAccessor& Color0Deltas = GetAttributeAccessor(EMeshAttributeType::COLOR_0);

		if (!Color0Deltas.IsValid())
		{
			return;
		}

		if (Color0Deltas.Type == FAccessor::EType::Vec4)
		{
			Color0Deltas.GetVec4Array(Buffer);
		}
		else if (Color0Deltas.Type == FAccessor::EType::Vec3)
		{
			const int32 N = Color0Deltas.Count;
			Buffer.Reserve(N);
			for (int32 Index = 0; Index < N; ++Index)
			{
				const FVector Vec = Color0Deltas.GetVec3(Index);
				Buffer.Emplace((FVector3f)Vec, 0.f);
			}
		}
	}
	int32 FMorphTarget::GetNumberOfColorDeltas() const
	{
		return GetAttributeAccessor(EMeshAttributeType::COLOR_0).Count;
	}


	const TArray<FPrimitive::EMode> FPrimitive::SupportedModes = { FPrimitive::EMode::Triangles, FPrimitive::EMode::TriangleStrip, FPrimitive::EMode::TriangleFan };
	FString FPrimitive::ToString(const FPrimitive::EMode& Mode)
	{
		switch (Mode)
		{
			case EMode::Points:         return TEXT("POINTS");
			case EMode::Lines:          return TEXT("LINES");
			case EMode::LineLoop:       return TEXT("LINE_LOOP");
			case EMode::LineStrip:      return TEXT("LINE_STRIP");
			case EMode::Triangles:      return TEXT("TRIANGLES");
			case EMode::TriangleStrip:  return TEXT("TRIANGLE_STRIP");
			case EMode::TriangleFan:    return TEXT("TRIANGLE_FAN");

			case EMode::Unknown:
			default:                    return TEXT("UNKNOWN");
		}
	}

	FPrimitive::FPrimitive(EMode InMode, int32 InMaterial, const FAccessor& InIndices, const TArray<TPair<EMeshAttributeType, FAccessor&>>& InAttributeAccessors)
	    : FAttributeAccessors(InAttributeAccessors, false)
		, Mode(InMode)
	    , MaterialIndex(InMaterial)
	    , Indices(InIndices)
	{
		
	}

	void FPrimitive::GenerateIsValidCache()
	{
		bIsValidCache = IsValidPrivate();
	}

	bool FPrimitive::IsValid() const
	{
		if (bIsValidCache.IsSet())
		{
			return bIsValidCache.GetValue();
		}
		else
		{
			return IsValidPrivate();
		}
	}

	bool FPrimitive::IsValidPrivate() const
	{
		//Position is always required:
		if (!HasPositions())
		{
			return false;
		}

		const uint32 VertexCount = GetAttributeAccessor(EMeshAttributeType::POSITION).Count;
		
		//Validate Index Ranges:
		{
			TArray<uint32> IndicesValues;
			GetTriangleIndices(IndicesValues);
			for (uint32 Index : IndicesValues)
			{
				if (VertexCount <= Index)
				{
					return false;
				}
			}
		}

		// make sure all semantic attributes meet the spec
		bool bHasData = false;
		if (!AreAttributeAccessorsValid(bHasData, VertexCount))
		{
			return false;
		}

		//Morph Targets:
		for (const FMorphTarget& MorphTarget : MorphTargets)
		{
			if (!MorphTarget.IsValid(VertexCount))
			{
				return false;
			}
		}

		return true;
	}

	FMD5Hash FPrimitive::GetHash() const
	{
		if (!IsValid())
		{
			return FMD5Hash();
		}

		auto HashAccessor = [](FMD5& MD5, const FAccessor& Accessor)
		{
			if (!Accessor.IsValid())
			{
				return;
			}

			FMD5Hash MD5Hash = Accessor.GetHash();
			if (MD5Hash.IsValid())
			{
				MD5.Update(MD5Hash.GetBytes(), MD5Hash.GetSize());
			}
		};

		FMD5 MD5;

		uint8 ModeInt = static_cast<uint8>(Mode);
		MD5.Update(&ModeInt, sizeof(ModeInt));
		MD5.Update(reinterpret_cast<const uint8*>(&MaterialIndex), sizeof(MaterialIndex));

		HashAccessor(MD5, Indices);

		HashAttributes(MD5);

		//Morph Targets:
		for (const FMorphTarget& MorphTarget : MorphTargets)
		{
			FMD5Hash MorphTargetHash = MorphTarget.GetHash();
			if (MorphTargetHash.IsValid())
			{
				MD5.Update(MorphTargetHash.GetBytes(), MorphTargetHash.GetSize());
			}
		}

		FMD5Hash Hash;
		Hash.Set(MD5);
		return Hash;
	}

	bool FPrimitive::HasTexCoords(uint32 Index) const
	{
		if (Index < SupportedTexCoordNum)
		{
			const uint32 TexcoordAttributeIndex = EMeshAttributeType::TEXCOORD_0 + Index;
			return HasAttributeAccessor(EMeshAttributeType(TexcoordAttributeIndex));
		}

		return false;
		
	}

	void FPrimitive::GetPositions(TArray<FVector3f>& Buffer) const
	{
		GetAttributeAccessor(EMeshAttributeType::POSITION).GetCoordArray(Buffer);
	}

	void FPrimitive::GetNormals(TArray<FVector3f>& Buffer) const
	{
		GetAttributeAccessor(EMeshAttributeType::NORMAL).GetCoordArray(Buffer);
	}

	void FPrimitive::GetTangents(TArray<FVector4f>& Buffer) const
	{
		const FAccessor& Accessor = GetAttributeAccessor(EMeshAttributeType::TANGENT);

		if (!Accessor.IsValid())
		{
			return;
		}

		Buffer.Reserve(Accessor.Count);
		for (uint32 Index = 0; Index < Accessor.Count; ++Index)
		{
			Buffer.Push((FVector4f)GLTF::ConvertTangent(Accessor.GetVec4(Index)));
		}
	}

	void FPrimitive::GetTexCoords(uint32 Index, TArray<FVector2f>& Buffer) const
	{
		if (Index < SupportedTexCoordNum)
		{
			const uint32 TexcoordAttributeIndex = EMeshAttributeType::TEXCOORD_0 + Index;
			return GetAttributeAccessor(EMeshAttributeType(TexcoordAttributeIndex)).GetVec2Array(Buffer);
		}
	}

	void FPrimitive::GetColors(TArray<FVector4f>& Buffer) const
	{
		const FAccessor& Accessor = GetAttributeAccessor(EMeshAttributeType::COLOR_0);

		if (!Accessor.IsValid())
		{
			return;
		}

		if (Accessor.Type == FAccessor::EType::Vec4)
		{
			Accessor.GetVec4Array(Buffer);
		}
		else if (Accessor.Type == FAccessor::EType::Vec3)
		{
			const int32 N = Accessor.Count;
			Buffer.Reserve(N);
			for (int32 Index = 0; Index < N; ++Index)
			{
				const FVector Vec = Accessor.GetVec3(Index);
				Buffer.Emplace((FVector3f)Vec, 1.f);
			}
		}
	}

	void FPrimitive::GetJointInfluences(TArray<FJointInfluence>& Buffer) const
	{
		const FAccessor& Joints0 = GetAttributeAccessor(EMeshAttributeType::JOINTS_0);
		const FAccessor& Weights0 = GetAttributeAccessor(EMeshAttributeType::WEIGHTS_0);

		// return a flat array that corresponds 1-to-1 with vertex positions
		const int32 N = Joints0.Count;
		Buffer.Reserve(N);
		for (int32 Index = 0; Index < N; ++Index)
		{
			FJointInfluence& Joint = Buffer.Emplace_GetRef(Weights0.GetVec4(Index));
			Joints0.GetUnsignedInt16x4(Index, Joint.ID);
		}
	}

	FTriangle FPrimitive::TriangleVerts(uint32 T) const
	{
		FTriangle Result;
		if (T >= TriangleCount())
			return Result;

		const bool Indexed = Indices.IsValid();
		switch (Mode)
		{
		case EMode::Triangles:
			if (Indexed)
			{
				Result.A = Indices.GetUnsignedInt(3 * T);
				Result.B = Indices.GetUnsignedInt(3 * T + 1);
				Result.C = Indices.GetUnsignedInt(3 * T + 2);
			}
			else
			{
				Result.A = 3 * T;
				Result.B = 3 * T + 1;
				Result.C = 3 * T + 2;
			}
			break;
		case EMode::TriangleStrip:
			// are indexed TriangleStrip & TriangleFan valid?
			// I don't see anything in the spec that says otherwise...
			if (Indexed)
			{
				if (T % 2 == 0)
				{
					Result.A = Indices.GetUnsignedInt(T);
					Result.B = Indices.GetUnsignedInt(T + 1);
				}
				else
				{
					Result.A = Indices.GetUnsignedInt(T + 1);
					Result.B = Indices.GetUnsignedInt(T);
				}
				Result.C = Indices.GetUnsignedInt(T + 2);
			}
			else
			{
				if (T % 2 == 0)
				{
					Result.A = T;
					Result.B = T + 1;
				}
				else
				{
					Result.A = T + 1;
					Result.B = T;
				}
				Result.C = T + 2;
			}
			break;
		case EMode::TriangleFan:
			if (Indexed)
			{
				Result.A = Indices.GetUnsignedInt(0);
				Result.B = Indices.GetUnsignedInt(T + 1);
				Result.C = Indices.GetUnsignedInt(T + 2);
			}
			else
			{
				Result.A = 0;
				Result.B = T + 1;
				Result.C = T + 2;
			}
			break;
		default:
			break;
		}

		return Result;
	}

	void FPrimitive::GetTriangleIndices(TArray<uint32>& Buffer) const
	{
		if (Mode == EMode::Triangles)
		{
			if (Indices.IsValid())
			{
				return Indices.GetUnsignedIntArray(Buffer);
			}
			else
			{
				// generate indices [0 1 2][3 4 5]...
				const uint32 N = TriangleCount() * 3;
				Buffer.Reserve(N);
				for (uint32 Index = 0; Index < N; ++Index)
				{
					Buffer.Push(Index);
				}
			}
		}
		else
		{
			const uint32 N = TriangleCount() * 3;
			Buffer.Reserve(N);
			for (uint32 Index = 0; Index < TriangleCount(); ++Index)
			{
				FTriangle Tri = TriangleVerts(Index);
				Buffer.Push(Tri.A);
				Buffer.Push(Tri.B);
				Buffer.Push(Tri.C);
			}
		}
	}

	uint32 FPrimitive::VertexCount() const
	{
		if (Indices.IsValid())
			return Indices.Count;
		else
			return GetAttributeAccessor(EMeshAttributeType::POSITION).Count;
	}

	uint32 FPrimitive::TriangleCount() const
	{
		switch (Mode)
		{
		case EMode::Triangles:
			return VertexCount() / 3;
		case EMode::TriangleStrip:
		case EMode::TriangleFan:
			return VertexCount() - 2;
		default:
			return 0;
		}
	}

	FMD5Hash FMesh::GetHash() const
	{
		if (!IsValid())
		{
			return FMD5Hash();
		}

		FMD5 MD5;

		for (const GLTF::FPrimitive& Primitive : Primitives)
		{
			FMD5Hash PrimitiveHash = Primitive.GetHash();
			if (PrimitiveHash.IsValid())
			{
				MD5.Update(PrimitiveHash.GetBytes(), PrimitiveHash.GetSize());
			}
		}

		FMD5Hash Hash;
		Hash.Set(MD5);
		return Hash;
	}

	//This is a helper functions for the validators.
	//We are allowed to check only the first primitive's morph target count,
	// because if there is a incosonsitency acorss primitive targets then the mesh IsValid will report false already.
	int32 FMesh::NumberOfMorphTargetsPerPrimitive() const
	{
		int32 MorphTargetCounter = Primitives.Num() > 0 ? Primitives[0].MorphTargets.Num() : 0;

		return MorphTargetCounter;
	}

}  // namespace GLTF
