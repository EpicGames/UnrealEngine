// Copyright Epic Games, Inc. All Rights Reserved.

#include "GLTFReader.h"

#include "ExtensionsHandler.h"
#include "GLTF/ConversionUtilities.h"
#include "GLTFAsset.h"
#include "GLTF/JsonUtilities.h"
#include "GLTFBinaryReader.h"
#include "GLTFNode.h"
#include "MaterialUtilities.h"

#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Base64.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonSerializer.h"

#include "GenericPlatform/GenericPlatformHttp.h"


#define LOCTEXT_NAMESPACE "InterchangeGLTFReader"

namespace GLTF
{
	namespace
	{
		bool DecodeDataURI(const FString& URI, FString& OutMimeType, uint8* OutData, uint32& OutDataSize)
		{
			// Data URIs look like "data:[<mime-type>][;encoding],<data>"
			// glTF always uses base64 encoding for data URIs

			if (!ensure(URI.StartsWith(TEXT("data:"))))
			{
				return false;
			}

			int32      Semicolon, Comma;
			const bool HasSemicolon = URI.FindChar(TEXT(';'), Semicolon);
			const bool HasComma     = URI.FindChar(TEXT(','), Comma);
			if (!(HasSemicolon && HasComma))
			{
				return false;
			}

			const FString Encoding = URI.Mid(Semicolon + 1, Comma - Semicolon - 1);
			if (Encoding != TEXT("base64"))
			{
				return false;
			}

			OutMimeType = URI.Mid(5, Semicolon - 5);

			const FString EncodedData = URI.RightChop(Comma + 1);
			OutDataSize               = FBase64::GetDecodedDataSize(EncodedData);
			return FBase64::Decode(*EncodedData, EncodedData.Len(), OutData);
		}

		uint32 GetDecodedDataSize(const FString& URI, FString& OutMimeType)
		{
			// Data URIs look like "data:[<mime-type>][;encoding],<data>"
			// glTF always uses base64 encoding for data URIs

			if (!ensure(URI.StartsWith(TEXT("data:"))))
			{
				return false;
			}

			int32      Semicolon, Comma;
			const bool HasSemicolon = URI.FindChar(TEXT(';'), Semicolon);
			const bool HasComma     = URI.FindChar(TEXT(','), Comma);
			if (!(HasSemicolon && HasComma))
			{
				return 0;
			}

			const FString Encoding = URI.Mid(Semicolon + 1, Comma - Semicolon - 1);
			if (Encoding != TEXT("base64"))
			{
				return 0;
			}

			OutMimeType = URI.Mid(5, Semicolon - 5);

			const FString EncodedData = URI.RightChop(Comma + 1);
			return FBase64::GetDecodedDataSize(EncodedData);
		}

		FAccessor& AccessorAtIndex(TArray<FAccessor>& Accessors, int32 Index)
		{
			if (Accessors.IsValidIndex(Index))
			{
				return Accessors[Index];
			}
			else
			{
				static FAccessor EmptyAccessor;
				return EmptyAccessor;
			}
		}

		template<typename T>
		void SetTransformFromMatrix(FTransform& Transform, const UE::Math::TMatrix<T>& InMatrix)
		{
			using namespace UE::Math;

			TMatrix<T> M = InMatrix;

			// Get the 3D scale from the matrix
			TVector<T> OutScale3D = M.ExtractScaling(0); //we want Precise Scaling with 0 tolerance.

			// If there is negative scaling going on, we handle that here
			if (InMatrix.Determinant() < 0.f)
			{
				// Assume it is along X and modify transform accordingly. 
				// It doesn't actually matter which axis we choose, the 'appearance' will be the same
				OutScale3D.X *= -1.f;
				M.SetAxis(0, -M.GetScaledAxis(EAxis::X));
			}

			TQuat<T> OutRotation = TQuat<T>(M);
			TVector<T> OutTranslation = InMatrix.GetOrigin();

			// Normalize rotation
			OutRotation.Normalize();

			OutRotation = GLTF::ConvertQuat(OutRotation);

			Transform.SetScale3D(OutScale3D);
			Transform.SetRotation(OutRotation);
			Transform.SetTranslation(OutTranslation);
		}

		void ProcessExtras(const FJsonObject& Object, TMap<FString, FString>& StorageForExtras, TSet<FString> KeyExceptions = {})
		{
			if (Object.HasField(TEXT("extras")))
			{
				const TSharedPtr<FJsonObject>& Extras = Object.GetObjectField(TEXT("extras"));
				if (KeyExceptions.Num() > 0)
				{
					for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : Extras->Values)
					{
						FString KeyStr(Pair.Key);
						if (KeyExceptions.Contains(KeyStr))
						{
							continue;
						}

						FString ExtraString;
						if (Pair.Value->TryGetString(ExtraString))
						{
							StorageForExtras.Add(MoveTemp(KeyStr), ExtraString);
						}
					}
				}
				else
				{
					for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : Extras->Values)
					{
						FString ExtraString;
						if (Pair.Value->TryGetString(ExtraString))
						{
							StorageForExtras.Add(FString(Pair.Key), ExtraString);
						}
					}
				}
			}
		}
	}

	FFileReader::FFileReader()
	    : BufferCount(0)
	    , BufferViewCount(0)
	    , ImageCount(0)
	    , BinaryReader(new FBinaryFileReader())
	    , ExtensionsHandler(new FExtensionsHandler(Messages))
	    , Asset(nullptr)
	{
	}

	FFileReader::~FFileReader() {}

	void FFileReader::SetupBuffer(const FJsonObject& Object, const FString& Path)
	{
		const uint64 ByteLength = GetUnsignedInt64(Object, TEXT("byteLength"), 0);
		Asset->Buffers.Emplace(ByteLength);
		FBuffer& Buffer = Asset->Buffers.Last();

		bool bUpdateOffset = false;
		if (Object.HasTypedField<EJson::String>(TEXT("uri")))
		{
			// set Buffer.Data from Object.uri

			FString URI = Object.GetStringField(TEXT("uri"));
			if (URI.StartsWith(TEXT("data:")))
			{
				FString MimeType;
				uint32  DataSize = 0;
				bool    bSuccess = DecodeDataURI(URI, MimeType, CurrentBufferOffset, DataSize);
				if (!bSuccess || (MimeType != TEXT("application/octet-stream") && MimeType != TEXT("application/gltf-buffer")) || !ensure(DataSize == ByteLength))
				{
					Messages.Emplace(EMessageSeverity::Error, LOCTEXT("BufferDecodingProblem", "Problem decoding buffer from data URI."));
				}
				else
				{
					bUpdateOffset = true;
				}
			}
			else
			{
				URI = FGenericPlatformHttp::UrlDecode(URI);

				// Load buffer from external file.
				const FString FullPath = Path / URI;
				FArchive*     Reader   = IFileManager::Get().CreateFileReader(*FullPath);
				if (Reader)
				{
					const int64 FileSize = Reader->TotalSize();
					if (ByteLength == FileSize)
					{
						Reader->Serialize(CurrentBufferOffset, ByteLength);
						bUpdateOffset = true;
					}
					else
					{
						Messages.Emplace(EMessageSeverity::Error, LOCTEXT("NonMatchingBufferFileSize", "Buffer file size does not match."));
					}

					Reader->Close();
					delete Reader;
				}
				else
				{
					Messages.Emplace(EMessageSeverity::Error, FText::Format(LOCTEXT("LoadingFileFailed", "Could not load file: '{0}'"), FText::FromString(FullPath)));
				}
			}
		}
		else
		{
			// Missing URI means use binary chunk of GLB
			const uint32 BinSize = Asset->BinData.Num();
			if (BinSize == 0)
			{
				Messages.Emplace(EMessageSeverity::Error, LOCTEXT("BINChunkMissing","Buffer from BIN chunk is missing or empty."));
			}
			else if (BinSize < ByteLength)
			{
				Messages.Emplace(EMessageSeverity::Error, LOCTEXT("BINChunkTooSmall","Buffer from BIN chunk is too small."));
			}
			else
			{
				Buffer.Data = Asset->BinData.GetData();
			}
		}

		if (bUpdateOffset)
		{
			Buffer.Data = CurrentBufferOffset;
			CurrentBufferOffset += ByteLength;
		}

		ExtensionsHandler->SetupBufferExtensions(Object, Buffer);
	}

	void FFileReader::SetupBufferView(const FJsonObject& Object) const
	{
		const uint32 BufferIdx = GetUnsignedInt(Object, TEXT("buffer"), BufferCount);
		if (BufferIdx < BufferCount)  // must be true
		{
			const uint64 ByteOffset = GetUnsignedInt64(Object, TEXT("byteOffset"), 0);
			const uint64 ByteLength = GetUnsignedInt64(Object, TEXT("byteLength"), 0);
			const uint32 ByteStride = GetUnsignedInt(Object, TEXT("byteStride"), 0);
			Asset->BufferViews.Emplace(Asset->Buffers[BufferIdx], ByteOffset, ByteLength, ByteStride);
			ExtensionsHandler->SetupBufferViewExtensions(Object, Asset->BufferViews.Last());
		}
	}

	void FFileReader::SetupAccessor(const FJsonObject& Object) const
	{
		uint32 AccessorIndex = Asset->Accessors.Num();

		const uint32 BufferViewIdx = GetUnsignedInt(Object, TEXT("bufferView"), BufferViewCount);
		if (BufferViewIdx < BufferViewCount)  // must be true
		{
			const uint64                    ByteOffset = GetUnsignedInt64(Object, TEXT("byteOffset"), 0);
			const FAccessor::EComponentType CompType   = ComponentTypeFromNumber(GetUnsignedInt(Object, TEXT("componentType"), 0));
			const uint32                    Count      = GetUnsignedInt(Object, TEXT("count"), 0);
			const FAccessor::EType          Type       = AccessorTypeFromString(Object.GetStringField(TEXT("type")));
			const bool                      Normalized = GetBool(Object, TEXT("normalized"));

			//Sparse:
			if (Object.HasField(TEXT("sparse")))
			{
				const FJsonObject& SparseObject = *Object.GetObjectField(TEXT("sparse"));

				const uint32 SparseCount = GetUnsignedInt(SparseObject, TEXT("count"), 0);

				const FJsonObject& IndicesObject = *SparseObject.GetObjectField(TEXT("indices"));
				const uint32 IndicesBufferViewIdx = GetUnsignedInt(IndicesObject, TEXT("bufferView"), BufferViewCount);
				const uint64 IndicesByteOffset = GetUnsignedInt64(IndicesObject, TEXT("byteOffset"), 0);
				const FAccessor::EComponentType IndicesCompType = ComponentTypeFromNumber(GetUnsignedInt(IndicesObject, TEXT("componentType"), 0));

				const FJsonObject& ValuesObject = *SparseObject.GetObjectField(TEXT("values"));
				const uint32 ValuesBufferViewIdx = GetUnsignedInt(ValuesObject, TEXT("bufferView"), BufferViewCount);
				const uint64 ValuesByteOffset = GetUnsignedInt64(ValuesObject, TEXT("byteOffset"), 0);

				Asset->Accessors.Emplace(AccessorIndex,
					Asset->BufferViews[BufferViewIdx], ByteOffset, 
					Count, Type, CompType, Normalized,
					FAccessor::FSparse(SparseCount, 
						Asset->BufferViews[IndicesBufferViewIdx], IndicesByteOffset, IndicesCompType,
						Asset->BufferViews[ValuesBufferViewIdx], ValuesByteOffset));
			}
			else
			{
				Asset->Accessors.Emplace(AccessorIndex,
					Asset->BufferViews[BufferViewIdx], ByteOffset, 
					Count, Type, CompType, Normalized, 
					FAccessor::FSparse());
			}
		}
		else
		{
			if (!Object.HasTypedField<EJson::Number>(TEXT("bufferView")))
			{
				//if bufferView does not exist in the Object, then the presumption is that it is a (Draco) CompressedAccessor:
				//Accessors with Sparse present can have no bufferView, in which case:
				//"When accessor.bufferView is undefined, the sparse accessor is initialized as an array of zeros of size (size of the accessor element) * (accessor.count) bytes"
				const uint32                    Count = GetUnsignedInt(Object, TEXT("count"), 0);
				const FAccessor::EType          Type = AccessorTypeFromString(Object.GetStringField(TEXT("type")));
				const FAccessor::EComponentType CompType = ComponentTypeFromNumber(GetUnsignedInt(Object, TEXT("componentType"), 0));
				const bool                      Normalized = GetBool(Object, TEXT("normalized"));

				//Sparse:
				if (Object.HasField(TEXT("sparse")))
				{
					const FJsonObject& SparseObject = *Object.GetObjectField(TEXT("sparse"));

					const uint32 SparseCount = GetUnsignedInt(SparseObject, TEXT("count"), 0);

					const FJsonObject& IndicesObject = *SparseObject.GetObjectField(TEXT("indices"));
					const uint32 IndicesBufferViewIdx = GetUnsignedInt(IndicesObject, TEXT("bufferView"), BufferViewCount);
					const uint64 IndicesByteOffset = GetUnsignedInt64(IndicesObject, TEXT("byteOffset"), 0);
					const FAccessor::EComponentType IndicesCompType = ComponentTypeFromNumber(GetUnsignedInt(IndicesObject, TEXT("componentType"), 0));

					const FJsonObject& ValuesObject = *SparseObject.GetObjectField(TEXT("values"));
					const uint32 ValuesBufferViewIdx = GetUnsignedInt(ValuesObject, TEXT("bufferView"), BufferViewCount);
					const uint64 ValuesByteOffset = GetUnsignedInt64(ValuesObject, TEXT("byteOffset"), 0);

					Asset->Accessors.Emplace(AccessorIndex,
						Count, Type, CompType, Normalized,
						FAccessor::FSparse(SparseCount,
							Asset->BufferViews[IndicesBufferViewIdx], IndicesByteOffset, IndicesCompType,
							Asset->BufferViews[ValuesBufferViewIdx], ValuesByteOffset));
				}
				else
				{
					Asset->Accessors.Emplace(AccessorIndex,
						Count, Type, CompType, Normalized, 
						FAccessor::FSparse());
				}
			}
			else
			{
				Asset->Accessors.AddDefaulted();
			}
		}
		
		ExtensionsHandler->SetupAccessorExtensions(Object, Asset->Accessors.Last());
	}

	void FFileReader::SetupMorphTarget(const FJsonObject& Object, GLTF::FPrimitive& Primitive, const bool bMeshQuantized) const
	{
		const TArray<FAccessor>& A = Asset->Accessors;

		//Morph targets do not support joints/weights, aka they go only until Color_0
		// all Accessors have to be updated to bQuantized except for Color_0

		TArray<TPair<EMeshAttributeType, FAccessor&>> AttributeAccessors;
		AttributeAccessors.Reserve(EMeshAttributeType::COLOR_0);
		for (uint8 AttributeTypeCounter = 0; AttributeTypeCounter < EMeshAttributeType::COLOR_0; AttributeTypeCounter++)
		{
			EMeshAttributeType AttributeType = EMeshAttributeType(AttributeTypeCounter);
			FAccessor& Accessor = AccessorAtIndex(Asset->Accessors, GetIndex(Object, *ToString(AttributeType)));
			Accessor.bQuantized = bMeshQuantized;
			AttributeAccessors.Emplace(AttributeType, Accessor);
		}

		FAccessor& AccessorColor0 = AccessorAtIndex(Asset->Accessors, GetIndex(Object, *ToString(EMeshAttributeType::COLOR_0)));
		AttributeAccessors.Emplace(EMeshAttributeType::COLOR_0, AccessorColor0);

		Primitive.MorphTargets.Emplace(AttributeAccessors);
	}

	void FFileReader::SetupPrimitive(const FJsonObject& Object, FMesh& Mesh, const bool bMeshQuantized, const uint32& PrimitiveIndex) const
	{
		const FPrimitive::EMode       Mode = PrimitiveModeFromNumber(GetUnsignedInt(Object, TEXT("mode"), (uint32)FPrimitive::EMode::Triangles));
		if (Mode == FPrimitive::EMode::Unknown)
		{
			return;
		}

		if (!FPrimitive::SupportedModes.Contains(Mode))
		{
			Messages.Emplace(EMessageSeverity::Warning, FText::Format(LOCTEXT("UnsupportedPrimitiveMode", 
				"Primitive Mode[{0}] in Primitive[{1}] (in Mesh[{2}]) is currently not supported. Geometry won't be imported."), 
				FText::FromString(FPrimitive::ToString(Mode)), 
				PrimitiveIndex, 
				FText::FromString(Mesh.Name)));
		}

		const int32 MaterialIndex = GetIndex(Object, TEXT("material"));
		const FAccessor& Indices = AccessorAtIndex(Asset->Accessors, GetIndex(Object, TEXT("indices")));
		
		const FJsonObject& Attributes = *Object.GetObjectField(TEXT("attributes"));

		TArray<TPair<EMeshAttributeType, FAccessor&>> AttributeAccessors;
		AttributeAccessors.Reserve(EMeshAttributeType::COUNT);
		for (uint8 AttributeTypeCounter = 0; AttributeTypeCounter < EMeshAttributeType::COLOR_0; AttributeTypeCounter++)
		{
			EMeshAttributeType AttributeType = EMeshAttributeType(AttributeTypeCounter);
			FAccessor& Accessor = AccessorAtIndex(Asset->Accessors, GetIndex(Attributes, *ToString(AttributeType)));
			Accessor.bQuantized = bMeshQuantized;
			AttributeAccessors.Emplace(AttributeType, Accessor);
		}

		//Color_0 ... do not need the bMeshQuantized setting.
		for (uint8 AttributeTypeCounter = EMeshAttributeType::COLOR_0; AttributeTypeCounter < EMeshAttributeType::COUNT; AttributeTypeCounter++)
		{
			EMeshAttributeType AttributeType = EMeshAttributeType(AttributeTypeCounter);
			FAccessor& Accessor = AccessorAtIndex(Asset->Accessors, GetIndex(Attributes, *ToString(AttributeType)));
			AttributeAccessors.Emplace(AttributeType, Accessor);
		}

		Mesh.Primitives.Emplace(Mode, MaterialIndex, Indices, AttributeAccessors);

		//Morph Targets:
		if (Object.HasField(TEXT("targets")))
		{
			const TArray<TSharedPtr<FJsonValue> >& MorphTargets = Object.GetArrayField(TEXT("targets"));
			for (TSharedPtr<FJsonValue> Value : MorphTargets)
			{
				const FJsonObject& MorphTargetObject = *Value->AsObject();
				SetupMorphTarget(MorphTargetObject, Mesh.Primitives.Last(), bMeshQuantized);
			}
		}

		ProcessExtras(Object, Mesh.Primitives.Last().Extras);

		ExtensionsHandler->SetupPrimitiveExtensions(Object, Mesh.Primitives.Last(), Mesh.Primitives.Num()-1, Mesh.UniqueId);
	}

	void FFileReader::SetupMesh(const FJsonObject& Object, const bool bMeshQuantized) const
	{
		Asset->Meshes.Emplace();
		FMesh& Mesh = Asset->Meshes.Last();

		const TArray<TSharedPtr<FJsonValue> >& PrimArray = Object.GetArrayField(TEXT("primitives"));

		Mesh.Name = GetString(Object, TEXT("name"));
		Mesh.Primitives.Reserve(PrimArray.Num());

		int32 NumberOfMorphTargets = -1;
		uint32 PrimitiveIndex = 0;
		for (TSharedPtr<FJsonValue> Value : PrimArray)
		{
			const FJsonObject& PrimObject = *Value->AsObject();
			SetupPrimitive(PrimObject, Mesh, bMeshQuantized, PrimitiveIndex);

			if (NumberOfMorphTargets == -1)
			{
				NumberOfMorphTargets = Mesh.Primitives.Last().MorphTargets.Num();
			}
			else
			{
				if (NumberOfMorphTargets != Mesh.Primitives.Last().MorphTargets.Num())
				{
					//All primitives MUST have the same number of morph targets in the same order.
					Messages.Emplace(EMessageSeverity::Error, LOCTEXT("InconsistentNumMorphTargets", "Number of Primitive.Targets is not consistent across the Mesh."));
				}
			}

			PrimitiveIndex++;
		}

		// Morph Target Weights:
		if (Object.HasField(TEXT("weights")))

		{
			const TArray<TSharedPtr<FJsonValue> >& MorphTargetWeightsArray = Object.GetArrayField(TEXT("weights"));
			for (TSharedPtr<FJsonValue> Value : MorphTargetWeightsArray)
			{
				Mesh.MorphTargetWeights.Add(Value->AsNumber());
			}
		}

		// Morph Target Names:
		if (Object.HasField(TEXT("extras")))
		{
			const TSharedPtr<FJsonObject>& Extras = Object.GetObjectField(TEXT("extras"));
			if (Extras->HasField(TEXT("targetNames")))
			{
				const TArray<TSharedPtr<FJsonValue>>& TargetNamesArray = Extras->GetArrayField(TEXT("targetNames"));
				for (TSharedPtr<FJsonValue> Value : TargetNamesArray)
				{
					Mesh.MorphTargetNames.Add(Value->AsString());
				}
			}

			ProcessExtras(Object, Mesh.Extras, { TEXT("targetNames") });
		}

		Mesh.GenerateIsValidCache();

		ExtensionsHandler->SetupMeshExtensions(Object, Mesh);
	}

	void FFileReader::SetupScene(const FJsonObject& Object) const
	{
		FScene& Scene = Asset->Scenes.Emplace_GetRef();

		Scene.Name = GetString(Object, TEXT("name"));
		if (Object.HasField(TEXT("nodes")))
		{
			const TArray<TSharedPtr<FJsonValue> >& NodesArray = Object.GetArrayField(TEXT("nodes"));
			Scene.Nodes.Reserve(NodesArray.Num());
			for (TSharedPtr<FJsonValue> Value : NodesArray)
			{
				const int32 NodeIndex = Value->AsNumber();
				Scene.Nodes.Add(NodeIndex);
			}
		}

		ProcessExtras(Object, Scene.Extras);

		ExtensionsHandler->SetupSceneExtensions(Object, Scene);
	}

	void FFileReader::SetupNode(const FJsonObject& Object) const
	{
		FNode& Node = Asset->Nodes.Emplace_GetRef();

		Node.Index = Asset->Nodes.Num() - 1;

		Node.Name = GetString(Object, TEXT("name"));

		if (Object.HasField(TEXT("matrix")))
		{
			FMatrix Matrix = GetMat4(Object, TEXT("matrix"));

			//we cannot use Transform.SetFromMatrix, because we want precise scaling
			// (GLTF::SetTransformFromMatrix also Converts the Rotation Quaternion)
			GLTF::SetTransformFromMatrix<double>(Node.Transform, Matrix);
		}
		else
		{
			Node.Transform.SetTranslation(GetVec3(Object, TEXT("translation")));
			Node.Transform.SetRotation(GetQuat(Object, TEXT("rotation")));
			Node.Transform.SetScale3D(GetVec3(Object, TEXT("scale"), FVector::OneVector));
		}
		Node.Transform.SetTranslation(GLTF::ConvertVec3(Node.Transform.GetTranslation()));
		Node.Transform.SetScale3D(GLTF::ConvertVec3(Node.Transform.GetScale3D()));

		if (Object.HasField(TEXT("children")))
		{
			const TArray<TSharedPtr<FJsonValue> >& ChildArray = Object.GetArrayField(TEXT("children"));
			Node.Children.Reserve(ChildArray.Num());
			for (TSharedPtr<FJsonValue> Value : ChildArray)
			{
				const int32 ChildIndex = Value->AsNumber();
				Node.Children.Add(ChildIndex);
			}
		}

		Node.MeshIndex   = GetIndex(Object, TEXT("mesh"));
		Node.Skindex     = GetIndex(Object, TEXT("skin"));
		Node.CameraIndex = GetIndex(Object, TEXT("camera"));

		if (Object.HasField(TEXT("weights")))
		{
			const TArray<TSharedPtr<FJsonValue> >& ChildArray = Object.GetArrayField(TEXT("weights"));
			Node.MorphTargetWeights.Reserve(ChildArray.Num());
			for (TSharedPtr<FJsonValue> Value : ChildArray)
			{
				Node.MorphTargetWeights.Add(Value->AsNumber());
			}
		}

		ProcessExtras(Object, Node.Extras);

		ExtensionsHandler->SetupNodeExtensions(Object, Node);
	}

	void FFileReader::SetupCamera(const FJsonObject& Object) const
	{
		const uint32 CameraIndex = Asset->Cameras.Num();
		const FNode* Found       = Asset->Nodes.FindByPredicate([CameraIndex](const FNode& Node) { return CameraIndex == Node.CameraIndex; });
		FString Name = GetString(Object, TEXT("name"));
		if (!Found)
		{
			Messages.Emplace(EMessageSeverity::Warning, FText::Format(LOCTEXT("NoCameraNodeFound", "No camera node found for camera {0}('{1}')"), CameraIndex, FText::FromString(Name)));
			return;
		}

		FCamera& Camera = Asset->Cameras.Emplace_GetRef(*Found);
		Camera.Name     = GetString(Object, TEXT("name"));

		const FString Type = GetString(Object, TEXT("type"));
		if (Type == TEXT("perspective"))
		{
			const FJsonObject& Perspective = *Object.GetObjectField(Type);

			Camera.ZNear                   = GetScalar(Perspective, TEXT("znear"), 0.f);
			Camera.ZFar                    = GetScalar(Perspective, TEXT("zfar"), Camera.ZNear + 10.f);
			Camera.Perspective.AspectRatio = GetScalar(Perspective, TEXT("aspectRatio"), 1.f);
			Camera.Perspective.Fov         = GetScalar(Perspective, TEXT("yfov"), 0.f);
			Camera.bIsPerspective          = true;
		}
		else if (Type == TEXT("orthographic"))
		{
			const FJsonObject& Orthographic = *Object.GetObjectField(Type);

			Camera.ZNear                       = GetScalar(Orthographic, TEXT("znear"), 0.f);
			Camera.ZFar                        = GetScalar(Orthographic, TEXT("zfar"), Camera.ZNear + 10.f);
			Camera.Orthographic.XMagnification = GetScalar(Orthographic, TEXT("xmag"), 0.f);
			Camera.Orthographic.YMagnification = GetScalar(Orthographic, TEXT("ymag"), 0.f);
			Camera.bIsPerspective              = false;
		}
		else
		{
			Messages.Emplace(EMessageSeverity::Error, FText::Format(LOCTEXT("InvalidCameraType", "Invalid camera type: {0}"), FText::FromString(Type)));
		}

		ProcessExtras(Object, Camera.Extras);
		
		ExtensionsHandler->SetupCameraExtensions(Object, Camera);
	}

	void FFileReader::SetupAnimation(const FJsonObject& Object) const
	{
		FAnimation& Animation = Asset->Animations.Emplace_GetRef();
		Animation.Name        = GetString(Object, TEXT("name"));

		// create samplers
		{
			const TArray<TSharedPtr<FJsonValue> >& SampplerArray = Object.GetArrayField(TEXT("samplers"));
			Animation.Samplers.Reserve(SampplerArray.Num());
			for (const TSharedPtr<FJsonValue>& Value : SampplerArray)
			{
				const FJsonObject& SamplerObject = *Value->AsObject();
				const int32        Input         = GetIndex(SamplerObject, TEXT("input"));
				const int32        Output        = GetIndex(SamplerObject, TEXT("output"));
				if (ensure((Input != INDEX_NONE) && (Output != INDEX_NONE)))
				{
					FAnimation::FSampler Sampler(Asset->Accessors[Input], Asset->Accessors[Output]);
					FString InterpolationStr = GetString(SamplerObject, TEXT("interpolation"), TEXT("LINEAR"));
					if (InterpolationStr == TEXT("LINEAR"))
					{
						Sampler.Interpolation = FAnimation::EInterpolation::Linear;
					}
					else if (InterpolationStr == TEXT("STEP"))
					{
						Sampler.Interpolation = FAnimation::EInterpolation::Step;
					}
					else if (InterpolationStr == TEXT("CUBICSPLINE"))
					{
						Sampler.Interpolation = FAnimation::EInterpolation::CubicSpline;
					}

					Animation.Samplers.Add(Sampler);
				}
			}
		}

		// create channels
		{
			const TArray<TSharedPtr<FJsonValue> >& ChannelsArray = Object.GetArrayField(TEXT("channels"));
			Animation.Channels.Reserve(ChannelsArray.Num());
			for (const TSharedPtr<FJsonValue>& Value : ChannelsArray)
			{
				const FJsonObject& ChannelObject = *Value->AsObject();
				const int32        Index         = GetIndex(ChannelObject, TEXT("sampler"));
				if (!ensure(Index != INDEX_NONE))
				{
					continue;
				}

				const FJsonObject& TargetObject = *ChannelObject.GetObjectField(TEXT("target"));
				const int32        NodeIndex    = GetIndex(TargetObject, TEXT("node"));
				if (!ensure(NodeIndex != INDEX_NONE))
				{
					continue;
				}

				FAnimation::FChannel Channel(Asset->Nodes[NodeIndex]);
				Channel.Sampler     = Index;
				Channel.Target.Path = AnimationPathFromString(GetString(TargetObject, TEXT("path")));
				Animation.Channels.Add(Channel);
			}
		}

		ProcessExtras(Object, Animation.Extras);
		
		ExtensionsHandler->SetupAnimationExtensions(Object, Animation);
	}

	void FFileReader::SetupSkin(const FJsonObject& Object) const
	{
		const FAccessor& InverseBindMatrices = AccessorAtIndex(Asset->Accessors, GetIndex(Object, TEXT("inverseBindMatrices")));

		FSkinInfo& Skin = Asset->Skins.Emplace_GetRef(InverseBindMatrices);
		Skin.Name       = GetString(Object, TEXT("name"));

		const TArray<TSharedPtr<FJsonValue> >& JointArray = Object.GetArrayField(TEXT("joints"));
		Skin.Joints.Reserve(JointArray.Num());
		for (TSharedPtr<FJsonValue> Value : JointArray)
		{
			const int32 JointIndex = Value->AsNumber();
			Skin.Joints.Add(JointIndex);
		}

		Skin.Skeleton = GetIndex(Object, TEXT("skeleton"));

		ProcessExtras(Object, Skin.Extras);
		
		ExtensionsHandler->SetupSkinExtensions(Object, Skin);
	}

	void FFileReader::SetupImage(const FJsonObject& Object, const FString& Path, bool bInLoadImageData)
	{
		FImage& Image = Asset->Images.Emplace_GetRef();
		Image.Name    = GetString(Object, TEXT("name"));

		bool bUpdateOffset = false;
		if (Object.HasTypedField<EJson::String>(TEXT("uri")))
		{
			// Get data now, so Unreal doesn't need to care about where the data came from.
			// Unreal *is* responsible for decoding Data based on Format.

			Image.URI = Object.GetStringField(TEXT("uri"));
			if (Image.URI.StartsWith(TEXT("data:")))
			{
				uint32  ImageSize = 0;
				FString MimeType;
				bool    bSuccess = DecodeDataURI(Image.URI, MimeType, CurrentBufferOffset, ImageSize);
				Image.Format     = ImageFormatFromMimeType(MimeType);
				if (!bSuccess || Image.Format == FImage::EFormat::Unknown)
				{
					Messages.Emplace(EMessageSeverity::Error, LOCTEXT("DecodingImageProblem", "Problem decoding image from data URI."));
				}
				else
				{
					Image.DataByteLength = ImageSize;
					bUpdateOffset        = true;
				}
			}
			else  // Load buffer from external file.
			{
				Image.URI = FGenericPlatformHttp::UrlDecode(Image.URI);
				Image.Format = ImageFormatFromFilename(Image.URI);

				Image.FilePath = Path / Image.URI;
				if (bInLoadImageData)
				{
					FArchive* Reader = IFileManager::Get().CreateFileReader(*Image.FilePath);
					if (Reader)
					{
						const int64 FileSize = Reader->TotalSize();
						Reader->Serialize(CurrentBufferOffset, FileSize);
						Reader->Close();
						delete Reader;

						Image.DataByteLength = FileSize;
						bUpdateOffset        = true;
					}
					else
					{
						Messages.Emplace(EMessageSeverity::Error, LOCTEXT("ErrorLoadingImageFile", "Could not load image file."));
					}
				}
			}
		}
		else
		{
			// Missing URI means use a BufferView
			const int32 Index = GetIndex(Object, TEXT("bufferView"));
			if (Asset->BufferViews.IsValidIndex(Index))
			{
				Image.Format = ImageFormatFromMimeType(GetString(Object, TEXT("mimeType")));

				const FBufferView& BufferView = Asset->BufferViews[Index];
				// We just created Image, so Image.Data is empty. Fill it with encoded bytes!
				Image.DataByteLength = BufferView.ByteLength;
				Image.Data           = static_cast<const uint8*>(BufferView.DataAt(0));
			}
		}

		if (bUpdateOffset)
		{
			Image.Data = CurrentBufferOffset;
			CurrentBufferOffset += Image.DataByteLength;
		}

		ProcessExtras(Object, Image.Extras);
		
		ExtensionsHandler->SetupImageExtensions(Object, Image);
	}

	void FFileReader::SetupSampler(const FJsonObject& Object) const
	{
		FSampler& Sampler = Asset->Samplers.Emplace_GetRef();

		// spec doesn't specify default value, use linear
		Sampler.MinFilter = FilterFromNumber(GetUnsignedInt(Object, TEXT("minFilter"), (uint32)FSampler::EFilter::Linear));
		Sampler.MagFilter = FilterFromNumber(GetUnsignedInt(Object, TEXT("magFilter"), (uint32)FSampler::EFilter::Linear));
		// default mode is Repeat according to spec
		Sampler.WrapS = WrapModeFromNumber(GetUnsignedInt(Object, TEXT("wrapS"), (uint32)FSampler::EWrap::Repeat));
		Sampler.WrapT = WrapModeFromNumber(GetUnsignedInt(Object, TEXT("wrapT"), (uint32)FSampler::EWrap::Repeat));

		ExtensionsHandler->SetupSamplerExtensions(Object, Sampler);
	}

	void FFileReader::SetupTexture(const FJsonObject& Object) const
	{
		int32 SourceIndex  = GetIndex(Object, TEXT("source"));
		int32 SamplerIndex = GetIndex(Object, TEXT("sampler"));

		// According to the spec it's possible to have a Texture with no Image source.
		// In that case use a default image (checkerboard?).

		if (Asset->Images.IsValidIndex(SourceIndex))
		{
			const bool HasSampler = Asset->Samplers.IsValidIndex(SamplerIndex);

			const FString TexName = GetString(Object, TEXT("name"));

			const FImage&   Source  = Asset->Images[SourceIndex];
			const FSampler& Sampler = HasSampler ? Asset->Samplers[SamplerIndex] : FSampler::DefaultSampler;

			Asset->Textures.Emplace(TexName, Source, Sampler);

			ProcessExtras(Object, Asset->Textures.Last().Extras);
			
			ExtensionsHandler->SetupTextureExtensions(Object, Asset->Textures.Last());
		}
		else
		{
			Messages.Emplace(EMessageSeverity::Warning, FText::Format(LOCTEXT("InvalidTextureSourceIndex", "Invalid texture source index: {0}"), SourceIndex));
		}
	}

	void FFileReader::SetupMaterial(const FJsonObject& Object) const
	{
		Asset->Materials.Emplace(GetString(Object, TEXT("name")));
		FMaterial& Material = Asset->Materials.Last();

		GLTF::SetTextureMap(Object, TEXT("emissiveTexture"), nullptr, Asset->Textures, Material.Emissive, ExtensionsHandler->GetMessages());
		Material.EmissiveFactor = (FVector3f)GetVec3(Object, TEXT("emissiveFactor"));

		Material.NormalScale       = GLTF::SetTextureMap(Object, TEXT("normalTexture"), TEXT("scale"), Asset->Textures, Material.Normal, ExtensionsHandler->GetMessages());
		Material.OcclusionStrength = GLTF::SetTextureMap(Object, TEXT("occlusionTexture"), TEXT("strength"), Asset->Textures, Material.Occlusion, ExtensionsHandler->GetMessages());

		if (Object.HasTypedField<EJson::Object>(TEXT("pbrMetallicRoughness")))
		{
			const FJsonObject& PBR = *Object.GetObjectField(TEXT("pbrMetallicRoughness"));

			GLTF::SetTextureMap(PBR, TEXT("baseColorTexture"), nullptr, Asset->Textures, Material.BaseColor, ExtensionsHandler->GetMessages());
			Material.BaseColorFactor = (FVector4f)GetVec4(PBR, TEXT("baseColorFactor"), FVector4(1.0f, 1.0f, 1.0f, 1.0f));

			GLTF::SetTextureMap(PBR, TEXT("metallicRoughnessTexture"), nullptr, Asset->Textures, Material.MetallicRoughness.Map, ExtensionsHandler->GetMessages());
			Material.MetallicRoughness.MetallicFactor  = GetScalar(PBR, TEXT("metallicFactor"), 1.0f);
			Material.MetallicRoughness.RoughnessFactor = GetScalar(PBR, TEXT("roughnessFactor"), 1.0f);
		}

		if (Object.HasTypedField<EJson::String>(TEXT("alphaMode")))
		{
			Material.AlphaMode = AlphaModeFromString(Object.GetStringField(TEXT("alphaMode")));
			if (Material.AlphaMode == FMaterial::EAlphaMode::Mask)
			{
				Material.AlphaCutoff = GetScalar(Object, TEXT("alphaCutoff"), 0.5f);
			}
		}

		Material.bIsDoubleSided = GetBool(Object, TEXT("doubleSided"));

		ProcessExtras(Object, Material.Extras);
		
		ExtensionsHandler->SetupMaterialExtensions(Object, Material);
	}

	void FFileReader::ReadFile(const FString& InFilePath, bool bInLoadImageData, bool bInLoadMetadata, GLTF::FAsset& OutAsset)
	{
		Messages.Empty();

		TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*InFilePath));
		TUniquePtr<FArchive> JsonFileReader;
		if (!FileReader)
		{
			Messages.Emplace(EMessageSeverity::Error, FText::Format(LOCTEXT("ErrorLoadingFile", "Can't load file: {0}"), FText::FromString(InFilePath)));
			return;
		}

		const FString Extension = FPaths::GetExtension(InFilePath);
		if (Extension == TEXT("gltf"))
		{
			// Convert to UTF8
			FFileHelper::LoadFileToString(JsonBuffer, *InFilePath);
		}
		else if (Extension == TEXT("glb"))
		{
			BinaryReader->SetBuffer(OutAsset.BinData);
			if (!BinaryReader->ReadFile(*FileReader))
			{
				Messages.Append(BinaryReader->GetLogMessages());
				return;
			}

			// Convert to UTF8
			const TArray<uint8>& Buffer = BinaryReader->GetJsonBuffer();
			FFileHelper::BufferToString(JsonBuffer, Buffer.GetData(), Buffer.Num());
		}
		else
		{
			Messages.Emplace(EMessageSeverity::Error, LOCTEXT("InvalidFileExtention", "Invalid extension."));
			return;
		}
		JsonFileReader.Reset(new FBufferReader(JsonBuffer.GetCharArray().GetData(), sizeof(FString::ElementType) * JsonBuffer.Len(), false));

		JsonRoot                                                  = MakeShareable(new FJsonObject);
		TSharedRef<TJsonReader<FString::ElementType> > JsonReader = TJsonReader<FString::ElementType>::Create(JsonFileReader.Get());
		if (!FJsonSerializer::Deserialize(JsonReader, JsonRoot))
		{
			JsonRoot.Reset();
			Messages.Emplace(EMessageSeverity::Error, LOCTEXT("JSONDeserializationError","Problem loading JSON."));
			return;
		}

		// Check file format version to make sure we can read it.
		const TSharedPtr<FJsonObject>& AssetInfo = JsonRoot->GetObjectField(TEXT("asset"));
		if (AssetInfo->HasTypedField<EJson::Number>(TEXT("minVersion")))
		{
			const double MinVersion = AssetInfo->GetNumberField(TEXT("minVersion"));
			if (MinVersion > 2.0)
			{
				Messages.Emplace(EMessageSeverity::Error, LOCTEXT("UnsupportedGLTFAssetMinVersion", "This importer supports glTF version 2.0 (or compatible) assets."));
				return;
			}
			OutAsset.Metadata.Version = MinVersion;
		}
		else
		{
			const double Version = AssetInfo->GetNumberField(TEXT("version"));
			if (Version < 2.0)
			{
				Messages.Emplace(EMessageSeverity::Error, LOCTEXT("UnsupportedGLTFAssetVersion", "This importer supports glTF asset version 2.0 or later."));
				return;
			}
			OutAsset.Metadata.Version = Version;
		}
		if (bInLoadMetadata)
			LoadMetadata(OutAsset);

		const FString ResourcesPath = FPaths::GetPath(InFilePath);
		ImportAsset(ResourcesPath, bInLoadImageData, OutAsset);

		// generate asset name
		{
			OutAsset.Name = FPaths::GetBaseFilename(InFilePath);
			if (OutAsset.Name.ToLower() == TEXT("scene"))
			{
				// change name, try to see if asset title is given
				if (const FMetadata::FExtraData* Extra = OutAsset.Metadata.GetExtraData(TEXT("title")))
				{
					OutAsset.Name = Extra->Value;
				}
				else
				{
					OutAsset.Name = FPaths::GetBaseFilename(FPaths::GetPath(InFilePath));
				}
			}
		}
		OutAsset.GenerateNames();
		
		ResolveConflictingTextures();

		if (OutAsset.ValidationCheck() != FAsset::Valid)
		{
			Messages.Emplace(EMessageSeverity::Warning, FText::Format(LOCTEXT("SomeImportedObjectsInvalid","For GLTF Asset [{0}] not all imported objects are valid."), FText::FromString(OutAsset.Name)));
		}

		JsonRoot.Reset();
	}

	void FFileReader::LoadMetadata(GLTF::FAsset& OutAsset)
	{
		const TSharedPtr<FJsonObject>& AssetInfo = JsonRoot->GetObjectField(TEXT("asset"));
		if (AssetInfo->HasField(TEXT("generator")))
			OutAsset.Metadata.GeneratorName = AssetInfo->GetStringField(TEXT("generator"));

		if (!AssetInfo->HasField(TEXT("extras")))
			return;

		const TSharedPtr<FJsonObject>& Extras = AssetInfo->GetObjectField(TEXT("extras"));
		for (const auto& ValuePair : Extras->Values)
		{
			const TSharedPtr<FJsonValue>& JsonValue = ValuePair.Value;
			OutAsset.Metadata.Extras.Emplace(FMetadata::FExtraData {FString(ValuePair.Key), JsonValue->AsString()});
		}
	}

	void FFileReader::AllocateExtraData(const FString& InResourcesPath, bool bInLoadImageData, TArray64<uint8>& OutExtraData)
	{
		uint64 ExtraBufferSize = 0;
		if (BufferCount > 0)
		{
			for (const TSharedPtr<FJsonValue>& Value : JsonRoot->GetArrayField(TEXT("buffers")))
			{
				const FJsonObject& Object     = *Value->AsObject();
				const uint64       ByteLength = GetUnsignedInt64(Object, TEXT("byteLength"), 0);
				if (!Object.HasTypedField<EJson::String>(TEXT("uri")))
					continue;

				FString URI = Object.GetStringField(TEXT("uri"));
				if (URI.StartsWith(TEXT("data:")))
				{
					FString      MimeType;
					const uint32 DataSize = GetDecodedDataSize(URI, MimeType);
					if (DataSize > 0 && (MimeType == TEXT("application/octet-stream") || MimeType == (TEXT("application/gltf-buffer"))))
					{
						ensure(DataSize == ByteLength);
						ExtraBufferSize += ByteLength;
					}
				}
				else
				{
					URI = FGenericPlatformHttp::UrlDecode(URI);
					const FString FullPath = InResourcesPath / URI;
					const int64   FileSize = FPlatformFileManager::Get().GetPlatformFile().FileSize(*FullPath);
					if (ByteLength == FileSize)
					{
						ExtraBufferSize += ByteLength;
					}
				}
			}
		}

		if (ImageCount)
		{
			for (const TSharedPtr<FJsonValue>& Value : JsonRoot->GetArrayField(TEXT("images")))
			{
				const FJsonObject& Object = *Value->AsObject();

				if (!Object.HasTypedField<EJson::String>(TEXT("uri")))
					continue;

				FString URI = Object.GetStringField(TEXT("uri"));
				if (URI.StartsWith(TEXT("data:")))
				{
					FString         MimeType;
					const uint32    DataSize = GetDecodedDataSize(URI, MimeType);
					FImage::EFormat Format   = ImageFormatFromMimeType(MimeType);
					if (DataSize > 0 && Format != FImage::EFormat::Unknown)
					{
						ExtraBufferSize += DataSize;
					}
				}
				else if (bInLoadImageData)
				{
					URI = FGenericPlatformHttp::UrlDecode(URI);

					FImage::EFormat Format = ImageFormatFromFilename(URI);
					if (Format != FImage::EFormat::Unknown)
					{
						const FString FullPath = InResourcesPath / URI;
						const int64   FileSize = FPlatformFileManager::Get().GetPlatformFile().FileSize(*FullPath);
						ExtraBufferSize += FileSize;
					}
				}
			}
		}

		OutExtraData.Reserve(ExtraBufferSize + 16);
		OutExtraData.SetNumUninitialized(ExtraBufferSize);
		CurrentBufferOffset = ExtraBufferSize ? OutExtraData.GetData() : nullptr;
	}

	void FFileReader::ImportAsset(const FString& InResourcesPath, bool bInLoadImageData, GLTF::FAsset& OutAsset)
	{
		BufferCount                = ArraySize(*JsonRoot, TEXT("buffers"));
		BufferViewCount            = ArraySize(*JsonRoot, TEXT("bufferViews"));
		const uint32 AccessorCount = ArraySize(*JsonRoot, TEXT("accessors"));
		const uint32 MeshCount     = ArraySize(*JsonRoot, TEXT("meshes"));

		const uint32 SceneCount      = ArraySize(*JsonRoot, TEXT("scenes"));
		const uint32 NodeCount       = ArraySize(*JsonRoot, TEXT("nodes"));
		const uint32 CameraCount     = ArraySize(*JsonRoot, TEXT("cameras"));
		const uint32 SkinCount       = ArraySize(*JsonRoot, TEXT("skins"));
		const uint32 AnimationsCount = ArraySize(*JsonRoot, TEXT("animations"));

		ImageCount                 = ArraySize(*JsonRoot, TEXT("images"));
		const uint32 SamplerCount  = ArraySize(*JsonRoot, TEXT("samplers"));
		const uint32 TextureCount  = ArraySize(*JsonRoot, TEXT("textures"));
		const uint32 MaterialCount = ArraySize(*JsonRoot, TEXT("materials"));

		const uint32 ExtensionsRequiredCount = ArraySize(*JsonRoot, TEXT("extensionsRequired"));

		{
			// cleanup and reserve
			OutAsset.Buffers.Empty(BufferCount);
			OutAsset.BufferViews.Empty(BufferViewCount);
			OutAsset.Accessors.Empty(AccessorCount);
			OutAsset.Meshes.Empty(MeshCount);
			OutAsset.Scenes.Empty(SceneCount);
			OutAsset.Nodes.Empty(NodeCount);
			OutAsset.Cameras.Empty(CameraCount);
			OutAsset.Lights.Empty(10);
			OutAsset.Skins.Empty(SkinCount);
			OutAsset.Animations.Empty(AnimationsCount);
			OutAsset.Images.Empty(ImageCount);
			OutAsset.Samplers.Empty(SamplerCount);
			OutAsset.Textures.Empty(TextureCount);
			OutAsset.Materials.Empty(MaterialCount);
			OutAsset.ProcessedExtensions.Empty((int)EExtension::Count);
			OutAsset.ExtensionsUsed.Empty();
			OutAsset.ExtensionsRequired.Empty();
		}

		// allocate asset mapped data for images and buffers
		AllocateExtraData(InResourcesPath, bInLoadImageData, OutAsset.ExtraBinData);

		Asset = &OutAsset;
		ExtensionsHandler->SetAsset(OutAsset);

		/**
		 * According to gltf specification checking only the top-level extensionsRequired property in order to decide
		 * if the model import is supported should be sufficient as the documentation states:
		 * "All glTF extensions required to load and/or render an asset MUST be listed in the top-level extensionsRequired array"
		 */
		bool bMeshQuantized = false;
		const FString KHR_MeshQuantizationString = ToString(GLTF::EExtension::KHR_MeshQuantization);
		const TArray<TSharedPtr<FJsonValue>>* ExtensionsRequired;
		if (JsonRoot->TryGetArrayField(TEXT("extensionsRequired"), ExtensionsRequired))
		{
			if (ensure(ExtensionsRequired))
			{
				OutAsset.ExtensionsRequired.Reserve(OutAsset.ExtensionsRequired.Num() + ExtensionsRequired->Num());
				for (const TSharedPtr<FJsonValue>& Extension : *ExtensionsRequired)
				{
					FString ExtensionString = Extension->AsString();
					OutAsset.ExtensionsRequired.Add(ExtensionString);

					if (ExtensionString == KHR_MeshQuantizationString)
					{
						bMeshQuantized = true;
					}
				}
			}
		}

		if (!SetupObjects(BufferCount, TEXT("buffers"), [this, InResourcesPath](const FJsonObject& Object) { SetupBuffer(Object, InResourcesPath); })) { return; }
		if (!SetupObjects(BufferViewCount, TEXT("bufferViews"), [this](const FJsonObject& Object) { SetupBufferView(Object); })) { return; }
		if (!SetupObjects(AccessorCount, TEXT("accessors"), [this](const FJsonObject& Object) { SetupAccessor(Object); })) { return; }

		if (!SetupObjects(MeshCount, TEXT("meshes"), [this, &bMeshQuantized](const FJsonObject& Object) { SetupMesh(Object, bMeshQuantized); })) { return; }
		if (!SetupObjects(NodeCount, TEXT("nodes"), [this](const FJsonObject& Object) { SetupNode(Object); })) { return; }
		if (!SetupObjects(SceneCount, TEXT("scenes"), [this](const FJsonObject& Object) { SetupScene(Object); })) { return; }
		if (!SetupObjects(CameraCount, TEXT("cameras"), [this](const FJsonObject& Object) { SetupCamera(Object); })) { return; }
		if (!SetupObjects(SkinCount, TEXT("skins"), [this](const FJsonObject& Object) { SetupSkin(Object); })) { return; }

		{//BuildRootJoints can affect the node hierarchy and data, as the Animation setup currently stores references to the Nodes, we have to do these setups before the SetupAnimation calls.
			SetupUsedSkins();

			SetupNodesType();

			for (FScene& Scene : Asset->Scenes)
			{
				for (int32 NodeIndex : Scene.Nodes)
				{
					BuildParentIndices(INDEX_NONE, INDEX_NONE, NodeIndex);
				}
			}

			FixNodeTypesForIndirectJoints();

			BuildRootJoints();

			GenerateInverseBindPosesPerSkinIndices();
			GenerateLocalBindPosesPerSkinIndices();
			SetLocalBindPosesForJoints();
		}

		if (!SetupObjects(AnimationsCount, TEXT("animations"), [this](const FJsonObject& Object) { SetupAnimation(Object); })) { return; }

		if (!SetupObjects(ImageCount, TEXT("images"), [this, InResourcesPath, bInLoadImageData](const FJsonObject& Object) { SetupImage(Object, InResourcesPath, bInLoadImageData); })) { return; }
		if (!SetupObjects(SamplerCount, TEXT("samplers"), [this](const FJsonObject& Object) { SetupSampler(Object); })) { return; }
		if (!SetupObjects(TextureCount, TEXT("textures"), [this](const FJsonObject& Object) { SetupTexture(Object); })) { return; }
		if (!SetupObjects(MaterialCount, TEXT("materials"), [this](const FJsonObject& Object) { SetupMaterial(Object); })) { return; }


		const TArray<TSharedPtr<FJsonValue>>* ExtensionsUsed;
		if (JsonRoot->TryGetArrayField(TEXT("extensionsUsed"), ExtensionsUsed))
		{
			OutAsset.ExtensionsUsed.Reserve(ExtensionsUsed->Num());
			for (const TSharedPtr<FJsonValue>& Extension : *ExtensionsUsed)
			{
				OutAsset.ExtensionsUsed.Add(Extension->AsString());
			}
		}

		ExtensionsHandler->SetupAssetExtensions(*JsonRoot, InResourcesPath);
	}

	bool FFileReader::CheckForErrors(int32 StartIndex) const
	{
		for (size_t Index = StartIndex; Index < Messages.Num(); Index++)
		{
			if (Messages[Index].Key == EMessageSeverity::Error)
			{
				return true;
			}
		}

		return false;
	}

	template <typename SetupFunc>
	bool FFileReader::SetupObjects(uint32 ObjectCount, const TCHAR* FieldName, SetupFunc Func) const
	{
		int32 StartIndex = Messages.Num();
		
		if (ObjectCount > 0)
		{
			for (const TSharedPtr<FJsonValue>& Value : JsonRoot->GetArrayField(FieldName))
			{
				const FJsonObject& Object = *Value->AsObject();
				Func(Object);
			}
		}

		if (CheckForErrors(StartIndex))
		{
			//Any error found should automatically halt the import.
			return false;
		}

		return true;
	}

	void FFileReader::SetupUsedSkins() const
	{
		//Acquire Skin usages:
		TBitArray SkinIndicesUsage(false, Asset->Skins.Num());
		for (size_t Index = 0; Index < Asset->Nodes.Num(); Index++)
		{
			int32 SkinIndex = Asset->Nodes[Index].Skindex;
			if (SkinIndex != INDEX_NONE)
			{
				SkinIndicesUsage[SkinIndex] = true;
			}
		}

		FString UnUsedIndicesString;
		for (size_t SkinIndex = 0; SkinIndex < SkinIndicesUsage.Num(); SkinIndex++)
		{
			FSkinInfo& Skin = Asset->Skins[SkinIndex];
			Skin.bUsed = SkinIndicesUsage[SkinIndex];

			if (!Skin.bUsed)
			{
				if (UnUsedIndicesString.Len() > 0)
				{
					UnUsedIndicesString += TEXT(",");
				}
				UnUsedIndicesString += FString::FromInt(SkinIndex);
			}
		}

		if (UnUsedIndicesString.Len() > 0)
		{
			Messages.Emplace(EMessageSeverity::Warning, FText::Format(LOCTEXT("UnusedSkinObjects","Skin objects unused. At indices: {0}."), FText::FromString(UnUsedIndicesString)));
		}
	}

	void FFileReader::SetupNodesType() const
	{
		// setup node types
		for (FNode& Node : Asset->Nodes)
		{
			if (Node.MeshIndex != INDEX_NONE)
			{
				Node.Type = Node.Skindex != INDEX_NONE ? FNode::EType::MeshSkinned : FNode::EType::Mesh;
			}
			else if (Node.CameraIndex != INDEX_NONE)
			{
				Node.Type = FNode::EType::Camera;
			}
			else if (Node.LightIndex != INDEX_NONE || Node.LightIES.Index != INDEX_NONE)
			{
				Node.Type = FNode::EType::Light;
			}
			else
			{
				ensure(Node.Transform.IsValid());
				if (!Node.Transform.GetRotation().IsIdentity() || !Node.Transform.GetTranslation().IsZero() ||
				    !Node.Transform.GetScale3D().Equals(FVector(1.f)))
				{
					Node.Type = FNode::EType::Transform;
				}
			}
		}

		for (const FSkinInfo& Skin : Asset->Skins)
		{
			if (!Skin.bUsed)
			{
				continue;
			}

			for (int32 JointIndex : Skin.Joints)
			{
				ensure(Asset->Nodes[JointIndex].Type == FNode::EType::None 
					|| Asset->Nodes[JointIndex].Type == FNode::EType::Transform
					|| Asset->Nodes[JointIndex].Type == FNode::EType::Joint);
				Asset->Nodes[JointIndex].Type = FNode::EType::Joint;
			}
		}
	}

	void FFileReader::GenerateInverseBindPosesPerSkinIndices() const
	{
		for (size_t SkinIndex = 0; SkinIndex < Asset->Skins.Num(); SkinIndex++)
		{
			const FSkinInfo& Skin = Asset->Skins[SkinIndex];
			if (Skin.InverseBindMatrices.Count == Skin.Joints.Num() &&
				Skin.InverseBindMatrices.IsValid())
			{
				for (size_t JointCounter = 0; JointCounter < Skin.Joints.Num(); JointCounter++)
				{
					FMatrix InverseBindMatrix = Skin.InverseBindMatrices.GetMat4(JointCounter);
					InverseBindMatrix = GLTF::ConvertMat(InverseBindMatrix);
					
					Asset->Nodes[Skin.Joints[JointCounter]].SkinIndexToGlobalInverseBindMatrix.Add(SkinIndex, InverseBindMatrix);
				}
			}
		}
	}

	void GenerateGlobalTransform(const TArray<FNode>& Nodes, int32 CurrentIndex, FTransform& GlobalTransform, const int32& SkeletonCommonRootIndex/*PivotPoint*/)
	{
		if (Nodes.IsValidIndex(CurrentIndex))
		{
			const FNode& CurrentNode = Nodes[CurrentIndex];

			if (CurrentIndex != SkeletonCommonRootIndex)
			{
				GenerateGlobalTransform(Nodes, CurrentNode.ParentIndex, GlobalTransform, SkeletonCommonRootIndex);
			}

			GlobalTransform = CurrentNode.Transform * GlobalTransform;
		}
	}

	void FFileReader::GenerateLocalBindPosesPerSkinIndices() const
	{
		auto GetTransform = [](const FMatrix& Matrix) -> FTransform
			{
				FTransform Transform;
				Transform.SetFromMatrix(Matrix);
				Transform.SetRotation(GLTF::ConvertQuat(Transform.GetRotation()));
				Transform.SetTranslation(GLTF::ConvertVec3(Transform.GetTranslation()));
				Transform.SetScale3D(GLTF::ConvertVec3(Transform.GetScale3D()));

				return Transform;
			};

		for (size_t SkinIndex = 0; SkinIndex < Asset->Skins.Num(); SkinIndex++)
		{
			const FSkinInfo& Skin = Asset->Skins[SkinIndex];
			if (Skin.InverseBindMatrices.Count == Skin.Joints.Num() &&
				Skin.InverseBindMatrices.IsValid())
			{
				for (size_t JointCounter = 0; JointCounter < Skin.Joints.Num(); JointCounter++)
				{
					FNode& CurrentNode = Asset->Nodes[Skin.Joints[JointCounter]];

					
					if (CurrentNode.DirectParentJointIndex != INDEX_NONE &&
						Asset->Nodes.IsValidIndex(CurrentNode.DirectParentJointIndex) &&
						(Asset->Nodes[CurrentNode.DirectParentJointIndex].SkinIndexToGlobalInverseBindMatrix.Contains(SkinIndex) || Asset->Nodes[CurrentNode.DirectParentJointIndex].SkinIndexToGlobalInverseBindMatrix.Num() > 0)
						)
					{
						FNode& ParentNode = Asset->Nodes[CurrentNode.DirectParentJointIndex];

						//LocalBindPose; //bind pose would be CurrentNode.GlobalInverseBindTransform.Inverse() * ParentNode.GlobalInverseBindTransform
						FMatrix ParentGlobalInverseBindMatrix;
						if (ParentNode.SkinIndexToGlobalInverseBindMatrix.Contains(SkinIndex))
						{
							ParentGlobalInverseBindMatrix = ParentNode.SkinIndexToGlobalInverseBindMatrix[SkinIndex];
						}
						else
						{
							//Scenario is that the a Skin is instantiated at the end of another skin
							//(Prime example is the RecursiveSkeleton gltf sample file.)
							ParentGlobalInverseBindMatrix = ParentNode.SkinIndexToGlobalInverseBindMatrix.FindArbitraryElement()->Value;
						}

						FTransform LocalBindTransform = FTransform::Identity;

						FTransform ParentGlobalInverseBindTransform = GetTransform(ParentGlobalInverseBindMatrix);
						FTransform GlobalBindTransform = GetTransform(CurrentNode.SkinIndexToGlobalInverseBindMatrix[SkinIndex].Inverse());

						if (CurrentNode.DirectParentJointIndex != CurrentNode.ParentJointIndex
							&& Asset->Nodes.IsValidIndex(CurrentNode.ParentJointIndex))
						{
							int32 ParentJointIndex = CurrentNode.ParentJointIndex;

							TArray<FTransform> ParentTransformsToApply;

							while (Asset->Nodes.IsValidIndex(ParentJointIndex)
								&& Asset->Nodes[ParentJointIndex].bHasLocalBindPose) //this cycle meant to go through the "JointGaps". "JointGaps" have their LocalBindPose already set here (set @FixNodeTypesForIndirectJoints)
							{
								ParentTransformsToApply.Add(Asset->Nodes[ParentJointIndex].Transform);

								ParentJointIndex = Asset->Nodes[ParentJointIndex].ParentJointIndex;
							}

							for (int32 MatrixToApplyIndex = ParentTransformsToApply.Num()-1; MatrixToApplyIndex >= 0; MatrixToApplyIndex--)
							{
								ParentGlobalInverseBindTransform = ParentGlobalInverseBindTransform * ParentTransformsToApply[MatrixToApplyIndex].Inverse();
							}
						}

						LocalBindTransform = GlobalBindTransform * ParentGlobalInverseBindTransform;

						CurrentNode.SkinIndexToLocalBindTransform.Add(SkinIndex, LocalBindTransform);
					}
					else
					{
						FTransform LocalBindPose = GetTransform(CurrentNode.SkinIndexToGlobalInverseBindMatrix[SkinIndex].Inverse());
						CurrentNode.SkinIndexToLocalBindTransform.Add(SkinIndex, LocalBindPose);
					}
				}
			}
		}
	}

	void FFileReader::SetLocalBindPosesForJoints() const
	{
		//Validate generated SkinIndexToLocalBindPose values, before setting.
		TArray<FString> OffendingJointsNames;

		for (const FNode& CurrentNode : Asset->Nodes)
		{
			TMap<int, FTransform>::TConstIterator Iter(CurrentNode.SkinIndexToLocalBindTransform.CreateConstIterator());
			FTransform ToCompareAgainst = Iter ? Iter.Value() : FTransform();
			++Iter;
			for (; Iter; ++Iter)
			{
				if (!ToCompareAgainst.Equals(Iter.Value()))
				{
					OffendingJointsNames.Add(CurrentNode.Name);
					break;
				}
			}
		}
		
		if (OffendingJointsNames.Num() > 0)
		{
			FString OffendingJointsNamesString;
			for (const FString& OffendingJointName : OffendingJointsNames)
			{
				if (OffendingJointsNamesString.Len() > 0)
				{
					OffendingJointsNamesString += TEXT(", ");
				}
				OffendingJointsNamesString += OffendingJointName;
			}
			
			Messages.Emplace(EMessageSeverity::Warning, FText::Format(LOCTEXT("MultipleSkinsUseSameJointProblem","The same Joint(s) are used in multiple Skins with multiple different InverseBindMatrix values, which is not supported. Ignoring InverseBindMatrices for the entire Import. Offending Joints' Names: {0}."), FText::FromString(OffendingJointsNamesString)));
			
			Asset->HasAbnormalInverseBindMatrices = true;

			return;
		}

		for (size_t SkinIndex = 0; SkinIndex < Asset->Skins.Num(); SkinIndex++)
		{
			const FSkinInfo& Skin = Asset->Skins[SkinIndex];
			if (Skin.InverseBindMatrices.Count == Skin.Joints.Num() &&
				Skin.InverseBindMatrices.IsValid())
			{
				for (size_t JointCounter = 0; JointCounter < Skin.Joints.Num(); JointCounter++)
				{
					FNode& CurrentNode = Asset->Nodes[Skin.Joints[JointCounter]];

					if (!CurrentNode.bHasLocalBindPose
						&& CurrentNode.SkinIndexToLocalBindTransform.Contains(SkinIndex))
					{
						CurrentNode.bHasLocalBindPose = true;

						CurrentNode.LocalBindPose = CurrentNode.SkinIndexToLocalBindTransform[SkinIndex];
					}
				}
			}
		}
	}

	void FFileReader::BuildParentJointIndices() const
	{
		for (size_t SkinIndex = 0; SkinIndex < Asset->Skins.Num(); SkinIndex++)
		{
			GLTF::FSkinInfo& Skin = Asset->Skins[SkinIndex];

			for (int32 BoneIndex : Skin.Joints)
			{
				int32 CurrentIndex = BoneIndex;
				while (Asset->Nodes.IsValidIndex(CurrentIndex)
					&& Asset->Nodes.IsValidIndex(Asset->Nodes[CurrentIndex].ParentIndex)
					&& Asset->Nodes[Asset->Nodes[CurrentIndex].ParentIndex].Type == GLTF::FNode::EType::Joint)
				{
					Asset->Nodes[CurrentIndex].ParentJointIndex = Asset->Nodes[CurrentIndex].ParentIndex;

					CurrentIndex = Asset->Nodes[CurrentIndex].ParentJointIndex;
				}
			}
		}
	}

	void FFileReader::BuildParentIndices(int32 ParentNodeIndex, int32 DirectParentJointIndex, int32 CurrentNodeIndex) const
	{
		if (!Asset->Nodes.IsValidIndex(CurrentNodeIndex))
		{
			return;
		}

		GLTF::FNode& Node = Asset->Nodes[CurrentNodeIndex];

		if (Node.ParentIndex != INDEX_NONE)
		{
			//it's already so stop the sequence here.
			return;
		}

		Node.ParentIndex = ParentNodeIndex;

		//TODO1: we also should differentiate between Skins
		//		currently while finding roots and joints, but especially roots, we just look at the type of the Node
		//		However this way we are taking into account that different skins can have different Skeleton compositions.
		//		We have to update the FNodes' to contain information as to which Skin are they Joint in.
		//		Then root calculations should take into account the SkinIdx information and create Roots based on said information.
		Node.DirectParentJointIndex = DirectParentJointIndex;

		for (const int32 ChildNodeIndex : Node.Children)
		{
			BuildParentIndices(CurrentNodeIndex, Node.Type == GLTF::FNode::EType::Joint ? CurrentNodeIndex : DirectParentJointIndex, ChildNodeIndex);
		}
	}

	int32 FFileReader::FindRootJointIndex(int32 CurrentIndex) const
	{
		if (!ensure(Asset->Nodes.IsValidIndex(CurrentIndex)))
		{
			return INDEX_NONE;
		}

		while (Asset->Nodes.IsValidIndex(Asset->Nodes[CurrentIndex].ParentJointIndex))
		{
			CurrentIndex = Asset->Nodes[CurrentIndex].ParentJointIndex;
		}
		return CurrentIndex;
	}

	void FFileReader::FixNodeTypesForIndirectJoints() const
	{
		for (size_t SkinIndex = 0; SkinIndex < Asset->Skins.Num(); SkinIndex++)
		{
			GLTF::FSkinInfo& Skin = Asset->Skins[SkinIndex];

			TSet<int32> BoneIndices;
			BoneIndices.Append(Skin.Joints);

			for (int32 BoneIndex : Skin.Joints)
			{
				if (!Asset->Nodes.IsValidIndex(BoneIndex))
				{
					continue;
				}

				int32 ParentIndex = Asset->Nodes[BoneIndex].ParentIndex;

				TArray<int32> NodeToUpdateToJointIndices;
				bool bSkeletonGapFound = false;

				while (Asset->Nodes.IsValidIndex(ParentIndex))
				{
					if (BoneIndices.Contains(ParentIndex))
					{
						if (NodeToUpdateToJointIndices.Num() > 0)
						{
							bSkeletonGapFound = true;
						}
						break;
					}
					NodeToUpdateToJointIndices.Add(ParentIndex);
					ParentIndex = Asset->Nodes[ParentIndex].ParentIndex;
				}

				if (bSkeletonGapFound)
				{
					for (int32 NodeToUpdate : NodeToUpdateToJointIndices)
					{
						GLTF::FNode& Node = Asset->Nodes[NodeToUpdate];

						ensure(Node.Type == FNode::EType::None
							|| Node.Type == FNode::EType::Transform
							|| Node.Type == FNode::EType::Joint);
						Node.Type = FNode::EType::Joint;

						Node.bHasLocalBindPose = true;
						Node.LocalBindPose = Node.Transform;
					}
				}
			}
		}

		BuildParentJointIndices();
	}

	void FFileReader::BuildRootJoints() const
	{
		//Fix Multi root problems (at least for the semi-trivial "same parent non-joint" scenario)
		{
			auto GetRootDistance = [&](const GLTF::FNode& Node)
				{
					int32 Distance = 0;
					int32 CurrentNodeIndex = Node.Index;
					while (Asset->Nodes[CurrentNodeIndex].ParentIndex != INDEX_NONE && Asset->Nodes.IsValidIndex(Asset->Nodes[CurrentNodeIndex].ParentIndex))
					{
						Distance++;
						CurrentNodeIndex = Asset->Nodes[CurrentNodeIndex].ParentIndex;
					}
					return Distance;
				};

			struct FRootJoints {
				TArray<int32> Indices;
				int32 ParentDistanceFromRoot;
			};
			auto SortPredicate = [](const FRootJoints& GroupA, const FRootJoints& GroupB)
				{
					return GroupA.ParentDistanceFromRoot > GroupB.ParentDistanceFromRoot;
				};

			for (size_t SkinIndex = 0; SkinIndex < Asset->Skins.Num(); SkinIndex++)
			{
				GLTF::FSkinInfo& Skin = Asset->Skins[SkinIndex];

				if (!Skin.bUsed)
				{
					continue;
				}

				//0. Group RootJointNodes by ParentIndices or by Index if ParentIndex is -1
				TMap<int32, FRootJoints> ParentToRootJointIndices;
				for (size_t JointIndex = 0; JointIndex < Skin.Joints.Num(); JointIndex++)
				{
					if (!Asset->Nodes.IsValidIndex(Skin.Joints[JointIndex]))
					{
						continue;
					}
					GLTF::FNode& JointNode = Asset->Nodes[Skin.Joints[JointIndex]];
					if (!Asset->Nodes.IsValidIndex(JointNode.ParentIndex) || Asset->Nodes[JointNode.ParentIndex].Type != GLTF::FNode::EType::Joint)
					{
						FRootJoints* RootJoints = ParentToRootJointIndices.Find(JointNode.ParentIndex);
						if (RootJoints)
						{
							RootJoints->Indices.Add(JointNode.Index);
						}
						else
						{
							RootJoints = &ParentToRootJointIndices.Add(JointNode.ParentIndex != INDEX_NONE ? JointNode.ParentIndex : JointNode.Index);
							RootJoints->Indices.Add(JointNode.Index);
							RootJoints->ParentDistanceFromRoot = GetRootDistance(JointNode);
						}
					}
				}
				
				//1. Sort groups based on distance from common root (furthest to closest)
				ParentToRootJointIndices.ValueSort(SortPredicate);

				//2. Validate Skin.Skeleton if it exists:
				if (Skin.Skeleton != INDEX_NONE && Asset->Nodes.IsValidIndex(Skin.Skeleton))
				{
					const GLTF::FNode SkinSkeletonNode = Asset->Nodes[Skin.Skeleton];
					int32 SkinSkeletonDistance = GetRootDistance(SkinSkeletonNode);
					

					const FText MessageText = LOCTEXT("NonCommonRootNode", "Skeleton node is not a common root.");
					for (TPair<int32, FRootJoints> Group : ParentToRootJointIndices)
					{
						if (Group.Value.Indices.Num() == 1 )
						{
							if (!Asset->Nodes.IsValidIndex(Group.Value.Indices[0]))
							{
								continue;
							}
							const GLTF::FNode RootNodeCandidate = Asset->Nodes[Group.Value.Indices[0]];
							int32 RootNodeCandidateDistance = GetRootDistance(RootNodeCandidate);
							if (SkinSkeletonDistance > RootNodeCandidateDistance)
							{
								Messages.Emplace(EMessageSeverity::Warning, MessageText);
								break;
							}
						}
						else if (Group.Value.Indices.Num() == 0)
						{
							continue;
						}
						else
						{
							const GLTF::FNode RootNodeCandidate = Asset->Nodes[Group.Key];
							int32 RootNodeCandidateDistance = GetRootDistance(RootNodeCandidate);
							if (SkinSkeletonDistance > RootNodeCandidateDistance)
							{
								Messages.Emplace(EMessageSeverity::Warning, MessageText);
								break;
							}
						}
						
					}
				}

				//3. Introduce new true root per group
				for (TPair<int32, FRootJoints> Group : ParentToRootJointIndices)
				{
					if (Group.Value.Indices.Num() < 2)
					{
						continue;
					}

					GLTF::FNode& Node = Asset->Nodes.Emplace_GetRef();
					Node.Index = Asset->Nodes.Num() - 1;
					Node.Type = GLTF::FNode::EType::Joint;

					if (Asset->Nodes.IsValidIndex(Group.Key))
					{
						GLTF::FNode& OriginalNode = Asset->Nodes[Group.Key];

						Node.Name = OriginalNode.Name + "_ProxyTrueRootJoint";
						Node.Children = OriginalNode.Children;
						OriginalNode.Children.Reset();
						OriginalNode.Children.Add(Node.Index);
						Node.ParentIndex = OriginalNode.Index;
						Node.DirectParentJointIndex = OriginalNode.DirectParentJointIndex;

						Node.Transform = OriginalNode.Transform;
						Node.bHasLocalBindPose = true;
						Node.LocalBindPose = FTransform::Identity;

						//also update the Scenes.Nodes idx in case the oroginal node is part of the Scenes idx list:
						for (GLTF::FScene& Scene : Asset->Scenes)
						{
							int32 OriginalNodeIndexInSceneNodes;
							if (Scene.Nodes.Find(OriginalNode.Index, OriginalNodeIndexInSceneNodes))
							{
								Scene.Nodes[OriginalNodeIndexInSceneNodes] = Node.Index;
							}
						}

						for (size_t RootJointIndex = 0; RootJointIndex < Group.Value.Indices.Num(); RootJointIndex++)
						{
							GLTF::FNode& JointNode = Asset->Nodes[Group.Value.Indices[RootJointIndex]];
							JointNode.ParentIndex = Node.Index;
							JointNode.ParentJointIndex = Node.Index;
						}
					}
					else
					{
						Node.Name = "_ProxyTrueRootJoint";
						Node.Transform = FTransform::Identity;
						Node.bHasLocalBindPose = true;
						Node.LocalBindPose = FTransform::Identity;

						for (size_t RootJointIndex = 0; RootJointIndex < Group.Value.Indices.Num(); RootJointIndex++)
						{
							GLTF::FNode& JointNode = Asset->Nodes[Group.Value.Indices[RootJointIndex]];
							JointNode.ParentIndex = Node.Index;
							JointNode.ParentJointIndex = Node.Index;
							Node.Children.Add(JointNode.Index);
						}

						for (GLTF::FScene& Scene : Asset->Scenes)
						{
							bool bHadChildAsSceneNode = false;
							Scene.Nodes.RemoveAll([&bHadChildAsSceneNode, &Node](const int32& NodeIdx)
								{
									int ElementIndex;
									if (Node.Children.Find(NodeIdx, ElementIndex))
									{
										bHadChildAsSceneNode = true;
										return true;
									}
									return false;
								});

							if (bHadChildAsSceneNode)
							{
								Scene.Nodes.Add(Node.Index);
							}
						}
					}
				}
			}
		}

		//Set up the RootJointIndices:
		for (size_t Index = 0; Index < Asset->Nodes.Num(); Index++)
		{
			if (Asset->Nodes[Index].Type == GLTF::FNode::EType::Joint)
			{
				Asset->Nodes[Index].RootJointIndex = FindRootJointIndex(Index);
			}
		}
	}

	void FFileReader::ResolveConflictingTextures()
	{
		TSet<int32> DuplicateTexturesForNormalMap;

		struct FAffectedTextureMaps
		{
			TArray<FTextureMap*> TextureMaps;
		};

		TMap<int32, FAffectedTextureMaps> TextureMapsRequiringUpdate;

		auto AddTextureIndex = [&](TSet<int32>& IndicesSet, int32 TextureIndex)
		{
			if (Asset->Textures.IsValidIndex(TextureIndex))
			{
				IndicesSet.Add(TextureIndex);
			}
		};

		auto AddTextureIndexToMap = [&](TMap<int32, GLTF::FTextureMap*>& IndexToTextureMap, GLTF::FTextureMap* TextureMap)
		{
			if (TextureMap && Asset->Textures.IsValidIndex(TextureMap->TextureIndex))
			{
				IndexToTextureMap.Emplace(TextureMap->TextureIndex, TextureMap);
			}
		};

		for (GLTF::FMaterial& Material : Asset->Materials)
		{
			TSet<int32> NormalMapTextureIndices;
			TSet<int32> ColoredTextureIndices;
			TMap<int32, GLTF::FTextureMap*> IdToTextureMap;

			// Textures that are used as normal maps
			AddTextureIndex(NormalMapTextureIndices, Material.Normal.TextureIndex);
			AddTextureIndex(NormalMapTextureIndices, Material.ClearCoat.NormalMap.TextureIndex);
			AddTextureIndex(NormalMapTextureIndices, Material.Packing.NormalMap.TextureIndex);

			// To quickly identify the FTextureMaps from the Texture Index
			AddTextureIndexToMap(IdToTextureMap, &Material.Normal);
			AddTextureIndexToMap(IdToTextureMap, &Material.ClearCoat.NormalMap);
			AddTextureIndexToMap(IdToTextureMap, &Material.Packing.NormalMap);

			// Textures that are not used as normal maps
			AddTextureIndex(ColoredTextureIndices, Material.BaseColor.TextureIndex);
			AddTextureIndex(ColoredTextureIndices, Material.MetallicRoughness.Map.TextureIndex);
			AddTextureIndex(ColoredTextureIndices, Material.SpecularGlossiness.Map.TextureIndex);
			AddTextureIndex(ColoredTextureIndices, Material.ClearCoat.ClearCoatMap.TextureIndex);
			AddTextureIndex(ColoredTextureIndices, Material.ClearCoat.RoughnessMap.TextureIndex);
			AddTextureIndex(ColoredTextureIndices, Material.Transmission.TransmissionMap.TextureIndex);
			AddTextureIndex(ColoredTextureIndices, Material.Sheen.SheenColorMap.TextureIndex);
			AddTextureIndex(ColoredTextureIndices, Material.Sheen.SheenRoughnessMap.TextureIndex);
			AddTextureIndex(ColoredTextureIndices, Material.Specular.SpecularMap.TextureIndex);
			AddTextureIndex(ColoredTextureIndices, Material.Specular.SpecularColorMap.TextureIndex);
			AddTextureIndex(ColoredTextureIndices, Material.Packing.Map.TextureIndex);
			AddTextureIndex(ColoredTextureIndices, Material.Iridescence.Texture.TextureIndex);
			AddTextureIndex(ColoredTextureIndices, Material.Iridescence.Thickness.Texture.TextureIndex);
			AddTextureIndex(ColoredTextureIndices, Material.Anisotropy.Texture.TextureIndex);
			AddTextureIndex(ColoredTextureIndices, Material.Occlusion.TextureIndex);
			AddTextureIndex(ColoredTextureIndices, Material.Emissive.TextureIndex);

			TSet<int32> ConflictingTextureIndices = NormalMapTextureIndices.Intersect(ColoredTextureIndices);
			if (!ConflictingTextureIndices.IsEmpty())
			{
				for (const int32& ConflictingTextureIndex : ConflictingTextureIndices)
				{
					if (!TextureMapsRequiringUpdate.Contains(ConflictingTextureIndex))
					{
						TextureMapsRequiringUpdate.Emplace(ConflictingTextureIndex, FAffectedTextureMaps());
					}

					DuplicateTexturesForNormalMap.Add(ConflictingTextureIndex);
					TextureMapsRequiringUpdate[ConflictingTextureIndex].TextureMaps.Add(IdToTextureMap[ConflictingTextureIndex]);
				}
			}
		}

		// Add the Duplicate textures for normal maps.
		int32 TextureIndex = Asset->Textures.Num();
		for (const int32& NormalMapIndex : DuplicateTexturesForNormalMap)
		{
			FTexture DuplicateTexture = Asset->Textures[NormalMapIndex];
			DuplicateTexture.Name += TEXT("_greenFlipped");
			DuplicateTexture.UniqueId = GLTFNodePrefixStaticHelper::GetTexturePrefix(Asset->Name) + FString::FromInt(TextureIndex);
			Asset->Textures.Add(DuplicateTexture);
			FAffectedTextureMaps& AffectedTextureMaps = TextureMapsRequiringUpdate.FindChecked(NormalMapIndex);
			for (FTextureMap* AffectedTextureMap : AffectedTextureMaps.TextureMaps)
			{
				AffectedTextureMap->TextureIndex = TextureIndex;
			}
			TextureIndex++;
		}
	}

}  // namespace GLTF

#undef LOCTEXT_NAMESPACE