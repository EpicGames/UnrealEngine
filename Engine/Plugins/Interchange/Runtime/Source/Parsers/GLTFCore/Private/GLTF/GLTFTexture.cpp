// Copyright Epic Games, Inc. All Rights Reserved.

#include "GLTFTexture.h"

namespace GLTF
{
	const FSampler FSampler::DefaultSampler;

}  // namespace GLTF
