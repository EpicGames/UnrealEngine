// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConversionUtilities.h"

#include "Dom/JsonObject.h"

namespace GLTF
{
	inline uint32 ArraySize(const FJsonObject& Object, const FString& Name)
	{
		if (Object.HasTypedField<EJson::Array>(Name))
		{
			return Object.GetArrayField(Name).Num();
		}

		return 0;  // for empty arrays & if array does not exist
	}

	inline FString GetString(const FJsonObject& Object, const TCHAR* Name, const FString& DefaultValue = FString())
	{
		if (Object.HasTypedField<EJson::String>(Name))
		{
			return Object.GetStringField(Name);
		}

		return DefaultValue;
	}

	inline bool GetBool(const FJsonObject& Object, const TCHAR* Name, bool DefaultValue = false)
	{
		if (Object.HasTypedField<EJson::Boolean>(Name))
		{
			return Object.GetBoolField(Name);
		}

		return DefaultValue;
	}

	inline uint64 GetUnsignedInt64(const FJsonObject& Object, const TCHAR* Name, uint64 DefaultValue)
	{
		if (Object.HasTypedField<EJson::Number>(Name))
		{
			uint64 UnsignedInt64Value;
			
			if (Object.TryGetNumberField(Name, UnsignedInt64Value))
			{
				return UnsignedInt64Value;
			}
		}

		return DefaultValue;
	}

	inline uint32 GetUnsignedInt(const FJsonObject& Object, const TCHAR* Name, uint32 DefaultValue)
	{
		if (Object.HasTypedField<EJson::Number>(Name))
		{
			int32 SignedValue = Object.GetIntegerField(Name);
			if (SignedValue >= 0)
			{
				return (uint32)SignedValue;
			}
			// complain if negative? if fractional?
		}

		return DefaultValue;
	}

	inline uint32 GetIndex(const FJsonObject& Object, const TCHAR* Name)
	{
		return GetUnsignedInt(Object, Name, INDEX_NONE);
	}

	inline bool GetIndex(const FJsonObject& Object, const TCHAR* Name, uint32& Value)
	{
		if (Object.HasTypedField<EJson::Number>(Name))
		{
			Value = Object.GetIntegerField(Name);
			return true;
		}

		return false;
	}

	inline float GetScalar(const FJsonObject& Object, const TCHAR* Name, float DefaultValue = 0.0f)
	{
		if (Object.HasTypedField<EJson::Number>(Name))
		{
			return Object.GetNumberField(Name);
		}

		return DefaultValue;
	}

	inline FVector GetVec3(const FJsonObject& Object, const TCHAR* Name, const FVector& DefaultValue = FVector::ZeroVector)
	{
		if (Object.HasTypedField<EJson::Array>(Name))
		{
			const TArray<TSharedPtr<FJsonValue> >& Array = Object.GetArrayField(Name);
			if (Array.Num() == 3)
			{
				float X = Array[0]->AsNumber();
				float Y = Array[1]->AsNumber();
				float Z = Array[2]->AsNumber();
				return FVector(X, Y, Z);
			}
		}

		return DefaultValue;
	}

	inline FVector4 GetVec4(const FJsonObject& Object, const TCHAR* Name, const FVector4& DefaultValue = FVector4())
	{
		if (Object.HasTypedField<EJson::Array>(Name))
		{
			const TArray<TSharedPtr<FJsonValue> >& Array = Object.GetArrayField(Name);
			if (Array.Num() == 4)
			{
				float X = Array[0]->AsNumber();
				float Y = Array[1]->AsNumber();
				float Z = Array[2]->AsNumber();
				float W = Array[3]->AsNumber();
				return FVector4(X, Y, Z, W);
			}
		}

		return DefaultValue;
	}

	inline FQuat GetQuat(const FJsonObject& Object, const TCHAR* Name, const FQuat& DefaultValue = FQuat(0, 0, 0, 1))
	{
		if (Object.HasTypedField<EJson::Array>(Name))
		{
			const TArray<TSharedPtr<FJsonValue> >& Array = Object.GetArrayField(Name);
			if (Array.Num() == 4)
			{
				float X = Array[0]->AsNumber();
				float Y = Array[1]->AsNumber();
				float Z = Array[2]->AsNumber();
				float W = Array[3]->AsNumber();
				return GLTF::ConvertQuat(FQuat(X, Y, Z, W));
			}
		}

		return DefaultValue;
	}

	inline FMatrix GetMat4(const FJsonObject& Object, const TCHAR* Name, const FMatrix& DefaultValue = FMatrix::Identity)
	{
		if (Object.HasTypedField<EJson::Array>(Name))
		{
			const TArray<TSharedPtr<FJsonValue> >& Array = Object.GetArrayField(Name);
			if (Array.Num() == 16)
			{
				FMatrix Matrix;
				for (int32 Row = 0; Row < 4; ++Row)
				{
					for (int32 Col = 0; Col < 4; ++Col)
					{
						Matrix.M[Row][Col] = Array[Col * 4 + Row]->AsNumber();
					}
				}

				return GLTF::ConvertMat(Matrix);
			}
		}

		return DefaultValue;
	}

}  // namespace GLTF
