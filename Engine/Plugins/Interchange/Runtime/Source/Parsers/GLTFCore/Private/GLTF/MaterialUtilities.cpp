// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialUtilities.h"
#include "GLTFAsset.h"

#define LOCTEXT_NAMESPACE "InterchangeGLTFMaterialUtilities"

namespace GLTF
{
	// Returns scale factor if JSON has it, 1.0 by default.
	float SetTextureMap(const FJsonObject& InObject, const TCHAR* InTexName, const TCHAR* InScaleName, const TArray<FTexture>& Textures,
	                           GLTF::FTextureMap& OutMap, TArray<FLogMessage>& OutMessages)
	{
		float Scale = 1.0f;

		if (InObject.HasTypedField<EJson::Object>(InTexName))
		{
			const FJsonObject& TexObj   = *InObject.GetObjectField(InTexName);
			int32              TexIndex = GetIndex(TexObj, TEXT("index"));
			if (Textures.IsValidIndex(TexIndex))
			{
				OutMap.TextureIndex = TexIndex;
				uint32 TexCoord     = GetUnsignedInt(TexObj, TEXT("texCoord"), 0);

				OutMap.TexCoord = TexCoord;
				if (InScaleName && TexObj.HasTypedField<EJson::Number>(InScaleName))
				{
					Scale = TexObj.GetNumberField(InScaleName);
				}

				// Handle textureInfo extensions
				if (TexObj.HasTypedField<EJson::Object>(TEXT("extensions")))
				{
					static const TArray<EExtension> SupportedTextureInfoExtensions = {
						EExtension::KHR_TextureTransform
					};
					TArray<FString> SupportedTextureInfoExtensionsStringified;
					for (size_t ExtensionIndex = 0; ExtensionIndex < SupportedTextureInfoExtensions.Num(); ExtensionIndex++)
					{
						SupportedTextureInfoExtensionsStringified.Add(GLTF::ToString(SupportedTextureInfoExtensions[ExtensionIndex]));
					}

					const FJsonObject& ExtensionsObj = *TexObj.GetObjectField(TEXT("extensions"));

					for (int32 Index = 0; Index < SupportedTextureInfoExtensions.Num(); ++Index)
					{
						const FString ExtensionName = SupportedTextureInfoExtensionsStringified[Index];
						if (!ExtensionsObj.HasTypedField<EJson::Object>(ExtensionName))
						{
							OutMessages.Emplace(RuntimeWarningSeverity(), FText::Format(LOCTEXT("UnsupportedExtension", "Extension is not supported: {0}"), FText::FromString(ExtensionName)));
							continue;
						}

						const FJsonObject& ExtObj = *ExtensionsObj.GetObjectField(ExtensionName);

						const EExtension Extension = SupportedTextureInfoExtensions[Index];
						switch (Extension)
						{
							case EExtension::KHR_TextureTransform:
							{
								OutMap.bHasTextureTransform = true;
								OutMap.TextureTransform.Offset[0] = 0.0f;
								OutMap.TextureTransform.Offset[1] = 0.0f;
								OutMap.TextureTransform.Scale[0] = 1.0f;
								OutMap.TextureTransform.Scale[1] = 1.0f;
								OutMap.TextureTransform.Rotation = GetScalar(ExtObj, TEXT("rotation"), 0.f);

								if (ExtObj.HasTypedField<EJson::Array>(TEXT("offset")))
								{
									const TArray<TSharedPtr<FJsonValue>>& Offset = ExtObj.GetArrayField(TEXT("offset"));
									if (Offset.Num() >= 2)
									{
										OutMap.TextureTransform.Offset[0] = static_cast<float>(Offset[0]->AsNumber());
										OutMap.TextureTransform.Offset[1] = static_cast<float>(Offset[1]->AsNumber());
									}
								}

								if (ExtObj.HasTypedField<EJson::Array>(TEXT("scale")))
								{
									const TArray<TSharedPtr<FJsonValue>>& TransformScale = ExtObj.GetArrayField(TEXT("scale"));
									if(TransformScale.Num() >= 2)
									{
										OutMap.TextureTransform.Scale[0] = static_cast<float>(TransformScale[0]->AsNumber());
										OutMap.TextureTransform.Scale[1] = static_cast<float>(TransformScale[1]->AsNumber());
									}
								}
							}
							break;
						}
					}
				}
			}
		}

		return Scale;
	}

}  // namespace GLTF

#undef LOCTEXT_NAMESPACE