// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GLTFLogger.h"
#include "JsonUtilities.h"

namespace GLTF { struct FTexture; }
namespace GLTF { struct FTextureMap; }

namespace GLTF
{
	// Returns scale factor if JSON has it, 1.0 by default.
	float SetTextureMap(const FJsonObject& InObject, const TCHAR* InTexName, const TCHAR* InScaleName, const TArray<FTexture>& Textures,
		GLTF::FTextureMap& OutMap, TArray<FLogMessage>& OutMessages);

}  // namespace GLTF
