// Copyright Epic Games, Inc. All Rights Reserved.

#include "GLTFLogger.h"

#include "CoreGlobals.h"

namespace GLTF
{
	EMessageSeverity RuntimeWarningSeverity()
	{
		if (GIsAutomationTesting)
		{
			return EMessageSeverity::Display;
		}
		else
		{
			return EMessageSeverity::Warning;
		}
	}
}  // namespace GLTF
