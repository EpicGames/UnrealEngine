// Copyright Epic Games, Inc. All Rights Reserved.

#include "GLTFMapFactory.h"

#include "GLTFMaterial.h"
#include "GLTFMaterialExpressions.h"

#include "GLTFMaterialFactory.h"

namespace GLTF
{
	namespace
	{
		void CreateTextureCoordinate(int32 TexCoord, FMaterialExpressionTexture& TexExpression, FMaterialElement& MaterialElement)
		{
			if (TexCoord)
			{
				FMaterialExpressionTextureCoordinate* CoordExpression = MaterialElement.AddMaterialExpression<FMaterialExpressionTextureCoordinate>();
				CoordExpression->SetCoordinateIndex(TexCoord);
				CoordExpression->ConnectExpression(TexExpression.GetInputCoordinate(), 0);
			}
		}

		void CreateTextureTransform(const FTextureTransform& InTextureTransform, FMaterialExpressionTexture& TexExpression, FMaterialElement& MaterialElement)
		{
			FMaterialExpression* CoordExpression = TexExpression.GetInputCoordinate().GetExpression();

			if (CoordExpression == nullptr)
			{
				CoordExpression = MaterialElement.AddMaterialExpression<FMaterialExpressionTextureCoordinate>();
			}

			FMaterialExpression* LastExprInChain = nullptr;

			FMaterialExpressionGeneric* UVMultiplyExpr = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			UVMultiplyExpr->SetExpressionName(TEXT("Multiply"));

			FMaterialExpressionGeneric* UVScale = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			UVScale->SetExpressionName(TEXT("Constant2Vector"));
			UVScale->SetFloatProperty(TEXT("R"), InTextureTransform.Scale[0]);
			UVScale->SetFloatProperty(TEXT("G"), InTextureTransform.Scale[1]);

			CoordExpression->ConnectExpression(*UVMultiplyExpr->GetInput(0), 0);
			UVScale->ConnectExpression(*UVMultiplyExpr->GetInput(1), 0);

			LastExprInChain = UVMultiplyExpr;

			if (!FMath::IsNearlyZero(InTextureTransform.Rotation))
			{
				FMaterialExpressionFunctionCall* UVRotate = MaterialElement.AddMaterialExpression<FMaterialExpressionFunctionCall>();
				UVRotate->SetFunctionPathName(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Texturing/CustomRotator.CustomRotator"));

				float AngleRadians = InTextureTransform.Rotation;

				if (AngleRadians < 0.0f)
				{
					AngleRadians = TWO_PI - AngleRadians;
				}

				FMaterialExpressionScalar* RotationAngle = MaterialElement.AddMaterialExpression<FMaterialExpressionScalar>();
				RotationAngle->GetScalar() = 1.0f - (AngleRadians / TWO_PI); // Normalize angle value

				FMaterialExpressionGeneric* RotationPivot = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
				RotationPivot->SetExpressionName(TEXT("Constant2Vector"));
				RotationPivot->SetFloatProperty(TEXT("R"), 0.0f);
				RotationPivot->SetFloatProperty(TEXT("G"), 0.0f);

				LastExprInChain->ConnectExpression(*UVRotate->GetInput(0), 0);
				RotationPivot->ConnectExpression(*UVRotate->GetInput(1), 0);
				RotationAngle->ConnectExpression(*UVRotate->GetInput(2), 0);

				LastExprInChain = UVRotate;
			}

			if (!FMath::IsNearlyZero(InTextureTransform.Offset[0]) || 
				!FMath::IsNearlyZero(InTextureTransform.Offset[1]))
			{
				FMaterialExpressionGeneric* UVOffset = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
				UVOffset->SetExpressionName(TEXT("Constant2Vector"));
				UVOffset->SetFloatProperty(TEXT("R"), InTextureTransform.Offset[0]);
				UVOffset->SetFloatProperty(TEXT("G"), InTextureTransform.Offset[1]);

				FMaterialExpressionGeneric* AddExpr = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
				AddExpr->SetExpressionName(TEXT("Add"));

				LastExprInChain->ConnectExpression(*AddExpr->GetInput(0), 0);
				UVOffset->ConnectExpression(*AddExpr->GetInput(1), 0);

				LastExprInChain = AddExpr;
			}

			LastExprInChain->ConnectExpression(TexExpression.GetInputCoordinate(), 0);
		}

		FMaterialExpressionInput& GetFirstInput(FMaterialExpression* Expression)
		{
			FMaterialExpressionInput* Input = Expression->GetInput(0);
			if (Input->GetExpression() == nullptr)
				return *Input;
			else
				return GetFirstInput(Input->GetExpression());
		}

		inline void SetExpresisonValue(float Value, FMaterialExpressionScalar* Expression)
		{
			Expression->GetScalar() = Value;
		}

		inline void SetExpresisonValue(const FVector3f& Color, FMaterialExpressionColor* Expression)
		{
			Expression->GetColor() = FLinearColor(Color);
		}

		inline void SetExpresisonValue(const FVector4f& Color, FMaterialExpressionColor* Expression)
		{
			Expression->GetColor() = FLinearColor(Color);
		}
	}

	FPBRMapFactory::FPBRMapFactory(ITextureFactory& TextureFactory)
	    : CurrentMaterialElement(nullptr)
	    , TextureFactory(TextureFactory)
	    , ParentPackage(nullptr)
	    , Flags(RF_NoFlags)
	{
	}

	void FPBRMapFactory::CreateNormalMap(const GLTF::FTexture& Map, int CoordinateIndex, float NormalScale, const GLTF::FTextureTransform* TextureTransform)
	{
		if (!ensure(CurrentMaterialElement))
		{
			return;
		}

		FMaterialExpressionTexture* TexExpression = CreateTextureMap(Map, CoordinateIndex, TEXT("Normal Map"), ETextureMode::Normal, TextureTransform);
		if (!TexExpression)
		{
			return;
		}

		FMaterialExpressionScalar* ScalarExpression = CurrentMaterialElement->AddMaterialExpression<FMaterialExpressionScalar>();
		ScalarExpression->SetName(TEXT("Normal Scale"));
		ScalarExpression->SetGroupName(*GroupName);
		ScalarExpression->GetScalar() = NormalScale;

		// GLTF specifies that the following formula is used:
		// scaledNormal = normalize((<sampled normal texture value> * 2.0 - 1.0) * vec3(<normal scale>, <normal scale>, 1.0)).
		FMaterialExpressionGeneric* NormalXY = CurrentMaterialElement->AddMaterialExpression<FMaterialExpressionGeneric>();
		NormalXY->SetExpressionName(TEXT("ComponentMask"));
		NormalXY->SetBoolProperty(TEXT("R"), true);
		NormalXY->SetBoolProperty(TEXT("G"), true);
		NormalXY->SetBoolProperty(TEXT("B"), false);
		TexExpression->ConnectExpression(*NormalXY->GetInput(0), 0);

		FMaterialExpressionGeneric* NormalZ = CurrentMaterialElement->AddMaterialExpression<FMaterialExpressionGeneric>();
		NormalZ->SetExpressionName(TEXT("ComponentMask"));
		NormalZ->SetBoolProperty(TEXT("R"), false);
		NormalZ->SetBoolProperty(TEXT("G"), false);
		NormalZ->SetBoolProperty(TEXT("B"), true);
		TexExpression->ConnectExpression(*NormalZ->GetInput(0), 0);

		FMaterialExpressionGeneric* MultiplyXYExpression = CurrentMaterialElement->AddMaterialExpression<FMaterialExpressionGeneric>();
		MultiplyXYExpression->SetExpressionName(TEXT("Multiply"));
		NormalXY->ConnectExpression(*MultiplyXYExpression->GetInput(0), 0);
		ScalarExpression->ConnectExpression(*MultiplyXYExpression->GetInput(1), 0);

		FMaterialExpressionGeneric* ReconstructNormalVector = CurrentMaterialElement->AddMaterialExpression<FMaterialExpressionGeneric>();
		ReconstructNormalVector->SetExpressionName(TEXT("AppendVector"));
		MultiplyXYExpression->ConnectExpression(*ReconstructNormalVector->GetInput(0), 0);
		NormalZ->ConnectExpression(*ReconstructNormalVector->GetInput(1), 0);

		FMaterialExpressionGeneric* Normalize = CurrentMaterialElement->AddMaterialExpression<FMaterialExpressionGeneric>();
		Normalize->SetExpressionName(TEXT("Normalize"));

		ReconstructNormalVector->ConnectExpression(*Normalize->GetInput(0), 0);

		Normalize->ConnectExpression(CurrentMaterialElement->GetNormal(), 0);
	}

	FMaterialExpression* FPBRMapFactory::CreateColorMap(const GLTF::FTexture& Map, int CoordinateIndex, const FVector3f& Color, const TCHAR* MapName,
	                                                    const TCHAR* ValueName, ETextureMode TextureMode, FMaterialExpressionInput& MaterialInput,
														const GLTF::FTextureTransform* TextureTransform)
	{
		return CreateMap<FMaterialExpressionColor>(Map, CoordinateIndex, Color, MapName, ValueName, TextureMode, MaterialInput, TextureTransform);
	}

	FMaterialExpression* FPBRMapFactory::CreateColorMap(const GLTF::FTexture& Map, int CoordinateIndex, const FVector4f& Color, const TCHAR* MapName,
	                                                    const TCHAR* ValueName, ETextureMode TextureMode, FMaterialExpressionInput& MaterialInput, 
														const GLTF::FTextureTransform* TextureTransform)
	{
		return CreateMap<FMaterialExpressionColor>(Map, CoordinateIndex, Color, MapName, ValueName, TextureMode, MaterialInput, TextureTransform);
	}

	void FPBRMapFactory::CreateMultiMap(const GLTF::FTexture& Map, int CoordinateIndex, const TCHAR* MapName, const FMapChannel* MapChannels,
	                                    uint32 MapChannelsCount, ETextureMode TextureMode, const GLTF::FTextureTransform* TextureTransform)
	{
		FExpressionList ValueExpressions;
		for (uint32 Index = 0; Index < MapChannelsCount; ++Index)
		{
			const FMapChannel& MapChannel = MapChannels[Index];

			FMaterialExpressionParameter* ValueExpression = nullptr;
			switch (MapChannel.Channel)
			{
				case EChannel::RG:
				case EChannel::RGB:
				case EChannel::All:
				{
					ValueExpression = CurrentMaterialElement->AddMaterialExpression<FMaterialExpressionColor>();
					FMaterialExpressionColor* ColorExpression = static_cast<FMaterialExpressionColor*>(ValueExpression);
					SetExpresisonValue(*reinterpret_cast<const FVector3f*>(MapChannel.VecValue), ColorExpression);
					ColorExpression->SetGroupName(*GroupName);
					break;
				}
				default:
					ValueExpression = CurrentMaterialElement->AddMaterialExpression<FMaterialExpressionScalar>();
					FMaterialExpressionScalar* ScalarExpression = static_cast<FMaterialExpressionScalar*>(ValueExpression);
					SetExpresisonValue(MapChannel.Value, ScalarExpression);
					ScalarExpression->SetGroupName(*GroupName);
					break;
			}

			ValueExpression->SetName(MapChannel.ValueName);

			ValueExpressions.Add(ValueExpression);
		}

		FMaterialExpressionTexture* TexExpression = CreateTextureMap(Map, CoordinateIndex, MapName, TextureMode, TextureTransform);
		if (TexExpression)
		{
			for (uint32 Index = 0; Index < MapChannelsCount; ++Index)
			{
				const FMapChannel& MapChannel = MapChannels[Index];

				FMaterialExpressionGeneric* MultiplyExpression = CurrentMaterialElement->AddMaterialExpression<FMaterialExpressionGeneric>();
				MultiplyExpression->SetExpressionName(TEXT("Multiply"));

				switch (MapChannel.Channel)
				{
				case EChannel::RG:
				{
					FMaterialExpressionFunctionCall* MakeFloat2 = CurrentMaterialElement->AddMaterialExpression<FMaterialExpressionFunctionCall>();
					MakeFloat2->SetFunctionPathName(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/MakeFloat2.MakeFloat2"));

					TexExpression->ConnectExpression(*MakeFloat2->GetInput(0), (int32)EChannel::Red);
					TexExpression->ConnectExpression(*MakeFloat2->GetInput(1), (int32)EChannel::Green);
					MakeFloat2->ConnectExpression(*MultiplyExpression->GetInput(0), 0);
					break;
				}
				// RGB is the top channel now
				case EChannel::RGB:
					TexExpression->ConnectExpression(*MultiplyExpression->GetInput(0), 0);
					break;
				case EChannel::All:
					TexExpression->ConnectExpression(*MultiplyExpression->GetInput(0), 5);
					break;
				default:
					// single channel connection
					TexExpression->ConnectExpression(*MultiplyExpression->GetInput(0), (int32)MapChannel.Channel);
					break;
				}

				ValueExpressions[Index]->ConnectExpression(*MultiplyExpression->GetInput(1), 0);
				if (MapChannel.OutputExpression)
				{
					MultiplyExpression->ConnectExpression(GetFirstInput(MapChannel.OutputExpression), 0);
					MapChannel.OutputExpression->ConnectExpression(*MapChannel.MaterialInput, 0);
				}
				else
				{
					MultiplyExpression->ConnectExpression(*MapChannel.MaterialInput, 0);
				}
			}
		}
		else
		{
			// no texture present, make value connections
			for (uint32 Index = 0; Index < MapChannelsCount; ++Index)
			{
				const FMapChannel& MapChannel = MapChannels[Index];

				if (MapChannel.OutputExpression)
				{
					ValueExpressions[Index]->ConnectExpression(GetFirstInput(MapChannel.OutputExpression), 0);
					MapChannel.OutputExpression->ConnectExpression(*MapChannel.MaterialInput, 0);
				}
				else
				{
					ValueExpressions[Index]->ConnectExpression(*MapChannel.MaterialInput, 0);
				}
			}
		}
	}

	FMaterialExpressionTexture* FPBRMapFactory::CreateTextureMap(const GLTF::FTexture& Map, int CoordinateIndex, const TCHAR* MapName,
	                                                             ETextureMode TextureMode, const GLTF::FTextureTransform* TextureTransform)
	{
		ITextureElement* Texture = TextureFactory.CreateTexture(Map, ParentPackage, Flags, TextureMode);

		FMaterialExpressionTexture* TexExpression = nullptr;
		if (Texture)
		{
			TexExpression = CurrentMaterialElement->AddMaterialExpression<FMaterialExpressionTexture>();
			TexExpression->SetTexture(Texture);
			TexExpression->SetName(*(FString(MapName) + TEXT(" Map")));
			TexExpression->SetGroupName(*GroupName);

			CreateTextureCoordinate(CoordinateIndex, *TexExpression, *CurrentMaterialElement);

			if (nullptr != TextureTransform)
			{
				CreateTextureTransform(*TextureTransform, *TexExpression, *CurrentMaterialElement);
			}
		}
		return TexExpression;
	}

	template <class ValueExpressionClass, class ValueClass>
	FMaterialExpression* FPBRMapFactory::CreateMap(const GLTF::FTexture& Map, int CoordinateIndex, const ValueClass& Value, const TCHAR* MapName,
	                                               const TCHAR* ValueName, ETextureMode TextureMode, FMaterialExpressionInput& MaterialInput, 
												   const GLTF::FTextureTransform* TextureTransform)
	{
		if (!ensure(MapName) || !ensure(CurrentMaterialElement))
		{
			return nullptr;
		}

		ValueExpressionClass* ValueExpression = CurrentMaterialElement->AddMaterialExpression<ValueExpressionClass>();
		if (ValueName)
			ValueExpression->SetName(*(FString(MapName) + TEXT(" ") + ValueName));
		else
			ValueExpression->SetName(MapName);
		ValueExpression->SetGroupName(*GroupName);
		SetExpresisonValue(Value, ValueExpression);

		FMaterialExpressionTexture* TexExpression = CreateTextureMap(Map, CoordinateIndex, MapName, TextureMode, TextureTransform);
		if (TexExpression)
		{
			FMaterialExpressionGeneric* MultiplyExpression = CurrentMaterialElement->AddMaterialExpression<FMaterialExpressionGeneric>();
			MultiplyExpression->SetExpressionName(TEXT("Multiply"));

			TexExpression->ConnectExpression(*MultiplyExpression->GetInput(0), 0);
			ValueExpression->ConnectExpression(*MultiplyExpression->GetInput(1), 0);
			MultiplyExpression->ConnectExpression(MaterialInput, 0);
			return MultiplyExpression;
		}
		else
		{
			ValueExpression->ConnectExpression(MaterialInput, 0);
			return ValueExpression;
		}
	}

}  // namespace GLTF
