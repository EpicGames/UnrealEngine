// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Math/Vector4.h"
#include "UObject/ObjectMacros.h"

namespace GLTF { class FMaterialElement; }
namespace GLTF { class FMaterialExpression; }
namespace GLTF { class FMaterialExpressionInput; }
namespace GLTF { class FMaterialExpressionTexture; }
namespace GLTF { class ITextureFactory; }
namespace GLTF { enum class ETextureMode; }
template <uint32 NumInlineElements> class TFixedAllocator;

namespace GLTF
{
	struct FTexture;
	struct FTextureTransform;
	class FTextureFactory;

	struct FPBRMapFactory
	{
		enum class EChannel
		{
			All,
			Red,
			Green,
			Blue,
			Alpha,
			RG,
			RGB
		};

		struct FMapChannel
		{
			union {
				float Value;
				float VecValue[3];
			};

			const TCHAR*              ValueName;
			EChannel                  Channel;
			FMaterialExpressionInput* MaterialInput;
			FMaterialExpression*      OutputExpression;

			FMapChannel(float InValue, const TCHAR* InValueName, EChannel InChannel, FMaterialExpressionInput* InMaterialInput, FMaterialExpression* InOutputExpression)
				: FMapChannel(InValueName, InChannel, InMaterialInput, InOutputExpression)
			{
				Value = InValue;
			}

			FMapChannel(const FVector& InVecValue, const TCHAR* InValueName, EChannel InChannel, FMaterialExpressionInput* InMaterialInput, FMaterialExpression* InOutputExpression)
				: FMapChannel(InValueName, InChannel, InMaterialInput, InOutputExpression)
			{
				SetValue(InVecValue);
			}

			void SetValue(const FVector& Vec)
			{
				VecValue[0] = (float)Vec.X;
				VecValue[1] = (float)Vec.Y;
				VecValue[2] = (float)Vec.Z;
			}

		private:
			FMapChannel(const TCHAR* InValueName, EChannel InChannel, FMaterialExpressionInput* InMaterialInput, FMaterialExpression* InOutputExpression)
				: ValueName(InValueName)
				, Channel(InChannel)
				, MaterialInput(InMaterialInput)
				, OutputExpression(InOutputExpression)
			{
			}
		};

		FMaterialElement* CurrentMaterialElement;
		FString           GroupName;

	public:
		FPBRMapFactory(ITextureFactory& TextureFactory);

		void SetParentPackage(UObject* ParentPackage, EObjectFlags Flags);

		void CreateNormalMap(const GLTF::FTexture& Map, int CoordinateIndex, float NormalScale, const GLTF::FTextureTransform* TextureTransform);

		FMaterialExpression* CreateColorMap(const GLTF::FTexture& Map, int CoordinateIndex, const FVector3f& Color, const TCHAR* MapName,
		                                    const TCHAR* ValueName, ETextureMode TextureMode, FMaterialExpressionInput& MaterialInput,
											const GLTF::FTextureTransform* TextureTransform);

		FMaterialExpression* CreateColorMap(const GLTF::FTexture& Map, int CoordinateIndex, const FVector4f& Color, const TCHAR* MapName,
		                                    const TCHAR* ValueName, ETextureMode TextureMode, FMaterialExpressionInput& MaterialInput,
											const GLTF::FTextureTransform* TextureTransform);

		FMaterialExpressionTexture* CreateTextureMap(const GLTF::FTexture& Map, int CoordinateIndex, const TCHAR* MapName, ETextureMode TextureMode, 
													 const GLTF::FTextureTransform* TextureTransform);

		void CreateMultiMap(const GLTF::FTexture& Map, int CoordinateIndex, const TCHAR* MapName, const FMapChannel* MapChannels,
		                    uint32 MapChannelsCount, ETextureMode TextureMode, const GLTF::FTextureTransform* TextureTransform);

	private:
		using FExpressionList = TArray<FMaterialExpression*, TFixedAllocator<4> >;

		template <class ValueExpressionClass, class ValueClass>
		FMaterialExpression* CreateMap(const GLTF::FTexture& Map, int CoordinateIndex, const ValueClass& Value, const TCHAR* MapName,
		                               const TCHAR* ValueName, ETextureMode TextureMode, FMaterialExpressionInput& MaterialInput, 
									   const GLTF::FTextureTransform* TextureTransform);

	private:
		ITextureFactory& TextureFactory;
		UObject*         ParentPackage;
		EObjectFlags     Flags;
	};

	inline void FPBRMapFactory::SetParentPackage(UObject* InParentPackage, EObjectFlags InFlags)
	{
		ParentPackage = InParentPackage;
		Flags         = InFlags;
	}

}  // namespace GLTF
