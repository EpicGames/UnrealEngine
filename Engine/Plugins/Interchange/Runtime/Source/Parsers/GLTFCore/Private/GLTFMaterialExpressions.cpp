// Copyright Epic Games, Inc. All Rights Reserved.

#include "GLTFMaterialExpressions.h"


namespace GLTF
{
	FMaterialExpressionInput::FMaterialExpressionInput(const FString& Name)
	    : Name(Name)
	    , Expression(nullptr)
	    , OutputIndex(INDEX_NONE)
	{
	}

	FMaterialExpression::FMaterialExpression() {}

	FMaterialExpression::FMaterialExpression(const FMaterialExpression& Expr)
	    : Type(Expr.Type)
	{
	}
	FMaterialExpression::FMaterialExpression(int Type)
	    : Type((EMaterialExpressionType)Type)
	{
	}

	void FMaterialExpression::ConnectExpression(FMaterialExpressionInput& ExpressionInput, int32 OutputIndex)
	{
		if (OutputIndex != INDEX_NONE)
		{
			ExpressionInput.Expression  = this;
			ExpressionInput.OutputIndex = OutputIndex;
		}
	}

	FMaterialExpressionColor::FMaterialExpressionColor()
	    : FMaterialExpressionParameter(Type)
	{
	}

	FMaterialExpressionTexture::FMaterialExpressionTexture()
	    : FMaterialExpressionParameter(Type)
	    , InputCoordinate(TEXT("InputCoordinate"))
	    , Texture(nullptr)
	{
	}

	FMaterialExpressionTextureCoordinate::FMaterialExpressionTextureCoordinate()
	    : FMaterialExpression(Type)
	    , CoordinateIndex(0)
	{
	}

	FMaterialExpressionGeneric::FMaterialExpressionGeneric()
	    : FMaterialExpression(Type)
	{
	}

	FMaterialExpressionFunctionCall::FMaterialExpressionFunctionCall()
	    : FMaterialExpression(Type)
	{
	}

	//

	FMaterialExpressionParameter::FMaterialExpressionParameter(int Type)
	    : FMaterialExpression(Type)
	{
	}

	void FMaterialExpressionParameter::SetName(const TCHAR* InName)
	{
		ensure(Type == EMaterialExpressionType::ConstantColor || Type == EMaterialExpressionType::ConstantScalar ||
		      Type == EMaterialExpressionType::Texture);
		Name = InName;
	}

	const TCHAR* FMaterialExpressionParameter::GetName() const
	{
		ensure(Type == EMaterialExpressionType::ConstantColor || Type == EMaterialExpressionType::ConstantScalar ||
		      Type == EMaterialExpressionType::Texture);
		return *Name;
	}

	void FMaterialExpressionParameter::SetGroupName(const TCHAR* InGroupName)
	{
		ensure(Type == EMaterialExpressionType::ConstantColor || Type == EMaterialExpressionType::ConstantScalar ||
		      Type == EMaterialExpressionType::Texture);
		GroupName = InGroupName;
	}

	const TCHAR* FMaterialExpressionParameter::GetGroupName() const
	{
		ensure(Type == EMaterialExpressionType::ConstantColor || Type == EMaterialExpressionType::ConstantScalar ||
		      Type == EMaterialExpressionType::Texture);
		return *GroupName;
	}

	//

	FMaterialExpressionInput* FMaterialExpressionTexture::GetInput(int32 Index)
	{
		ensure(Index == 0);
		return &InputCoordinate;
	}

	int32 FMaterialExpressionTexture::GetInputCount() const
	{
		return 1;
	}

	FMaterialExpressionInput* FMaterialExpressionGeneric::GetInput(int32 Index)
	{
		while (!Inputs.IsValidIndex(Index))
		{
			Inputs.Emplace(*FString::FromInt(Inputs.Num()));
		}

		return &Inputs[Index];
	}

	int32 FMaterialExpressionGeneric::GetInputCount() const
	{
		return Inputs.Num();
	}

	FMaterialExpressionInput* FMaterialExpressionFunctionCall::GetInput(int32 Index)
	{
		while (!Inputs.IsValidIndex(Index))
		{
			Inputs.Emplace(*FString::FromInt(Inputs.Num()));
		}

		return &Inputs[Index];
	}

	int32 FMaterialExpressionFunctionCall::GetInputCount() const
	{
		return Inputs.Num();
	}

	//

	FMaterialElement::FMaterialElement(const FString& Name)
	    : Name(Name)
	    , BaseColor(TEXT("BaseColor"))
	    , Metallic(TEXT("Metallic"))
	    , Specular(TEXT("Specular"))
	    , Roughness(TEXT("Roughness"))
	    , EmissiveColor(TEXT("EmissiveColor"))
	    , Opacity(TEXT("Opacity"))
	    , Normal(TEXT("Normal"))
	    , WorldDisplacement(TEXT("WorldDisplacement"))
	    , Refraction(TEXT("Refraction"))
	    , AmbientOcclusion(TEXT("AmbientOcclusion"))
		, ClearCoat(TEXT("ClearCoat"))
		, ClearCoatRoughness(TEXT("ClearCoatRoughness"))
		, ThinTranslucentMaterialOutput(nullptr)
		, ClearCoatBottomNormalOutput(nullptr)
	    , bIsFinal(false)
	{
	}

	FMaterialElement::~FMaterialElement()
	{
		for (GLTF::FMaterialExpression* Expr : Expressions)
		{
			delete Expr;
		}
	}

    FMD5Hash FMaterialElement::GetGLTFMaterialHash() const
	{
		return GLTFMaterialHash;
	}

	void FMaterialElement::SetGLTFMaterialHash(FMD5Hash Hash)
	{
		GLTFMaterialHash = Hash;
	}

	int32 FMaterialElement::GetExpressionsCount() const
	{
		return Expressions.Num();
	}

	FMaterialExpression* FMaterialElement::GetExpression(int32 Index)
	{
		if (!ensure(Index < Expressions.Num()))
		{
			return nullptr;
		}
		return Expressions[Index];
	}

	FMaterialExpression* FMaterialElement::AddMaterialExpression(EMaterialExpressionType ExpressionType)
	{
		FMaterialExpression* Expression = nullptr;
		switch (ExpressionType)
		{
			case GLTF::EMaterialExpressionType::ConstantColor:
				Expression = new FMaterialExpressionColor();
				break;
			case GLTF::EMaterialExpressionType::ConstantScalar:
				Expression = new FMaterialExpressionScalar();
				break;
			case GLTF::EMaterialExpressionType::FunctionCall:
				Expression = new FMaterialExpressionFunctionCall();
				break;
			case GLTF::EMaterialExpressionType::Generic:
				Expression = new FMaterialExpressionGeneric();
				break;
			case GLTF::EMaterialExpressionType::Texture:
				Expression = new FMaterialExpressionTexture();
				break;
			case GLTF::EMaterialExpressionType::TextureCoordinate:
				Expression = new FMaterialExpressionTextureCoordinate();
				break;
			default:
				ensure(false);
		}
		if (Expression)
			Expressions.Add(Expression);
		return Expression;
	}

}  // namespace GLTF
