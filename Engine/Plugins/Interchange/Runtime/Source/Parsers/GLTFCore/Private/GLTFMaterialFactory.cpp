// Copyright Epic Games, Inc. All Rights Reserved.

#include "GLTFMaterialFactory.h"

// IWYU pragma: begin_keep
#include "GLTFAnimation.h"
#include "GLTFMapFactory.h"
#include "GLTFMaterial.h"
#include "GLTFMaterialExpressions.h"
#include "GLTFAsset.h"
#include "Engine/EngineTypes.h"
#include "GLTFMesh.h"
#include "GLTFNode.h"
#include "GLTFTexture.h"
// IWYU pragma: end_keep

#define LOCTEXT_NAMESPACE "InterchangeGLTFMaterialFactory"

namespace GLTF
{
	namespace
	{
		const GLTF::FTexture& GetTexture(const GLTF::FTextureMap& Map, const TArray<GLTF::FTexture>& Textures)
		{
			static const GLTF::FImage   Image;
			static const GLTF::FTexture None(FString(), Image, GLTF::FSampler::DefaultSampler);
			return Map.TextureIndex != INDEX_NONE ? Textures[Map.TextureIndex] : None;
		}
	}

	class FMaterialFactoryImpl : public GLTF::FBaseLogger
	{
	public:
		FMaterialFactoryImpl(IMaterialElementFactory* MaterialElementFactory, ITextureFactory* TextureFactory)
		    : MaterialElementFactory(MaterialElementFactory)
		    , TextureFactory(TextureFactory)
		{
			if (!ensure(MaterialElementFactory))
			{
				Messages.Emplace(EMessageSeverity::Warning, LOCTEXT("MaterialElementFactoryNull", "MaterialElementFactory is unexpected nullptr."));
			}
		}

		const TArray<FMaterialElement*>& CreateMaterials(const GLTF::FAsset& Asset, UObject* ParentPackage, EObjectFlags Flags);

	private:
		void HandleOpacity(const TArray<GLTF::FTexture>& Texture, const GLTF::FMaterial& GLTFMaterial, FMaterialElement& MaterialElement);
		void HandleShadingModel(const TArray<GLTF::FTexture>& Texture, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement);
		void HandleOcclusion(const TArray<GLTF::FTexture>& Texture, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement);
		void HandleEmissive(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement);
		void HandleNormal(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement);
		void HandleClearCoat(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement);
		void HandleTransmission(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement);
		void HandleSheen(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement);
		void HandleIOR(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement);
		void HandleSpecular(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement);


	private:
		TUniquePtr<IMaterialElementFactory> MaterialElementFactory;
		TUniquePtr<ITextureFactory>         TextureFactory;
		TArray<FMaterialElement*>           Materials;

		friend class FMaterialFactory;
	};

	namespace
	{
		EBlendMode ConvertAlphaMode(FMaterial::EAlphaMode Mode)
		{
			switch (Mode)
			{
				case FMaterial::EAlphaMode::Opaque:
					return EBlendMode::BLEND_Opaque;
				case FMaterial::EAlphaMode::Blend:
					return EBlendMode::BLEND_Translucent;
				case FMaterial::EAlphaMode::Mask:
					return EBlendMode::BLEND_Masked;
				default:
					return EBlendMode::BLEND_Opaque;
			}
		}

		template <class ReturnClass>
		ReturnClass* FindExpression(const FString& Name, FMaterialElement& MaterialElement)
		{
			ReturnClass* Result = nullptr;
			for (int32 Index = 0; Index < MaterialElement.GetExpressionsCount(); ++Index)
			{
				FMaterialExpression* Expression = MaterialElement.GetExpression(Index);

				if (!Expression)
				{
					continue;
				}

				if (Expression->GetType() != EMaterialExpressionType::ConstantColor &&
				    Expression->GetType() != EMaterialExpressionType::ConstantScalar && Expression->GetType() != EMaterialExpressionType::Texture)
					continue;

				FMaterialExpressionParameter* ExpressionParameter = static_cast<FMaterialExpressionParameter*>(Expression);
				if (ExpressionParameter->GetName() == Name)
				{
					if (ensure(Expression->GetType() == (EMaterialExpressionType)ReturnClass::Type))
					{
						Result = static_cast<ReturnClass*>(ExpressionParameter);
					}
					break;
				}
			}
			return Result;
		}

	}

	const TArray<FMaterialElement*>& FMaterialFactoryImpl::CreateMaterials(const GLTF::FAsset& Asset, UObject* ParentPackage, EObjectFlags Flags)
	{
		TextureFactory->CleanUp();
		Materials.Empty();

		if (!MaterialElementFactory)
		{
			return Materials;
		}

		Materials.Reserve(Asset.Materials.Num() + 1);

		Messages.Empty();

		FPBRMapFactory MapFactory(*TextureFactory);
		MapFactory.SetParentPackage(ParentPackage, Flags);

		for (const GLTF::FMaterial& GLTFMaterial : Asset.Materials)
		{
			if (!ensure(!GLTFMaterial.Name.IsEmpty()))
			{
				continue;
			}

			FMaterialElement* MaterialElement = MaterialElementFactory->CreateMaterial(*GLTFMaterial.Name, ParentPackage, Flags);
			MaterialElement->SetTwoSided(GLTFMaterial.bIsDoubleSided);
			MaterialElement->SetBlendMode(ConvertAlphaMode(GLTFMaterial.AlphaMode));

			MapFactory.CurrentMaterialElement = MaterialElement;

			HandleShadingModel(Asset.Textures, GLTFMaterial, MapFactory, *MaterialElement);
			HandleOpacity(Asset.Textures, GLTFMaterial, *MaterialElement);
			HandleClearCoat(Asset.Textures, GLTFMaterial, MapFactory, *MaterialElement);
			HandleTransmission(Asset.Textures, GLTFMaterial, MapFactory, *MaterialElement);
			HandleSheen(Asset.Textures, GLTFMaterial, MapFactory, *MaterialElement);
			HandleIOR(Asset.Textures, GLTFMaterial, MapFactory, *MaterialElement);
			HandleSpecular(Asset.Textures, GLTFMaterial, MapFactory, *MaterialElement);

			// Additional maps
			HandleOcclusion(Asset.Textures, GLTFMaterial, MapFactory, *MaterialElement);
			HandleEmissive(Asset.Textures, GLTFMaterial, MapFactory, *MaterialElement);
			HandleNormal(Asset.Textures, GLTFMaterial, MapFactory, *MaterialElement);

			MaterialElement->SetGLTFMaterialHash(GLTFMaterial.GetHash());
            MaterialElement->Finalize();
			Materials.Add(MaterialElement);
		}

		return Materials;
	}

	void FMaterialFactoryImpl::HandleOpacity(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FMaterialElement& MaterialElement)
	{
		if (GLTFMaterial.IsOpaque() || GLTFMaterial.bHasTransmission) // With transmission, we handle opacity different
		{
			return;
		}

		const TCHAR* GroupName = TEXT("Opacity");

		FMaterialExpressionTexture* BaseColorMap = FindExpression<FMaterialExpressionTexture>(TEXT("BaseColor Map"), MaterialElement);
		switch (GLTFMaterial.AlphaMode)
		{
			case FMaterial::EAlphaMode::Mask:
			{
				FMaterialExpressionColor* BaseColorFactor = FindExpression<FMaterialExpressionColor>(TEXT("BaseColor"), MaterialElement);

				FMaterialExpressionGeneric* MultiplyExpression = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
				MultiplyExpression->SetExpressionName(TEXT("Multiply"));

				BaseColorFactor->ConnectExpression(*MultiplyExpression->GetInput(1), (int)FPBRMapFactory::EChannel::Alpha);
				BaseColorMap->ConnectExpression(*MultiplyExpression->GetInput(0), (int)FPBRMapFactory::EChannel::Alpha);

				FMaterialExpressionFunctionCall* CuttofExpression = MaterialElement.AddMaterialExpression<FMaterialExpressionFunctionCall>();
				CuttofExpression->SetFunctionPathName(TEXT("/Engine/Functions/Engine_MaterialFunctions02/SmoothStep.SmoothStep"));

				FMaterialExpressionScalar* ValueExpression = MaterialElement.AddMaterialExpression<FMaterialExpressionScalar>();
				ValueExpression->SetName(TEXT("Alpha Cuttof"));
				ValueExpression->SetGroupName(GroupName);
				ValueExpression->GetScalar() = GLTFMaterial.AlphaCutoff;

				MultiplyExpression->ConnectExpression(*CuttofExpression->GetInput(0), 0);
				ValueExpression->ConnectExpression(*CuttofExpression->GetInput(1), 0);
				ValueExpression->ConnectExpression(*CuttofExpression->GetInput(2), 0);

				CuttofExpression->ConnectExpression(MaterialElement.GetOpacity(), 0);
				break;
			}
			case FMaterial::EAlphaMode::Blend:
			{
				FMaterialExpressionScalar* ValueExpression = MaterialElement.AddMaterialExpression<FMaterialExpressionScalar>();
				ValueExpression->SetName(TEXT("IOR"));
				ValueExpression->SetGroupName(GroupName);
				ValueExpression->GetScalar() = 1.f;
				ValueExpression->ConnectExpression(MaterialElement.GetRefraction(), 0);

				FMaterialExpressionColor* BaseColorFactor = FindExpression<FMaterialExpressionColor>(TEXT("BaseColor"), MaterialElement);
				if (BaseColorMap)
				{
					FMaterialExpressionGeneric* MultiplyExpression = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
					MultiplyExpression->SetExpressionName(TEXT("Multiply"));

					BaseColorFactor->ConnectExpression(*MultiplyExpression->GetInput(1), (int)FPBRMapFactory::EChannel::Alpha);
					BaseColorMap->ConnectExpression(*MultiplyExpression->GetInput(0), (int)FPBRMapFactory::EChannel::Alpha);
					MultiplyExpression->ConnectExpression(MaterialElement.GetOpacity(), 0);
				}
				else
				{
					BaseColorFactor->ConnectExpression(MaterialElement.GetOpacity(), (int)FPBRMapFactory::EChannel::Alpha);
				}
				break;
			}
			default:
				if (!ensure(false))
				{
					Messages.Emplace(EMessageSeverity::Warning, LOCTEXT("InvalidAlphaMode", "AlphaMode Opaque not supported for Opacity"));
				}
				break;
		}
	}

	void FMaterialFactoryImpl::HandleShadingModel(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement)
	{
		MapFactory.GroupName = TEXT("GGX");

		FMaterialExpressionInput* BaseColorInput = &MaterialElement.GetBaseColor();

		if (GLTFMaterial.bIsUnlitShadingModel)
		{
			BaseColorInput = &MaterialElement.GetEmissiveColor();
			MaterialElement.SetShadingModel(EGLTFMaterialShadingModel::Unlit);
		}

		TArray<FPBRMapFactory::FMapChannel, TFixedAllocator<4>> Maps;
		if (GLTFMaterial.ShadingModel == FMaterial::EShadingModel::MetallicRoughness)
		{
			// Base Color
			MapFactory.GroupName = TEXT("Base Color");
			MapFactory.CreateColorMap(GetTexture(GLTFMaterial.BaseColor, Textures),
									  GLTFMaterial.BaseColor.TexCoord,
									  GLTFMaterial.BaseColorFactor,
									  TEXT("BaseColor"),
									  nullptr,
									  ETextureMode::Color,
									  *BaseColorInput,
									  GLTFMaterial.BaseColor.bHasTextureTransform ? &GLTFMaterial.BaseColor.TextureTransform : nullptr);

			if (!GLTFMaterial.bIsUnlitShadingModel)
			{
				// Metallic
				Maps.Emplace(GLTFMaterial.MetallicRoughness.MetallicFactor,
							 TEXT("Metallic Factor"),
							 FPBRMapFactory::EChannel::Blue,
							 &MaterialElement.GetMetallic(),
							 nullptr);

				// Roughness
				Maps.Emplace(GLTFMaterial.MetallicRoughness.RoughnessFactor,
							 TEXT("Roughness Factor"),
							 FPBRMapFactory::EChannel::Green,
							 &MaterialElement.GetRoughness(),
							 nullptr);

				MapFactory.CreateMultiMap(GetTexture(GLTFMaterial.MetallicRoughness.Map, Textures),
										  GLTFMaterial.MetallicRoughness.Map.TexCoord,
										  TEXT("MetallicRoughness"),
										  Maps.GetData(),
										  Maps.Num(),
										  ETextureMode::Grayscale,
										  GLTFMaterial.MetallicRoughness.Map.bHasTextureTransform ? &GLTFMaterial.MetallicRoughness.Map.TextureTransform : nullptr);
			}
		}
		else if (GLTFMaterial.ShadingModel == FMaterial::EShadingModel::SpecularGlossiness)
		{
			// We'll actually just convert it into MetalRoughness in the material graph
			FMaterialExpressionFunctionCall* SpecGlossToMetalRough = MaterialElement.AddMaterialExpression<FMaterialExpressionFunctionCall>();
			SpecGlossToMetalRough->SetFunctionPathName(TEXT("/InterchangeAssets/gltf/SpecGlossToMetalRoughness.SpecGlossToMetalRoughness"));
			SpecGlossToMetalRough->ConnectExpression(*BaseColorInput, 0);

			if (!GLTFMaterial.bIsUnlitShadingModel)
			{
				SpecGlossToMetalRough->ConnectExpression(MaterialElement.GetMetallic(), 1);

				FMaterialExpressionGeneric* GlossToRoughness = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
				GlossToRoughness->SetExpressionName(TEXT("OneMinus"));
				GlossToRoughness->ConnectExpression(MaterialElement.GetRoughness(), 0);

				// Diffuse Color (BaseColor/BaseColorFactor are used to store the Diffuse alternatives for Spec/Gloss)
				MapFactory.GroupName = TEXT("Diffuse Color");
				FMaterialExpression* Diffuse = MapFactory.CreateColorMap(GetTexture(GLTFMaterial.BaseColor, Textures),
																		 GLTFMaterial.BaseColor.TexCoord,
																		 GLTFMaterial.BaseColorFactor,
																		 TEXT("Diffuse"),
																		 TEXT("Color"),
																		 ETextureMode::Color,
																		 *SpecGlossToMetalRough->GetInput(1),
																		 GLTFMaterial.BaseColor.bHasTextureTransform ? &GLTFMaterial.BaseColor.TextureTransform : nullptr);

				// Specular (goes into SpecGlossToMetalRough conversion)
				Maps.Emplace(GLTFMaterial.SpecularGlossiness.SpecularFactor,
							 TEXT("Specular Factor"),
							 FPBRMapFactory::EChannel::RGB,
							 SpecGlossToMetalRough->GetInput(0),
							 nullptr);

				// Glossiness (converted to Roughness)
				Maps.Emplace(GLTFMaterial.SpecularGlossiness.GlossinessFactor,
							 TEXT("Glossiness Factor"),
							 FPBRMapFactory::EChannel::Alpha,
							 GlossToRoughness->GetInput(0),
							 nullptr);

				// Creates the multimap for Specular and Glossiness
				MapFactory.CreateMultiMap(GetTexture(GLTFMaterial.SpecularGlossiness.Map, Textures),
										  GLTFMaterial.SpecularGlossiness.Map.TexCoord,
										  TEXT("SpecularGlossiness"),
										  Maps.GetData(),
										  Maps.Num(),
										  ETextureMode::Color,
										  GLTFMaterial.SpecularGlossiness.Map.bHasTextureTransform ? &GLTFMaterial.SpecularGlossiness.Map.TextureTransform : nullptr);
			}
		}
	}

	void FMaterialFactoryImpl::HandleOcclusion(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement)
	{
		MapFactory.GroupName = TEXT("Occlusion");

		FMaterialExpressionTexture* TexExpression = MapFactory.CreateTextureMap(
			GetTexture(GLTFMaterial.Occlusion, Textures), GLTFMaterial.Occlusion.TexCoord, TEXT("Occlusion"), ETextureMode::Grayscale, 
			GLTFMaterial.Occlusion.bHasTextureTransform ? &GLTFMaterial.Occlusion.TextureTransform : nullptr);

		if (!TexExpression)
			return;

		FMaterialExpressionScalar* ConstantExpression = MaterialElement.AddMaterialExpression<FMaterialExpressionScalar>();
		ConstantExpression->GetScalar()               = 1.f;

		FMaterialExpressionGeneric* LerpExpression = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
		LerpExpression->SetExpressionName(TEXT("LinearInterpolate"));

		FMaterialExpressionScalar* ValueExpression = MaterialElement.AddMaterialExpression<FMaterialExpressionScalar>();
		ValueExpression->SetName(TEXT("Occlusion Strength"));
		ValueExpression->SetGroupName(*MapFactory.GroupName);
		ValueExpression->GetScalar() = GLTFMaterial.OcclusionStrength;

		ConstantExpression->ConnectExpression(*LerpExpression->GetInput(0), 0);
		TexExpression->ConnectExpression(*LerpExpression->GetInput(1), (int)FPBRMapFactory::EChannel::Red);  // ignore other channels
		ValueExpression->ConnectExpression(*LerpExpression->GetInput(2), 0);

		LerpExpression->ConnectExpression(MaterialElement.GetAmbientOcclusion(), 0);
	}

	void FMaterialFactoryImpl::HandleEmissive(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement)
	{
		if (GLTFMaterial.Emissive.TextureIndex == INDEX_NONE || GLTFMaterial.EmissiveFactor.IsNearlyZero() || GLTFMaterial.bIsUnlitShadingModel)
		{
			return;
		}

		MapFactory.GroupName = TEXT("Emission");
		MapFactory.CreateColorMap(GetTexture(GLTFMaterial.Emissive, Textures),
								  GLTFMaterial.Emissive.TexCoord,
								  GLTFMaterial.EmissiveFactor,
								  TEXT("Emissive"),
								  TEXT("Color"),
								  ETextureMode::Color, // emissive map is in sRGB space
								  MaterialElement.GetEmissiveColor(),
								  GLTFMaterial.Emissive.bHasTextureTransform ? &GLTFMaterial.Emissive.TextureTransform : nullptr);
	}

	void FMaterialFactoryImpl::HandleNormal(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement)
	{
		MapFactory.GroupName = TEXT("Normal");
		MapFactory.CreateNormalMap(GetTexture(GLTFMaterial.Normal, Textures),
								   GLTFMaterial.Normal.TexCoord,
								   GLTFMaterial.NormalScale,
								   GLTFMaterial.Normal.bHasTextureTransform ? &GLTFMaterial.Normal.TextureTransform : nullptr);
	}

	void FMaterialFactoryImpl::HandleClearCoat(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement)
	{
		if (!GLTFMaterial.bHasClearCoat || GLTFMaterial.bIsUnlitShadingModel || FMath::IsNearlyEqual(GLTFMaterial.ClearCoat.ClearCoatFactor, 0.0f))
		{
			return;
		}

		MaterialElement.SetShadingModel(EGLTFMaterialShadingModel::ClearCoat);

		FMaterialExpressionScalar* ClearCoatFactor = MaterialElement.AddMaterialExpression<FMaterialExpressionScalar>();
		ClearCoatFactor->GetScalar() = GLTFMaterial.ClearCoat.ClearCoatFactor;
		ClearCoatFactor->SetName(TEXT("ClearCoatFactor"));

		FMaterialExpressionScalar* ClearCoatRoughnessFactor = MaterialElement.AddMaterialExpression<FMaterialExpressionScalar>();
		ClearCoatRoughnessFactor->GetScalar() = GLTFMaterial.ClearCoat.Roughness;
		ClearCoatRoughnessFactor->SetName(TEXT("ClearCoatRoughnessFactor"));

		FMaterialExpressionTexture* ClearCoatTexture = MapFactory.CreateTextureMap(
			GetTexture(GLTFMaterial.ClearCoat.ClearCoatMap, Textures),
			GLTFMaterial.ClearCoat.ClearCoatMap.TexCoord,
			TEXT("ClearCoat"),
			ETextureMode::Color,
			GLTFMaterial.ClearCoat.ClearCoatMap.bHasTextureTransform ? &GLTFMaterial.ClearCoat.ClearCoatMap.TextureTransform : nullptr);

		FMaterialExpressionTexture* ClearCoatRoughnessTexture = MapFactory.CreateTextureMap(
			GetTexture(GLTFMaterial.ClearCoat.RoughnessMap, Textures),
			GLTFMaterial.ClearCoat.RoughnessMap.TexCoord,
			TEXT("ClearCoatRoughness"),
			ETextureMode::Color,
			GLTFMaterial.ClearCoat.RoughnessMap.bHasTextureTransform ? &GLTFMaterial.ClearCoat.RoughnessMap.TextureTransform : nullptr);

		FMaterialExpression* ClearCoatExpr = ClearCoatFactor;
		if (ClearCoatTexture)
		{
			FMaterialExpressionGeneric* MultiplyExpr = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			MultiplyExpr->SetExpressionName(TEXT("Multiply"));

			ClearCoatFactor->ConnectExpression(*MultiplyExpr->GetInput(0), 0);
			ClearCoatTexture->ConnectExpression(*MultiplyExpr->GetInput(1), (int)FPBRMapFactory::EChannel::Red);
			ClearCoatExpr = MultiplyExpr;
		}

		FMaterialExpression* ClearCoatRougnessExpr = ClearCoatRoughnessFactor;
		if (ClearCoatRoughnessTexture)
		{
			FMaterialExpressionGeneric* MultiplyExpr = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			MultiplyExpr->SetExpressionName(TEXT("Multiply"));

			ClearCoatRoughnessFactor->ConnectExpression(*MultiplyExpr->GetInput(0), 0);
			ClearCoatRoughnessTexture->ConnectExpression(*MultiplyExpr->GetInput(1), (int)FPBRMapFactory::EChannel::Green);
			ClearCoatRougnessExpr = MultiplyExpr;
		}

		ClearCoatExpr->ConnectExpression(MaterialElement.GetClearCoat(), 0);
		ClearCoatRougnessExpr->ConnectExpression(MaterialElement.GetClearCoatRoughness(), 0);

		FMaterialExpressionTexture* ClearCoatNormalTexture = MapFactory.CreateTextureMap(
			GetTexture(GLTFMaterial.ClearCoat.NormalMap, Textures),
			GLTFMaterial.ClearCoat.NormalMap.TexCoord,
			TEXT("ClearCoatNormal"),
			ETextureMode::Normal,
			GLTFMaterial.ClearCoat.NormalMap.bHasTextureTransform ? &GLTFMaterial.ClearCoat.NormalMap.TextureTransform : nullptr);

		if (ClearCoatNormalTexture)
		{
			FMaterialExpressionScalar* UVScale = MaterialElement.AddMaterialExpression<FMaterialExpressionScalar>();
			UVScale->GetScalar() = GLTFMaterial.ClearCoat.NormalMapUVScale;
			UVScale->SetName(TEXT("NormalMapUVScale"));

			FMaterialExpressionGeneric* Mult = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			Mult->SetExpressionName(TEXT("Multiply"));

			FMaterialExpressionTextureCoordinate* TexCoord = MaterialElement.AddMaterialExpression<FMaterialExpressionTextureCoordinate>();
			TexCoord->SetCoordinateIndex(GLTFMaterial.ClearCoat.NormalMap.TexCoord);
			
			FMaterialExpressionGeneric* ClearCoatNormalCustomOutput = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			ClearCoatNormalCustomOutput->SetExpressionName(TEXT("ClearCoatNormalCustomOutput"));
			
			UVScale->ConnectExpression(*Mult->GetInput(0), 0);
			TexCoord->ConnectExpression(*Mult->GetInput(1), 0);
			Mult->ConnectExpression(*ClearCoatNormalTexture->GetInput(0), 0);
			ClearCoatNormalTexture->ConnectExpression(*ClearCoatNormalCustomOutput->GetInput(0), 0);
		}
	}

	void FMaterialFactoryImpl::HandleTransmission(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement)
	{
		if (!GLTFMaterial.bHasTransmission)
		{
			return;
		}

		MaterialElement.SetBlendMode(ConvertAlphaMode(GLTF::FMaterial::EAlphaMode::Blend));
		MaterialElement.SetShadingModel(GLTF::EGLTFMaterialShadingModel::ThinTranslucent);
		MaterialElement.SetTranslucencyLightingMode(ETranslucencyLightingMode::TLM_SurfacePerPixelLighting);
		MaterialElement.SetTwoSided(GLTFMaterial.bIsDoubleSided);

		FMaterialExpressionGeneric* ThinTranslucentOutput = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
		ThinTranslucentOutput->SetExpressionName(TEXT("ThinTranslucentMaterialOutput"));

		FMaterialExpressionScalar* TransmissionFactorExpr = MaterialElement.AddMaterialExpression<FMaterialExpressionScalar>();
		TransmissionFactorExpr->GetScalar() = GLTFMaterial.Transmission.TransmissionFactor;
		TransmissionFactorExpr->SetName(TEXT("TransmissionFactor"));

		FMaterialExpressionTexture* TransmissionTexture = MapFactory.CreateTextureMap(
			GetTexture(GLTFMaterial.Transmission.TransmissionMap, Textures),
			GLTFMaterial.Transmission.TransmissionMap.TexCoord,
			TEXT("Transmission"),
			ETextureMode::Color,
			GLTFMaterial.Transmission.TransmissionMap.bHasTextureTransform ? &GLTFMaterial.Transmission.TransmissionMap.TextureTransform : nullptr);

		FMaterialExpression* MetallicFactor = MaterialElement.GetMetallic().GetExpression();
		FMaterialExpression* RoughnessFactor = MaterialElement.GetRoughness().GetExpression();

		FMaterialExpression* MetallicExpr = nullptr;
		FMaterialExpression* RoughnessExpr = nullptr;

		FMaterialExpressionTexture* MetallicRoughnessTexture = FindExpression<FMaterialExpressionTexture>(TEXT("MetallicRoughness"), MaterialElement);
		
		if (MetallicRoughnessTexture)
		{
			FMaterialExpressionGeneric* MultMetallic = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			MultMetallic->SetExpressionName(TEXT("Multiply"));
			FMaterialExpressionGeneric* MultRoughness = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			MultRoughness->SetExpressionName(TEXT("Multiply"));

			MetallicRoughnessTexture->ConnectExpression(*MultMetallic->GetInput(0), (int)FPBRMapFactory::EChannel::Blue);
			MetallicFactor->ConnectExpression(*MultMetallic->GetInput(1), 0);

			MetallicRoughnessTexture->ConnectExpression(*MultRoughness->GetInput(0), (int)FPBRMapFactory::EChannel::Green);
			RoughnessFactor->ConnectExpression(*MultMetallic->GetInput(1), 0);

			MetallicExpr = MultMetallic;
			RoughnessExpr = MultRoughness;
		}
		else
		{
			MetallicExpr = MetallicFactor;
			RoughnessExpr = RoughnessFactor;
		}

		FMaterialExpressionTexture* BaseColorMap = FindExpression<FMaterialExpressionTexture>(TEXT("BaseColor Map"), MaterialElement);

		if (GLTFMaterial.AlphaMode == FMaterial::EAlphaMode::Mask)
		{
			// Build opacity mask graph

			FMaterialExpressionColor* BaseColorFactor = FindExpression<FMaterialExpressionColor>(TEXT("BaseColor"), MaterialElement);

			FMaterialExpression* BaseColor1 = nullptr;
			FMaterialExpression* BaseColor2 = nullptr;

			if (BaseColorMap)
			{
				BaseColor1 = BaseColorMap;
			}
			else
			{
				FMaterialExpressionColor* DefaultColor = MaterialElement.AddMaterialExpression<FMaterialExpressionColor>();
				DefaultColor->GetColor() = FLinearColor::White;
				BaseColor1 = DefaultColor;
			}

			if (BaseColorFactor)
			{
				BaseColor2 = BaseColorFactor;
			}
			else
			{
				FMaterialExpressionColor* DefaultColor = MaterialElement.AddMaterialExpression<FMaterialExpressionColor>();
				DefaultColor->GetColor() = FLinearColor::White;
				BaseColor2 = DefaultColor;
			}

			FMaterialExpressionGeneric* Mult1 = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			Mult1->SetExpressionName(TEXT("Multiply"));

			BaseColor1->ConnectExpression(*Mult1->GetInput(0), 0);
			BaseColor2->ConnectExpression(*Mult1->GetInput(1), 0);

			FMaterialExpressionGeneric* Step1 = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			Step1->SetExpressionName(TEXT("Step"));
			Step1->SetFloatProperty(TEXT("ConstX"), 0.333f);

			FMaterialExpressionGeneric* OneMinus1 = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			OneMinus1->SetExpressionName(TEXT("OneMinus"));
			
			BaseColor1->ConnectExpression(*Step1->GetInput(0), (int)FPBRMapFactory::EChannel::Alpha);
			Step1->ConnectExpression(*OneMinus1->GetInput(0), 0);
			Step1->ConnectExpression(MaterialElement.GetSpecular(), 0);

			FMaterialExpressionGeneric* OpacityMult = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			OpacityMult->SetExpressionName(TEXT("Multiply"));

			OneMinus1->ConnectExpression(*OpacityMult->GetInput(0), 0);
			OneMinus1->ConnectExpression(MaterialElement.GetSpecular(), 0);

			OpacityMult->ConnectExpression(MaterialElement.GetOpacity(), 0);

			FMaterialExpressionGeneric* Max1 = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			Max1->SetExpressionName(TEXT("Max"));

			FMaterialExpressionGeneric* Max2 = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			Max2->SetExpressionName(TEXT("Max"));

			MetallicExpr->ConnectExpression(*Max1->GetInput(0), 0);
			RoughnessExpr->ConnectExpression(*Max1->GetInput(1), 0);
	
			Max1->ConnectExpression(*Max2->GetInput(1), 0);
			Max2->ConnectExpression(*OpacityMult->GetInput(1), 0);

			FMaterialExpression* TransmissionExpr = nullptr;

			if (TransmissionTexture)
			{
				FMaterialExpressionGeneric* MultTransm = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
				MultTransm->SetExpressionName(TEXT("Multiply"));

				TransmissionTexture->ConnectExpression(*MultTransm->GetInput(0), (int)FPBRMapFactory::EChannel::Red);
				TransmissionFactorExpr->ConnectExpression(*MultTransm->GetInput(1), 0);

				TransmissionExpr = MultTransm;
			}
			else
			{
				TransmissionExpr = TransmissionFactorExpr;
			}

			// Transmission network
			FMaterialExpressionGeneric* Mult2 = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			Mult2->SetExpressionName(TEXT("Multiply"));

			FMaterialExpressionGeneric* OneMinus2 = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			OneMinus2->SetExpressionName(TEXT("OneMinus"));

			FMaterialExpressionGeneric* LerpThinTransl = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			LerpThinTransl->SetExpressionName(TEXT("LinearInterpolate"));

			TransmissionExpr->ConnectExpression(*Mult2->GetInput(0), 0);
			TransmissionExpr->ConnectExpression(*OneMinus2->GetInput(0), 0);
			Mult1->ConnectExpression(*Mult2->GetInput(1), 0);

			Mult2->ConnectExpression(*LerpThinTransl->GetInput(0), 0);
			Step1->ConnectExpression(*LerpThinTransl->GetInput(2), 0);

			OneMinus2->ConnectExpression(*Max2->GetInput(0), 0);

			LerpThinTransl->ConnectExpression(*ThinTranslucentOutput->GetInput(0), 0);
		
			Mult1->ConnectExpression(MaterialElement.GetBaseColor(), 0);
		}
		else
		{
			FMaterialExpressionGeneric* BaseColorMult = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			BaseColorMult->SetExpressionName(TEXT("Multiply"));

			BaseColorMult->ConnectExpression(*ThinTranslucentOutput->GetInput(0), 0);

			FMaterialExpressionGeneric* OneMinus = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			OneMinus->SetExpressionName(TEXT("OneMinus"));

			FMaterialExpressionGeneric* Max1 = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			Max1->SetExpressionName(TEXT("Max"));

			FMaterialExpressionGeneric* Max2 = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			Max2->SetExpressionName(TEXT("Max"));

			MetallicExpr->ConnectExpression(*Max1->GetInput(0), 0);
			RoughnessExpr->ConnectExpression(*Max1->GetInput(1), 0);
			OneMinus->ConnectExpression(*Max2->GetInput(0), 0);
			Max1->ConnectExpression(*Max2->GetInput(1), 0);

			FMaterialExpression* BaseColorExpr = MaterialElement.GetBaseColor().GetExpression();
			if (BaseColorExpr)
			{
				BaseColorExpr->ConnectExpression(*BaseColorMult->GetInput(0), MaterialElement.GetBaseColor().GetOutputIndex());
			}

			if (TransmissionTexture)
			{
				FMaterialExpressionGeneric* TransmissionMult = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
				TransmissionMult->SetExpressionName(TEXT("Multiply"));
				TransmissionFactorExpr->ConnectExpression(*TransmissionMult->GetInput(0), 0);
				TransmissionTexture->ConnectExpression(*TransmissionMult->GetInput(1), (int)FPBRMapFactory::EChannel::Red);
				TransmissionMult->ConnectExpression(*BaseColorMult->GetInput(1), 0);
				TransmissionMult->ConnectExpression(*OneMinus->GetInput(0), 0);
			}
			else
			{
				TransmissionFactorExpr->ConnectExpression(*BaseColorMult->GetInput(1), 0);
				TransmissionFactorExpr->ConnectExpression(*OneMinus->GetInput(0), 0);
			}

			Max2->ConnectExpression(MaterialElement.GetOpacity(), 0);
		}
	}
	
	void FMaterialFactoryImpl::HandleSheen(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement)
	{
		if (!GLTFMaterial.bHasSheen || GLTFMaterial.bIsUnlitShadingModel)
		{
			return;
		}

		FMaterialExpressionFunctionCall* FuzzyShadingCall = MaterialElement.AddMaterialExpression<FMaterialExpressionFunctionCall>();
		FuzzyShadingCall->SetFunctionPathName(TEXT("/Engine/Functions/Engine_MaterialFunctions01/Shading/FuzzyShading.FuzzyShading"));

		FMaterialExpressionColor* Fuzzyness = MaterialElement.AddMaterialExpression<FMaterialExpressionColor>();
		Fuzzyness->SetName(TEXT("Fuzzyness"));
		Fuzzyness->GetColor() = FLinearColor(0.9f, 0.8f, 2.0f, 1.0f);
		Fuzzyness->ConnectExpression(*FuzzyShadingCall->GetInput(2), (int)FPBRMapFactory::EChannel::Red);	// CoreDarkness
		Fuzzyness->ConnectExpression(*FuzzyShadingCall->GetInput(4), (int)FPBRMapFactory::EChannel::Green);	// EdgeBrightness
		Fuzzyness->ConnectExpression(*FuzzyShadingCall->GetInput(3), (int)FPBRMapFactory::EChannel::Blue);	// Power

		if (FMaterialExpression* BaseColorExpr = MaterialElement.GetBaseColor().GetExpression())
		{
			BaseColorExpr->ConnectExpression(*FuzzyShadingCall->GetInput(0), MaterialElement.GetBaseColor().GetOutputIndex());

			FMaterialExpressionGeneric* LerpExpression = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			LerpExpression->SetExpressionName(TEXT("LinearInterpolate"));

			FuzzyShadingCall->ConnectExpression(*LerpExpression->GetInput(0), 0);
			BaseColorExpr->ConnectExpression(*LerpExpression->GetInput(1), MaterialElement.GetBaseColor().GetOutputIndex());
			Fuzzyness->ConnectExpression(*LerpExpression->GetInput(2), (int)FPBRMapFactory::EChannel::Alpha);

			LerpExpression->ConnectExpression(MaterialElement.GetBaseColor(), 0);
		}

		if (FMaterialExpression* NormalExpr = MaterialElement.GetNormal().GetExpression())
		{
			NormalExpr->ConnectExpression(*FuzzyShadingCall->GetInput(1), MaterialElement.GetNormal().GetOutputIndex());
		}
	}

	void FMaterialFactoryImpl::HandleIOR(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement)
	{
		if (!GLTFMaterial.bHasIOR || GLTFMaterial.ShadingModel == FMaterial::EShadingModel::SpecularGlossiness || GLTFMaterial.bIsUnlitShadingModel)
		{
			// As per glTF 2.0 spec, IOR extension is not compatible with specular-glossiness workflow or unlit shading model
			return;
		}

		if (!GLTFMaterial.bHasTransmission)
		{
			MaterialElement.SetBlendMode(ConvertAlphaMode(GLTF::FMaterial::EAlphaMode::Blend));
			MaterialElement.SetTranslucencyLightingMode(ETranslucencyLightingMode::TLM_Surface);
			MaterialElement.SetTwoSided(GLTFMaterial.bIsDoubleSided);
		}

		FMaterialExpressionGeneric* LerpExpr = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
		LerpExpr->SetExpressionName(TEXT("LinearInterpolate"));

		FMaterialExpressionScalar* ConstExpr = MaterialElement.AddMaterialExpression<FMaterialExpressionScalar>();
		ConstExpr->GetScalar() = 1.f;

		FMaterialExpressionScalar* IORExpr = MaterialElement.AddMaterialExpression<FMaterialExpressionScalar>();
		IORExpr->SetName(TEXT("IOR"));
		IORExpr->GetScalar() = GLTFMaterial.IOR;

		FMaterialExpressionGeneric* FresnelExpr = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
		FresnelExpr->SetExpressionName(TEXT("Fresnel"));

		ConstExpr->ConnectExpression(*LerpExpr->GetInput(0), 0);
		IORExpr->ConnectExpression(*LerpExpr->GetInput(1), 0);
		FresnelExpr->ConnectExpression(*LerpExpr->GetInput(2), 0);

		LerpExpr->ConnectExpression(MaterialElement.GetRefraction(), 0);
	}

	void FMaterialFactoryImpl::HandleSpecular(const TArray<GLTF::FTexture>& Textures, const GLTF::FMaterial& GLTFMaterial, FPBRMapFactory& MapFactory, FMaterialElement& MaterialElement)
	{
		if (!GLTFMaterial.bHasSpecular || GLTFMaterial.ShadingModel == FMaterial::EShadingModel::SpecularGlossiness || GLTFMaterial.bIsUnlitShadingModel)
		{
			// As per glTF 2.0 spec, specular extension is not compatible with specular-glossiness workflow or unlit shading model
			return;
		}

		FMaterialExpressionScalar* SpecularFactorExpr = MaterialElement.AddMaterialExpression<FMaterialExpressionScalar>();
		SpecularFactorExpr->SetName(TEXT("SpecularFactor"));
		SpecularFactorExpr->GetScalar() = GLTFMaterial.Specular.SpecularFactor;

		FMaterialExpressionTexture* SpecularTextureExpr = MapFactory.CreateTextureMap(
			GetTexture(GLTFMaterial.Specular.SpecularMap, Textures), GLTFMaterial.Specular.SpecularMap.TexCoord, TEXT("SpecularMap"), ETextureMode::Color,
			GLTFMaterial.Specular.SpecularMap.bHasTextureTransform ? &GLTFMaterial.Specular.SpecularMap.TextureTransform : nullptr);

		FMaterialExpression* SpecularExpr = SpecularFactorExpr;

		if (SpecularTextureExpr)
		{
			FMaterialExpressionGeneric* MultiplyExpr = MaterialElement.AddMaterialExpression<FMaterialExpressionGeneric>();
			MultiplyExpr->SetExpressionName(TEXT("Multiply"));
			
			SpecularFactorExpr->ConnectExpression(*MultiplyExpr->GetInput(0), 0);
			SpecularTextureExpr->ConnectExpression(*MultiplyExpr->GetInput(1), (int)FPBRMapFactory::EChannel::Alpha);

			SpecularExpr = MultiplyExpr;
		}

		SpecularExpr->ConnectExpression(MaterialElement.GetSpecular(), 0);
	}

	FMaterialFactory::FMaterialFactory(IMaterialElementFactory* MaterialElementFactory, ITextureFactory* TextureFactory)
	    : Impl(new FMaterialFactoryImpl(MaterialElementFactory, TextureFactory))
	{
	}

	FMaterialFactory::~FMaterialFactory() {}

	const TArray<FMaterialElement*>& FMaterialFactory::CreateMaterials(const GLTF::FAsset& Asset, UObject* ParentPackage, EObjectFlags Flags)
	{
		return Impl->CreateMaterials(Asset, ParentPackage, Flags);
	}

	const TArray<FLogMessage>& FMaterialFactory::GetLogMessages() const
	{
		return Impl->GetLogMessages();
	}

	const TArray<FMaterialElement*>& FMaterialFactory::GetMaterials() const
	{
		return Impl->Materials;
	}

	IMaterialElementFactory& FMaterialFactory::GetMaterialElementFactory()
	{
		return *Impl->MaterialElementFactory;
	}

	ITextureFactory& FMaterialFactory::GetTextureFactory()
	{
		return *Impl->TextureFactory;
	}

	void FMaterialFactory::CleanUp()
	{
		for (FMaterialElement* MaterialElement : Impl->Materials)
		{
			delete MaterialElement;
		}
		Impl->Materials.Empty();
	}

}  // namespace GLTF

#undef LOCTEXT_NAMESPACE