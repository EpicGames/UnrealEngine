// Copyright Epic Games, Inc. All Rights Reserved.

#include "GLTFMeshFactory.h"

#include "GLTF/ConversionUtilities.h"
#include "GLTFMesh.h"
#include "StaticMeshAttributes.h"
#include "Util/ColorConstants.h"

#define LOCTEXT_NAMESPACE "InterchangeGLTFMeshFactory"

namespace GLTF
{
	class FMeshFactoryImpl : public GLTF::FBaseLogger
	{
	public:
		FMeshFactoryImpl();

		void FillMeshDescription(const GLTF::FMesh &Mesh, const FTransform& MeshGlobalTransform, FMeshDescription* MeshDescription, const TArray<float>& MorphTargetWeights);

		void CleanUp();

	private:

		bool ImportPrimitive(const GLTF::FPrimitive&                        Primitive,  //
			const FString&								   MeshName,
			int32                                          PrimitiveIndex,
			int32                                          NumUVs,
			bool                                           bMeshHasTagents,
			bool                                           bMeshHasColors,
			const TVertexInstanceAttributesRef<FVector3f>&   VertexInstanceNormals,
			const TVertexInstanceAttributesRef<FVector3f>&   VertexInstanceTangents,
			const TVertexInstanceAttributesRef<float>&     VertexInstanceBinormalSigns,
			const TVertexInstanceAttributesRef<FVector2f>& VertexInstanceUVs,
			const TVertexInstanceAttributesRef<FVector4f>&  VertexInstanceColors,
			const TEdgeAttributesRef<bool>&                EdgeHardnesses,
			TPolygonAttributesRef<FName>&				PolygonObjectNames,
			const FTransform& MeshGlobalTransform,
			FMeshDescription* MeshDescription,
			const TArray<float>& MorphTargetWeights);

		inline TArray<FVector4f>& GetVector4dBuffer(int32 Index)
		{
			check(Index < (sizeof(Vector4dBuffers) / sizeof(Vector4dBuffers[0])));
			uint32 ReserveSize = Vector4dBuffers[Index].Num() + Vector4dBuffers[Index].GetSlack();
			Vector4dBuffers[Index].Empty(ReserveSize);
			return Vector4dBuffers[Index];
		}

		inline TArray<FVector3f>& GetVectorBuffer(int32 Index)
		{
			check(Index < (sizeof(VectorBuffers) / sizeof(VectorBuffers[0])));
			uint32 ReserveSize = VectorBuffers[Index].Num() + VectorBuffers[Index].GetSlack();
			VectorBuffers[Index].Empty(ReserveSize);
			return VectorBuffers[Index];
		}

		inline TArray<FVector2f>& GetVector2dBuffer(int32 Index)
		{
			check(Index < (sizeof(Vector2dBuffers) / sizeof(Vector2dBuffers[0])));
			uint32 ReserveSize = Vector2dBuffers[Index].Num() + Vector2dBuffers[Index].GetSlack();
			Vector2dBuffers[Index].Empty(ReserveSize);
			return Vector2dBuffers[Index];
		}

		inline TArray<uint32>& GetIntBuffer()
		{
			uint32 ReserveSize = IntBuffer.Num() + IntBuffer.GetSlack();
			IntBuffer.Empty(ReserveSize);
			return IntBuffer;
		}

	private:

		enum Vector3fMorphTargetAttributes
		{
			POSITIONS,
			NORMALS
		};
		void UpdateVector3fFromMorphTargets(TArray<FVector3f>& AttrbiuteValues, const FPrimitive& Primitive, const TArray<float>& MorphTargetWeights, const Vector3fMorphTargetAttributes& AttributeToProcess);

		enum Vector4dMorphTargetAttributes
		{
			TANGENTS,
			COLORS
		};
		void UpdateVector4dFromMorphTargets(TArray<FVector4f>& AttrbiuteValues, const FPrimitive& Primitive, const TArray<float>& MorphTargetWeights, const Vector4dMorphTargetAttributes& AttributeToProcess);

		void UpdateTexCoordsFromMorphTargets(TArray<FVector2f>& AttrbiuteValues, const FPrimitive& Primitive, const TArray<float>& MorphTargetWeights, int32 Index);

		template <typename T>
		void ReIndex(const TArray<T>& Source, const TArray<uint32>& Indices, TArray<T>& Dst, const FString& AttributeName, const FString& MeshName);
		
		void GenerateFlatNormals(const TArray<FVector3f>& Positions, const TArray<uint32>& Indices, TArray<FVector3f>& Normals, const FString& MeshName);

		enum
		{
			NormalBufferIndex = 0,
			PositionBufferIndex = 1,
			ReindexBufferIndex = 2,
			MorphTargetBufferIndex = 3,
			VectorBufferCount = 4,
			UvReindexBufferIndex = MAX_MESH_TEXTURE_COORDS_MD,
			MorphTargetUvBufferIndex = UvReindexBufferIndex+1,
			ColorBufferIndex = 0,
			TangentBufferIndex = 1,
			Reindex4dBufferIndex = 2,
			MorphTarget4dBufferIndex = 3,
			Vector4dBufferCount = 4,
		};

		float                ImportUniformScale;

		TSet<int32>                  MaterialIndicesUsed;
		TMap<int32, FPolygonGroupID> MaterialIndexToPolygonGroupID;
		TArray<FMeshFactory::FIndexVertexIdMap>    PositionIndexToVertexIdPerPrim;

		TArray<FVector2f>                       Vector2dBuffers[MAX_MESH_TEXTURE_COORDS_MD + 2];
		TArray<FVector3f>                       VectorBuffers[VectorBufferCount];
		TArray<FVector4f>                       Vector4dBuffers[Vector4dBufferCount];
		TArray<uint32>                          IntBuffer;
		TArray<FVertexInstanceID>				CornerVertexInstanceIDs;
		uint32                                  MaxReserveSize;

		friend class FMeshFactory;
	};

	
		template <typename T>
		void FMeshFactoryImpl::ReIndex(const TArray<T>& Source, const TArray<uint32>& Indices, TArray<T>& Dst, const FString& AttributeName, const FString& MeshName)
		{
			if (!ensure(&Source != &Dst))
			{
				return;
			}

			uint32 NumOfSource = Source.Num();
			if (NumOfSource == 0)
			{
				return;
			}

			Dst.Reserve(Indices.Num());

			bool bHasOOBIndex = false;
			for (uint32 Index : Indices)
			{
				if (Index >= NumOfSource)
				{
					bHasOOBIndex = true;
					Index = NumOfSource - 1;
				}

				Dst.Add(Source[Index]);
			}

			if (bHasOOBIndex)
			{
				FString WarningMsg = TEXT("Out of Bounds indices found in ") + AttributeName + TEXT(" for: '") + MeshName + TEXT("'. Clamping given indices.");
				Messages.Emplace(EMessageSeverity::Warning, FText::FromString(WarningMsg));
			}
		}

		void FMeshFactoryImpl::GenerateFlatNormals(const TArray<FVector3f>& Positions, const TArray<uint32>& Indices, TArray<FVector3f>& Normals, const FString& MeshName)
		{
			Normals.Empty();

			uint32 N = Indices.Num();
			if (!ensure(N % 3 == 0))
			{
				N -= N % 3;
			}
			Normals.AddDefaulted(N);

			uint32 NumOfPositions = Positions.Num();

			if (NumOfPositions == 0)
			{
				return;
			}

			bool bHasOOBIndex = false;
			auto ValidateIndex = [&NumOfPositions, &bHasOOBIndex](uint32& Index)
				{
					if (Index >= NumOfPositions)
					{
						bHasOOBIndex = true;
						Index = NumOfPositions - 1;
					}
				};

			for (uint32 i = 0; i < N; i += 3)
			{
				uint32 AIndex = Indices[i];
				uint32 BIndex = Indices[i + 1];
				uint32 CIndex = Indices[i + 2];

				ValidateIndex(AIndex);
				ValidateIndex(BIndex);
				ValidateIndex(CIndex);

				const FVector3f& A = Positions[AIndex];
				const FVector3f& B = Positions[BIndex];
				const FVector3f& C = Positions[CIndex];

				// We have a left-handed coordinate system, but a counter-clockwise winding order
				// Hence normal calculation has to take the triangle vectors cross product in reverse.
				const FVector3f Normal = FVector3f::CrossProduct(C - A, B - A).GetSafeNormal();
				// Same for each corner of the triangle.
				Normals[i] = Normal;
				Normals[i + 1] = Normal;
				Normals[i + 2] = Normal;
			}

			if (bHasOOBIndex)
			{
				FString WarningMsg = TEXT("Out of Bounds indices found while generating Flat Normals for: '") + MeshName + TEXT("'. Clamping given indices.");
				Messages.Emplace(EMessageSeverity::Warning, FText::FromString(WarningMsg));
			}
		}

	namespace
	{
		int32 GetNumUVs(const GLTF::FMesh& Mesh)
		{
			int32 NumUVs = 0;
			for (int32 UVIndex = 0; UVIndex < MAX_MESH_TEXTURE_COORDS_MD; ++UVIndex)
			{
				if (Mesh.HasTexCoords(UVIndex))
				{
					NumUVs++;
				}
				else
				{
					break;
				}
			}
			return NumUVs;
		}
	}

	FMeshFactoryImpl::FMeshFactoryImpl()
		: ImportUniformScale(1.f)
	{
		CornerVertexInstanceIDs.SetNum(3);
	}

	void FMeshFactoryImpl::UpdateVector3fFromMorphTargets(TArray<FVector3f>& AttributeValues, const FPrimitive& Primitive, const TArray<float>& MorphTargetWeights, const Vector3fMorphTargetAttributes& AttributeToProcess)
	{
		if (MorphTargetWeights.Num() != Primitive.MorphTargets.Num())
		{
			return;
		}

		for (int32 MorphTargetIndex = 0; MorphTargetIndex < Primitive.MorphTargets.Num(); MorphTargetIndex++)
		{
			float MorphTargetWeight = MorphTargetWeights.Num() > MorphTargetIndex ? MorphTargetWeights[MorphTargetIndex] : 0;

			if (MorphTargetWeight > 0 && MorphTargetWeight <= 1.0)
			{
				TArray<FVector3f>& Displacements = GetVectorBuffer(MorphTargetBufferIndex);

				switch (AttributeToProcess)
				{
				case POSITIONS:
					if (!Primitive.MorphTargets[MorphTargetIndex].HasPositionDisplacements()) continue;
					Primitive.MorphTargets[MorphTargetIndex].GetPositionDisplacements(Displacements);
					break;
				case NORMALS:
					if (!Primitive.MorphTargets[MorphTargetIndex].HasNormalDisplacements()) continue;
					Primitive.MorphTargets[MorphTargetIndex].GetNormalDisplacements(Displacements);
					break;
				default:
					return;
				}

				if (AttributeValues.Num() != Displacements.Num())
				{
					continue;
				}
				
				for (int32 Index = 0; Index < Displacements.Num(); Index++)
				{
					AttributeValues[Index] += MorphTargetWeight * Displacements[Index];
				}
			}
		}
	}
	void FMeshFactoryImpl::UpdateVector4dFromMorphTargets(TArray<FVector4f>& AttributeValues, const FPrimitive& Primitive, const TArray<float>& MorphTargetWeights, const Vector4dMorphTargetAttributes& AttributeToProcess)
	{
		if (MorphTargetWeights.Num() != Primitive.MorphTargets.Num())
		{
			return;
		}

		for (int32 MorphTargetIndex = 0; MorphTargetIndex < Primitive.MorphTargets.Num(); MorphTargetIndex++)
		{
			float MorphTargetWeight = MorphTargetWeights.Num() > MorphTargetIndex ? MorphTargetWeights[MorphTargetIndex] : 0;

			if (MorphTargetWeight > 0 && MorphTargetWeight <= 1.0)
			{
				TArray<FVector4f>& Displacements = GetVector4dBuffer(MorphTargetBufferIndex);

				switch (AttributeToProcess)
				{
				case TANGENTS:
					if (!Primitive.MorphTargets[MorphTargetIndex].HasTangentDisplacements()) continue;
					Primitive.MorphTargets[MorphTargetIndex].GetTangentDisplacements(Displacements);
					break;
				case COLORS:
					if (!Primitive.MorphTargets[MorphTargetIndex].HasColorDeltas()) continue;
					Primitive.MorphTargets[MorphTargetIndex].GetColorDeltas(Displacements);
					break;
				default:
					return;
				}

				if (AttributeValues.Num() != Displacements.Num())
				{
					continue;
				}

				for (int32 Index = 0; Index < Displacements.Num(); Index++)
				{
					AttributeValues[Index] += MorphTargetWeight * Displacements[Index];

					if (AttributeToProcess == Vector4dMorphTargetAttributes::COLORS)
					{
						//After applying color deltas, all components of each COLOR_0 morphed accessor element MUST be clamped to [0.0, 1.0] range.
						AttributeValues[Index].X = FMath::Clamp(AttributeValues[Index].X, .0f, 1.f);
						AttributeValues[Index].Y = FMath::Clamp(AttributeValues[Index].Y, .0f, 1.f);
						AttributeValues[Index].Z = FMath::Clamp(AttributeValues[Index].W, .0f, 1.f);
						AttributeValues[Index].W = FMath::Clamp(AttributeValues[Index].Z, .0f, 1.f);
					}
				}
			}
		}
	}
	void FMeshFactoryImpl::UpdateTexCoordsFromMorphTargets(TArray<FVector2f>& AttributeValues, const FPrimitive& Primitive, const TArray<float>& MorphTargetWeights, int32 UvIndex)
	{
		if (MorphTargetWeights.Num() != Primitive.MorphTargets.Num())
		{
			return;
		}

		for (int32 MorphTargetIndex = 0; MorphTargetIndex < Primitive.MorphTargets.Num(); MorphTargetIndex++)
		{
			float MorphTargetWeight = MorphTargetWeights.Num() > MorphTargetIndex ? MorphTargetWeights[MorphTargetIndex] : 0;

			if (MorphTargetWeight > 0 && MorphTargetWeight <= 1.0)
			{
				TArray<FVector2f>& Displacements = GetVector2dBuffer(MorphTargetUvBufferIndex);

				if (!Primitive.MorphTargets[MorphTargetIndex].HasTexCoordDisplacements(UvIndex)) continue;

				Primitive.MorphTargets[MorphTargetIndex].GetTexCoordDisplacements(UvIndex, Displacements);

				if (AttributeValues.Num() != Displacements.Num())
				{
					continue;
				}

				for (int32 Index = 0; Index < Displacements.Num(); Index++)
				{
					AttributeValues[Index] += MorphTargetWeight * Displacements[Index];
				}
			}
		}
	}

	void FMeshFactoryImpl::FillMeshDescription(const FMesh &Mesh, const FTransform& MeshGlobalTransform, FMeshDescription* MeshDescription, const TArray<float>& MorphTargetWeights)
	{
		const int32 NumUVs = FMath::Max(1, GetNumUVs(Mesh));

		FStaticMeshAttributes StaticMeshAttributes(*MeshDescription);
		StaticMeshAttributes.Register();
		StaticMeshAttributes.RegisterPolygonObjectNameAttribute();

		FMatrix TotalMatrix = MeshGlobalTransform.ToMatrixWithScale();

		TVertexAttributesRef<FVector3f> VertexPositions = StaticMeshAttributes.GetVertexPositions();
		TEdgeAttributesRef<bool>  EdgeHardnesses = StaticMeshAttributes.GetEdgeHardnesses();
		TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = StaticMeshAttributes.GetPolygonGroupMaterialSlotNames();
		TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = StaticMeshAttributes.GetVertexInstanceNormals();
		TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = StaticMeshAttributes.GetVertexInstanceTangents();
		TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = StaticMeshAttributes.GetVertexInstanceBinormalSigns();
		TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = StaticMeshAttributes.GetVertexInstanceUVs();
		TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = StaticMeshAttributes.GetVertexInstanceColors();
		TPolygonAttributesRef<FName> PolygonObjectNames = StaticMeshAttributes.GetPolygonObjectNames();

		VertexInstanceUVs.SetNumChannels(NumUVs);

		MaterialIndicesUsed.Empty(10);
		// Add the vertex position
		PositionIndexToVertexIdPerPrim.SetNum(FMath::Max(Mesh.Primitives.Num(), PositionIndexToVertexIdPerPrim.Num()));
		for (int32 Index = 0; Index < Mesh.Primitives.Num(); ++Index)
		{
			const FPrimitive& Primitive = Mesh.Primitives[Index];

			if (!Primitive.IsValid())
			{
				Messages.Emplace(EMessageSeverity::Warning, FText::Format(LOCTEXT("InvalidMeshPrimitive", "Mesh has an invalid primitive : {0}"), FText::FromString(Mesh.Name)));
				continue;
			}

			// Remember which primitives use which materials.
			MaterialIndicesUsed.Add(Primitive.MaterialIndex);

			TArray<FVector3f>& Positions = GetVectorBuffer(PositionBufferIndex);
			Primitive.GetPositions(Positions);
			
			UpdateVector3fFromMorphTargets(Positions, Primitive, MorphTargetWeights, Vector3fMorphTargetAttributes::POSITIONS);

			FMeshFactory::FIndexVertexIdMap& PositionIndexToVertexId = PositionIndexToVertexIdPerPrim[Index];
			PositionIndexToVertexId.Empty(Positions.Num());

			if (TotalMatrix != FMatrix::Identity)
			{
				for (int32 PositionIndex = 0; PositionIndex < Positions.Num(); ++PositionIndex)
				{
					const FVertexID& VertexID = MeshDescription->CreateVertex();
					FVector3f FinalPosition = Positions[PositionIndex] * ImportUniformScale;
					const FVector TransformedPosition = TotalMatrix.TransformPosition(FVector(FinalPosition));
					FinalPosition = static_cast<FVector3f>(TransformedPosition);
					VertexPositions[VertexID] = FinalPosition;
					PositionIndexToVertexId.Add(PositionIndex, VertexID);
				}
			}
			else
			{
				for (int32 PositionIndex = 0; PositionIndex < Positions.Num(); ++PositionIndex)
				{
					const FVertexID& VertexID = MeshDescription->CreateVertex();
					VertexPositions[VertexID] = Positions[PositionIndex] * ImportUniformScale;
					PositionIndexToVertexId.Add(PositionIndex, VertexID);
				}
			}
		}

		// Add the PolygonGroup
		MaterialIndexToPolygonGroupID.Empty(10);
		for (int32 MaterialIndex : MaterialIndicesUsed)
		{
			const FPolygonGroupID& PolygonGroupID = MeshDescription->CreatePolygonGroup();
			MaterialIndexToPolygonGroupID.Add(MaterialIndex, PolygonGroupID);

			const FName ImportedSlotName(*FString::FromInt(MaterialIndex));
			PolygonGroupImportedMaterialSlotNames[PolygonGroupID] = ImportedSlotName;
		}

		// Add the VertexInstance
		bool bMeshUsesEmptyMaterial = false;
		bool bDidGenerateTexCoords = false;
		for (int32 Index = 0; Index < Mesh.Primitives.Num(); ++Index)
		{
			const FPrimitive& Primitive = Mesh.Primitives[Index];

			if (!Primitive.IsValid())
			{
				continue;
			}

			const bool        bHasDegenerateTriangles =
				ImportPrimitive(Primitive, Mesh.Name, Index, NumUVs, Mesh.HasTangents(), Mesh.HasColors(),  //
					VertexInstanceNormals, VertexInstanceTangents, VertexInstanceBinormalSigns, VertexInstanceUVs,
					VertexInstanceColors,  //
					EdgeHardnesses, PolygonObjectNames, MeshGlobalTransform, MeshDescription,
					MorphTargetWeights);

			bMeshUsesEmptyMaterial |= Primitive.MaterialIndex == INDEX_NONE;
			for (int32 UVIndex = 0; UVIndex < NumUVs; ++UVIndex)
			{
				if (!Primitive.HasTexCoords(UVIndex))
				{
					bDidGenerateTexCoords = true;
					break;
				}
			}

			if (bHasDegenerateTriangles)
			{
				Messages.Emplace(EMessageSeverity::Warning, FText::Format(LOCTEXT("PrimitiveFoundDegenerateTriangles", "Mesh {0} has primitive with degenerate triangles: {1}"), FText::FromString(Mesh.Name), Index));
			}
		}
		if (bMeshUsesEmptyMaterial)
		{
			Messages.Emplace(EMessageSeverity::Warning, FText::Format(LOCTEXT("PrimitiveNoMaterialsAssigned", "Mesh has primitives with no materials assigned: {0}") , FText::FromString(Mesh.Name)));
		}
	}

	bool FMeshFactoryImpl::ImportPrimitive(const GLTF::FPrimitive&                        Primitive,  //
		const FString&                                 MeshName,
		int32                                          PrimitiveIndex,
		int32                                          NumUVs,
		bool                                           bMeshHasTangents,
		bool                                           bMeshHasColors,
		const TVertexInstanceAttributesRef<FVector3f>&   VertexInstanceNormals,
		const TVertexInstanceAttributesRef<FVector3f>&   VertexInstanceTangents,
		const TVertexInstanceAttributesRef<float>&     VertexInstanceBinormalSigns,
		const TVertexInstanceAttributesRef<FVector2f>& VertexInstanceUVs,
		const TVertexInstanceAttributesRef<FVector4f>&  VertexInstanceColors,
		const TEdgeAttributesRef<bool>&                EdgeHardnesses,
		TPolygonAttributesRef<FName>&				PolygonObjectNames,
		const FTransform& MeshGlobalTransform,
		FMeshDescription* MeshDescription,
		const TArray<float>& MorphTargetWeights)
	{
		FName MeshFName(MeshName);
		const FPolygonGroupID CurrentPolygonGroupID = MaterialIndexToPolygonGroupID[Primitive.MaterialIndex];
		const uint32          TriCount = Primitive.TriangleCount();

		TArray<uint32>& Indices = GetIntBuffer();
		Primitive.GetTriangleIndices(Indices);

		//Note from gltf 2.0 specification:
		//	When the base mesh primitive does not specify normals, client implementations MUST calculate flat normals for each morph target; the provided tangents and their displacements (if present) MUST be ignored.
		bool bIgnoreTangents = false;

		FMatrix TotalMatrix = MeshGlobalTransform.ToMatrixWithScale();
		FMatrix TotalMatrixForNormal;
		TotalMatrixForNormal = TotalMatrix.Inverse();
		TotalMatrixForNormal = TotalMatrixForNormal.GetTransposed();

		const bool bIsMirrored = TotalMatrix.Determinant() < 0.f;
		const float BinormalMultipler = bIsMirrored ? -1.f : 1.f;

		auto TransformPosition = [](const FMatrix& Matrix, FVector3f& Position)
		{
			const FVector TransformedPosition = Matrix.TransformPosition(FVector(Position));
			Position = static_cast<FVector3f>(TransformedPosition);
		};

		auto TransformPosition4f = [](const FMatrix& Matrix, FVector4f& Position)
		{
			const FVector4d TransformedPosition = Matrix.TransformPosition(FVector4d(Position));
			Position = static_cast<FVector4f>(TransformedPosition);
		};

		TArray<FVector3f>& Normals = GetVectorBuffer(NormalBufferIndex);
		// glTF does not guarantee each primitive within a mesh has the same attributes.
		// Fill in gaps as needed:
		// - missing normals will be flat, based on triangle orientation
		// - missing UVs will be (0,0)
		// - missing tangents will be (0,0,1)
		if (Primitive.HasNormals())
		{
			TArray<FVector3f>& ReindexBuffer = GetVectorBuffer(ReindexBufferIndex);
			Primitive.GetNormals(Normals);

			UpdateVector3fFromMorphTargets(Normals, Primitive, MorphTargetWeights, Vector3fMorphTargetAttributes::NORMALS);

			if (TotalMatrixForNormal != FMatrix::Identity)
			{
				for (FVector3f& Normal : Normals)
				{
					FVector TransformedPosition = TotalMatrixForNormal.TransformVector(FVector(Normal)).GetSafeNormal();
					Normal = static_cast<FVector3f>(TransformedPosition);
				}
			}

			ReIndex(Normals, Indices, ReindexBuffer, TEXT("Normals"), MeshName);
			Swap(Normals, ReindexBuffer);
		}
		
		if (Normals.IsEmpty())
		{
			if (Primitive.HasNormals())
			{
				Messages.Emplace(EMessageSeverity::Warning,
					FText::Format(LOCTEXT("InvalidNormalsAccessor",
						"[{0}] Normal accessor declared but produced no data (invalid or empty); falling back to flat normals. Tangents discarded."),
						FText::FromString(MeshName)));
			}

			TArray<FVector3f>& Positions = GetVectorBuffer(PositionBufferIndex);
			Primitive.GetPositions(Positions);

			//update positions with morph targets before generating flat normals:
			UpdateVector3fFromMorphTargets(Positions, Primitive, MorphTargetWeights, Vector3fMorphTargetAttributes::POSITIONS);

			if (TotalMatrix != FMatrix::Identity)
			{
				for (FVector3f& Position : Positions)
				{
					TransformPosition(TotalMatrix, Position);
				}
			}

			GenerateFlatNormals(Positions, Indices, Normals, MeshName);
			bIgnoreTangents = true;
		}

		TArray<FVector4f>& Tangents = GetVector4dBuffer(TangentBufferIndex);
		if (!bIgnoreTangents && Primitive.HasTangents())
		{
			TArray<FVector4f>& ReindexBuffer = GetVector4dBuffer(Reindex4dBufferIndex);
			Primitive.GetTangents(Tangents);

			UpdateVector4dFromMorphTargets(Tangents, Primitive, MorphTargetWeights, Vector4dMorphTargetAttributes::TANGENTS);

			if (TotalMatrixForNormal != FMatrix::Identity)
			{
				for (FVector4f& TangentBinormal : Tangents)
				{
					const float Binormal = TangentBinormal.W * BinormalMultipler;
					FVector4 TransformedTangentBinormal = TotalMatrixForNormal.TransformVector(FVector(TangentBinormal)).GetSafeNormal();
					TransformedTangentBinormal.W = Binormal;
					TangentBinormal = FVector4f(TransformedTangentBinormal);
				}
			}

			ReIndex(Tangents, Indices, ReindexBuffer, TEXT("Tangents"), MeshName);
			Swap(Tangents, ReindexBuffer);
		}
		
		if (bMeshHasTangents && Tangents.IsEmpty())
		{
			// If other primitives in this mesh have tangents, generate filler ones for this primitive, to avoid gaps.
			Tangents.Init(FVector4f(1.0f, 0.0f, 0.0f, 1.0f), Primitive.VertexCount());
		}

		TArray<FVector4f>& Colors = GetVector4dBuffer(ColorBufferIndex);
		if (Primitive.HasColors())
		{
			TArray<FVector4f>& ReindexBuffer = GetVector4dBuffer(Reindex4dBufferIndex);
			Primitive.GetColors(Colors);

			UpdateVector4dFromMorphTargets(Colors, Primitive, MorphTargetWeights, Vector4dMorphTargetAttributes::COLORS);

			ReIndex(Colors, Indices, ReindexBuffer, TEXT("Colors"), MeshName);
			Swap(Colors, ReindexBuffer);
		}
		
		if (bMeshHasColors && Colors.IsEmpty())
		{
			// If other primitives in this mesh have colors, generate filler ones for this primitive, to avoid gaps.
			Colors.Init(FVector4f(1.0f), Primitive.VertexCount());
		}

		int32_t            AvailableBufferIndex = 0;
		TArray<FVector2f>* UVs[MAX_MESH_TEXTURE_COORDS_MD];
		for (int32 UVIndex = 0; UVIndex < NumUVs; ++UVIndex)
		{
			FString UVNameSpace = TEXT("UVs[") + FString::FromInt(UVIndex) + TEXT("]");

			UVs[UVIndex] = &GetVector2dBuffer(AvailableBufferIndex++);
			if (Primitive.HasTexCoords(UVIndex))
			{
				TArray<FVector2f>& ReindexBuffer = GetVector2dBuffer(UvReindexBufferIndex);
				Primitive.GetTexCoords(UVIndex, *UVs[UVIndex]);

				UpdateTexCoordsFromMorphTargets(*UVs[UVIndex], Primitive, MorphTargetWeights, UVIndex);

				ReIndex(*UVs[UVIndex], Indices, ReindexBuffer, UVNameSpace, MeshName);
				Swap(*UVs[UVIndex], ReindexBuffer);
			}
			
			if ((*UVs[UVIndex]).IsEmpty())
			{
				// Unreal StaticMesh must have UV channel 0.
				// glTF doesn't require this since not all materials need texture coordinates.
				// We also fill UV channel > 1 for this primitive if other primitives have it, to avoid gaps.
				(*UVs[UVIndex]).AddZeroed(TriCount * 3);
			}
		}

		bool bHasDegenerateTriangles = false;
		
		// Now add all vertexInstances
		FVertexID         CornerVertexIDs[3];
		uint32 TriangleCornerMaxIndex = (TriCount * 3) - 1;

		//Note: below code is mostly duplicated, 
		// difference is the IndiceIndex calculations in the 2 inner for loops.
		// Duplicated for performance reasons.
		if (bIsMirrored)
		{
			for (uint32 TriangleIndex = 0; TriangleIndex < TriCount; ++TriangleIndex)
			{
				for (uint32 Corner = 0; Corner < 3; ++Corner)
				{
					const int32 IndiceIndex = Indices[TriangleCornerMaxIndex - (TriangleIndex * 3 + Corner)];
					const FVertexID VertexID = PositionIndexToVertexIdPerPrim[PrimitiveIndex][IndiceIndex];
					CornerVertexIDs[Corner] = VertexID;
				}

				// Check for degenerate triangles
				const FVertexID& Vertex1 = CornerVertexIDs[0];
				const FVertexID& Vertex2 = CornerVertexIDs[1];
				const FVertexID& Vertex3 = CornerVertexIDs[2];

				if (Vertex1 == Vertex2 || Vertex2 == Vertex3 || Vertex1 == Vertex3)
				{
					bHasDegenerateTriangles = true;
					continue; // Triangle is degenerate, skip it
				}

				for (uint32 Corner = 0; Corner < 3; ++Corner)
				{
					const FVertexInstanceID VertexInstanceID = MeshDescription->CreateVertexInstance(CornerVertexIDs[Corner]);
					CornerVertexInstanceIDs[Corner] = VertexInstanceID;

					const uint32 IndiceIndex = TriangleCornerMaxIndex - (TriangleIndex * 3 + Corner);

					VertexInstanceNormals[VertexInstanceID] = Normals[IndiceIndex];

					if (!bIgnoreTangents && Tangents.Num() > 0)
					{
						VertexInstanceTangents[VertexInstanceID] = Tangents[IndiceIndex];
						VertexInstanceBinormalSigns[VertexInstanceID] = Tangents[IndiceIndex].W;
					}

					for (int32 UVIndex = 0; UVIndex < NumUVs; ++UVIndex)
					{
						VertexInstanceUVs.Set(VertexInstanceID, UVIndex, (*UVs[UVIndex])[IndiceIndex]);
					}

					if (Colors.Num() > 0)
					{
						FVector4f Color = Colors[IndiceIndex];
						// Because the engine will auto convert our vertex colors to sRGB, we need to counter the conversion here to ensure the end result is still linear space
						UE::Geometry::LinearColors::SRGBToLinear(Color);
						VertexInstanceColors[VertexInstanceID] = Color;
					}
				}

				// Insert a polygon into the mesh
				TArray<FEdgeID> NewEdgeIDs;
				const FPolygonID NewPolygonID = MeshDescription->CreatePolygon(CurrentPolygonGroupID, CornerVertexInstanceIDs, &NewEdgeIDs);

				for (const FEdgeID& NewEdgeID : NewEdgeIDs)
				{
					// Make all faces part of the same smoothing group, so Unreal will combine identical adjacent verts.
					// (Is there a way to set auto-gen smoothing threshold? glTF spec says to generate flat normals if they're not specified.
					//   We want to combine identical verts whether they're smooth neighbors or triangles belonging to the same flat polygon.)
					EdgeHardnesses[NewEdgeID] = false;
				}
			}
		}
		else
		{
			for (uint32 TriangleIndex = 0; TriangleIndex < TriCount; ++TriangleIndex)
			{
				for (uint32 Corner = 0; Corner < 3; ++Corner)
				{
					const int32 VertexIndex = Indices[(TriangleIndex * 3 + Corner)];
					const FVertexID VertexID = PositionIndexToVertexIdPerPrim[PrimitiveIndex][VertexIndex];
					CornerVertexIDs[Corner] = VertexID;
				}

				// Check for degenerate triangles
				const FVertexID& Vertex1 = CornerVertexIDs[0];
				const FVertexID& Vertex2 = CornerVertexIDs[1];
				const FVertexID& Vertex3 = CornerVertexIDs[2];

				if (Vertex1 == Vertex2 || Vertex2 == Vertex3 || Vertex1 == Vertex3)
				{
					bHasDegenerateTriangles = true;
					continue; // Triangle is degenerate, skip it
				}

				for (uint32 Corner = 0; Corner < 3; ++Corner)
				{
					const FVertexInstanceID VertexInstanceID = MeshDescription->CreateVertexInstance(CornerVertexIDs[Corner]);
					CornerVertexInstanceIDs[Corner] = VertexInstanceID;

					const uint32 IndiceIndex = (TriangleIndex * 3 + Corner);

					VertexInstanceNormals[VertexInstanceID] = Normals[IndiceIndex];

					if (!bIgnoreTangents && Tangents.Num() > 0)
					{
						VertexInstanceTangents[VertexInstanceID] = Tangents[IndiceIndex];
						VertexInstanceBinormalSigns[VertexInstanceID] = Tangents[IndiceIndex].W;
					}

					for (int32 UVIndex = 0; UVIndex < NumUVs; ++UVIndex)
					{
						VertexInstanceUVs.Set(VertexInstanceID, UVIndex, (*UVs[UVIndex])[IndiceIndex]);
					}

					if (Colors.Num() > 0)
					{
						FVector4f Color = Colors[IndiceIndex];
						// Because the engine will auto convert our vertex colors to sRGB, we need to counter the conversion here to ensure the end result is still linear space
						UE::Geometry::LinearColors::SRGBToLinear(Color);
						VertexInstanceColors[VertexInstanceID] = Color;
					}
				}

				// Insert a polygon into the mesh
				TArray<FEdgeID> NewEdgeIDs;
				const FPolygonID NewPolygonID = MeshDescription->CreatePolygon(CurrentPolygonGroupID, CornerVertexInstanceIDs, &NewEdgeIDs);

				for (const FEdgeID& NewEdgeID : NewEdgeIDs)
				{
					// Make all faces part of the same smoothing group, so Unreal will combine identical adjacent verts.
					// (Is there a way to set auto-gen smoothing threshold? glTF spec says to generate flat normals if they're not specified.
					//   We want to combine identical verts whether they're smooth neighbors or triangles belonging to the same flat polygon.)
					EdgeHardnesses[NewEdgeID] = false;
				}

				PolygonObjectNames.Set(NewPolygonID, MeshFName);
			}
		}
		
		return bHasDegenerateTriangles;
	}

	inline void FMeshFactoryImpl::CleanUp()
	{
		const uint32 ReserveSize = FMath::Min(uint32(IntBuffer.Num() + IntBuffer.GetSlack()), MaxReserveSize);  // cap reserved size
		IntBuffer.Empty(ReserveSize);
		Vector2dBuffers[0].Empty(ReserveSize);
		for (TArray<FVector3f>& Array : VectorBuffers)
		{
			Array.Empty(ReserveSize);
		}
		for (int32 Index = 1; Index < MAX_MESH_TEXTURE_COORDS_MD + 1; ++Index)
		{
			TArray<FVector2f>& Array = Vector2dBuffers[Index];
			Array.Empty();
		}
		for (TArray<FVector4f>& Array : Vector4dBuffers)
		{
			Array.Empty();
		}

		Messages.Empty();
	}

	//

	FMeshFactory::FMeshFactory()
		: Impl(new FMeshFactoryImpl())
	{
	}

	FMeshFactory::~FMeshFactory() {}

	void FMeshFactory::FillMeshDescription(const GLTF::FMesh &Mesh, const FTransform& MeshGlobalTransform, FMeshDescription* MeshDescription, const TArray<float>& MorphTargetWeights)
	{
		Impl->FillMeshDescription(Mesh, MeshGlobalTransform, MeshDescription, MorphTargetWeights);
	}

	const TArray<FLogMessage>& FMeshFactory::GetLogMessages() const
	{
		return Impl->GetLogMessages();
	}

	float FMeshFactory::GetUniformScale() const
	{
		return Impl->ImportUniformScale;
	}

	void FMeshFactory::SetUniformScale(float Scale)
	{
		Impl->ImportUniformScale = Scale;
	}

	void FMeshFactory::SetReserveSize(uint32 Size)
	{
		Impl->MaxReserveSize = Size;
	}

	void FMeshFactory::CleanUp()
	{
		Impl->CleanUp();
	}

	TArray<FMeshFactory::FIndexVertexIdMap>& FMeshFactory::GetPositionIndexToVertexIdPerPrim() const
	{
		return Impl->PositionIndexToVertexIdPerPrim;
	}
} //namespace GLTF

#undef LOCTEXT_NAMESPACE