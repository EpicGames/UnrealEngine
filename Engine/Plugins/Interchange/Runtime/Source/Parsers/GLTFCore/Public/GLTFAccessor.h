// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components.h" //For MAX_STATIC_TEXCOORDS check.

#define UE_API GLTFCORE_API

struct FMD5Hash;

namespace GLTF
{
	struct FBuffer
	{
		uint64 ByteLength;
		uint8* Data;

		FBuffer()
			: ByteLength(0)
			, Data(nullptr)
		{

		}

		explicit FBuffer(uint64 InByteLength)
		    : ByteLength(InByteLength)
		    , Data(nullptr)
		{
		}

		bool IsValid() const
		{
			return Data != nullptr;
		}

		const uint8* DataAt(uint64 Offset) const
		{
			checkSlow(Data);
			return Data + Offset;
		}

		void operator=(const FBuffer& InBuffer)
		{
			ByteLength = InBuffer.ByteLength;
			Data = InBuffer.Data;
		}
	};

	struct FBufferView
	{
		FBuffer Buffer;
		uint64 ByteOffset;
		uint64 ByteLength;
		// if zero then accessor elements are tightly packed, i.e., effective stride equals the size of the element
		uint32 ByteStride;  // range 4..252

		FBufferView()
			: Buffer()
			, ByteOffset(0)
			, ByteLength(0)
			, ByteStride(0)
		{

		}
		explicit FBufferView(const FBuffer& InBuffer, uint64 InOffset, uint64 InLength, uint32 InStride)
		    : Buffer(InBuffer)
		    , ByteOffset(InOffset)
		    , ByteLength(InLength)
		    , ByteStride(InStride)
		{
			// check that view fits completely inside the buffer
		}

		void operator=(const FBufferView& InBufferView)
		{
			Buffer = InBufferView.Buffer;
			ByteOffset = InBufferView.ByteOffset;
			ByteLength = InBufferView.ByteLength;
			ByteStride = InBufferView.ByteStride;
		}

		bool IsValid() const
		{
			return Buffer.IsValid();
		}

		bool HasCapacityFor(const uint32 InCount, const uint64 InByteOffset, const uint32 InElementSize, const uint32 InByteStride) const
		{
			if (!IsValid())
			{
				return false;
			}
		
			if (InElementSize == 0 || InByteStride == 0)
			{
				return false;
			}

			if (InByteOffset >= ByteLength)
			{
				return false;
			}

			if (InCount > 0)
			{
				const uint64 AvailableBytes = ByteLength - InByteOffset;

				if (InElementSize > AvailableBytes)
				{
					return false;
				}
				if (InCount - 1 > (AvailableBytes - InElementSize) / InByteStride)
				{
					return false;
				}
			}

			return true;
		}

		const uint8* DataAt(uint64 Offset) const
		{
			return Buffer.DataAt(Offset + ByteOffset);
		}
	};

	enum EMeshAttributeType : uint8
	{
		POSITION = 0,
		NORMAL,
		TANGENT,
		
		TEXCOORD_0, 
		TEXCOORD_1,
		TEXCOORD_2,
		TEXCOORD_3,
		TEXCOORD_4,
		TEXCOORD_5,
		TEXCOORD_6,
		TEXCOORD_7,
		
		COLOR_0,

		JOINTS_0,
		WEIGHTS_0,

		COUNT,
	};

	constexpr uint8 SupportedTexCoordNum = EMeshAttributeType::TEXCOORD_7 - EMeshAttributeType::TEXCOORD_0 + 1;
	static_assert(SupportedTexCoordNum == MAX_STATIC_TEXCOORDS, "The maximum UE supported Texcoord count has changed. Update the EMeshAttributeType and related accordingly.");

	GLTFCORE_API FString ToString(const EMeshAttributeType& Type);

	struct FAccessor
	{
		// accessor stores the data but has no usage semantics

		enum class EType
		{
			Unknown,
			Scalar,
			Vec2,
			Vec3,
			Vec4,
			Mat2,
			Mat3,
			Mat4,
			Count
		};

		enum class EComponentType
		{
			None,
			S8,   // signed byte
			U8,   // unsigned byte
			S16,  // signed short
			U16,  // unsigned short
			U32,  // unsigned int -- only valid for indices, not attributes
			F32,  // float
			Count
		};

		struct FSparse
		{
			bool                     bHasSparse;

			const uint32              Count;

			//Indices:
			struct FIndices
			{
				const int32          Count; //Helper for creating cache, equals to FSparse.Count

				FBufferView          BufferView;
				const uint64         ByteOffset;
				const EComponentType ComponentType;

				FIndices()
					: Count(0)
					, BufferView()
					, ByteOffset(0)
					, ComponentType(EComponentType::None)
				{
				}

				FIndices(uint32 InCount, FBufferView& InBufferView, uint64 InByteOffset, EComponentType InComponentType);
			} Indices;
			
			//Values:
			struct FValues
			{
				FBufferView          BufferView;
				const uint64         ByteOffset;
				
				FValues()
					: BufferView()
					, ByteOffset(0)
				{
				}

				FValues(FBufferView& InBufferView, uint64 InByteOffset)
					: BufferView(InBufferView)
					, ByteOffset(InByteOffset)
				{
				}
			} Values;
			
			FSparse()
				: bHasSparse(false)
				, Count(0)
				, Indices(FIndices())
				, Values(FValues())
			{
			}

			FSparse(uint32 InCount,
				FBufferView& InIndicesBufferView, uint64 InIndicesByteOffset, EComponentType InIndicesComponentType,
				FBufferView& InValuesBufferView, uint64 InValuesByteOffset)
				: bHasSparse(true)
				, Count(InCount)
				, Indices(InCount, InIndicesBufferView, InIndicesByteOffset, InIndicesComponentType)
				, Values(InValuesBufferView, InValuesByteOffset)
			{
			}
		};

		uint32                AccessorIndex; //Index of the Accessor in Asset->Accessors list

		const uint32          Count;
		const EType           Type;
		const EComponentType  ComponentType;
		const bool            bNormalized;
		bool                  bQuantized;
		const FSparse         Sparse;

		FBufferView           BufferView;
		uint64                ByteOffset;
		const uint32          NumberOfComponents;
		const uint32          ElementSize;
		const uint32	      ByteStride;

		UE_API FAccessor();
		UE_API FAccessor(uint32 InAccessorIndex, FBufferView& InBufferView, uint64 InOffset, uint32 InCount, EType InType, EComponentType InCompType, bool bInNormalized, const FSparse& InSparse);

		/**
		* Compressed Data sets (FAccessor does not have FBufferView at processing of Accessors, as FBufferview will be created with the processing of the KHR_draco_mesh_compression extension)
		*/
		UE_API FAccessor(uint32 InAccessorIndex, uint32 InCount, EType InType, EComponentType InCompType, bool bInNormalized, const FSparse& InSparse);

		UE_API bool IsValid() const;
		UE_API FMD5Hash GetHash() const;

		UE_API uint32       GetUnsignedInt(uint32 Index) const;
		UE_API void         GetUnsignedInt16x4(uint32 Index, uint16 Values[4]) const;

		UE_API float        GetFloat(uint32 Index) const;
		UE_API FVector2D    GetVec2(uint32 Index) const;
		UE_API FVector      GetVec3(uint32 Index) const;
		UE_API FVector4     GetVec4(uint32 Index) const;
		UE_API FMatrix      GetMat4(uint32 Index) const;

		UE_API void         GetUnsignedIntArray(TArray<uint32>& Buffer) const;
		UE_API void         GetUnsignedIntArray(uint32* Buffer) const;

		UE_API void         GetFloatArray(TArray<float>& Buffer) const;
		UE_API void         GetFloatArray(float* Buffer) const;

		UE_API void         GetVec2Array(TArray<FVector2f>& Buffer) const;
		UE_API void         GetVec2Array(FVector2f* Buffer) const;

		UE_API void         GetVec3Array(TArray<FVector3f>& Buffer) const;
		UE_API void         GetVec3Array(FVector3f* Buffer) const;

		///@note Performs axis conversion for vec3s(i.e. from glTF right-handed and Y-up to left-handed and Z-up).
		UE_API void         GetCoordArray(TArray<FVector3f>& Buffer) const;
		UE_API void         GetCoordArray(FVector3f* Buffer) const;

		UE_API void         GetVec4Array(TArray<FVector4f>& Buffer) const;
		UE_API void         GetVec4Array(FVector4f* Buffer) const;

		///@note Performs axis conversion for quaternion(i.e. from glTF right-handed and Y-up to left-handed and Z-up).
		UE_API void         GetQuatArray(TArray<FVector4f>& Buffer) const;
		UE_API void         GetQuatArray(FVector4f* Buffer) const;

		UE_API void         GetMat4Array(TArray<FMatrix44f>& Buffer) const;
		UE_API void         GetMat4Array(FMatrix44f* Buffer) const;

		UE_API bool IsValidDataType(EMeshAttributeType MeshAttributeType, bool bMorphTargetProperty) const;
		UE_API bool CheckAccessorTypeForDataType(EMeshAttributeType MeshAttributeType, bool bMorphTargetProperty) const;
		UE_API bool CheckNonQuantizedComponentTypeForDataType(EMeshAttributeType MeshAttributeType, bool bMorphTargetProperty) const;
		UE_API bool CheckQuantizedComponentTypeForDataType(EMeshAttributeType MeshAttributeType, bool bMorphTargetProperty) const;

		UE_API const uint8* DataAt(uint32 Index) const;

		//Sparse related helpers:
		UE_API void UpdateUnsignedIntWithSparse(uint32 Index, uint32& Data) const;
		UE_API void UpdateUnsignedInt16x4WithSparse(uint32 Index, uint16 Data[4]) const;

		UE_API void UpdateFloatWithSparse(uint32 Index, float& Data) const;
		UE_API void UpdateVec2WithSparse(uint32 Index, FVector2D& Data) const;
		UE_API void UpdateVec3WithSparse(uint32 Index, FVector& Data) const;
		UE_API void UpdateVec4WithSparse(uint32 Index, FVector4& Data) const;

		UE_API void UpdateMat4WithSparse(uint32 Index, FMatrix& Data) const;

		UE_API void UpdateFloatArrayWithSparse(float* Data) const;
		UE_API void UpdateUnsignedIntArrayWithSparse(uint32* Data) const;

		template<typename ItemType, uint32 ItemElementCount>
		void UpdateArrayWithSparse(ItemType* Data) const;
	};
}  // namespace GLTF

#undef UE_API
