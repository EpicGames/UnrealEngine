// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FScriptContainerElement;

namespace GLTF
{
	struct FNode;
	struct FAccessor;

	struct FAnimation
	{
		enum class EInterpolation
		{
			Linear,
			Step,
			CubicSpline,
		};

		enum class EPath
		{
			Translation,
			Rotation,
			Scale,
			Weights
		};

		struct FTarget
		{
			const FNode& Node;
			EPath        Path;

			FTarget(const FNode& Node)
			    : Node(Node)
			{
			}
		};

		struct FChannel
		{
			// The index of a sampler in this animation used to compute the value for the target.
			int32 Sampler;
			// The index of the node and TRS property to target.
			FTarget Target;

			FChannel(const FNode& Node)
			    : Target(Node)
			{
			}
		};

		struct FSampler
		{
			EInterpolation Interpolation;
			// The accessor containing keyframe input values, e.g., time. Always float and seconds.
			const FAccessor& Input;
			// The accessor containing keyframe output values.
			const FAccessor& Output;

			FSampler(const FAccessor& Input, const FAccessor& Output)
			    : Interpolation(EInterpolation::Linear)
			    , Input(Input)
			    , Output(Output)
			{
			}
		};

		FString          Name;
		TArray<FSampler> Samplers;
		TArray<FChannel> Channels;

		TMap<FString, FString> Extras;

		FString          UniqueId; //will be generated in FAsset::GenerateNames
	};

}  // namespace GLTF
