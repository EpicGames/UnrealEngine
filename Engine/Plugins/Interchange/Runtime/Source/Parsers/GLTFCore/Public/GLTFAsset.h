// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GLTFAnimation.h"
#include "GLTFMaterial.h"
#include "GLTFMesh.h"
#include "GLTFNode.h"
#include "GLTFTexture.h"

namespace GLTF { struct FAnimation; }
namespace GLTF { struct FBuffer; }
namespace GLTF { struct FBufferView; }
namespace GLTF { struct FCamera; }
namespace GLTF { struct FImage; }
namespace GLTF { struct FLight; }
namespace GLTF { struct FMaterial; }
namespace GLTF { struct FMesh; }
namespace GLTF { struct FNode; }
namespace GLTF { struct FSampler; }
namespace GLTF { struct FSkinInfo; }
namespace GLTF { struct FTexture; }
namespace GLTF { struct FAccessor; }
struct FScriptContainerElement;

namespace GLTF
{
	enum class EExtension
	{
		KHR_MaterialsPbrSpecularGlossiness,
		KHR_MaterialsUnlit,
		KHR_MaterialsClearCoat,
		KHR_MaterialsTransmission,
		KHR_MaterialsSheen,
		KHR_MaterialsVariants,
		KHR_MaterialsIOR,
		KHR_MaterialsSpecular,
		KHR_MaterialsEmissiveStrength,
		KHR_MaterialsIridescence,
		KHR_MaterialsAnisotropy,
		KHR_MaterialsVolume,
		KHR_TextureTransform,
		KHR_DracoMeshCompression,
		KHR_LightsPunctual,
		KHR_Lights,
		EXT_LightsIES,
		KHR_Blend,
		MSFT_TextureDDS,
		MSFT_PackingNormalRoughnessMetallic,
		MSFT_PackingOcclusionRoughnessMetallic,
		KHR_MeshQuantization,
		Count
	};

	struct FScene
	{
		FString       Name;
		TArray<int32> Nodes;
		TMap<FString, FString> Extras;
		FString	      UniqueId; //will be generated in FAsset::GenerateNames
	};

	struct FMetadata
	{
		struct FExtraData
		{
			FString Name;
			FString Value;
		};
		FString            GeneratorName;
		float              Version;
		TArray<FExtraData> Extras;

		GLTFCORE_API const FExtraData* GetExtraData(const FString& Name) const;
	};

	struct FAsset : FNoncopyable
	{
		static GLTFCORE_API const TSet<EExtension> SupportedExtensions;

		enum EValidationCheck
		{
			Valid,
			InvalidMeshPresent   = 0x1,
			InvalidNodePresent = 0x2,
		};

		FString Name;

		TArray<FBuffer>          Buffers;
		TArray<FBufferView>      BufferViews;
		TArray<FAccessor>        Accessors; //Note: We rely on order of the Accessors (both from glTF point of view and from internal Identification point of view).
		TArray<FMesh>            Meshes;

		TArray<FScene>     Scenes;
		TArray<FNode>      Nodes;
		TArray<FCamera>    Cameras;
		TArray<FLight>     Lights;
		TArray<FLightIES>  LightsIES;
		TArray<FSkinInfo>  Skins;
		TArray<FAnimation> Animations;

		TArray<FImage>    Images;
		TArray<FSampler>  Samplers;
		TArray<FTexture>  Textures;
		TArray<FMaterial> Materials;

		TArray<FString>   Variants;

		TArray<FString>	 ExtensionsUsed;      //marked in the gltf
		TArray<FString>	 ExtensionsRequired;  //marked in the gltf

		TSet<EExtension> ProcessedExtensions;
		FMetadata        Metadata;

		bool             HasAbnormalInverseBindMatrices = false; //True: in cases where at least 1 Joint node has multiple InverseBindMatrices that do not equal.

		/**
		 * Will clear the asset's buffers.
		 *
		 * @param BinBufferKBytes - bytes to reserve for the bin chunk buffer.
		 * @param ExtraBinBufferKBytes -  bytes to reserve for the extra binary buffer(eg. image, mime data, etc.)
		 * @note Only reserves buffers if they had any existing data.
		 */
		GLTFCORE_API void Clear(uint32 BinBufferKBytes, uint32 ExtraBinBufferKBytes);

		/**
		 * Will generate names for any entities(nodes, meshes, etc.) that have the name field missing.
		 * Is called from GLTF::FFileReader::ReadFile
		 *
		 * @param Prefix - prefix to add to the entities name.
		 */
		GLTFCORE_API void GenerateNames();

		/**
		 * Finds the indices for the nodes which are root nodes.
		 *
		 * @param NodeIndices - array with the results
		 */
		GLTFCORE_API void GetRootNodes(TArray<int32>& NodeIndices);

		/**
		 * Returns EValidationCheck::Valid if the asset passes the post-import validation checks.
		 */
		GLTFCORE_API EValidationCheck ValidationCheck() const;

		/**
		* Creates and sets up FBuffer and FBufferView for a given FAccessor (at the given index 'id')
		*/
		GLTFCORE_API FAccessor& CreateBuffersForAccessorIndex(uint32 AccessorIndex);

	private:
		// Binary glTF files can have embedded data after JSON.
		// This will be empty when reading from a text glTF (common) or a binary glTF with no BIN chunk (rare).
		TArray64<uint8> BinData;

		// Extra binary data used for images from disk, mime data and so on.
		TArray64<uint8> ExtraBinData;

		//Stores (Draco) Uncompressed Binary Data
		TArray<TArray64<uint8>>  UncompressedDracoBinData;

		friend class FFileReader;
	};

	GLTFCORE_API const TCHAR* ToString(EExtension Extension);

	struct GLTFNodePrefixStaticHelper
	{
		GLTFCORE_API static FString GetScenePrefix(const FString& AssetName);
		GLTFCORE_API static FString GetNodePrefix(const FString& AssetName);
		GLTFCORE_API static FString GetJointPrefix(const FString& AssetName);
		GLTFCORE_API static FString GetTexturePrefix(const FString& AssetName);
		GLTFCORE_API static FString GetCameraPrefix(const FString& AssetName);
		GLTFCORE_API static FString GetLightPrefix(const FString& AssetName);
		GLTFCORE_API static FString GetMaterialPrefix(const FString& AssetName);
		GLTFCORE_API static FString GetSkinPrefix(const FString& AssetName);
		GLTFCORE_API static FString GetAnimationPrefix(const FString& AssetName);
		GLTFCORE_API static FString GetImagePrefix(const FString& AssetName);
		GLTFCORE_API static FString GetMeshPrefix(const FString& AssetName);
		GLTFCORE_API static FString GetIESPrefix(const FString& AssetName);
	};

}  // namespace GLTF
