// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

/**
 * The public interface of the GLTFCore module
 */
class IGLTFCoreModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to IGLTFCore
	 *
	 * @return Returns IGLTFCore singleton instance, loading the module on demand if needed
	 */
	static IGLTFCoreModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IGLTFCoreModule>("GLTFCore");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("GLTFCore");
	}
};
