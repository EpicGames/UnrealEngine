// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Tuple.h"
#include "Internationalization/Text.h"

namespace GLTF
{
	enum class EMessageSeverity
	{
		Display,
		Warning,
		Error,
	};

	/**
	* There are certain warnings that we acknowledge and accept when running AutomationTests.
	* For that reason we want Display severity for these specific warnings when Automation is running.
	* For example not supported extension warnings are acknowledged and accepted when Automation is running,
	* but we want those shown still as warnings when running otherwise.
	*/
	EMessageSeverity RuntimeWarningSeverity();

	using FLogMessage = TTuple<EMessageSeverity, FText>;

	class FBaseLogger
	{
	public:
		const TArray<FLogMessage>& GetLogMessages() const;

	protected:
		mutable TArray<FLogMessage> Messages;
	};

	inline const TArray<FLogMessage>& FBaseLogger::GetLogMessages() const
	{
		return Messages;
	}
}  // namespace GLTF

