// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/SecureHash.h"

namespace GLTF
{
	struct FTextureTransform
	{
		float Offset[2];
		float Rotation;
		float Scale[2];

		FTextureTransform()
		{
			Offset[0] = Offset[1] = 0.0f;
			Scale[0] = Scale[1] = 1.0f;
			Rotation = 0.0f;
		}
	};

	struct FTextureMap
	{
		int32 TextureIndex;
		uint8 TexCoord;

		bool bHasTextureTransform;
		FTextureTransform TextureTransform;

		FTextureMap()
		    : TextureIndex(INDEX_NONE)
		    , TexCoord(0)
			, bHasTextureTransform(false)
		{
		}

		FMD5Hash GetHash() const
		{
			FMD5 MD5;
			MD5.Update(reinterpret_cast<const uint8*>(&TextureIndex), sizeof(TextureIndex));
			MD5.Update(&TexCoord, sizeof(TexCoord));
			FMD5Hash Hash;
			Hash.Set(MD5);
			return Hash;
		}
	};

	struct FMaterial
	{
		enum class EAlphaMode
		{
			Opaque,
			Mask,
			Blend
		};

		enum class EShadingModel
		{
			MetallicRoughness,
			SpecularGlossiness,
		};

		enum class EPackingFlags
		{
			// no packing, i.e. default: Unused (R) Roughness (G), Metallic (B) map
			None = 0x0,
			// packing two channel (RG) normal map
			NormalRG = 0x1,
			// packing Occlusion (R), Roughness (G), Metallic (B) map
			OcclusionRoughnessMetallic = 0x2,
			// packing Roughness (R), Metallic (G), Occlusion (B) map
			RoughnessMetallicOcclusion = 0x4,
			// packing Normal (RG), Roughness (B), Metallic (A) map
			NormalRoughnessMetallic = NormalRG | 0x8,
		};

		struct FMetallicRoughness
		{
			FTextureMap Map;
			float       MetallicFactor = 1.0f;
			float       RoughnessFactor = 1.0f;
		};
		// "pbrSpecularGlossiness" extension
		// Implements a specular glossiness shading model and is mutually exclusive with "specular" extension, which uses metallic roughness model
		struct FSpecularGlossiness 
		{
			FTextureMap Map;
			FVector     SpecularFactor = FVector(1.0f);
			float       GlossinessFactor = 1.0f;
		};
		struct FClearCoat
		{
			float ClearCoatFactor = 0.0f;
			FTextureMap ClearCoatMap;

			float Roughness = 0.0f;
			FTextureMap RoughnessMap;

			float NormalMapUVScale = 1.0f;
			FTextureMap NormalMap;
		};
		struct FTransmission
		{
			float TransmissionFactor = 0.0f;
			FTextureMap TransmissionMap;
		};
		struct FSheen
		{
			FVector SheenColorFactor = FVector::Zero();
			FTextureMap SheenColorMap;
			float SheenRoughnessFactor = 0.f;
			FTextureMap SheenRoughnessMap;
		};
		struct FSpecular // "specular" extension
		{
			float SpecularFactor = 1.0f;
			FTextureMap SpecularMap;
			FVector SpecularColorFactor = FVector(1.0f, 1.0f, 1.0f);
			FTextureMap SpecularColorMap;
		};

		struct FPacking
		{
			int         Flags = (int)EPackingFlags::None;  // see EPackingFlags
			FTextureMap Map;
			FTextureMap NormalMap;
		};

		struct FIridescence
		{
			struct FThickness
			{
				float       Minimum = 100.0f;
				float       Maximum = 400.0f;
				FTextureMap	Texture;
			};

			float Factor = 0.0f;
			FTextureMap Texture;
			float IOR = 1.3f;

			FThickness Thickness;

			bool bHasIridescence = false;
		};

		struct FAnisotropy
		{
			FTextureMap Texture;

			float Strength = 0.0;
			float Rotation = 0.0;

			bool bHasAnisotropy = false;
		};

		struct FVolume
		{
			FTextureMap ThicknessTexture;
			float Thickness = 0.f;
			float AttenuationDistance = 10.f; // Theoretically +∞, should definitely not be 0, 10.f if a good compromise as it renders no attenuation
			FVector AttenuationColor{ 1, 1, 1 };
			bool bHasVolume = false;
		};

		FString Name;

		// PBR properties
		FTextureMap         BaseColor;		 // Used for DiffuseColor on Specular-Glossiness mode
		FVector4f           BaseColorFactor; // Used for DiffuseFactor on Specular-Glossiness mode
		EShadingModel       ShadingModel;
		FMetallicRoughness  MetallicRoughness;
		FSpecularGlossiness SpecularGlossiness;
		FClearCoat          ClearCoat;
		FTransmission       Transmission;
		FSheen              Sheen;
		FSpecular           Specular;
		float               IOR;
		float               EmissiveStrength;

		// base properties
		FTextureMap Normal;
		FTextureMap Occlusion;
		FTextureMap Emissive;
		float       NormalScale;
		float       OcclusionStrength;
		FVector3f   EmissiveFactor;

		// material properties
		bool       bIsDoubleSided;
		EAlphaMode AlphaMode;
		float      AlphaCutoff;  // only used when AlphaMode == Mask

		// extension properties
		FPacking        Packing;
		bool            bIsUnlitShadingModel;
		bool            bHasClearCoat;
		bool            bHasSheen;
		bool            bHasTransmission;
		bool            bHasIOR;
		bool            bHasSpecular;
		bool            bHasEmissiveStrength;
		FIridescence    Iridescence;
		FAnisotropy		Anisotropy;
		FVolume         Volume;

		TMap<FString, FString> Extras;

		FString UniqueId; //will be generated in FAsset::GenerateNames

		FMaterial(const FString& Name)
		    : Name(Name)
		    , BaseColorFactor {1.0f, 1.0f, 1.0f, 1.0f}
		    , ShadingModel(EShadingModel::MetallicRoughness)
			, IOR(1.0f)
			, EmissiveStrength(1.0f)
		    , NormalScale(1.f)
		    , OcclusionStrength(1.f)
		    , EmissiveFactor(FVector::ZeroVector)
		    , bIsDoubleSided(false)
		    , AlphaMode(EAlphaMode::Opaque)
		    , AlphaCutoff(0.5f)
		    , bIsUnlitShadingModel(false)
			, bHasClearCoat(false)
			, bHasSheen(false)
			, bHasTransmission(false)
			, bHasIOR(false)
			, bHasSpecular(false)
			, bHasEmissiveStrength(false)
		{
		}

		bool IsOpaque() const
		{
			return AlphaMode == EAlphaMode::Opaque;
		}

		GLTFCORE_API FMD5Hash GetHash() const;
	};
}  // namespace GLTF
