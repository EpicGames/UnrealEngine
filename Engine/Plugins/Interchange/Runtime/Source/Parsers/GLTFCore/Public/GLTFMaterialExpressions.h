// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/StrongObjectPtr.h"
#include "Misc/SecureHash.h"

#define UE_API GLTFCORE_API

namespace GLTF
{
	class FMaterialExpressionInput;
	class FMaterialExpressionTexture;
	class FMaterialExpressionFunctionCall;
	class FMaterialElement;

	enum class EMaterialExpressionType
	{
		ConstantColor,
		ConstantScalar,
		FunctionCall,
		Generic,
		Texture,
		TextureCoordinate,
	};

	enum class EGLTFMaterialShadingModel
	{
		DefaultLit,
		Subsurface,
		ClearCoat,
		ThinTranslucent,
		Unlit
	};

	class ITextureElement
	{
	public:
		virtual ~ITextureElement() = default;
	};

	class FMaterialExpression
	{
	public:
		virtual ~FMaterialExpression() = default;

		virtual FMaterialExpressionInput* GetInput(int32 Index)
		{
			return nullptr;
		}

		virtual int32 GetInputCount() const
		{
			return 0;
		}

		EMaterialExpressionType GetType() const
		{
			return Type;
		}

		template <EMaterialExpressionType T>
		bool IsA() const
		{
			return T == Type;
		}

		UE_API void ConnectExpression(FMaterialExpressionInput& ExpressionInput, int32 OutputIndex);

	protected:
		UE_API FMaterialExpression();
		UE_API FMaterialExpression(const FMaterialExpression& Expr);
		UE_API explicit FMaterialExpression(int Type);

	protected:
		EMaterialExpressionType Type;

		friend class FMaterialElement;
	};

	class FMaterialExpressionInput
	{
	public:
		FMaterialExpression* GetExpression()
		{
			return Expression;
		}

		const FMaterialExpression* GetExpression() const
		{
			return Expression;
		}

		int32 GetOutputIndex() const
		{
			return OutputIndex;
		}

	private:
		UE_API FMaterialExpressionInput(const FString& Name);

	private:
		FString              Name;
		FMaterialExpression* Expression;
		int32                OutputIndex;

		friend class FMaterialElement;
		friend class FMaterialExpression;
		friend class FMaterialExpressionTexture;
		friend class TArray<FMaterialExpressionInput>;
	};

	class FMaterialExpressionParameter : public FMaterialExpression
	{
	public:
		virtual ~FMaterialExpressionParameter() = default;

		UE_API void         SetName(const TCHAR* Name);
		UE_API const TCHAR* GetName() const;
		UE_API void         SetGroupName(const TCHAR* GroupName);
		UE_API const TCHAR* GetGroupName() const;

	protected:
		UE_API explicit FMaterialExpressionParameter(int Type);

	private:
		FString Name;
		FString GroupName;
	};

	class FMaterialExpressionScalar : public FMaterialExpressionParameter
	{
	public:
		enum
		{
			Type = (int)EMaterialExpressionType::ConstantScalar
		};

		FMaterialExpressionScalar()
		    : FMaterialExpressionParameter(Type)
		{
		}

		float& GetScalar()
		{
			return Scalar;
		}

		float GetScalar() const
		{
			return Scalar;
		}

	private:
		float Scalar;
	};

	class FMaterialExpressionColor : public FMaterialExpressionParameter
	{
	public:
		enum
		{
			Type = (int)EMaterialExpressionType::ConstantColor
		};

		UE_API FMaterialExpressionColor();

		FLinearColor& GetColor()
		{
			return Color;
		}

		const FLinearColor& GetColor() const
		{
			return Color;
		}

	private:
		FLinearColor Color;
	};

	class FMaterialExpressionTexture : public FMaterialExpressionParameter
	{
	public:
		enum
		{
			Type = (int)EMaterialExpressionType::Texture
		};

		UE_API FMaterialExpressionTexture();

		void SetTexture(ITextureElement* InTexture)
		{
			Texture = InTexture;
		}

		ITextureElement* GetTexture() const
		{
			return Texture;
		}

		FMaterialExpressionInput& GetInputCoordinate()
		{
			return InputCoordinate;
		}

		UE_API virtual FMaterialExpressionInput* GetInput(int32 Index) override;
		UE_API virtual int32                     GetInputCount() const override;

	private:
		FMaterialExpressionInput InputCoordinate;
		ITextureElement*         Texture;
	};

	class FMaterialExpressionTextureCoordinate : public FMaterialExpression
	{
	public:
		enum
		{
			Type = (int)EMaterialExpressionType::TextureCoordinate
		};

		UE_API FMaterialExpressionTextureCoordinate();

		void SetCoordinateIndex(int32 InCoordinateIndex)
		{
			CoordinateIndex = InCoordinateIndex;
		}

		int32 GetCoordinateIndex() const
		{
			return CoordinateIndex;
		}

	private:
		int32 CoordinateIndex;
	};

	class FMaterialExpressionGeneric : public FMaterialExpression
	{
	public:
		enum
		{
			Type = (int)EMaterialExpressionType::Generic
		};

		UE_API FMaterialExpressionGeneric();

		void SetExpressionName(const TCHAR* InExpressionName)
		{
			ExpressionName = InExpressionName;
		}

		const TCHAR* GetExpressionName() const
		{
			return *ExpressionName;
		}

		UE_API virtual FMaterialExpressionInput* GetInput(int32 Index) override;
		UE_API virtual int32                     GetInputCount() const override;

		void SetBoolProperty(const TCHAR* Name, bool Value)
		{
			BoolProperties.Add(Name, Value);
		}

		void SetFloatProperty(const TCHAR* Name, float Value)
		{
			FloatProperties.Add(Name, Value);
		}

		const TMap<FString, bool>& GetBoolProperties() const
		{
			return BoolProperties;
		}

		const TMap<FString, float>& GetFloatProperties() const
		{
			return FloatProperties;
		}

	private:
		FString                          ExpressionName;
		TArray<FMaterialExpressionInput> Inputs;

		TMap<FString, bool> BoolProperties;
		TMap<FString, float> FloatProperties;
	};

	class FMaterialExpressionFunctionCall : public FMaterialExpression
	{
	public:
		enum
		{
			Type = (int)EMaterialExpressionType::FunctionCall
		};

		UE_API FMaterialExpressionFunctionCall();

		void SetFunctionPathName(const TCHAR* InFunctionPathName)
		{
			FunctionPathName = InFunctionPathName;
		}

		const TCHAR* GetFunctionPathName() const
		{
			return *FunctionPathName;
		}

		UE_API virtual FMaterialExpressionInput* GetInput(int32 Index) override;
		UE_API virtual int32                     GetInputCount() const override;

	private:
		FString                          FunctionPathName;
		TArray<FMaterialExpressionInput> Inputs;
	};

	class FMaterialElement
	{
	public:
		UE_API FMaterialElement(const FString& Name);
		UE_API virtual ~FMaterialElement();

		virtual int  GetBlendMode() const          = 0;
		virtual void SetBlendMode(int InBlendMode) = 0;
		virtual bool GetTwoSided() const           = 0;
		virtual void SetTwoSided(bool bTwoSided)   = 0;
		virtual bool GetIsThinSurface() const = 0;
		virtual void SetIsThinSurface(bool bIsThinSurface) = 0;
		virtual void SetShadingModel(EGLTFMaterialShadingModel InShadingModel) = 0;
		virtual void SetTranslucencyLightingMode(int InLightingMode) = 0;

		virtual void Finalize() = 0;

		UE_API FMD5Hash GetGLTFMaterialHash() const;
		UE_API void SetGLTFMaterialHash(FMD5Hash Hash);

		const FString& GetName() const;

		FMaterialExpressionInput& GetBaseColor();
		FMaterialExpressionInput& GetMetallic();
		FMaterialExpressionInput& GetSpecular();
		FMaterialExpressionInput& GetRoughness();
		FMaterialExpressionInput& GetEmissiveColor();
		FMaterialExpressionInput& GetOpacity();
		FMaterialExpressionInput& GetNormal();
		FMaterialExpressionInput& GetWorldDisplacement();
		FMaterialExpressionInput& GetRefraction();
		FMaterialExpressionInput& GetAmbientOcclusion();
		FMaterialExpressionInput& GetClearCoat();
		FMaterialExpressionInput& GetClearCoatRoughness();

		UE_API int32                GetExpressionsCount() const;
		UE_API FMaterialExpression* GetExpression(int32 Index);

		UE_API FMaterialExpression* AddMaterialExpression(EMaterialExpressionType ExpressionType);
		template <typename T>
		T* AddMaterialExpression();

	protected:
		FString                  Name;
		FMaterialExpressionInput BaseColor;
		FMaterialExpressionInput Metallic;
		FMaterialExpressionInput Specular;
		FMaterialExpressionInput Roughness;
		FMaterialExpressionInput EmissiveColor;
		FMaterialExpressionInput Opacity;
		FMaterialExpressionInput Normal;
		FMaterialExpressionInput WorldDisplacement;
		FMaterialExpressionInput Refraction;
		FMaterialExpressionInput AmbientOcclusion;
		FMaterialExpressionInput ClearCoat;
		FMaterialExpressionInput ClearCoatRoughness;

		TArray<FMaterialExpression*> Expressions;

		GLTF::FMaterialExpression* ThinTranslucentMaterialOutput;
		GLTF::FMaterialExpression* ClearCoatBottomNormalOutput;

		bool bIsFinal;

        FMD5Hash GLTFMaterialHash;
	};

	inline const FString& FMaterialElement::GetName() const
	{
		return Name;
	}

	inline FMaterialExpressionInput& FMaterialElement::GetBaseColor()
	{
		return BaseColor;
	}
	inline FMaterialExpressionInput& FMaterialElement::GetMetallic()
	{
		return Metallic;
	}
	inline FMaterialExpressionInput& FMaterialElement::GetSpecular()
	{
		return Specular;
	}
	inline FMaterialExpressionInput& FMaterialElement::GetRoughness()
	{
		return Roughness;
	}
	inline FMaterialExpressionInput& FMaterialElement::GetEmissiveColor()
	{
		return EmissiveColor;
	}
	inline FMaterialExpressionInput& FMaterialElement::GetOpacity()
	{
		return Opacity;
	}
	inline FMaterialExpressionInput& FMaterialElement::GetNormal()
	{
		return Normal;
	}
	inline FMaterialExpressionInput& FMaterialElement::GetWorldDisplacement()
	{
		return WorldDisplacement;
	}
	inline FMaterialExpressionInput& FMaterialElement::GetRefraction()
	{
		return Refraction;
	}
	inline FMaterialExpressionInput& FMaterialElement::GetAmbientOcclusion()
	{
		return AmbientOcclusion;
	}
	inline FMaterialExpressionInput& FMaterialElement::GetClearCoat()
	{
		return ClearCoat;
	}
	inline FMaterialExpressionInput& FMaterialElement::GetClearCoatRoughness()
	{
		return ClearCoatRoughness;
	}

	template <>
	inline FMaterialExpressionScalar* FMaterialElement::AddMaterialExpression<FMaterialExpressionScalar>()
	{
		return static_cast<FMaterialExpressionScalar*>(AddMaterialExpression(EMaterialExpressionType(FMaterialExpressionScalar::Type)));
	}

	template <>
	inline FMaterialExpressionColor* FMaterialElement::AddMaterialExpression<FMaterialExpressionColor>()
	{
		return static_cast<FMaterialExpressionColor*>(AddMaterialExpression(EMaterialExpressionType(FMaterialExpressionColor::Type)));
	}

	template <>
	inline FMaterialExpressionTexture* FMaterialElement::AddMaterialExpression<FMaterialExpressionTexture>()
	{
		return static_cast<FMaterialExpressionTexture*>(AddMaterialExpression(EMaterialExpressionType(FMaterialExpressionTexture::Type)));
	}

	template <>
	inline FMaterialExpressionTextureCoordinate* FMaterialElement::AddMaterialExpression<FMaterialExpressionTextureCoordinate>()
	{
		return static_cast<FMaterialExpressionTextureCoordinate*>(
		    AddMaterialExpression(EMaterialExpressionType(FMaterialExpressionTextureCoordinate::Type)));
	}

	template <>
	inline FMaterialExpressionFunctionCall* FMaterialElement::AddMaterialExpression<FMaterialExpressionFunctionCall>()
	{
		return static_cast<FMaterialExpressionFunctionCall*>(AddMaterialExpression(EMaterialExpressionType(FMaterialExpressionFunctionCall::Type)));
	}

	template <>
	inline FMaterialExpressionGeneric* FMaterialElement::AddMaterialExpression<FMaterialExpressionGeneric>()
	{
		return static_cast<FMaterialExpressionGeneric*>(AddMaterialExpression(EMaterialExpressionType(FMaterialExpressionGeneric::Type)));
	}

}  // namespace GLTF

#undef UE_API
