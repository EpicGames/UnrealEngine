// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GLTFLogger.h"

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#define UE_API GLTFCORE_API

namespace GLTF
{
	struct FAsset;
	struct FTexture;
	class FMaterialElement;
	class ITextureElement;
	class FMaterialFactoryImpl;

	enum class ETextureMode
	{
		Color,
		Grayscale,
		Normal
	};

	class ITextureFactory
	{
	public:
		virtual ~ITextureFactory() = default;

		virtual ITextureElement* CreateTexture(const GLTF::FTexture& Texture, UObject* ParentPackage, EObjectFlags Flags,
		                                       GLTF::ETextureMode TextureMode) = 0;
		virtual void             CleanUp()                                     = 0;
	};

	class IMaterialElementFactory
	{
	public:
		virtual ~IMaterialElementFactory() = default;

		virtual FMaterialElement* CreateMaterial(const TCHAR* Name, UObject* ParentPackage, EObjectFlags Flags) = 0;
	};

	class FMaterialFactory
	{
	public:
		UE_API FMaterialFactory(IMaterialElementFactory* MaterialElementFactory, ITextureFactory* TextureFactory);
		UE_API ~FMaterialFactory();

		UE_API const TArray<FMaterialElement*>& CreateMaterials(const GLTF::FAsset& Asset, UObject* ParentPackage, EObjectFlags Flags);

		UE_API const TArray<FLogMessage>&       GetLogMessages() const;
		UE_API const TArray<FMaterialElement*>& GetMaterials() const;

		UE_API IMaterialElementFactory& GetMaterialElementFactory();
		UE_API ITextureFactory&         GetTextureFactory();

		UE_API void CleanUp();

	private:
		TUniquePtr<FMaterialFactoryImpl> Impl;
	};

}  // namespace GLTF

#undef UE_API
