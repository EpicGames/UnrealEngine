// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GLTFAccessor.h"
#include "CoreMinimal.h"

class FMD5;
struct FMD5Hash;

namespace GLTF
{
	// for storing triangle indices as a unit
	struct FTriangle
	{
		uint32 A, B, C;

		FTriangle()
		    : A(0)
		    , B(0)
		    , C(0)
		{
		}
	};

	struct FJointInfluence
	{
		FVector4 Weight;
		uint16   ID[4];

		FJointInfluence(const FVector4& InWeight)
		    : Weight(InWeight)
		    , ID {0, 0, 0, 0}
		{
		}
	};

	struct FVariantMapping
	{
		int32 MaterialIndex;
		TArray<int32> VariantIndices;
	};

	struct FAttributeAccessors
	{
		//Expects elements in the exact same order as the EMeshAttributeType.
		GLTFCORE_API FAttributeAccessors(const TArray<TPair<EMeshAttributeType, FAccessor&>>& InAttributeAccessors, bool bInMorphTarget);

	protected:
		GLTFCORE_API bool HasAttributeAccessor(const EMeshAttributeType& Type) const;
		GLTFCORE_API const FAccessor& GetAttributeAccessor(const EMeshAttributeType& Type) const;
		
		GLTFCORE_API bool AreAttributeAccessorsValid(bool& bHasData, uint32 ExpectedElementCount) const;

		GLTFCORE_API void HashAttributes(FMD5& MD5) const;

	private:
		TArray<TPair<EMeshAttributeType, FAccessor&>> AttributeAccessors;
		bool bMorphTarget;
	};

	struct FMorphTarget : FAttributeAccessors
	{
		GLTFCORE_API FMorphTarget(const TArray<TPair<EMeshAttributeType, FAccessor&>>& InAttributeAccessors /*Displacements and Deltas*/);

		GLTFCORE_API bool IsValid(uint32 ExpectedElementCount) const;
		GLTFCORE_API FMD5Hash GetHash() const;

		bool HasPositionDisplacements() const { return HasAttributeAccessor(EMeshAttributeType::POSITION); };
		GLTFCORE_API void GetPositionDisplacements(TArray<FVector3f>& Buffer) const;
		GLTFCORE_API int32 GetNumberOfPositionDisplacements() const;

		bool HasNormalDisplacements() const { return HasAttributeAccessor(EMeshAttributeType::NORMAL); };
		GLTFCORE_API void GetNormalDisplacements(TArray<FVector3f>& Buffer) const;
		GLTFCORE_API int32 GetNumberOfNormalDisplacements() const;

		bool HasTangentDisplacements() const { return HasAttributeAccessor(EMeshAttributeType::TANGENT); };
		GLTFCORE_API void GetTangentDisplacements(TArray<FVector4f>& Buffer) const;
		GLTFCORE_API int32 GetNumberOfTangentDisplacements() const;

		GLTFCORE_API bool HasTexCoordDisplacements(uint32 Index) const;
		GLTFCORE_API void GetTexCoordDisplacements(uint32 Index, TArray<FVector2f>& Buffer) const;
		GLTFCORE_API int32 GetNumberOfTexCoordDisplacements(uint32 Index) const;

		bool HasColorDeltas() const { return HasAttributeAccessor(EMeshAttributeType::COLOR_0); };
		GLTFCORE_API void GetColorDeltas(TArray<FVector4f>& Buffer) const;
		GLTFCORE_API int32 GetNumberOfColorDeltas() const;
	};

	struct FPrimitive : FAttributeAccessors
	{
		enum class EMode
		{
			// valid but unsupported
			Points    = 0,
			Lines     = 1,
			LineLoop  = 2,
			LineStrip = 3,
			// initially supported
			Triangles = 4,
			TriangleStrip = 5,
			TriangleFan   = 6,

			//
			Unknown = 7
		};

		static GLTFCORE_API const TArray<EMode> SupportedModes;
		static GLTFCORE_API FString ToString(const EMode& Mode);

		const EMode Mode;
		const int32 MaterialIndex;
		TArray<FVariantMapping>	VariantMappings;

		TMap<FString, FString> Extras;

		GLTFCORE_API FPrimitive(EMode InMode, int32 InMaterial, const FAccessor& InIndices, const TArray<TPair<EMeshAttributeType, FAccessor&>>& InAttributeAccessors);

		GLTFCORE_API void GenerateIsValidCache();
		GLTFCORE_API bool IsValid() const;
		GLTFCORE_API FMD5Hash GetHash() const;

		bool HasPositions() const { return HasAttributeAccessor(EMeshAttributeType::POSITION); }
		bool HasNormals() const { return HasAttributeAccessor(EMeshAttributeType::NORMAL); }
		bool HasTangents() const { return HasAttributeAccessor(EMeshAttributeType::TANGENT); }
		GLTFCORE_API bool HasTexCoords(uint32 Index) const;
		bool HasColors() const { return HasAttributeAccessor(EMeshAttributeType::COLOR_0); }
		bool HasJointWeights() const { return HasAttributeAccessor(EMeshAttributeType::JOINTS_0) && HasAttributeAccessor(EMeshAttributeType::WEIGHTS_0); }

		GLTFCORE_API void GetPositions(TArray<FVector3f>& Buffer) const;
		GLTFCORE_API void GetNormals(TArray<FVector3f>& Buffer) const;
		GLTFCORE_API void GetTangents(TArray<FVector4f>& Buffer) const;
		GLTFCORE_API void GetTexCoords(uint32 Index, TArray<FVector2f>& Buffer) const;
		GLTFCORE_API void GetColors(TArray<FVector4f>& Buffer) const;
		GLTFCORE_API void GetJointInfluences(TArray<FJointInfluence>& Buffer) const;

		GLTFCORE_API FTriangle TriangleVerts(uint32 T) const;
		GLTFCORE_API void      GetTriangleIndices(TArray<uint32>& Buffer) const;

		GLTFCORE_API uint32 VertexCount() const;
		GLTFCORE_API uint32 TriangleCount() const;

		//
		TArray<FMorphTarget> MorphTargets;

		//
		uint32 GetIndicesAccessorIndex() { return Indices.AccessorIndex; }
		uint32 GetAttributeAccessorIndex(const EMeshAttributeType& Type) { return GetAttributeAccessor(Type).AccessorIndex; }

	private:
		bool IsValidPrivate() const;

		// index buffer
		const FAccessor& Indices;

		//Validity cache:
		TOptional<bool> bIsValidCache;
	};


	struct FMesh
	{
		FString				Name;
		TArray<FPrimitive>	Primitives;

		TArray<float>		MorphTargetWeights;
		TArray<FString>		MorphTargetNames;

		TMap<FString, FString> Extras;

		FString				UniqueId; //will be generated in FAsset::GenerateNames
	
		bool HasNormals() const;
		bool HasTangents() const;
		bool HasTexCoords(uint32 Index) const;
		bool HasColors() const;
		bool HasJointWeights() const;

		void GenerateIsValidCache(bool GenerateIsValidCacheForPrimitives = true);
		bool IsValid() const;
		GLTFCORE_API FMD5Hash GetHash() const;

		GLTFCORE_API int32 NumberOfMorphTargetsPerPrimitive() const;

	private:
		bool IsValidPrivate() const;
		//Validity cache:
		TOptional<bool> bIsValidCache;
	};

	//

	inline bool FMesh::HasNormals() const
	{
		return Primitives.FindByPredicate([](const FPrimitive& Prim) { return Prim.HasNormals(); }) != nullptr;
	}

	inline bool FMesh::HasTangents() const
	{
		return Primitives.FindByPredicate([](const FPrimitive& Prim) { return Prim.HasTangents(); }) != nullptr;
	}

	inline bool FMesh::HasTexCoords(uint32 Index) const
	{
		return Primitives.FindByPredicate([Index](const FPrimitive& Prim) { return Prim.HasTexCoords(Index); }) != nullptr;
	}

	inline bool FMesh::HasColors() const
	{
		return Primitives.FindByPredicate([](const FPrimitive& Prim) { return Prim.HasColors(); }) != nullptr;
	}

	inline bool FMesh::HasJointWeights() const
	{
		const bool Result = Primitives.FindByPredicate([](const auto& Prim) { return Prim.HasJointWeights(); }) != nullptr;

		if (Result)
		{
			// According to spec, *all* primitives of a skinned mesh must have joint weights.
			int32 Count = 0;
			for (const FPrimitive& Prim : Primitives)
			{
				Count += Prim.HasJointWeights();
			}
			ensure(Primitives.Num() == Count);
		}

		return Result;
	}

	inline void FMesh::GenerateIsValidCache(bool GenerateIsValidCacheForPrimitives)
	{
		if (GenerateIsValidCacheForPrimitives)
		{
			for (FPrimitive& Primitive : Primitives)
			{
				Primitive.GenerateIsValidCache();
			}
		}

		bIsValidCache = IsValidPrivate();
	}
	inline bool FMesh::IsValid() const
	{
		if (bIsValidCache.IsSet())
		{
			return bIsValidCache.GetValue();
		}
		else
		{
			return IsValidPrivate();
		}
	}

	inline bool FMesh::IsValidPrivate() const
	{
		//Validate Primitives:
		bool bIsValid = Primitives.FindByPredicate([](const FPrimitive& Prim) { return !Prim.IsValid(); }) == nullptr;

		//if MorphTargetNames are not set, but mesh has morph targets (which is likely), this will generate isValid false;
		//IsValidCache will have to be (re-)generated post FAsset::GenerateNames (which is called at the end of GLTF::FReader) to overcome this issue.

		//Validate Morph Target Names and Morph Target Weights:
		if ((MorphTargetNames.Num() > 0) && (MorphTargetWeights.Num() > 0))
		{
			if (MorphTargetNames.Num() != MorphTargetWeights.Num())
			{
				bIsValid = false;
			}
		}

		//Validate Morph Target (and Morph Target Names) Counts:
		int32 MorphTargetCounter = MorphTargetNames.Num();

		for (const FPrimitive& Primitive : Primitives)
		{
			if (MorphTargetCounter != Primitive.MorphTargets.Num())
			{
				bIsValid = false;
				break;
			}
		}

		return bIsValid;
	}

}  // namespace GLTF
