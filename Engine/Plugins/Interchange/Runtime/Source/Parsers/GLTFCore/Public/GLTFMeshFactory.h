// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshTypes.h"
#include "GLTFLogger.h"

#define UE_API GLTFCORE_API

struct FVertexID;

struct FMeshDescription;

namespace GLTF
{
	struct FMesh;
	class FMeshFactoryImpl;

	class FMeshFactory
	{
	public:
		UE_API FMeshFactory();
		UE_API ~FMeshFactory();

		using FIndexVertexIdMap = TMap<int32, FVertexID>;

		UE_API void FillMeshDescription(const GLTF::FMesh &Mesh, const FTransform& MeshGlobalTransform /*In UE Space*/, FMeshDescription* MeshDescription, const TArray<float>& MorphTargetWeights = TArray<float>());

		UE_API float GetUniformScale() const;
		UE_API void  SetUniformScale(float Scale);

		UE_API const TArray<FLogMessage>&  GetLogMessages() const;

		UE_API void SetReserveSize(uint32 Size);

		UE_API TArray<FMeshFactory::FIndexVertexIdMap>& GetPositionIndexToVertexIdPerPrim() const;

		UE_API void CleanUp();

	private:
		TUniquePtr<FMeshFactoryImpl> Impl;
	};
}

#undef UE_API
