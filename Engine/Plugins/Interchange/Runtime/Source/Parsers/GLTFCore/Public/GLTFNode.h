// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GLTFMaterial.h"
#include "GLTFMesh.h"
#include "GLTFTexture.h"

namespace GLTF { struct FAccessor; }

namespace GLTF
{
	struct FNode
	{
		struct FLightInstanceIES
		{
			int32 Index = INDEX_NONE; // index into FAsset::LightsIES
			
			float IntensityMultipler = 1.f;
			bool bHasIntensityMultiplier = false; //IES brightness scale
			
			FVector Color = FVector(1.f);
			bool bHasColor = false;

			FString GetHash() const
			{
				
				if (!bHasIntensityMultiplier && !bHasColor)
				{
					return TEXT("");
				}

				FMD5 MD5;
				
				MD5.Update(reinterpret_cast<const uint8*>(&Index), sizeof(Index));
				MD5.Update(reinterpret_cast<const uint8*>(&IntensityMultipler), sizeof(IntensityMultipler));
				MD5.Update(reinterpret_cast<const uint8*>(&Color), sizeof(Color));

				FMD5Hash Hash;
				Hash.Set(MD5);

				FString HashString = LexToString(Hash);

				return HashString;
			}
		};

		enum class EType
		{
			None,
			Transform,
			Joint,
			Mesh,
			MeshSkinned,
			Camera,
			Light
		};

		FString       Name;
		FTransform    Transform;
		TArray<int32> Children;  // each is an index into FAsset::Nodes
		EType         Type;

		int32 MeshIndex;  // index into FAsset::Meshes

		// Skindex is the skin used by the mesh at this node.
		// It's not the joints belonging *to* a skin
		int32 Skindex;      // index into FAsset::Skins
		int32 CameraIndex;  // index into FAsset::Cameras
		int32 LightIndex;   // index into FAsset::Lights
		FLightInstanceIES LightIES;

		int32 Index;		// index of FNode in GLTFAsset.Nodes array.
		int32 ParentIndex;
		int32 ParentJointIndex; //glTF marks the Nodes in Skins.Joints as a joint, however there are certain indirections that still make a node into a joint. (such as skin.joints having gaps/skips in the node hierarchy)
		int32 DirectParentJointIndex; //With this variable we are tracking the original Parent Joint Index. Joint index that was actually marked by the glTF file without any indirections, aka index is part of the Skin.Joints list.
		int32 RootJointIndex; //only valid if node is of Joint Type

		TArray<float> MorphTargetWeights; //for the instantiated mesh with morph targets.

		TMap<int, FMatrix> SkinIndexToGlobalInverseBindMatrix;
		TMap<int, FTransform> SkinIndexToLocalBindTransform;
														//bind pose would be CurrentNode.GlobalInverseBindTransform.Inverse() * ParentNode.GlobalInverseBindTransform
		bool bHasLocalBindPose;
		FTransform LocalBindPose;	// First Skin that's using the joint will fill the LocalBindPose.
									//	Edge case Scenario which is currently not supported:
									//	Where multiple skins use the same Joint. Currently expected bad outcome if the different skins have different inversebindmatrices on the joint.

		TMap<FString, FString> Extras;

		FString UniqueId;

		FNode()
		    : Type(EType::None)
		    , MeshIndex(INDEX_NONE)
		    , Skindex(INDEX_NONE)
		    , CameraIndex(INDEX_NONE)
		    , LightIndex(INDEX_NONE)
			, Index(INDEX_NONE)
			, ParentIndex(INDEX_NONE)
			, ParentJointIndex(INDEX_NONE)
			, DirectParentJointIndex(INDEX_NONE)
			, RootJointIndex(INDEX_NONE)
			, bHasLocalBindPose(false)
		{
		}
	};

	struct FCamera
	{
		struct FPerspective
		{
			// vertical field of view in radians
			float Fov;
			// aspect ratio of the field of view
			float AspectRatio;
		};
		struct FOrthographic
		{
			// horizontal magnification of the view
			float XMagnification;
			// vertical magnification of the view
			float YMagnification;
		};

		const FNode& Node;
		FString      Name;
		union {
			FOrthographic Orthographic;
			FPerspective  Perspective;
		};
		float ZNear;
		float ZFar;
		bool  bIsPerspective;

		TMap<FString, FString> Extras;

		FString UniqueId; //will be generated in FAsset::GenerateNames

		FCamera(const FNode& Node)
		    : Node(Node)
		    , ZNear(0.f)
			, ZFar(100.f)
		    , bIsPerspective(true)
		{
			Perspective.Fov         = 0.f;
			Perspective.AspectRatio = 1.f;
		}
	};

	struct FLightIES
	{
		int32 Index; //serves as idintifer as well.
		
		FString URI;
		FString FilePath;
		
		int32 BufferViewIndex = INDEX_NONE;
		uint32 DataByteLength;
		const uint8* Data;
		
		FString MimeType;
		
		FString Name;
		FString UniqueId; //will be generated in FAsset::GenerateNames
	};

	struct FLight
	{
		enum class EType
		{
			Directional,
			Point,
			Spot
		};

		struct FSpot
		{
			float InnerConeAngle;
			float OuterConeAngle;

			FSpot()
			    : InnerConeAngle(0.f)
			    , OuterConeAngle(PI / 4.f)
			{
			}
		};

		const FNode* Node;
		FString      Name;
		EType        Type;
		FVector      Color;
		float        Intensity;
		// Must be > 0. When undefined, range is assumed to be infinite.
		float Range;
		FSpot Spot;

		FString UniqueId; //will be generated in FAsset::GenerateNames

		FLight(const FNode* Node)
		    : Node(Node)
			, Type(EType::Point)
		    , Color(1.f)
		    , Intensity(1.f)
		    , Range(1e+20f)
		{
		}
	};

	struct FSkinInfo
	{
		const FAccessor& InverseBindMatrices;
		FString          Name;
		TArray<int32>    Joints;    // each is an index into FAsset::Nodes
		int32            Skeleton = INDEX_NONE;  // root node, index into FAsset::Nodes

		TMap<FString, FString> Extras;

		bool bUsed = false;

		FString          UniqueId; //will be generated in FAsset::GenerateNames

		FSkinInfo(const FAccessor& InverseBindMatrices)
		    : InverseBindMatrices(InverseBindMatrices)
		{
		}
	};

}  // namespace GLTF
