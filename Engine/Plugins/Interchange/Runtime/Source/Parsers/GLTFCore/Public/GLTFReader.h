// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GLTFLogger.h"

#include "CoreMinimal.h"

#define UE_API GLTFCORE_API

class FJsonObject;

namespace GLTF
{
	struct FAsset;
	struct FMesh;
	struct FPrimitive;
	struct FTextureMap;
	struct FMaterial;
	class FBinaryFileReader;
	class FExtensionsHandler;

	class FFileReader : public FBaseLogger
	{
	public:
		UE_API FFileReader();
		UE_API ~FFileReader();

		/**
		 * Loads the contents of a glTF file into the given asset.
		 *
		 * @param InFilePath - disk file path
		 * @param bInLoadImageData - if false then doesn't loads the data field of FImage
		 * @param bInLoadMetadata - load extra asset metadata
		 * images from a disk path, images from data sources will always be loaded.
		 * @param OutAsset - asset data destination
		 */
		UE_API void ReadFile(const FString& InFilePath, bool bInLoadImageData, bool bInLoadMetadata, GLTF::FAsset& OutAsset);

	private:
		void LoadMetadata(GLTF::FAsset& Asset);
		void ImportAsset(const FString& FilePath, bool bInLoadImageData, GLTF::FAsset& Asset);
		void AllocateExtraData(const FString& InResourcesPath, bool bInLoadImageData, TArray64<uint8>& OutExtraData);

		void SetupBuffer(const FJsonObject& Object, const FString& Path);
		void SetupBufferView(const FJsonObject& Object) const;
		void SetupAccessor(const FJsonObject& Object) const;
		void SetupMorphTarget(const FJsonObject& Object, GLTF::FPrimitive& Primitive, const bool bMeshQuantized) const;
		void SetupPrimitive(const FJsonObject& Object, GLTF::FMesh& Mesh, const bool bMeshQuantized, const uint32& PrimitiveIndex) const;
		void SetupMesh(const FJsonObject& Object, const bool bMeshQuantized) const;

		void SetupScene(const FJsonObject& Object) const;
		void SetupNode(const FJsonObject& Object) const;
		void SetupCamera(const FJsonObject& Object) const;
		void SetupAnimation(const FJsonObject& Object) const;
		void SetupSkin(const FJsonObject& Object) const;

		void SetupImage(const FJsonObject& Object, const FString& Path, bool bInLoadImageData);
		void SetupSampler(const FJsonObject& Object) const;
		void SetupTexture(const FJsonObject& Object) const;
		void SetupMaterial(const FJsonObject& Object) const;

		template <typename SetupFunc>
		bool SetupObjects(uint32 ObjectCount, const TCHAR* FieldName, SetupFunc Func) const;
		void SetupUsedSkins() const;
		void SetupNodesType() const;

		void GenerateInverseBindPosesPerSkinIndices() const; //Per node
		void GenerateLocalBindPosesPerSkinIndices() const; //Per node
		void SetLocalBindPosesForJoints() const; //Per node

		void BuildParentJointIndices() const;
		void BuildParentIndices(int32 ParentNodeIndex, int32 DirectParentJointIndex, int32 CurrentNodeIndex) const;
		int32 FindRootJointIndex(int32 CurrentNodeIndex) const;

		//Sets the Node Types to Joint for Nodes that were not marked Joint in Skins, however they have descendant and ancestor both in Skins.
		void FixNodeTypesForIndirectJoints() const;
		void BuildRootJoints() const;

		void ResolveConflictingTextures();
	private:
		bool CheckForErrors(int32 StartIndex = 0) const;

		uint32 BufferCount;
		uint32 BufferViewCount;
		uint32 ImageCount;

		TSharedPtr<FJsonObject>  JsonRoot;
		FString                        JsonBuffer;
		TUniquePtr<FBinaryFileReader>  BinaryReader;
		TUniquePtr<FExtensionsHandler> ExtensionsHandler;

		// Each mesh has its own primitives array.
		// Each primitive refers to the asset's global accessors array.
		GLTF::FAsset* Asset;

		// Current offset for extra data buffer.
		uint8* CurrentBufferOffset;
	};

}  // namespace GLTF

#undef UE_API
