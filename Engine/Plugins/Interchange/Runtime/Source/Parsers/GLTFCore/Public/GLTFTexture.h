// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FScriptContainerElement;

namespace GLTF
{
	struct FImage
	{
		enum class EFormat
		{
			Unknown,
			PNG,
			JPEG
		};

		FString Name;
		FString URI;
		EFormat Format;
		FString FilePath;

		// Image data is kept encoded in Format, to be decoded when needed by Unreal.
		uint32       DataByteLength;
		const uint8* Data;

		TMap<FString, FString> Extras;

		FString      UniqueId; //will be generated in FAsset::GenerateNames

		FImage()
		    : DataByteLength(0)
		    , Data(nullptr)
		{
		}
	};

	struct FSampler
	{
		enum class EFilter
		{
			// valid for Min & Mag
			Nearest = 9728,
			Linear  = 9729,
			// valid for Min only
			NearestMipmapNearest = 9984,
			LinearMipmapNearest  = 9985,
			NearestMipmapLinear  = 9986,
			LinearMipmapLinear   = 9987
		};

		enum class EWrap
		{
			Repeat         = 10497,
			MirroredRepeat = 33648,
			ClampToEdge    = 33071
		};

		static GLTFCORE_API const FSampler DefaultSampler;

		EFilter MinFilter;
		EFilter MagFilter;

		EWrap WrapS;
		EWrap WrapT;

		FSampler()
		    // Spec defines no default min/mag filter.
		    : MinFilter(EFilter::Linear)
		    , MagFilter(EFilter::Linear)
		    , WrapS(EWrap::Repeat)
		    , WrapT(EWrap::Repeat)
		{
		}
	};

	struct FTexture
	{
		const FImage&   Source;
		const FSampler& Sampler;
		FString         Name;

		TMap<FString, FString> Extras;

		FString			UniqueId; //will be generated in FAsset::GenerateNames

		FTexture(const FString& InName, const FImage& InSource, const FSampler& InSampler)
		    : Source(InSource)
		    , Sampler(InSampler)
		    , Name(InName)
		{
		}
	};

}  // namespace GLTF
