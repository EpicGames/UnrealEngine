// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class InterchangePipelines : ModuleRules
	{
		public InterchangePipelines(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"InterchangeCore",
					"InterchangeCommon",
					"InterchangeEngine",
					"InterchangeFactoryNodes",
					"InterchangeNodes",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ApplicationCore",
					"CinematicCamera",
					"DeveloperSettings",
					"Foliage",
					"GeometryCache",
					"GLTFCore",
					"HairStrandsCore",
					"ImageCore",
					"InputCore",
					"InterchangeImport",
					"LevelSequence",
					"Slate",
					"SlateCore",
					"VariantManagerContent",
				}
			);

			if (Target.Type == TargetType.Editor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"MainFrame",
						"PhysicsUtilities",
						"TextureUtilitiesCommon",
						"UnrealEd",
					}
				);
			}

			bDisableAutoRTFMInstrumentation = true;
		}
	}
}
