// Copyright Epic Games, Inc. All Rights Reserved. 
#include "InterchangeGenericAnimationPipeline.h"

#include "Animation/AnimationSettings.h"
#include "Animation/AnimSequence.h"
#include "CoreMinimal.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "InterchangeGenericAssetsPipeline.h"
#include "InterchangeGenericMeshPipeline.h"
#include "InterchangeLevelSequenceFactoryNode.h"
#include "InterchangeAnimationTrackSetNode.h"
#include "InterchangeAnimSequenceFactoryNode.h"
#include "InterchangeHelper.h"
#include "InterchangeMeshNode.h"
#include "InterchangePipelineLog.h"
#include "InterchangePipelineObjectVersion.h"
#include "InterchangeJointNode.h"
#include "InterchangeSkeletalMeshFactoryNode.h"
#include "InterchangeSkeletalMeshLodDataNode.h"
#include "InterchangeSkeletonFactoryNode.h"
#include "InterchangeSkeletonHelper.h"
#include "InterchangeSourceData.h"
#include "LevelSequence.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeSourceNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGenericAnimationPipeline)

namespace UE::Interchange::Private
{
	const FString ConvertedFromRigidAnimationPrefixIdentifier = TEXT("\\SkeletalAnimation\\ConvertedFromRigidAnimation\\");

	bool IsTranslatedDataContainOnlyJointAnimation(const UInterchangeBaseNodeContainer* InBaseNodeContainer, bool bConvertStaticsWithMorphTargetsToSkeletals, bool bConvertStaticsInBoneHierarchyToSkeletals)
	{
		//Its valid to call GetMeshesInformationFromTranslatedData with a null container
		if (!InBaseNodeContainer)
		{
			return false;
		}
		bool bContainOnlyJointAnimation = false;
		bool bContainJointAnimation = false;
		InBaseNodeContainer->BreakableIterateNodesOfType<UInterchangeSkeletalAnimationTrackNode>([&bContainJointAnimation](const FString& NodeUid, UInterchangeSkeletalAnimationTrackNode* AnimationNode)
			{
				bContainJointAnimation = true;
				return bContainJointAnimation;
			});
		if (bContainJointAnimation)
		{
			bool bContainSkinnedMeshNode = false;
			InBaseNodeContainer->BreakableIterateNodesOfType<UInterchangeMeshNode>([&bContainSkinnedMeshNode, &bConvertStaticsWithMorphTargetsToSkeletals, &bConvertStaticsInBoneHierarchyToSkeletals](const FString& NodeUid, UInterchangeMeshNode* MeshNode)
				{
					if (!MeshNode->IsMorphTarget())
					{
						if (MeshNode->IsSkinnedMesh() || bConvertStaticsInBoneHierarchyToSkeletals)
						{
							bContainSkinnedMeshNode = true;
						}
					}
					else
					{
						if (bConvertStaticsWithMorphTargetsToSkeletals)
						{
							bContainSkinnedMeshNode = true;
						}
					}
					return bContainSkinnedMeshNode;
				});
			bContainOnlyJointAnimation = !bContainSkinnedMeshNode;
		}

		return bContainOnlyJointAnimation;
	}

	//Check Compatibility: (as in does the animation target the skeleton at all)
	bool DoesSkeletalAnimationTargetSkeleton(const UInterchangeBaseNodeContainer* BaseNodeContainer, const UInterchangeSkeletalAnimationTrackNode& TrackNode, USkeleton* Skeleton)
	{
		//Check if the animation is targeting any of the joints on said skeleton:
		TMap<FString, FString> SceneNodeAnimationPayloadKeyUids;
		TMap<FString, uint8> SceneNodeAnimationPayloadKeyTypes;
		TrackNode.GetSceneNodeAnimationPayloadKeys(SceneNodeAnimationPayloadKeyUids, SceneNodeAnimationPayloadKeyTypes);

		TSet<FString> SceneNodeUIDs;
		SceneNodeAnimationPayloadKeyUids.GetKeys(SceneNodeUIDs);
		const FReferenceSkeleton& SkeletonRef = Skeleton->GetReferenceSkeleton();

		//Check compatibility by comparing bones to an eventual SceneNode already in the container.
		for (const FString& SceneNodeUID : SceneNodeUIDs)
		{
			if (const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(SceneNodeUID)))
			{
				FString DisplayName = SceneNode->GetDisplayLabel();
				int32 BoneIndex = SkeletonRef.FindBoneIndex(*DisplayName);
				if (BoneIndex != INDEX_NONE)
				{
					//We only need one bone to be found to consider this skeleton compatible
					return true;
				}
			}
		}

		//If no SceneNode was found, we can leverage morph targets name to check for compatibility.
		USkeletalMesh* PreviewMesh = Skeleton->GetPreviewMesh();
		if (PreviewMesh)
		{
			const TArray<TObjectPtr<UMorphTarget>>& SkeletalMeshMorphTargets = PreviewMesh->GetMorphTargets();
			TSet<FString> MorphTargetNames;
			for (const TObjectPtr<UMorphTarget>& MorphTarget : SkeletalMeshMorphTargets)
			{
				if (MorphTarget)
				{
					MorphTargetNames.Add(MorphTarget->GetName());
				}
			}

			if (MorphTargetNames.Num() > 0)
			{
				TMap<FString, FString> MorphTargetNodeAnimationPayloadKeyUids;
				TMap<FString, uint8> MorphTargetNodeAnimationPayloadKeyTypes;
				TrackNode.GetMorphTargetNodeAnimationPayloadKeys(MorphTargetNodeAnimationPayloadKeyUids, MorphTargetNodeAnimationPayloadKeyTypes);

				TSet<FString> MorphTargetNodeUIDs;
				MorphTargetNodeAnimationPayloadKeyUids.GetKeys(MorphTargetNodeUIDs);

				for (const FString& MorphTargetNodeUID : MorphTargetNodeUIDs)
				{
					if (const UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(BaseNodeContainer->GetNode(MorphTargetNodeUID)))
					{
						FString MorphTargetName = MeshNode->GetDisplayLabel();
						if (MorphTargetNames.Contains(MorphTargetName))
						{
							//If at least one morph target name matches the animated morphs targets of this TrackNode then we consider the skeleton compatible.
							return true;
						}
					}
				}
			}
		}

		//Skeleton not compatible with the TrackNode.
		return false;
	}
}

void UInterchangeGenericAnimationPipeline::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainInterchangePipelineObjectVersion::GUID);

	Super::Serialize(Ar);

	//If we load an old pipeline, we must transfer the value of the bAddCurveMetadataToSkeleton hold by 
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FFortniteMainInterchangePipelineObjectVersion::GUID) < FFortniteMainInterchangePipelineObjectVersion::InterchangeAddCurveMetadataToSkeletonPropertyMove)
		{
			if (ensure(CommonSkeletalMeshesAndAnimationsProperties.IsValid()))
			{
				CommonSkeletalMeshesAndAnimationsProperties->bAddCurveMetadataToSkeleton = bAddCurveMetadataToSkeleton_DEPRECATED;
			}
		}

		if (Ar.CustomVer(FFortniteMainInterchangePipelineObjectVersion::GUID) < FFortniteMainInterchangePipelineObjectVersion::InterchangeAddFrameAlignmentGenericAnimationPipelineConfig)
		{
			if (ensure(CommonSkeletalMeshesAndAnimationsProperties.IsValid()))
			{
				FrameAlignment = bSnapToClosestFrameBoundary_DEPRECATED ? EInterchangeFrameAlignment::SnapToClosest : EInterchangeFrameAlignment::None;
			}
		}
	}

	
}

FString UInterchangeGenericAnimationPipeline::GetPipelineCategory(UClass* AssetClass)
{
	return TEXT("Animations");
}

#if WITH_EDITOR
bool UInterchangeGenericAnimationPipeline::CanEditChange(const FProperty* InProperty) const
{
	// If other logic prevents editing, we want to respect that
	const bool ParentVal = Super::CanEditChange(InProperty);

	// Can we edit flower color?
	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UInterchangeGenericAnimationPipeline, FrameImportRange))
	{
		return ParentVal && bImportAnimations && bImportBoneTracks && AnimationRange == EInterchangeAnimationRange::SetRange;
	}
	return ParentVal;
}
#endif

void UInterchangeGenericAnimationPipeline::AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams)
{
	Super::AdjustSettingsForContext(ContextParams);

#if WITH_EDITOR
	check(CommonSkeletalMeshesAndAnimationsProperties.IsValid());
	
	bSceneImport = ContextParams.ContextType == EInterchangePipelineContext::SceneImport
				|| ContextParams.ContextType == EInterchangePipelineContext::SceneReimport;

	if (ContextParams.ContextType == EInterchangePipelineContext::AssetCustomLODImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomLODReimport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetAlternateSkinningImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetAlternateSkinningReimport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomMorphTargetImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomMorphTargetReImport)
	{
		bImportAnimations = false;
		CommonSkeletalMeshesAndAnimationsProperties->Skeleton.Reset();
		CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations = false;
	}
	
	const FString CommonMeshesCategory =  UInterchangeGenericCommonMeshesProperties::GetPipelineCategory(nullptr);
	const FString StaticMeshesCategory = UInterchangeGenericMeshPipeline::GetPipelineCategory(UStaticMesh::StaticClass());
	const FString SkeletalMeshesCategory = UInterchangeGenericMeshPipeline::GetPipelineCategory(USkeletalMesh::StaticClass());
	const FString AnimationCategory = UInterchangeGenericAnimationPipeline::GetPipelineCategory(nullptr);

	TArray<FString> HideCategories;
	if(ContextParams.ContextType == EInterchangePipelineContext::AssetImport)
	{
		if(UE::Interchange::Private::IsTranslatedDataContainOnlyJointAnimation(
			ContextParams.BaseNodeContainer, 
			CommonMeshesProperties->bConvertStaticsWithMorphTargetsToSkeletals,
			CommonMeshesProperties->bConvertStaticsInBoneHierarchyToSkeletals))
		{
			bImportAnimations = true;
			CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations = true;

			HideCategories.Add(StaticMeshesCategory);
			HideCategories.Add(SkeletalMeshesCategory);
			HideCategories.Add(CommonMeshesCategory);
		}
	}

	
	if (ContextParams.ContextType == EInterchangePipelineContext::AssetReimport)
	{
		if (UAnimSequence* AnimSequence = Cast<UAnimSequence>(ContextParams.ReimportAsset))
		{
			//Set the skeleton to the current asset skeleton and re-import only the animation
			CommonSkeletalMeshesAndAnimationsProperties->Skeleton = AnimSequence->GetSkeleton();
			CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations = true;
			bImportAnimations = true;
		}
		else
		{
			HideCategories.Add(AnimationCategory);
		}
	}

	if (CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations)
	{
		bImportAnimations = true;
	}

	if (UInterchangePipelineBase* OuterMostPipeline = GetMostPipelineOuter())
	{
		for (const FString& HideCategoryName : HideCategories)
		{
			HidePropertiesOfCategory(OuterMostPipeline, this, HideCategoryName);
		}
	}
#endif //WITH_EDITOR
}

#if WITH_EDITOR

bool UInterchangeGenericAnimationPipeline::IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UInterchangeGenericAnimationPipeline, bImportAnimations))
	{
		return true;
	}
	return Super::IsPropertyChangeNeedRefresh(PropertyChangedEvent);
}


void UInterchangeGenericAnimationPipeline::GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const
{
	PipelineSupportAssetClasses.Add(UAnimSequence::StaticClass());
	PipelineSupportAssetClasses.Add(ULevelSequence::StaticClass());
}

#endif //WITH_EDITOR

void UInterchangeGenericAnimationPipeline::ImplementUseSourceNameForAssetOptionAnimSequence(bool bUseSourceNameForAsset, const FString& AssetName)
{
	const bool bShouldChangeAssetName = bUseSourceNameForAsset || !AssetName.IsEmpty();

	// Prefix anim sequences with the asset name (mostly used by FBX)
	if (bShouldChangeAssetName && !SourceDatas.IsEmpty())
	{
		FString DisplayLabelName = AssetName.IsEmpty() ? (SourceDatas.Num() ? FPaths::GetBaseFilename(SourceDatas[0]->GetFilename()) : TEXT("No Source File Specified")) : AssetName;
		BaseNodeContainer->IterateNodesOfType<UInterchangeAnimSequenceFactoryNode>(
			[&DisplayLabelName](const FString&, UInterchangeAnimSequenceFactoryNode* AnimSequenceFactoryNode)
			{
				AnimSequenceFactoryNode->SetDisplayLabel(DisplayLabelName + AnimSequenceFactoryNode->GetDisplayLabel());
			});
	}
}


void UInterchangeGenericAnimationPipeline::ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath)
{
	if (!InBaseNodeContainer)
	{
		UE_LOGF(LogInterchangePipeline, Warning, "UInterchangeGenericAnimationPipeline: Cannot execute pre-import pipeline because InBaseNodeContainer is null.");
		return;
	}

	BaseNodeContainer = InBaseNodeContainer;

	if (!CommonSkeletalMeshesAndAnimationsProperties.IsValid())
	{
		return;
	}

	CommonSkeletalMeshesAndAnimationsProperties->SetResultsContainer(Results);

	if (CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations)
	{
		bImportAnimations = true;
	}

	if (!bImportAnimations)
	{
		//Nothing to import
		return;
	}

	TArray<UInterchangeAnimationTrackSetNode*> TrackSetNodes;
	BaseNodeContainer->IterateNodesOfType<UInterchangeAnimationTrackSetNode>([&](const FString& NodeUid, UInterchangeAnimationTrackSetNode* Node)
		{
			TrackSetNodes.Add(Node);
		});


	//Create AnimSequences(UInterchangeSkeletalAnimationTrackNode) for Mesh Instances having MorphTargetCurveWeights:
	{
		TArray<UInterchangeSceneNode*> SceneNodesWithMorphTargetCurveWeights;
		BaseNodeContainer->IterateNodesOfType<UInterchangeSceneNode>([&SceneNodesWithMorphTargetCurveWeights](const FString& NodeUid, UInterchangeSceneNode* SceneNode)
			{
				TMap<FString, float> MorphTargetCurveWeights;
				SceneNode->GetMorphTargetCurveWeights(MorphTargetCurveWeights);

				bool CreateAnimSequence = false;
				for (const TTuple<FString, float>& MorphTargetCurveWeight : MorphTargetCurveWeights)
				{
					if (MorphTargetCurveWeight.Value != 0)
					{
						CreateAnimSequence = true;
						break;
					}
				}

				if (CreateAnimSequence)
				{
					SceneNodesWithMorphTargetCurveWeights.Add(SceneNode);
				}
			});

		for (UInterchangeSceneNode* SceneNode : SceneNodesWithMorphTargetCurveWeights)
		{
			UInterchangeSkeletalAnimationTrackNode* SkeletalAnimationNode = NewObject< UInterchangeSkeletalAnimationTrackNode >(BaseNodeContainer);
			FString SkeletalAnimationNodeUid = UE::Interchange::Private::ConvertedFromRigidAnimationPrefixIdentifier + SceneNode->GetUniqueID();
			BaseNodeContainer->SetupNode(SkeletalAnimationNode, SkeletalAnimationNodeUid, SceneNode->GetDisplayLabel(), EInterchangeNodeContainerType::TranslatedAsset);

			SkeletalAnimationNode->SetCustomAnimationSampleRate(30.f);
			SkeletalAnimationNode->SetCustomAnimationStartTime(0);
			SkeletalAnimationNode->SetCustomAnimationStopTime(1.f / 30.f); //we want a single frame

			SkeletalAnimationNode->SetCustomSkeletonNodeUid(SceneNode->GetUniqueID());

			TMap<FString, float> MorphTargetCurveWeights;
			SceneNode->GetMorphTargetCurveWeights(MorphTargetCurveWeights);

			for (const TPair<FString, float>& MorphTargetCurveWeight : MorphTargetCurveWeights)
			{
				FString PayloadUid = MorphTargetCurveWeight.Key + TEXT(":") + LexToString(MorphTargetCurveWeight.Value);

				//add the payload key:
				SkeletalAnimationNode->SetAnimationPayloadKeyForMorphTargetNodeUid(MorphTargetCurveWeight.Key, PayloadUid, EInterchangeAnimationPayLoadType::MORPHTARGETCURVEWEIGHTINSTANCE);
			}

			SceneNode->SetCustomAnimationAssetUidToPlay(SkeletalAnimationNodeUid);
		}
	}

	if (!bSceneImport)
	{
		//Extract any skeleton node use by the skeletal mesh animation track
		TArray<FString> SceneNodesUsedBySkeleton;
		BaseNodeContainer->IterateNodesOfType<UInterchangeSkeletalAnimationTrackNode>([&](const FString& NodeUid, UInterchangeSkeletalAnimationTrackNode* Node)
			{
				if (Node)
				{
					FString SkeletonNodeUid;
					if (Node->GetCustomSkeletonNodeUid(SkeletonNodeUid))
					{
						SceneNodesUsedBySkeleton.Add(SkeletonNodeUid);
						BaseNodeContainer->IterateNodeChildren(SkeletonNodeUid, [&SceneNodesUsedBySkeleton](const UInterchangeBaseNode* Node)
							{
								if (const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(Node))
								{
									SceneNodesUsedBySkeleton.Add(Node->GetUniqueID());
								}
							});
					}
				}
			});

		auto IsTrackOverrideBySkeletalMeshAnimation = [this, &SceneNodesUsedBySkeleton](TArray<FString>& AnimationTrackUids)
			{
				//Skip track node that is using one or more scene node used by any skeletal mesh skeleton.
				bool bSomeTrackNodeUsedBySkeletalMesh = false;
				for (const FString& AnimationTrackUid : AnimationTrackUids)
				{
					if (const UInterchangeTransformAnimationTrackNode* TransformTrackNode = Cast<UInterchangeTransformAnimationTrackNode>(BaseNodeContainer->GetNode(AnimationTrackUid)))
					{
						FString ActorNodeUid;
						if (TransformTrackNode->GetCustomActorDependencyUid(ActorNodeUid))
						{
							if (SceneNodesUsedBySkeleton.Contains(ActorNodeUid))
							{
								bSomeTrackNodeUsedBySkeletalMesh = true;
								break;
							}
						}
					}
				}
				return bSomeTrackNodeUsedBySkeletalMesh;
			};

		bool bAllowSceneRootAsJoint = true;
		if (const UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::GetUniqueInstance(BaseNodeContainer))
		{
			SourceNode->GetCustomAllowSceneRootAsJoint(bAllowSceneRootAsJoint);
		}

		//Support rigid mesh animation animation data  (UAnimSequence for rigid mesh)
		for (UInterchangeAnimationTrackSetNode* TrackSetNode : TrackSetNodes)
		{
			if (!TrackSetNode)
			{
				continue;
			}

			TArray<FString> AnimationTrackUids;
			TrackSetNode->GetCustomAnimationTrackUids(AnimationTrackUids);
			
			if (IsTrackOverrideBySkeletalMeshAnimation(AnimationTrackUids))
			{
				continue;
			}

			UInterchangeSkeletalAnimationTrackNode* SkeletalAnimationNode = NewObject< UInterchangeSkeletalAnimationTrackNode >(BaseNodeContainer);
			FString SkeletalAnimationNodeUid = UE::Interchange::Private::ConvertedFromRigidAnimationPrefixIdentifier + TrackSetNode->GetUniqueID();
			BaseNodeContainer->SetupNode(SkeletalAnimationNode, SkeletalAnimationNodeUid, TrackSetNode->GetDisplayLabel(), EInterchangeNodeContainerType::TranslatedAsset);

			bool bCustomSkeletonNodeUidSet = false;

			
			float CustomFrameRate;
			if (!TrackSetNode->GetCustomFrameRate(CustomFrameRate))
			{
				CustomFrameRate = 30.0f;
			}
			SkeletalAnimationNode->SetCustomAnimationSampleRate(CustomFrameRate);
			SkeletalAnimationNode->SetCustomAnimationStartTime(0);
			//stop time will be calculated once we get the curves from the Translators.
			//However to avoid error reports we set stoptime for 1 subframe.
			SkeletalAnimationNode->SetCustomAnimationStopTime(1./CustomFrameRate);

			for (const FString& AnimationTrackUid : AnimationTrackUids)
			{
				if (const UInterchangeTransformAnimationTrackNode* TransformTrackNode = Cast<UInterchangeTransformAnimationTrackNode>(BaseNodeContainer->GetNode(AnimationTrackUid)))
				{
					FString ActorNodeUid;
					if (TransformTrackNode->GetCustomActorDependencyUid(ActorNodeUid))
					{
						if (const UInterchangeSceneNode* ActorNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(ActorNodeUid)))
						{
							FInterchangeAnimationPayLoadKey AnimationPayLoadKey;
							if (TransformTrackNode->GetCustomAnimationPayloadKey(AnimationPayLoadKey))
							{
								if (!bCustomSkeletonNodeUidSet)
								{
									FString SkeletonRootUid;
									FString LastSceneNode = ActorNodeUid;
									if (const UInterchangeSceneNode* SceneNode = ActorNode)
									{
										FString ParentUid = SceneNode->GetParentUid();
										while (!ParentUid.Equals(UInterchangeBaseNode::InvalidNodeUid()))
										{
											if (const UInterchangeSceneNode* ParentNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(ParentUid)))
											{
												bool bIsSceneRoot = false;
												ParentNode->GetCustomIsSceneRoot(bIsSceneRoot);
												if (!bAllowSceneRootAsJoint && bIsSceneRoot)
												{
													//Do not include scene root node in Skeleton.
													break;
												}

												if(ParentNode->IsA(UInterchangeJointNode::StaticClass()))
												{
													SkeletonRootUid = ParentUid;
												}
												LastSceneNode = ParentUid;
												ParentUid = ParentNode->GetParentUid();
											}
											else
											{
												break;
											}
										}
										if (SkeletonRootUid.IsEmpty())
										{
											SkeletonRootUid = LastSceneNode;
										}
									}

									SkeletalAnimationNode->SetCustomSkeletonNodeUid(SkeletonRootUid);
									bCustomSkeletonNodeUidSet = true;
								}

								//add the payload key:
								SkeletalAnimationNode->SetAnimationPayloadKeyForSceneNodeUid(ActorNode->GetUniqueID(), AnimationPayLoadKey.UniqueId, AnimationPayLoadKey.Type);
							}
						}
					}
				}
			}

			if (!bCustomSkeletonNodeUidSet)
			{
				//if no SkeletonNodeUid can be set, then the conversion failed and should not be added to the BaseNodeContainer.
				BaseNodeContainer->RemoveNode(SkeletalAnimationNode->GetUniqueID());
			}
		}
	}
	else
	{
		//Support scene node animation (ULevelSequence, only supported when doing scene import)
		for (UInterchangeAnimationTrackSetNode* TrackSetNode : TrackSetNodes)
		{
			if (TrackSetNode)
			{
				CreateLevelSequenceFactoryNode(*TrackSetNode);
			}
		}
	}

	if (CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations && !CommonSkeletalMeshesAndAnimationsProperties->IsSkeletonValid())
	{
		UE_LOGF(LogInterchangePipeline, Warning, "UInterchangeGenericAnimationPipeline: Cannot execute pre-import pipeline because importing animation only requires a valid skeleton.");
		return;
	}
	SourceDatas.Empty(InSourceDatas.Num());
	for (const UInterchangeSourceData* SourceData : InSourceDatas)
	{
		SourceDatas.Add(SourceData);
	}

	TArray<UInterchangeSkeletalAnimationTrackNode*> TrackNodes;
	BaseNodeContainer->IterateNodesOfType<UInterchangeSkeletalAnimationTrackNode>([&](const FString& NodeUid, UInterchangeSkeletalAnimationTrackNode* Node)
		{
			TrackNodes.Add(Node);
		});

	//Support skeletal mesh animation (UAnimSequence)
	for (UInterchangeSkeletalAnimationTrackNode* TrackNode : TrackNodes)
	{
		if (TrackNode)
		{
			CreateAnimSequenceFactoryNode(*TrackNode);
		}
	}
}

void UInterchangeGenericAnimationPipeline::CreateLevelSequenceFactoryNode(UInterchangeAnimationTrackSetNode& TranslatedNode)
{
	const FString FactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TranslatedNode.GetUniqueID());

	UInterchangeLevelSequenceFactoryNode* FactoryNode = NewObject<UInterchangeLevelSequenceFactoryNode>(BaseNodeContainer, NAME_None);

	BaseNodeContainer->SetupNode(FactoryNode, FactoryNodeUid, TranslatedNode.GetDisplayLabel(), EInterchangeNodeContainerType::FactoryData);
	FactoryNode->SetEnabled(true);

	TArray<FString> AnimationTrackUids;
	TranslatedNode.GetCustomAnimationTrackUids(AnimationTrackUids);

	for (const FString& AnimationTrackUid : AnimationTrackUids)
	{
		FactoryNode->AddCustomAnimationTrackUid(AnimationTrackUid);

		// Update factory's dependencies
		if (const UInterchangeAnimationTrackBaseNode* TrackNode = Cast<UInterchangeAnimationTrackBaseNode>(BaseNodeContainer->GetNode(AnimationTrackUid)))
		{
			if (const UInterchangeAnimationTrackNode* AnimationTrackNode = Cast<UInterchangeAnimationTrackNode>(TrackNode))
			{
				FString ActorNodeUid;
				if (AnimationTrackNode->GetCustomActorDependencyUid(ActorNodeUid))
				{
					const FString ActorFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(ActorNodeUid);
					FactoryNode->AddFactoryDependencyUid(ActorFactoryNodeUid);
				}
			}
			else if (const UInterchangeAnimationTrackSetInstanceNode* InstanceTrackNode = Cast<UInterchangeAnimationTrackSetInstanceNode>(TrackNode))
			{
				FString TrackSetNodeUid;
				if (InstanceTrackNode->GetCustomTrackSetDependencyUid(TrackSetNodeUid))
				{
					const FString TrackSetFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TrackSetNodeUid);
					FactoryNode->AddFactoryDependencyUid(TrackSetFactoryNodeUid);
				}

			}
		}
	}

	float FrameRate;
	if (TranslatedNode.GetCustomFrameRate(FrameRate))
	{
		FactoryNode->SetCustomFrameRate(FrameRate);
	}

	UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(&TranslatedNode, FactoryNode, false);

	FactoryNode->AddTargetNodeUid(TranslatedNode.GetUniqueID());
	TranslatedNode.AddTargetNodeUid(FactoryNode->GetUniqueID());
}

void UInterchangeGenericAnimationPipeline::CreateAnimSequenceFactoryNode(UInterchangeSkeletalAnimationTrackNode& TrackNode)
{
	FString SkeletonNodeUid;
	if (!ensure(TrackNode.GetCustomSkeletonNodeUid(SkeletonNodeUid)))
	{
		// TODO: Warn something wrong happened
		return;
	}
	const bool bImportOnlyAnimation = CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations;
	const bool bAddCurveMetadataToSkeleton = CommonSkeletalMeshesAndAnimationsProperties->bAddCurveMetadataToSkeleton;

	const UInterchangeSkeletonFactoryNode* SkeletonFactoryNode = nullptr;

	const FString SkeletonFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(SkeletonNodeUid);
	SkeletonFactoryNode = Cast<const UInterchangeSkeletonFactoryNode>(BaseNodeContainer->GetFactoryNode(SkeletonFactoryNodeUid));

	const bool bRigidAnimationConverted = TrackNode.GetUniqueID().StartsWith(UE::Interchange::Private::ConvertedFromRigidAnimationPrefixIdentifier);

	//If we import anim only and we do not have meshes and skeleton. We need to create a skeleton factory node
	//base on the specified skeleton
	bool bSkeletonCompatible = true;
	if (bImportOnlyAnimation && !SkeletonFactoryNode && CommonSkeletalMeshesAndAnimationsProperties->IsSkeletonValid())
	{
		TWeakObjectPtr<USkeleton> Skeleton(nullptr);
		if (USkeleton* LoadedSkeleton = CommonSkeletalMeshesAndAnimationsProperties->Skeleton.LoadSynchronous())
		{
			Skeleton = TWeakObjectPtr<USkeleton>(LoadedSkeleton);
		}

		TPair<int32, FString> SkeletonRootNodeUidAndBoneIndex = TPair<int32, FString>(INDEX_NONE, FString());
		BaseNodeContainer->IterateNodesOfType<UInterchangeSceneNode>([&SkeletonRootNodeUidAndBoneIndex, BaseNodeContainerClosure = BaseNodeContainer, Skeleton, bRigidAnimationConverted](const FString& NodeUid, UInterchangeSceneNode* Node)
		{
			//In case the AnimationTrackNode is created from a RigidAnimation conversion the SceneNodes won't have Joint Specialization:
			if (Skeleton.IsValid() && (Node->IsA(UInterchangeJointNode::StaticClass()) || bRigidAnimationConverted))
			{
				const FReferenceSkeleton& ReferenceSkeleton = Skeleton->GetReferenceSkeleton();
				int32 RefBoneIndex = ReferenceSkeleton.FindBoneIndex(FName(*Node->GetDisplayLabel()));
				if(RefBoneIndex != INDEX_NONE)
				{
					if (SkeletonRootNodeUidAndBoneIndex.Key == INDEX_NONE || RefBoneIndex < SkeletonRootNodeUidAndBoneIndex.Key)
					{
						SkeletonRootNodeUidAndBoneIndex = TPair<int32, FString>(RefBoneIndex, NodeUid);
					}
				}
			}
		});
		FString SkeletonRootUid;
		//Use the lower uid we found
		if (SkeletonRootNodeUidAndBoneIndex.Key != INDEX_NONE && !SkeletonRootNodeUidAndBoneIndex.Value.IsEmpty())
		{
			SkeletonRootUid = SkeletonRootNodeUidAndBoneIndex.Value;
		}

		if(!SkeletonRootUid.IsEmpty())
		{
			//Create a skeleton node from all the joint in the translated nodes
			SkeletonFactoryNode = CommonSkeletalMeshesAndAnimationsProperties->CreateSkeletonFactoryNode(BaseNodeContainer, SkeletonRootUid, SourceDatas.Num() ? SourceDatas[0]->GetFilename() : TEXT("No Source File Specified"));
		}
		
		bSkeletonCompatible = UE::Interchange::Private::DoesSkeletalAnimationTargetSkeleton(BaseNodeContainer, TrackNode, Skeleton.Get());
		if (!bSkeletonCompatible)
		{
			UInterchangeResultDisplay_Generic* Message = AddMessage<UInterchangeResultDisplay_Generic>();
			Message->Text = FText::Format(NSLOCTEXT("UInterchangeGenericAnimationPipeline", "IncompatibleSkeleton", "Incompatible skeleton {0} when importing AnimSequence {1}."),
				FText::FromString(Skeleton->GetName()),
				FText::FromString(TrackNode.GetDisplayLabel()));

			return;
		}
	}

	if (!SkeletonFactoryNode)
	{
		// It can happen if we force static mesh import, in that case no skeleton will be create
		return;
	}

	double SampleRate = 30.;
	double StartTime = 0.;
	double StopTime = 0.;
	bool bTimeRangeIsValid = false;

	if (bImportBoneTracks)
	{
		int32 Numerator, Denominator;
		const UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::GetUniqueInstance(BaseNodeContainer);

		//Get the sample rate from the options and the data
		if (!bUse30HzToBakeBoneAnimation)
		{
			if (CustomBoneAnimationSampleRate > 0)
			{
				SampleRate = static_cast<double>(CustomBoneAnimationSampleRate);
			}
			else if (SourceNode && SourceNode->GetCustomSourceFrameRateNumerator(Numerator) && SourceNode->GetCustomSourceFrameRateDenominator(Denominator) && Denominator > 0 && Numerator > 0)
			{
				SampleRate = static_cast<double>(Numerator) / static_cast<double>(Denominator);
			}
			else
			{
				TrackNode.GetCustomAnimationSampleRate(SampleRate);
			}
		}

		//Get the animation start and stop time (range) from the options and the data
		//Some format don't fill the animation range data we fall back on the trackNode
		if (AnimationRange == EInterchangeAnimationRange::Timeline)
		{
			bTimeRangeIsValid = TrackNode.GetCustomSourceTimelineAnimationStartTime(StartTime) && TrackNode.GetCustomSourceTimelineAnimationStopTime(StopTime);
		}
		else if (AnimationRange == EInterchangeAnimationRange::SetRange)
		{
			StartTime = static_cast<double>(FrameImportRange.Min) / SampleRate;
			StopTime = static_cast<double>(FrameImportRange.Max) / SampleRate;
			bTimeRangeIsValid = true;
		}
		//No custom time specified, use the track default
		if(!bTimeRangeIsValid)
		{
			StartTime = 0.;
			StopTime = 0.;
			bTimeRangeIsValid = TrackNode.GetCustomAnimationStartTime(StartTime);
			bTimeRangeIsValid &= TrackNode.GetCustomAnimationStopTime(StopTime);
		}

		FFrameRate FrameRate = UE::Interchange::Animation::ConvertSampleRateToFrameRate(SampleRate);

		const double SequenceLength = FMath::Max<double>(StopTime - StartTime, MINIMUM_ANIMATION_LENGTH);

		const float SubFrame = FrameRate.AsFrameTime(SequenceLength).GetSubFrame();

		if (!FMath::IsNearlyZero(SubFrame, KINDA_SMALL_NUMBER) && !FMath::IsNearlyEqual(SubFrame, 1.0f, KINDA_SMALL_NUMBER))
		{
			if (FrameAlignment != EInterchangeFrameAlignment::None)
			{
				// Figure out whether start or stop has to be adjusted
				const FFrameTime StartFrameTime = FrameRate.AsFrameTime(StartTime);
				const FFrameTime StopFrameTime = FrameRate.AsFrameTime(StopTime);
				FFrameNumber StartFrameNumber = StartFrameTime.GetFrame().Value, StopFrameNumber = StopFrameTime.GetFrame().Value;
				double NewStartTime = StartTime, NewStopTime = StopTime;

				auto GenerateFrameAlignedFrameNumber = [FrameAlignment = this->FrameAlignment](const FFrameTime& FrameTime) -> FFrameNumber
					{
						switch (FrameAlignment)
						{
							case EInterchangeFrameAlignment::SnapToFloor:
								return FrameTime.FloorToFrame();
							case EInterchangeFrameAlignment::SnapToClosest:
								return FrameTime.RoundToFrame();
							case EInterchangeFrameAlignment::SnapToCeiling:
								return FrameTime.CeilToFrame();
							case EInterchangeFrameAlignment::None:
							default:
								break;
						}
						return FrameTime.FrameNumber;
					};

				if (!FMath::IsNearlyZero(StartFrameTime.GetSubFrame()))
				{
					
					StartFrameNumber = GenerateFrameAlignedFrameNumber(StartFrameTime);
					NewStartTime = FrameRate.AsSeconds(StartFrameNumber);
				}

				if (!FMath::IsNearlyZero(StopFrameTime.GetSubFrame()))
				{
					StopFrameNumber = GenerateFrameAlignedFrameNumber(StopFrameTime);
					NewStopTime = FrameRate.AsSeconds(StopFrameNumber);
				}
			
				UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
				Message->SourceAssetName = SourceDatas.Num() ? SourceDatas[0]->GetFilename() : TEXT("No Source File Specified");
				Message->DestinationAssetName = TrackNode.GetDisplayLabel();
				Message->AssetType = UAnimSequence::StaticClass();
				Message->Text = FText::Format(NSLOCTEXT("UInterchangeGenericAnimationPipeline", "Info_ImportLengthSnap", "Animation length has been adjusted to align with frame borders using import frame-rate {0}.\n\nOriginal timings:\n\t\tStart: {1} ({2})\n\t\tStop: {3} ({4})\nAligned timings:\n\t\tStart: {5} ({6})\n\t\tStop: {7} ({8})"),
					FrameRate.ToPrettyText(),
					FText::AsNumber(StartTime),
					FText::AsNumber(StartFrameTime.AsDecimal()),
					FText::AsNumber(StopTime),
					FText::AsNumber(StopFrameTime.AsDecimal()),
					FText::AsNumber(NewStartTime),
					FText::AsNumber(StartFrameNumber.Value),
					FText::AsNumber(NewStopTime),
					FText::AsNumber(StopFrameNumber.Value));

				StartTime = NewStartTime;
				StopTime = NewStopTime;
			}
			else
			{
				UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
				Message->SourceAssetName = SourceDatas.Num() ? SourceDatas[0]->GetFilename() : TEXT("No Source File Specified");
				Message->DestinationAssetName = TrackNode.GetDisplayLabel();
				Message->AssetType = UAnimSequence::StaticClass();
				Message->Text = FText::Format(NSLOCTEXT("UInterchangeGenericAnimationPipeline", "WrongSequenceLength", "Animation length {0} is not compatible with import frame-rate {1} (sub frame {2}). The animation has to be frame-border aligned if the 'Snap to Closest Frame Boundary' pipeline option is disabled."),
					FText::AsNumber(SequenceLength), FrameRate.ToPrettyText(), FText::AsNumber(SubFrame));
				//Skip this anim sequence factory node
				return;
			}
		}
	}

	const FString AnimSequenceUid = TEXT("\\AnimSequence") + TrackNode.GetUniqueID();

	UInterchangeAnimSequenceFactoryNode* AnimSequenceFactoryNode = NewObject<UInterchangeAnimSequenceFactoryNode>(BaseNodeContainer, NAME_None);

	FString DisplayString = TrackNode.GetDisplayLabel();

	AnimSequenceFactoryNode->InitializeAnimSequenceNode(AnimSequenceUid, DisplayString, BaseNodeContainer);

	if (SkeletonFactoryNode)
	{
		AnimSequenceFactoryNode->SetCustomSkeletonFactoryNodeUid(SkeletonFactoryNode->GetUniqueID());
	}

	AnimSequenceFactoryNode->SetCustomImportBoneTracks(bImportBoneTracks);
	AnimSequenceFactoryNode->SetCustomImportBoneTracksSampleRate(SampleRate);
	if (bTimeRangeIsValid)
	{
		AnimSequenceFactoryNode->SetCustomImportBoneTracksRangeStart(StartTime);
		AnimSequenceFactoryNode->SetCustomImportBoneTracksRangeStop(StopTime);
	}

	AnimSequenceFactoryNode->SetCustomImportAttributeCurves(bImportCustomAttribute);
	AnimSequenceFactoryNode->SetCustomAddCurveMetadataToSkeleton(bAddCurveMetadataToSkeleton);
	AnimSequenceFactoryNode->SetCustomDoNotImportCurveWithZero(bDoNotImportCurveWithZero);
	AnimSequenceFactoryNode->SetCustomRemoveCurveRedundantKeys(bRemoveCurveRedundantKeys);
	AnimSequenceFactoryNode->SetCustomDeleteExistingMorphTargetCurves(bDeleteExistingMorphTargetCurves);
	AnimSequenceFactoryNode->SetCustomDeleteExistingCustomAttributeCurves(bDeleteExistingCustomAttributeCurves);
	AnimSequenceFactoryNode->SetCustomDeleteExistingNonCurveCustomAttributes(bDeleteExistingNonCurveCustomAttributes);

	AnimSequenceFactoryNode->SetCustomMaterialDriveParameterOnCustomAttribute(bSetMaterialDriveParameterOnCustomAttribute);
	for (const FString& MaterialSuffixe : MaterialCurveSuffixes)
	{
		AnimSequenceFactoryNode->SetAnimatedMaterialCurveSuffixe(MaterialSuffixe);
	}

	//Animation sequence cannot be created without a valid skeleton
	FString SkeletonUid;
	if(SkeletonFactoryNode)
	{
		SkeletonUid = SkeletonFactoryNode->GetUniqueID();
		AnimSequenceFactoryNode->AddFactoryDependencyUid(SkeletonUid);
	}



	FString RootJointUid;
	if (SkeletonFactoryNode && SkeletonFactoryNode->GetCustomRootJointUid(RootJointUid))
	{
		// NOTE: Could this be added as an array of FString attributes on the UInterchangeSkeletalAnimationTrackNode
#if WITH_EDITOR
		//Iterate all joints to set the meta data value in the anim sequence factory node
		
		//Note: We shouldn't be able to get to this point without the ChildrenCache already set, so there is no need to call ComputeChildrenCache.
		// (Also trying to avoid recreating the ChildrenCache for every AnimSequenceFactoryNode creation call.)

		UE::Interchange::Private::FSkeletonHelper::RecursiveAddSkeletonMetaDataValues(BaseNodeContainer, AnimSequenceFactoryNode, RootJointUid);
#endif //WITH_EDITOR

		const TArray<FString> CustomAttributeNamesToImport = UAnimationSettings::Get()->GetBoneCustomAttributeNamesToImport();

		BaseNodeContainer->IterateNodeChildren(RootJointUid, [&AnimSequenceFactoryNode, &CustomAttributeNamesToImport](const UInterchangeBaseNode* Node)
			{
				if (const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(Node))
				{
					using namespace UE::Interchange;

					FString BoneName = SceneNode->GetDisplayLabel();
					bool bImportAllAttributesOnBone = UAnimationSettings::Get()->BoneNamesWithCustomAttributes.Contains(BoneName);

					TArray<FInterchangeUserDefinedAttributeInfo> AttributeInfos;
					UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttributeInfos(SceneNode, AttributeInfos);
					for (const FInterchangeUserDefinedAttributeInfo& AttributeInfo : AttributeInfos)
					{
						if (AttributeInfo.PayloadKey.IsSet())
						{
							bool bDecimalType = AttributeInfo.Type == EAttributeTypes::Float ||
								AttributeInfo.Type == EAttributeTypes::Float16 ||
								AttributeInfo.Type == EAttributeTypes::Double;

							const bool bForceImportBoneCustomAttribute = CustomAttributeNamesToImport.Contains(AttributeInfo.Name);
							
							EInterchangeAnimationPayLoadType AnimationPayloadType = EInterchangeAnimationPayLoadType::NONE;
							if (SceneNode->GetAnimationCurveTypeForCurveName(AttributeInfo.Name, AnimationPayloadType))
							{
								if (AnimationPayloadType == EInterchangeAnimationPayLoadType::CURVE)
								{
									AnimSequenceFactoryNode->SetAnimatedAttributeCurveName(AttributeInfo.Name);
								}
								else if (AnimationPayloadType == EInterchangeAnimationPayLoadType::STEPCURVE)
								{
									AnimSequenceFactoryNode->SetAnimatedAttributeStepCurveName(AttributeInfo.Name);
								}
							}
							else
							{
								//Material attribute curve
								if (!bImportAllAttributesOnBone && bDecimalType && !bForceImportBoneCustomAttribute)
								{
									AnimSequenceFactoryNode->SetAnimatedAttributeCurveName(AttributeInfo.Name);
								}
								else if (bForceImportBoneCustomAttribute || bImportAllAttributesOnBone)
								{
									AnimSequenceFactoryNode->SetAnimatedAttributeStepCurveName(AttributeInfo.Name);
								}
							}

							
						}
					}
				}
			});
	}

	//Iterate dependencies
	//Skeletal Meshes factories are responsible to populate USkeletons using MergeAllBonesToBoneTree.
	//For this reason, Animation sequence has a dependency on all the Skeletal Meshes using that Skeleton.
	{
		TArray<FString> SkeletalMeshNodeUids;
		BaseNodeContainer->GetNodes(UInterchangeSkeletalMeshFactoryNode::StaticClass(), SkeletalMeshNodeUids);
		for (const FString& SkelMeshFactoryNodeUid : SkeletalMeshNodeUids)
		{
			if (const UInterchangeSkeletalMeshFactoryNode* ConstSkeletalMeshFactoryNode = Cast<const UInterchangeSkeletalMeshFactoryNode>(BaseNodeContainer->GetFactoryNode(SkelMeshFactoryNodeUid)))
			{
				if (ConstSkeletalMeshFactoryNode->HasFactoryDependency(SkeletonUid))
				{
					AnimSequenceFactoryNode->AddFactoryDependencyUid(SkelMeshFactoryNodeUid);
				}
			}
		}
	}

	if (CommonSkeletalMeshesAndAnimationsProperties->IsSkeletonValid())
	{
		if (USkeleton* Skeleton = CommonSkeletalMeshesAndAnimationsProperties->Skeleton.LoadSynchronous())
		{
			//TODO: support skeleton helper in runtime
#if WITH_EDITOR
			bSkeletonCompatible = bSkeletonCompatible && UE::Interchange::Private::FSkeletonHelper::IsCompatibleSkeleton(Skeleton
				, RootJointUid
				, BaseNodeContainer
				, CommonMeshesProperties->bConvertStaticsWithMorphTargetsToSkeletals || CommonMeshesProperties->ForceAllMeshAsType == EInterchangeForceMeshType::IFMT_SkeletalMesh
				, false /*bCheckForIdenticalSkeleton*/
				, CommonMeshesProperties->bImportSockets);
#endif
			if (bSkeletonCompatible)
			{
				AnimSequenceFactoryNode->SetCustomSkeletonSoftObjectPath(CommonSkeletalMeshesAndAnimationsProperties->Skeleton.ToSoftObjectPath());
			}
			else
			{
				UInterchangeResultDisplay_Generic* Message = AddMessage<UInterchangeResultDisplay_Generic>();
				Message->Text = FText::Format(NSLOCTEXT("UInterchangeGenericAnimationPipeline", "IncompatibleSkeleton", "Incompatible skeleton {0} when importing AnimSequence {1}."),
					FText::FromString(Skeleton->GetName()),
					FText::FromString(TrackNode.GetDisplayLabel()));
			}
		}
		else
		{
			UInterchangeResultWarning_Generic* Warning = AddMessage<UInterchangeResultWarning_Generic>();
			Warning->Text = FText::Format(NSLOCTEXT("UInterchangeGenericAnimationPipeline", "InvalidSkeleton", "Failed to load skeleton at path {0} to test compatibility when importing AnimSequence {1}."),
				FText::FromString(CommonSkeletalMeshesAndAnimationsProperties->Skeleton.ToString()),
				FText::FromString(TrackNode.GetDisplayLabel()));
		}
	}

	{
		TMap<FString, FString> SceneNodeAnimationPayloadKeyUids;
		TMap<FString, uint8> SceneNodeAnimationPayloadKeyTypes;
		TrackNode.GetSceneNodeAnimationPayloadKeys(SceneNodeAnimationPayloadKeyUids, SceneNodeAnimationPayloadKeyTypes);
		AnimSequenceFactoryNode->SetAnimationPayloadKeysForSceneNodeUids(SceneNodeAnimationPayloadKeyUids, SceneNodeAnimationPayloadKeyTypes);
	}

	{
		TMap<FString, FString> MorphTargetNodeAnimationPayloadKeyUids;
		TMap<FString, uint8> MorphTargetNodeAnimationPayloadKeyTypes;
		TrackNode.GetMorphTargetNodeAnimationPayloadKeys(MorphTargetNodeAnimationPayloadKeyUids, MorphTargetNodeAnimationPayloadKeyTypes);
		AnimSequenceFactoryNode->SetAnimationPayloadKeysForMorphTargetNodeUids(MorphTargetNodeAnimationPayloadKeyUids, MorphTargetNodeAnimationPayloadKeyTypes);
	}

	UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(&TrackNode, AnimSequenceFactoryNode, false);

	AnimSequenceFactoryNode->AddTargetNodeUid(TrackNode.GetUniqueID());
	TrackNode.AddTargetNodeUid(AnimSequenceFactoryNode->GetUniqueID());
}