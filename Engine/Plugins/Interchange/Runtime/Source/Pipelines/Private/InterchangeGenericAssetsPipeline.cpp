// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeGenericAssetsPipeline.h"

#include "Animation/Skeleton.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CoreMinimal.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/Texture.h"
#include "Framework/Docking/TabManager.h"
#include "GameFramework/Actor.h"
#include "GeometryCache.h"
#include "HAL/PlatformApplicationMisc.h"
#include "InterchangeAnimationTrackSetNode.h"
#include "InterchangeAnimSequenceFactoryNode.h"
#include "InterchangeAssetUserData.h"
#include "InterchangeBlueprintPipelineBase.h"
#include "InterchangeCommonPipelineDataFactoryNode.h"
#include "InterchangeGenericAnimationPipeline.h"
#include "InterchangeGenericGroomPipeline.h"
#include "InterchangeGenericMaterialPipeline.h"
#include "InterchangeGenericMeshPipeline.h"
#include "InterchangeGenericTexturePipeline.h"
#include "InterchangeManager.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangePipelineHelper.h"
#include "InterchangePipelineLog.h"
#include "InterchangePipelineMeshesUtilities.h"
#include "InterchangePythonPipelineBase.h"
#include "InterchangeJointNode.h"
#include "InterchangeSkeletalMeshFactoryNode.h"
#include "InterchangeSkeletalMeshLodDataNode.h"
#include "InterchangeSkeletonFactoryNode.h"
#include "InterchangeSkeletonHelper.h"
#include "InterchangeSourceData.h"
#include "InterchangeStaticMeshFactoryNode.h"
#include "InterchangeStaticMeshLodDataNode.h"
#include "LevelSequence.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeSourceNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "Styling/AppStyle.h"
#include "Templates/SubclassOf.h"
#include "UObject/MetaData.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ReferencerFinder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGenericAssetsPipeline)

const FName SInterchangeGenericAssetMaterialConflictWidget::NAME_Import = FName(TEXT("Imported"));
const FName SInterchangeGenericAssetMaterialConflictWidget::NAME_Asset = FName(TEXT("Asset"));

const FSlateColor SInterchangeGenericAssetMaterialConflictWidget::SlateColorFullConflict = FSlateColor(FLinearColor(0.9f, 0.25f, 0.0f));
const FSlateColor SInterchangeGenericAssetMaterialConflictWidget::SlateColorSubConflict = FSlateColor(FLinearColor(0.7f, 0.45f, 0.1f));

UInterchangeGenericAssetsPipeline::UInterchangeGenericAssetsPipeline()
{
	MaterialPipeline = CreateDefaultSubobject<UInterchangeGenericMaterialPipeline>("MaterialPipeline");
	CommonMeshesProperties = CreateDefaultSubobject<UInterchangeGenericCommonMeshesProperties>("CommonMeshesProperties");
	CommonSkeletalMeshesAndAnimationsProperties = CreateDefaultSubobject<UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties>("CommonSkeletalMeshesAndAnimationsProperties");
	MeshPipeline = CreateDefaultSubobject<UInterchangeGenericMeshPipeline>("MeshPipeline");
	MeshPipeline->CommonMeshesProperties = CommonMeshesProperties;
	MeshPipeline->CommonSkeletalMeshesAndAnimationsProperties = CommonSkeletalMeshesAndAnimationsProperties;
	GroomPipeline = CreateDefaultSubobject<UInterchangeGenericGroomPipeline>("GroomPipeline");
	AnimationPipeline = CreateDefaultSubobject<UInterchangeGenericAnimationPipeline>("AnimationPipeline");
	AnimationPipeline->CommonSkeletalMeshesAndAnimationsProperties = CommonSkeletalMeshesAndAnimationsProperties;
	AnimationPipeline->CommonMeshesProperties = CommonMeshesProperties;
}

void UInterchangeGenericAssetsPipeline::PreDialogCleanup(const FName PipelineStackName)
{
	check(CommonSkeletalMeshesAndAnimationsProperties);

	CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations = bImportOnlyAnimationAdjusted;

	if (MaterialPipeline)
	{
		MaterialPipeline->PreDialogCleanup(PipelineStackName);
	}
	
	if (MeshPipeline)
	{
		MeshPipeline->PreDialogCleanup(PipelineStackName);
	}

	if (GroomPipeline)
	{
		GroomPipeline->PreDialogCleanup(PipelineStackName);
	}

	if (AnimationPipeline)
	{
		AnimationPipeline->PreDialogCleanup(PipelineStackName);
	}
	
	if (IsStandAlonePipeline())
	{
		SaveSettings(PipelineStackName);
	}
}

void UInterchangeGenericAssetsPipeline::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FInterchangeCustomVersion::GUID);
}

void UInterchangeGenericAssetsPipeline::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(FInterchangeCustomVersion::GUID) < FInterchangeCustomVersion::SkeletalAndStaticMeshOptionsCleanUp)
	{
		if (CommonSkeletalMeshesAndAnimationsProperties && CommonMeshesProperties)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			CommonMeshesProperties->bConvertStaticsWithAnimatedTransformToSkeletals = CommonMeshesProperties->bAutoDetectMeshType_DEPRECATED;
			CommonMeshesProperties->bConvertStaticsInBoneHierarchyToSkeletals = CommonSkeletalMeshesAndAnimationsProperties->bImportMeshesInBoneHierarchy_DEPRECATED;
			CommonMeshesProperties->bConvertStaticsWithMorphTargetsToSkeletals = CommonSkeletalMeshesAndAnimationsProperties->bConvertStaticsWithMorphTargetsToSkeletals_DEPRECATED;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}



bool UInterchangeGenericAssetsPipeline::IsSettingsAreValid(TOptional<FText>& OutInvalidReason) const
{
	if (MaterialPipeline && !MaterialPipeline->IsSettingsAreValid(OutInvalidReason))
	{
		return false;
	}

	if (CommonMeshesProperties && !CommonMeshesProperties->IsSettingsAreValid(OutInvalidReason))
	{
		return false;
	}

	if (CommonSkeletalMeshesAndAnimationsProperties && !CommonSkeletalMeshesAndAnimationsProperties->IsSettingsAreValid(OutInvalidReason))
	{
		return false;
	}

	if (MeshPipeline && !MeshPipeline->IsSettingsAreValid(OutInvalidReason))
	{
		return false;
	}

	if (GroomPipeline && !GroomPipeline->IsSettingsAreValid(OutInvalidReason))
	{
		return false;
	}

	if (AnimationPipeline && !AnimationPipeline->IsSettingsAreValid(OutInvalidReason))
	{
		return false;
	}

	return Super::IsSettingsAreValid(OutInvalidReason);
}

void UInterchangeGenericAssetsPipeline::AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams)
{
	Super::AdjustSettingsForContext(ContextParams);

	if (ContextParams.ContextType == EInterchangePipelineContext::AssetImport)
	{
		if (CommonSkeletalMeshesAndAnimationsProperties
			&& CommonSkeletalMeshesAndAnimationsProperties->bTryAutoSelectSkeleton
			&& !CommonSkeletalMeshesAndAnimationsProperties->IsSkeletonValid())
		{
			//Search if there is a valid skeleton in this path
			FARFilter Filter;
			Filter.PackagePaths.Add(*ContentImportPath);
			Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());

			IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
			TArray<FAssetData> SkeletonAssets;
			AssetRegistry.GetAssets(Filter, SkeletonAssets);
			if (SkeletonAssets.Num() > 0)
			{
				CommonSkeletalMeshesAndAnimationsProperties->Skeleton = CastChecked<USkeleton>(SkeletonAssets[0].GetAsset());
			}
		}
	}
	else if (ContextParams.ContextType == EInterchangePipelineContext::AssetCustomMorphTargetImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomMorphTargetReImport)
	{
		bUseSourceNameForAsset = true;
	}

	if (MaterialPipeline)
	{
		MaterialPipeline->AdjustSettingsForContext(ContextParams);
	}

	if (MeshPipeline)
	{
		MeshPipeline->AdjustSettingsForContext(ContextParams);
	}

	if (GroomPipeline)
	{
		GroomPipeline->AdjustSettingsForContext(ContextParams);
	}

	if (AnimationPipeline)
	{
		AnimationPipeline->AdjustSettingsForContext(ContextParams);
	}

	//Store the adjusted settings
	bImportOnlyAnimationAdjusted = CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations;
}

#if WITH_EDITOR

void UInterchangeGenericAssetsPipeline::FilterPropertiesFromTranslatedData(UInterchangeBaseNodeContainer* InBaseNodeContainer)
{
	Super::FilterPropertiesFromTranslatedData(InBaseNodeContainer);

	if (MaterialPipeline)
	{
		MaterialPipeline->FilterPropertiesFromTranslatedData(InBaseNodeContainer);
	}

	if (CommonMeshesProperties && CommonSkeletalMeshesAndAnimationsProperties && MeshPipeline && AnimationPipeline)
	{
		CommonMeshesProperties->FilterPropertiesFromTranslatedData(InBaseNodeContainer);
		CommonSkeletalMeshesAndAnimationsProperties->FilterPropertiesFromTranslatedData(InBaseNodeContainer);
		MeshPipeline->FilterPropertiesFromTranslatedData(InBaseNodeContainer);
		AnimationPipeline->FilterPropertiesFromTranslatedData(InBaseNodeContainer);

		UInterchangePipelineMeshesUtilities* PipelineMeshesUtilities = UInterchangeGenericMeshPipeline::CreateMeshPipelineUtilities(InBaseNodeContainer, MeshPipeline);

		TArray<FString> SkeletalMeshes;
		TArray<FString> StaticMeshes;
		TArray<FString> GeometryCaches;
		int32 RawStaticMesh = 0;
		int32 RawSkeletalMesh = 0;
		int32 RawMorphTargetShape = 0;
		if (!CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations)
		{
			PipelineMeshesUtilities->GetAllSkinnedMeshInstance(SkeletalMeshes);
			if (SkeletalMeshes.Num() == 0)
			{
				PipelineMeshesUtilities->GetAllSkinnedMeshGeometry(SkeletalMeshes);
			}
			
			PipelineMeshesUtilities->GetAllStaticMeshInstance(StaticMeshes);
			if (StaticMeshes.Num() == 0)
			{
				PipelineMeshesUtilities->GetAllStaticMeshGeometry(StaticMeshes);
			}

			PipelineMeshesUtilities->GetAllGeometryCacheInstance(GeometryCaches);
			if (GeometryCaches.Num() == 0)
			{
				PipelineMeshesUtilities->GetAllGeometryCacheGeometry(GeometryCaches);
			}

			InBaseNodeContainer->IterateNodesOfType<UInterchangeMeshNode>([&RawStaticMesh, &RawSkeletalMesh, &RawMorphTargetShape](const FString& NodeUid, UInterchangeMeshNode* MeshNode)
				{
					if (MeshNode->IsMorphTarget())
					{
						RawMorphTargetShape++;
					}
					else
					{
						MeshNode->IsSkinnedMesh() ? RawSkeletalMesh++ : RawStaticMesh++;
					}
				});
		}

		int32 RawAnimationNode = 0;
		InBaseNodeContainer->IterateNodesOfType<UInterchangeAnimationTrackBaseNode>([&RawAnimationNode](const FString& NodeUid, UInterchangeAnimationTrackBaseNode* AnimationNode)
			{
				RawAnimationNode++;
			});
		InBaseNodeContainer->IterateNodesOfType<UInterchangeAnimationTrackSetNode>([&RawAnimationNode](const FString& NodeUid, UInterchangeAnimationTrackSetNode* AnimationNode)
			{
				RawAnimationNode++;
			});

		//Found which categories to hide
		bool bHideStaticMeshes = false;
		bool bHideSkeletalMeshes = false;
		bool bHideGeometryCaches = GeometryCaches.Num() == 0;
		bool bHideCommonMeshes = false;
		bool bHideCommonSkeletalMeshesAndAnimations = false;
		bool bHideCommonSkeletalMeshesAndAnimations_StaticMesh = false;
		bool bHideAnimations = false;

		if (RawStaticMesh == 0 || RawMorphTargetShape == 0)
		{
			bHideCommonSkeletalMeshesAndAnimations_StaticMesh = true;
		}

		if (SkeletalMeshes.Num() == 0 && StaticMeshes.Num() == 0)
		{
			bHideStaticMeshes = true;
			bHideSkeletalMeshes = true;
			bHideCommonMeshes = true;
		}
		else if (StaticMeshes.Num() > 0 && SkeletalMeshes.Num() == 0)
		{
			bHideSkeletalMeshes = true;
			bHideCommonSkeletalMeshesAndAnimations = true;
			bHideAnimations = true;
		}
		else if (SkeletalMeshes.Num() > 0 && StaticMeshes.Num() == 0)
		{
			bHideStaticMeshes = true;
		}

		UInterchangePipelineBase* OuterMostPipeline = GetMostPipelineOuter();

		if (SkeletalMeshes.Num() > 0)
		{
			if (MeshPipeline->SkeletalMeshImportContentType == EInterchangeSkeletalMeshContentType::SkinningWeights)
			{
				HideProperty(OuterMostPipeline, CommonMeshesProperties, GET_MEMBER_NAME_CHECKED(UInterchangeGenericCommonMeshesProperties, VertexOverrideColor));
				HideProperty(OuterMostPipeline, CommonMeshesProperties, GET_MEMBER_NAME_CHECKED(UInterchangeGenericCommonMeshesProperties, VertexColorImportOption));
				HideProperty(OuterMostPipeline, MeshPipeline, GET_MEMBER_NAME_CHECKED(UInterchangeGenericMeshPipeline, bImportMorphTargets));
				HideProperty(OuterMostPipeline, MeshPipeline, GET_MEMBER_NAME_CHECKED(UInterchangeGenericMeshPipeline, ThresholdPosition));
				HideProperty(OuterMostPipeline, MeshPipeline, GET_MEMBER_NAME_CHECKED(UInterchangeGenericMeshPipeline, ThresholdTangentNormal));
				HideProperty(OuterMostPipeline, MeshPipeline, GET_MEMBER_NAME_CHECKED(UInterchangeGenericMeshPipeline, ThresholdUV));
				HideProperty(OuterMostPipeline, MeshPipeline, GET_MEMBER_NAME_CHECKED(UInterchangeGenericMeshPipeline, MorphThresholdPosition));
			}
			else if (MeshPipeline->SkeletalMeshImportContentType == EInterchangeSkeletalMeshContentType::Geometry)
			{
				HideProperty(OuterMostPipeline, MeshPipeline, GET_MEMBER_NAME_CHECKED(UInterchangeGenericMeshPipeline, bUpdateSkeletonReferencePose));
			}
		}

		if (RawAnimationNode == 0)
		{
			bHideAnimations = true;
		}

		if (bHideAnimations && bHideSkeletalMeshes)
		{
			bHideCommonSkeletalMeshesAndAnimations = true;
		}


		if (CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations)
		{
			if (MaterialPipeline)
			{
				HidePropertiesOfCategory(OuterMostPipeline, MaterialPipeline, UInterchangeGenericMaterialPipeline::GetPipelineCategory(nullptr));
				if (MaterialPipeline->TexturePipeline)
				{
					HidePropertiesOfCategory(OuterMostPipeline, MaterialPipeline->TexturePipeline, UInterchangeGenericTexturePipeline::GetPipelineCategory(nullptr));
				}
			}
		}

		//Hide the categories
		if (bHideStaticMeshes)
		{
			HidePropertiesOfCategory(OuterMostPipeline, MeshPipeline, UInterchangeGenericMeshPipeline::GetPipelineCategory(UStaticMesh::StaticClass()));
		}
		if (bHideSkeletalMeshes)
		{
			HidePropertiesOfCategory(OuterMostPipeline, MeshPipeline, UInterchangeGenericMeshPipeline::GetPipelineCategory(USkeletalMesh::StaticClass()));
		}
		if (bHideGeometryCaches)
		{
			HidePropertiesOfCategory(OuterMostPipeline, MeshPipeline, UInterchangeGenericMeshPipeline::GetPipelineCategory(UGeometryCache::StaticClass()));
		}
		if (bHideCommonMeshes)
		{
			HidePropertiesOfCategory(OuterMostPipeline, CommonMeshesProperties, UInterchangeGenericCommonMeshesProperties::GetPipelineCategory(nullptr));
		}
		if (bHideCommonSkeletalMeshesAndAnimations)
		{
			//We cannot hide common sk and anim if import only animation is true.
			if (!CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations)
			{
				HidePropertiesOfCategory(OuterMostPipeline, CommonSkeletalMeshesAndAnimationsProperties, UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties::GetPipelineCategory(nullptr));
			}
		}
		if (bHideCommonSkeletalMeshesAndAnimations_StaticMesh)
		{
			HidePropertiesOfCategory(OuterMostPipeline, CommonSkeletalMeshesAndAnimationsProperties, UInterchangeGenericMeshPipeline::GetPipelineCategory(UStaticMesh::StaticClass()));
		}
		if (bHideAnimations)
		{
			HidePropertiesOfCategory(OuterMostPipeline, AnimationPipeline, UInterchangeGenericAnimationPipeline::GetPipelineCategory(nullptr));
		}
	}

	if (GroomPipeline)
	{
		GroomPipeline->FilterPropertiesFromTranslatedData(InBaseNodeContainer);
	}
}

bool UInterchangeGenericAssetsPipeline::IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const
{
	static const TSet<FName> NeedRefreshProperties =
	{
		GET_MEMBER_NAME_CHECKED(UInterchangeGenericAssetsPipeline, bUseSourceNameForAsset),
		GET_MEMBER_NAME_CHECKED(UInterchangeGenericAssetsPipeline, bSceneNameSubFolder),
		GET_MEMBER_NAME_CHECKED(UInterchangeGenericAssetsPipeline, bAssetTypeSubFolders),
		GET_MEMBER_NAME_CHECKED(UInterchangeGenericAssetsPipeline, AssetName)
	};

	if (NeedRefreshProperties.Contains(PropertyChangedEvent.GetPropertyName()))
	{
		return true;
	}

	if ((CommonMeshesProperties && CommonMeshesProperties->IsPropertyChangeNeedRefresh(PropertyChangedEvent))
		|| (CommonSkeletalMeshesAndAnimationsProperties && CommonSkeletalMeshesAndAnimationsProperties->IsPropertyChangeNeedRefresh(PropertyChangedEvent))
		|| (MeshPipeline && MeshPipeline->IsPropertyChangeNeedRefresh(PropertyChangedEvent))
		|| (MaterialPipeline && MaterialPipeline->IsPropertyChangeNeedRefresh(PropertyChangedEvent))
		|| (AnimationPipeline && AnimationPipeline->IsPropertyChangeNeedRefresh(PropertyChangedEvent))
		|| (GroomPipeline && GroomPipeline->IsPropertyChangeNeedRefresh(PropertyChangedEvent)))
	{
		return true;
	}
	return Super::IsPropertyChangeNeedRefresh(PropertyChangedEvent);
}

void UInterchangeGenericAssetsPipeline::GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const
{
	if (MeshPipeline)
	{
		MeshPipeline->GetSupportAssetClasses(PipelineSupportAssetClasses);
	}
	if (GroomPipeline)
	{
		GroomPipeline->GetSupportAssetClasses(PipelineSupportAssetClasses);
	}
	if (MaterialPipeline)
	{
		MaterialPipeline->GetSupportAssetClasses(PipelineSupportAssetClasses);
	}
	if (AnimationPipeline)
	{
		AnimationPipeline->GetSupportAssetClasses(PipelineSupportAssetClasses);
	}
}

bool UInterchangeGenericAssetsPipeline::GetPropertyPossibleValues(const FName PropertyPath, TArray<FString>& PossibleValues)
{
	const FString PropertyPathString = PropertyPath.ToString();
	if (MaterialPipeline && PropertyPathString.StartsWith(UInterchangeGenericMaterialPipeline::StaticClass()->GetPathName()))
	{
		if (MaterialPipeline->GetPropertyPossibleValues(PropertyPath, PossibleValues))
		{
			return true;
		}
	}
	
	if (MeshPipeline && PropertyPathString.StartsWith(UInterchangeGenericMeshPipeline::StaticClass()->GetPathName()))
	{
		if (MeshPipeline->GetPropertyPossibleValues(PropertyPath, PossibleValues))
		{
			return true;
		}
	}

	if (GroomPipeline && PropertyPathString.StartsWith(UInterchangeGenericGroomPipeline::StaticClass()->GetPathName()))
	{
		if (GroomPipeline->GetPropertyPossibleValues(PropertyPath, PossibleValues))
		{
			return true;
		}
	}

	if (AnimationPipeline && PropertyPathString.StartsWith(UInterchangeGenericAnimationPipeline::StaticClass()->GetPathName()))
	{
		if (AnimationPipeline->GetPropertyPossibleValues(PropertyPath, PossibleValues))
		{
			return true;
		}
	}

	//If we did not find any property call the super implementation
	return Super::GetPropertyPossibleValues(PropertyPath, PossibleValues);
}

void UInterchangeGenericAssetsPipeline::CreateMaterialConflict(UStaticMesh* StaticMesh, USkeletalMesh* SkeletalMesh, UInterchangeBaseNodeContainer* TransientBaseNodeContainer)
{
	UObject* ReimportObject = nullptr;
	MaterialConflictData.Reset();

	TArray<FString> AssetImportMaterialNames;
	TMap<FString, FString> MaterialNodePerMaterialSlotName;
	FString NoName = TEXT("Material_");
	uint32 NameIndex = 1;
	if (StaticMesh)
	{
		ReimportObject = StaticMesh;
		for (FStaticMaterial& Material : StaticMesh->GetStaticMaterials())
		{
			if (Material.ImportedMaterialSlotName != NAME_None)
			{
				AssetImportMaterialNames.Add(Material.ImportedMaterialSlotName.ToString());
			}
			else if (Material.MaterialSlotName != NAME_None)
			{
				AssetImportMaterialNames.Add(Material.MaterialSlotName.ToString());
			}
			else if (Material.MaterialInterface)
			{
				AssetImportMaterialNames.Add(Material.MaterialInterface->GetName());
			}
			else
			{
				AssetImportMaterialNames.Add(NoName + FString::FromInt(NameIndex++));
			}
		}

		TArray<FString> StaticMeshes;
		TransientBaseNodeContainer->GetNodes(UInterchangeStaticMeshFactoryNode::StaticClass(), StaticMeshes);
		if (StaticMeshes.Num() == 1)
		{
			//Grab all imported materials
			UInterchangeStaticMeshFactoryNode* MeshFactoryNode = Cast<UInterchangeStaticMeshFactoryNode>(TransientBaseNodeContainer->GetFactoryNode(StaticMeshes[0]));
			MeshFactoryNode->GetSlotMaterialDependencies(MaterialNodePerMaterialSlotName);
		}
	}
	else if (ensure(SkeletalMesh))
	{
		ReimportObject = SkeletalMesh;
		for (FSkeletalMaterial& Material : SkeletalMesh->GetMaterials())
		{
#if WITH_EDITORONLY_DATA
			if (Material.ImportedMaterialSlotName != NAME_None)
			{
				AssetImportMaterialNames.Add(Material.ImportedMaterialSlotName.ToString());
			}
			else
#endif // WITH_EDITORONLY_DATA
				if (Material.MaterialSlotName != NAME_None)
				{
					AssetImportMaterialNames.Add(Material.MaterialSlotName.ToString());
				}
				else if (Material.MaterialInterface)
				{
					AssetImportMaterialNames.Add(Material.MaterialInterface->GetName());
				}
				else
				{
					AssetImportMaterialNames.Add(NoName + FString::FromInt(NameIndex++));
				}
		}

		TArray<FString> SkeletalMeshes;
		TransientBaseNodeContainer->GetNodes(UInterchangeSkeletalMeshFactoryNode::StaticClass(), SkeletalMeshes);
		if (SkeletalMeshes.Num() == 1)
		{
			//Grab all imported materials
			UInterchangeSkeletalMeshFactoryNode* MeshFactoryNode = Cast<UInterchangeSkeletalMeshFactoryNode>(TransientBaseNodeContainer->GetFactoryNode(SkeletalMeshes[0]));
			MeshFactoryNode->GetSlotMaterialDependencies(MaterialNodePerMaterialSlotName);
		}
	}
	//Compare and cache the results
	bool bHasConflict = MaterialNodePerMaterialSlotName.IsEmpty() && AssetImportMaterialNames.Num() > 1; //If there is no material dependencies for the mesh factory node and we have more then one material in the static mesh, we have a conflict
	TArray<int32> MatchMaterials;
	int32 MatchMaterialCount = 0;
	int32 ImportMaterialIndex = 0;
	const bool bImportMoreMaterial = MaterialNodePerMaterialSlotName.Num() > AssetImportMaterialNames.Num();
	for (TPair<FString, FString> MaterialSlotNameAndMaterialNode : MaterialNodePerMaterialSlotName)
	{
		int32& MatchMaterial = MatchMaterials.Add_GetRef(INDEX_NONE);
		const FString& ImportMaterialName = MaterialSlotNameAndMaterialNode.Key;
		bool bFoundMatch = false;
		for (int32 AssetMaterialIndex = 0; AssetMaterialIndex < AssetImportMaterialNames.Num(); ++AssetMaterialIndex)
		{
			const FString& AssetMaterialName = AssetImportMaterialNames[AssetMaterialIndex];
			if (ImportMaterialName.Equals(AssetMaterialName))
			{
				bFoundMatch = true;
				MatchMaterial = AssetMaterialIndex;
				MatchMaterialCount++;
				break;
			}
		}
		if (!bFoundMatch)
		{
			bHasConflict = true;
		}
		ImportMaterialIndex++;
	}

	//Remove the conflict if we match all original asset materials
	if (bHasConflict && bImportMoreMaterial && MatchMaterialCount == AssetImportMaterialNames.Num())
	{
		bHasConflict = false;
	}

	//We have a conflict, make sure the MatchMaterials array is in sync with MaterialNodePerMaterialSlotName.
	if (bHasConflict && MaterialNodePerMaterialSlotName.Num() == MatchMaterials.Num())
	{
		//Throw all unmatch material into the conflict data
		FInterchangeConflictInfo& MaterialConflict = ConflictInfos.AddDefaulted_GetRef();
		MaterialConflict.DisplayName = TEXT("Materials");
		MaterialConflict.Description = TEXT("There is some unmatched materials");
		MaterialConflict.Pipeline = this;
		MaterialConflict.UniqueId = FGuid::NewGuid();
		MaterialConflict.AffectedAssetClasses.Add(UMaterialInterface::StaticClass());

		//Cache the data so we do not have to redo the works when we will show the conflict
		MaterialConflictData.ConflictUniqueId = MaterialConflict.UniqueId;
		MaterialConflictData.AssetMaterialNames = AssetImportMaterialNames;
		for (TPair<FString, FString> MaterialSlotNameAndMaterialNode : MaterialNodePerMaterialSlotName)
		{
			const FString& ImportMaterialName = MaterialSlotNameAndMaterialNode.Key;
			MaterialConflictData.ImportMaterialNames.Add(ImportMaterialName);
		}
		MaterialConflictData.MatchMaterialIndexes = MatchMaterials;
		MaterialConflictData.ReimportObject = ReimportObject;
	}
}

void UInterchangeGenericAssetsPipeline::InternalRecursiveFillJointsFromReferenceSkeleton(TSharedPtr<FSkeletonJoint> ParentJoint, TMap<FString, TSharedPtr<FSkeletonJoint>>& Joints, const int32 BoneIndex, const FReferenceSkeleton& ReferenceSkeleton)
{
	const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton.GetRawRefBoneInfo();
	if (!BoneInfos.IsValidIndex(BoneIndex))
	{
		return;
	}
	const FMeshBoneInfo& BoneInfo = BoneInfos[BoneIndex];
	const FString BoneName = BoneInfo.Name.ToString();
	//We should not have any name collision
	if (!ensure(!Joints.Contains(BoneName)))
	{
		return;
	}
	TSharedPtr<FSkeletonJoint> SkeletonJoint = MakeShared<FSkeletonJoint>();
	SkeletonJoint->JointName = BoneName;
	SkeletonJoint->Parent = ParentJoint;
	if (ParentJoint.IsValid())
	{
		ParentJoint->Children.Add(SkeletonJoint);
	}

	Joints.Add(BoneName, SkeletonJoint);

	TArray<int32> Children;
	ReferenceSkeleton.GetRawDirectChildBones(BoneIndex, Children);
	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		InternalRecursiveFillJointsFromReferenceSkeleton(SkeletonJoint, Joints, Children[ChildIndex], ReferenceSkeleton);
	}
}

void UInterchangeGenericAssetsPipeline::InternalRecursiveFillJointsFromNodeContainer(TSharedPtr<FSkeletonJoint> ParentJoint, TMap<FString, TSharedPtr<FSkeletonJoint>>& Joints, const FString& JoinUid, const UInterchangeBaseNodeContainer* BaseNodeContainer, const bool bConvertStaticToSkeletalActive)
{
	const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(JoinUid));
	if (!ensure(SceneNode))
	{
		return;
	}

	//If we are not converting static to skeleton then we should only accept UInterchangeJointNodes
	if (!bConvertStaticToSkeletalActive && !SceneNode->IsA(UInterchangeJointNode::StaticClass()))
	{
		return;
	}

	const FString ImportBoneName = SceneNode->GetDisplayLabel();
	//We should not have any name collision
	if (Joints.Contains(ImportBoneName))
	{
		ensure(false);
		return;
	}

	TSharedPtr<FSkeletonJoint> SkeletonJoint = MakeShared<FSkeletonJoint>();
	SkeletonJoint->JointName = ImportBoneName;
	SkeletonJoint->Parent = ParentJoint;
	if (ParentJoint.IsValid())
	{
		ParentJoint->Children.Add(SkeletonJoint);
	}

	Joints.Add(ImportBoneName, SkeletonJoint);

	//Iterate children
	const TArray<FString> ChildrenIds = BaseNodeContainer->GetNodeChildrenUids(JoinUid);
	for (int32 ChildIndex = 0; ChildIndex < ChildrenIds.Num(); ++ChildIndex)
	{
		InternalRecursiveFillJointsFromNodeContainer(SkeletonJoint, Joints, ChildrenIds[ChildIndex], BaseNodeContainer, bConvertStaticToSkeletalActive);
	}
}

namespace UE::Interchange::Private
{
	void SetParentChildConflict(TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> ParentJoint)
	{
		TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> ParentJointIter = ParentJoint;
		while (ParentJointIter.IsValid() && !ParentJointIter->bChildConflict)
		{
			ParentJointIter->bChildConflict = true;
			ParentJointIter = ParentJointIter->Parent;
		}
	}

	void RecursivelyFillJointRemoved(TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> AssetJoint
		, TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> ParentJoint
		, TMap<FString, TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint>>& ConflictDataJoints)
	{
		if (!AssetJoint.IsValid())
		{
			return;
		}
		auto SetRemove = [](TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> JointMatch)
			{
				JointMatch->bMatch = false;
				JointMatch->bAdded = false;
				JointMatch->bRemoved = true;
			};
		TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> NewJoint = MakeShared<UInterchangeGenericAssetsPipeline::FSkeletonJoint>();
		NewJoint->JointName = AssetJoint->JointName;
		if (ParentJoint.IsValid())
		{
			NewJoint->Parent = ParentJoint;
			ParentJoint->Children.Add(NewJoint);
		}
		else
		{
			//Add only the root nodes
			ConflictDataJoints.Add(NewJoint->JointName, NewJoint);
		}
		SetRemove(NewJoint);
		SetParentChildConflict(ParentJoint);
		const int32 ChildCount = AssetJoint->Children.Num();
		for (int32 ChildIndex = 0; ChildIndex < ChildCount; ++ChildIndex)
		{
			TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> AssetJointChild = AssetJoint->Children[ChildIndex];
			RecursivelyFillJointRemoved(AssetJointChild, NewJoint, ConflictDataJoints);
		}
	}

	void RecursivelyFillJointAdded(TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> ImportJoint
		, TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> ParentJoint
		, TMap<FString, TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint>>& AssetJoints
		, TMap<FString, TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint>>& ConflictDataJoints)
	{
		if (!ImportJoint.IsValid())
		{
			return;
		}

		auto SetAdded = [](TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> Joint)
			{
				Joint->bMatch = false;
				Joint->bAdded = true;
				Joint->bRemoved = false;
			};
		auto SetConflict = [](TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> Joint )
			{
				Joint->bMatch = false;
				Joint->bAdded = true;
				Joint->bRemoved = false;
				Joint->bConflict = true;
			};
		TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> NewJoint = MakeShared<UInterchangeGenericAssetsPipeline::FSkeletonJoint>();
		NewJoint->JointName = ImportJoint->JointName;
		if (ParentJoint.IsValid())
		{
			NewJoint->Parent = ParentJoint;
			ParentJoint->Children.Add(NewJoint);
		}
		else
		{
			//Add only the root nodes
			ConflictDataJoints.Add(NewJoint->JointName, NewJoint);
		}
		const FString ImportJointName = NewJoint->JointName;
		if (AssetJoints.Contains(ImportJointName))
		{
			//We have a conflict
			SetConflict(NewJoint);
		}
		else
		{
			SetAdded(NewJoint);
		}
		
		SetParentChildConflict(ParentJoint);

		const int32 ChildCount = ImportJoint->Children.Num();
		for (int32 ChildIndex = 0; ChildIndex < ChildCount; ++ChildIndex)
		{
			TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> ImportJointChild = ImportJoint->Children[ChildIndex];
			RecursivelyFillJointAdded(ImportJointChild, NewJoint, AssetJoints, ConflictDataJoints);
		}
	}

	void RecursivelyFillJointMatch(TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> AssetJoint
		, TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> ImportJoint
		, TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> ParentJoint
		, TMap<FString, TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint>>& AssetJoints
		, TMap<FString, TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint>>& ConflictDataJoints)
	{
		if (!ImportJoint.IsValid() || !AssetJoint.IsValid())
		{
			return;
		}
		auto SetMatch = [](TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> JointMatch)
			{
				JointMatch->bMatch = true;
				JointMatch->bAdded = false;
				JointMatch->bRemoved = false;
				JointMatch->bChildConflict = false;
			};
		if (ImportJoint->JointName.Equals(AssetJoint->JointName, ESearchCase::IgnoreCase))
		{
			TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> NewJoint = MakeShared<UInterchangeGenericAssetsPipeline::FSkeletonJoint>();
			NewJoint->JointName = ImportJoint->JointName;
			if (ParentJoint.IsValid())
			{
				NewJoint->Parent = ParentJoint;
				ParentJoint->Children.Add(NewJoint);
			}
			else
			{
				//Add only the root nodes
				ConflictDataJoints.Add(NewJoint->JointName, NewJoint);
			}
			SetMatch(NewJoint);
			
			TArray<TPair<int32, int32>> ChildrenMatched;
			TArray<int32> ChildrenRemoved;
			TArray<int32> ChildrenAdded;
			for (int32 AssetChildIndex = 0; AssetChildIndex < AssetJoint->Children.Num(); ++AssetChildIndex)
			{
				TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> AssetJointChild = AssetJoint->Children[AssetChildIndex];
				int32 ImportMatchIndex = INDEX_NONE;
				for (int32 ImportChildIndex = 0; ImportChildIndex < ImportJoint->Children.Num(); ++ImportChildIndex)
				{
					TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> ImportJointChild = ImportJoint->Children[ImportChildIndex];
					if (ImportJointChild->JointName.Equals(AssetJointChild->JointName, ESearchCase::IgnoreCase))
					{
						ImportMatchIndex = ImportChildIndex;
						ChildrenMatched.Add(TPair<int32, int32>(AssetChildIndex, ImportChildIndex));
						break;
					}
				}
				if (ImportMatchIndex == INDEX_NONE)
				{
					ChildrenRemoved.Add(AssetChildIndex);
				}
			}

			for (int32 ImportChildIndex = 0; ImportChildIndex < ImportJoint->Children.Num(); ++ImportChildIndex)
			{
				TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> ImportJointChild = ImportJoint->Children[ImportChildIndex];
				int32 ImportMatchIndex = INDEX_NONE;
				for (int32 AssetChildIndex = 0; AssetChildIndex < AssetJoint->Children.Num(); ++AssetChildIndex)
				{
					TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> AssetJointChild = AssetJoint->Children[AssetChildIndex];
					if (ImportJointChild->JointName.Equals(AssetJointChild->JointName, ESearchCase::IgnoreCase))
					{
						ImportMatchIndex = ImportChildIndex;
						break;
					}
				}
				if (ImportMatchIndex == INDEX_NONE)
				{
					ChildrenAdded.Add(ImportChildIndex);
				}
			}
			
			//build the matched nodes
			for (TPair<int32, int32> MatchIndices : ChildrenMatched)
			{
				TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> AssetJointChild = AssetJoint->Children[MatchIndices.Key];
				TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> ImportJointChild = ImportJoint->Children[MatchIndices.Value];
				RecursivelyFillJointMatch(AssetJointChild, ImportJointChild, NewJoint, AssetJoints, ConflictDataJoints);
			}
			//build the removed nodes
			for (int32 AssetChildIndex : ChildrenRemoved)
			{
				TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> AssetJointChild = AssetJoint->Children[AssetChildIndex];
				RecursivelyFillJointRemoved(AssetJointChild, NewJoint, ConflictDataJoints);
			}
			//build the added nodes
			for (int32 ImportChildIndex : ChildrenAdded)
			{
				TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> ImportJointChild = ImportJoint->Children[ImportChildIndex];
				RecursivelyFillJointAdded(ImportJointChild, NewJoint, AssetJoints, ConflictDataJoints);
			}
		}
		else
		{
			//when match fail we have two separate branch to display
			RecursivelyFillJointRemoved(AssetJoint, ParentJoint, ConflictDataJoints);
			RecursivelyFillJointAdded(ImportJoint, ParentJoint, AssetJoints, ConflictDataJoints);
		}
	}
}

void UInterchangeGenericAssetsPipeline::CreateSkeletonConflict(USkeleton* SpecifiedSkeleton, USkeletalMesh* SkeletalMesh, UInterchangeBaseNodeContainer* TransientBaseNodeContainer)
{
	SkeletonConflictData.Reset();
	if (!SkeletalMesh && !SpecifiedSkeleton)
	{
		return;
	}
	TArray<FString> SkeletalMeshes;
	TransientBaseNodeContainer->GetNodes(UInterchangeSkeletalMeshFactoryNode::StaticClass(), SkeletalMeshes);
	if (SkeletalMeshes.Num() != 1)
	{
		return;
	}

	//Get the skeleton from the container
	UInterchangeSkeletalMeshFactoryNode* MeshFactoryNode = Cast<UInterchangeSkeletalMeshFactoryNode>(TransientBaseNodeContainer->GetFactoryNode(SkeletalMeshes[0]));
	int32 LodCount = MeshFactoryNode->GetLodDataCount();
	TArray<FString> LodDataUniqueIds;
	MeshFactoryNode->GetLodDataUniqueIds(LodDataUniqueIds);
	ensure(LodDataUniqueIds.Num() == LodCount);
	if (LodCount <= 0)
	{
		return;
	}

	TMap<FString, TSharedPtr<FSkeletonJoint>> AssetJoints;
	TMap<FString, TSharedPtr<FSkeletonJoint>> ImportedJoints;

	//Only test the LOD 0
	constexpr int32 LodIndex = 0;
	{
		FString LodUniqueId = LodDataUniqueIds[LodIndex];
		const UInterchangeSkeletalMeshLodDataNode* LodDataNode = Cast<UInterchangeSkeletalMeshLodDataNode>(TransientBaseNodeContainer->GetNode(LodUniqueId));
		if (!LodDataNode)
		{
			return;
		}
		
		FString SkeletonNodeUid;
		if (!LodDataNode->GetCustomSkeletonUid(SkeletonNodeUid))
		{
			return;
		}
		const UInterchangeSkeletonFactoryNode* SkeletonNode = Cast<UInterchangeSkeletonFactoryNode>(TransientBaseNodeContainer->GetNode(SkeletonNodeUid));
		if (!SkeletonNode)
		{
			return;
		}

		FString RootJointNodeId;
		if (!SkeletonNode->GetCustomRootJointUid(RootJointNodeId))
		{
			return;
		}

		const UInterchangeSceneNode* RootJointNode = Cast<UInterchangeSceneNode>(TransientBaseNodeContainer->GetNode(RootJointNodeId));
		if (!RootJointNode)
		{
			return;
		}

		if (!SpecifiedSkeleton && !(SkeletalMesh && SkeletalMesh->GetSkeleton()))
		{
			return;
		}

		const bool bConvertStaticToSkeletalActive = CommonMeshesProperties->bConvertStaticsWithMorphTargetsToSkeletals || CommonMeshesProperties->ForceAllMeshAsType == EInterchangeForceMeshType::IFMT_SkeletalMesh;
		//If we have a compatible skeleton we do not need to create a conflict, for stricter comparisons we check for identical skeletons to report any deviations.
		constexpr bool bCheckForIdenticalSkeleton = true;
		if (UE::Interchange::Private::FSkeletonHelper::IsCompatibleSkeleton(SpecifiedSkeleton ? SpecifiedSkeleton : SkeletalMesh->GetSkeleton()
			, RootJointNodeId
			, TransientBaseNodeContainer
			, bConvertStaticToSkeletalActive
			, bCheckForIdenticalSkeleton
			, CommonMeshesProperties->bImportSockets))
		{
			return;
		}
		InternalRecursiveFillJointsFromNodeContainer(nullptr, ImportedJoints, RootJointNodeId, TransientBaseNodeContainer, bConvertStaticToSkeletalActive);
	}

	//Get the current asset reference skeleton
	const FReferenceSkeleton& ReferenceSkeleton = SpecifiedSkeleton ? SpecifiedSkeleton->GetReferenceSkeleton() : SkeletalMesh->GetSkeleton()->GetReferenceSkeleton();
	const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton.GetRawRefBoneInfo();

	for (int32 BoneIndex = 0; BoneIndex < BoneInfos.Num(); ++BoneIndex)
	{
		const FMeshBoneInfo& BoneInfo = BoneInfos[BoneIndex];
		if (BoneInfo.ParentIndex == INDEX_NONE)
		{
			InternalRecursiveFillJointsFromReferenceSkeleton(nullptr, AssetJoints, BoneIndex, ReferenceSkeleton);
		}
	}

	TSharedPtr<FSkeletonJoint> ImportedRootJoint = nullptr;
	for (TPair<FString, TSharedPtr<FSkeletonJoint>>& NameAndImportedJoint : ImportedJoints)
	{
		if (!NameAndImportedJoint.Value->Parent.IsValid())
		{
			ImportedRootJoint = NameAndImportedJoint.Value;
			//We use the first root we found, we do not support multiple root
			break;
		}
	}
	
	TSharedPtr<FSkeletonJoint> AssetRootJoint = nullptr;
	for (TPair<FString, TSharedPtr<FSkeletonJoint>>& NameAndAssetJoint : AssetJoints)
	{
		if (!NameAndAssetJoint.Value->Parent.IsValid())
		{
			AssetRootJoint = NameAndAssetJoint.Value;
			//We use the first root we found, we do not support multiple root
			break;
		}
	}

	if (!ImportedRootJoint.IsValid() || !AssetRootJoint.IsValid())
	{
		SkeletonConflictData.Reset();
		return;
	}
	UE::Interchange::Private::RecursivelyFillJointMatch(AssetRootJoint, ImportedRootJoint, nullptr, AssetJoints, SkeletonConflictData.Joints);

	//Compare and cache the results
	FInterchangeConflictInfo& SkeletonConflict = ConflictInfos.AddDefaulted_GetRef();
	SkeletonConflict.DisplayName = TEXT("Skeleton");
	SkeletonConflict.Description = TEXT("Imported skeleton is incompatible with the asset skeleton");
	SkeletonConflict.Pipeline = this;
	SkeletonConflict.UniqueId = FGuid::NewGuid();
	SkeletonConflict.AffectedAssetClasses.Add(USkeletalMesh::StaticClass());

	SkeletonConflictData.ConflictUniqueId = SkeletonConflict.UniqueId;
	SkeletonConflictData.ReimportObject = SkeletalMesh;
}
#endif //WITH_EDITOR

TArray<FInterchangeConflictInfo> UInterchangeGenericAssetsPipeline::GetConflictInfos(UObject* ReimportObject, UInterchangeBaseNodeContainer* InBaseNodeContainer, UInterchangeSourceData* SourceData)
{
	//Dont touch any conflict data outside of the game thread
	if (!ensure(IsInGameThread()))
	{
		return {};
	}

	ConflictInfos.Reset();

#if WITH_EDITOR
	USkeleton* SpecifiedSkeleton = CommonSkeletalMeshesAndAnimationsProperties->IsSkeletonValid() 
									? CommonSkeletalMeshesAndAnimationsProperties->Skeleton.LoadSynchronous() 
									: nullptr;
	UStaticMesh* StaticMesh = Cast<UStaticMesh>(ReimportObject);
	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(ReimportObject);
	if (!StaticMesh && !SkeletalMesh && !SpecifiedSkeleton)
	{
		return ConflictInfos;
	}

	UInterchangeBaseNodeContainer* TransientBaseNodeContainer = DuplicateObject<UInterchangeBaseNodeContainer>(InBaseNodeContainer, GetTransientPackage());
    TransientBaseNodeContainer->SetChildrenCache(InBaseNodeContainer->GetChildrenCache());

	TArray<UInterchangeSourceData*> SourceDatas;
	SourceDatas.Add(SourceData);

	UInterchangeResultsContainer* OldResults = Results;
	Results = NewObject<UInterchangeResultsContainer>(GetTransientPackage());
	ExecutePipeline(TransientBaseNodeContainer, SourceDatas, FString());
	Results = OldResults;

	//Create the materials conflict
	if (StaticMesh || SkeletalMesh)
	{
		CreateMaterialConflict(StaticMesh, SkeletalMesh, TransientBaseNodeContainer);
	}

	if (SpecifiedSkeleton || SkeletalMesh)
	{
		CreateSkeletonConflict(SpecifiedSkeleton, SkeletalMesh, TransientBaseNodeContainer);
	}
#endif
	return ConflictInfos;
}

void UInterchangeGenericAssetsPipeline::ShowConflictDialog(const FGuid& ConflictUniqueId)
{
	for (const FInterchangeConflictInfo& ConflictInfo : ConflictInfos)
	{
		if (ConflictInfo.UniqueId == ConflictUniqueId)
		{
			if (ensure(ConflictInfo.Pipeline == this))
			{
				if (MaterialConflictData.ConflictUniqueId == ConflictUniqueId)
				{
					//Create the dialog widget
					TSharedRef<SInterchangeGenericAssetMaterialConflictWidget> WidgetContent = SNew(SInterchangeGenericAssetMaterialConflictWidget)
						.AssetMaterialNames(MaterialConflictData.AssetMaterialNames)
						.ImportMaterialNames(MaterialConflictData.ImportMaterialNames)
						.MatchMaterialIndexes(MaterialConflictData.MatchMaterialIndexes)
						.ReimportObject(MaterialConflictData.ReimportObject);

					UE::Interchange::PipelineHelper::ShowModalDialog(WidgetContent, MaterialConflictData.DialogTitle, FVector2D(450.0f, 200.0f));
				}
				else if (SkeletonConflictData.ConflictUniqueId == ConflictUniqueId)
				{
					TArray<TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint>> Joints;
					Joints.Reserve(SkeletonConflictData.Joints.Num());
					for (TPair<FString, TSharedPtr<FSkeletonJoint>> NameAndJoint : SkeletonConflictData.Joints)
					{
						Joints.Add(NameAndJoint.Value);
					}
					
					//Grab all skeleton reference so user can know the impact of changing the skeleton
					TArray<TSharedPtr<FString>> AssetReferencingSkeleton;
					USkeletalMesh* ReimportSkeletalMesh = Cast<USkeletalMesh>(SkeletonConflictData.ReimportObject);
					USkeleton* Skeleton = [this, ReimportSkeletalMesh]() -> USkeleton* {
						if (CommonSkeletalMeshesAndAnimationsProperties->IsSkeletonValid())
						{
							return CommonSkeletalMeshesAndAnimationsProperties->Skeleton.LoadSynchronous();
						}

						if (ReimportSkeletalMesh)
						{
							return ReimportSkeletalMesh->GetSkeleton();
						}
						
						return nullptr;
					}();

					if (Skeleton)
					{
						//Get in memory reference
						TArray<FName> MemoryDependencies;
						{
							TArray<UObject*> ReferencedObjects;
							ReferencedObjects.Add(Skeleton);
							TSet<UObject*> ReferencerObjects;
							// Use the fast reference collector to recursively find referencers until no more new ones are found.
							int32 LastObjectCount = 0;
							TArray<UObject*> FoundReferencerObjects = FReferencerFinder::GetAllReferencers(ReferencedObjects, nullptr, EReferencerFinderFlags::None);
							do
							{
								LastObjectCount = ReferencerObjects.Num();
								//Do not list transient package objects
								for (UObject* FoundReferencerObject : FoundReferencerObjects)
								{
									UClass* ObjectClass = FoundReferencerObject->GetClass();
									if (FoundReferencerObject->GetOutermost() != GetTransientPackage()
										&& !ObjectClass->IsChildOf(UInterchangePipelineBase::StaticClass())
										&& !ObjectClass->IsChildOf(UInterchangeBlueprintPipelineBase::StaticClass())
										&& !ObjectClass->IsChildOf(UInterchangePythonPipelineBase::StaticClass()))
									{
										ReferencerObjects.Add(FoundReferencerObject);
									}
								}
								FoundReferencerObjects = FReferencerFinder::GetAllReferencers(FoundReferencerObjects, nullptr, EReferencerFinderFlags::SkipInnerReferences);

							} while (LastObjectCount != ReferencerObjects.Num());

							// Add the full path name to the memory dependencies array.
							MemoryDependencies.Reserve(ReferencerObjects.Num());
							for (UObject* ReferencerObject : ReferencerObjects)
							{
								if (ReferencerObject->GetOuter()->GetClass() == UPackage::StaticClass()
									&& ReferencerObject != Skeleton)
								{
									MemoryDependencies.Add(*ReferencerObject->GetFullName());
								}
							}
						}

						//Get not in memory assets dependencies
						FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
						const FName SelectedPackageName = Skeleton->GetOutermost()->GetFName();
						//Get the Hard dependencies
						TArray<FName> HardDependencies;
						AssetRegistryModule.Get().GetReferencers(SelectedPackageName, HardDependencies, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);
						//Get the Soft dependencies
						TArray<FName> SoftDependencies;
						AssetRegistryModule.Get().GetReferencers(SelectedPackageName, SoftDependencies, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Soft);

						//Compose the All dependencies array with unique entries
						TArray<FName> AllDependencies = MemoryDependencies;

						auto AddAssetName = [&AllDependencies, &AssetRegistryModule](TArray<FName>& AssetDependencies)
							{
								for (const FName& AssetDependencyName : AssetDependencies)
								{
									const FString PackageString = AssetDependencyName.ToString();
									const FSoftObjectPath FullAssetPath(FTopLevelAssetPath(*PackageString, *FPackageName::GetLongPackageAssetName(PackageString)));
									FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FullAssetPath);
									if (AssetData.GetClass() != nullptr)
									{
										AllDependencies.AddUnique(*FString(AssetData.GetClass()->GetName() + TEXT(" ") + AssetData.GetObjectPathString()));
									}
								}
							};

						AddAssetName(HardDependencies);
						AddAssetName(SoftDependencies);

						//Construct shared string for each dependencies 
						for (const FName& AssetDependencyName : AllDependencies)
						{
							TSharedPtr<FString> AssetReferencing = MakeShareable(new FString(AssetDependencyName.ToString()));
							AssetReferencingSkeleton.Add(AssetReferencing);
						}
					}

					//Create the dialog widget
					TSharedRef<SInterchangeGenericAssetSkeletonConflictWidget> WidgetContent = SNew(SInterchangeGenericAssetSkeletonConflictWidget)
						.Joints(Joints)
						.ReimportObject(SkeletonConflictData.ReimportObject)
						.AssetReferencingSkeleton(AssetReferencingSkeleton);

					UE::Interchange::PipelineHelper::ShowModalDialog(WidgetContent, SkeletonConflictData.DialogTitle, FVector2D(650.0f, 600.0f));
				}
			}
			break;
		}
	}
}

void UInterchangeGenericAssetsPipeline::ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath)
{
	check(CommonSkeletalMeshesAndAnimationsProperties);

	if (!InBaseNodeContainer)
	{
		UE_LOGF(LogInterchangePipeline, Warning, "UInterchangeGenericAssetsPipeline: Cannot execute pre-import pipeline because InBaseNodeContrainer is null");
		return;
	}

	//Set the result container to allow error message
	//The parent Results container should be set at this point
	ensure(Results);
	{
		if (MaterialPipeline)
		{
			MaterialPipeline->SetResultsContainer(Results);
		}
		if (MeshPipeline)
		{
			MeshPipeline->SetResultsContainer(Results);
		}
		if (GroomPipeline)
		{
			GroomPipeline->SetResultsContainer(Results);
		}
		if (AnimationPipeline)
		{
			AnimationPipeline->SetResultsContainer(Results);
		}
	}

	//////////////////////////////////////////////////////////////////////////
	//Make sure all options go together
	
	//When we import only animation we need to prevent material and physic asset to be created
	if (CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations)
	{
		//We need to turn on ImportAnimation
		AnimationPipeline->bImportAnimations = true;
		
		//Turn off mesh, material, texture, physics
		MaterialPipeline->bImportMaterials = false;
		MeshPipeline->bImportStaticMeshes = false;
		MeshPipeline->bCreatePhysicsAsset = false;
		MeshPipeline->PhysicsAsset = nullptr;
		MaterialPipeline->TexturePipeline->bImportTextures = false;
	}

	//////////////////////////////////////////////////////////////////////////


	//Setup the Global import offset
	{
		//Make sure the scale value is greater than zero, warn the user in this case and set the scale to the default value 1.0f
		if (ImportOffsetUniformScale < UE_SMALL_NUMBER)
		{
			FNumberFormattingOptions FormatingOptions;
			FormatingOptions.SetMaximumFractionalDigits(6);
			FormatingOptions.SetMinimumFractionalDigits(1);
			float DefaultScaleValue = 1.0f;
			UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
			Message->Text = FText::Format(NSLOCTEXT("UInterchangeGenericAssetsPipeline", "BadImportOffsetUniformScale", "Value [{0}] for ImportOffsetUniformScale setting is too small, we will use the default value [{1}]."),
				FText::AsNumber(ImportOffsetUniformScale, &FormatingOptions),
				FText::AsNumber(DefaultScaleValue, &FormatingOptions));
			ImportOffsetUniformScale = DefaultScaleValue;
		}

		FTransform ImportOffsetTransform;
		ImportOffsetTransform.SetTranslation(ImportOffsetTranslation);
		ImportOffsetTransform.SetRotation(FQuat(ImportOffsetRotation));
		ImportOffsetTransform.SetScale3D(FVector(ImportOffsetUniformScale));

		UInterchangeCommonPipelineDataFactoryNode* CommonPipelineDataFactoryNode = UInterchangeCommonPipelineDataFactoryNode::FindOrCreateUniqueInstance(InBaseNodeContainer);
		CommonPipelineDataFactoryNode->SetCustomGlobalOffsetTransform(InBaseNodeContainer, ImportOffsetTransform);

		CommonPipelineDataFactoryNode->SetBakeMeshes(CommonMeshesProperties->bBakeMeshes);
		CommonPipelineDataFactoryNode->SetBakePivotMeshes(CommonMeshesProperties->bBakePivotMeshes);
	}

	if (MaterialPipeline)
	{
		MaterialPipeline->ScriptedExecutePipeline(InBaseNodeContainer, InSourceDatas, ContentBasePath);
	}
	if (MeshPipeline)
	{
		MeshPipeline->ScriptedExecutePipeline(InBaseNodeContainer, InSourceDatas, ContentBasePath);
	}
	if (GroomPipeline)
	{
		GroomPipeline->ScriptedExecutePipeline(InBaseNodeContainer, InSourceDatas, ContentBasePath);
	}
	if (AnimationPipeline)
	{
		AnimationPipeline->ScriptedExecutePipeline(InBaseNodeContainer, InSourceDatas, ContentBasePath);
	}

	ImplementUseSourceNameForAssetOption(InBaseNodeContainer, InSourceDatas);

	// Setup the source node sub paths, which will be used here but also by all other pipelines that run after us and create asset factory nodes
	UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(InBaseNodeContainer);

	SourceNode->SetCustomReimportStrategyFlags(int8(ReimportStrategy));

	if (bAssetTypeSubFolders)
	{
		SourceNode->SetCustomUseAssetTypeSubPathSuffix(bAssetTypeSubFolders);
	}
	if (bSceneNameSubFolder)
	{
		// Update SceneNameFolderPrefix only if it is not a reimport otherwise reimport strategy will be impacted
		// when reimporting with new file which usually has a different name
		if (CacheContextParam.ContextType != EInterchangePipelineContext::AssetReimport && CacheContextParam.ContextType != EInterchangePipelineContext::SceneReimport)
		{
			SceneNameFolderPrefix = FPaths::GetBaseFilename(InSourceDatas[0]->GetFilename());

			UInterchangeManager::GetInterchangeManager().SanitizeNameInline(
				SceneNameFolderPrefix,
				ESanitizeNameTypeFlags::ObjectName | ESanitizeNameTypeFlags::ObjectPath | ESanitizeNameTypeFlags::LongPackage
			);
		}

		SourceNode->SetCustomSubPathPrefix(SceneNameFolderPrefix);
	}

	InBaseNodeContainer->IterateNodesOfType<UInterchangeFactoryBaseNode>(
		[SourceNode, ReimportStrategyClosure = ReimportStrategy](const FString& NodeUid, UInterchangeFactoryBaseNode* FactoryNode)
		{
			// This obviously only covers the factory nodes that were created so far. Any pipeline after this will need
			// to call this themselves if they create additional asset factory nodes
			UE::Interchange::PipelineHelper::FillSubPathFromSourceNode(FactoryNode, SourceNode);

			// Make sure all factory nodes have the specified strategy
			FactoryNode->SetReimportStrategyFlags(ReimportStrategyClosure);
		}
	);
}

void UInterchangeGenericAssetsPipeline::ExecutePostFactoryPipeline(const UInterchangeBaseNodeContainer* InBaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport)
{
	if (MaterialPipeline)
	{
		MaterialPipeline->ScriptedExecutePostFactoryPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
	}
	if (MeshPipeline)
	{
		MeshPipeline->ScriptedExecutePostFactoryPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
	}
	if (GroomPipeline)
	{
		GroomPipeline->ScriptedExecutePostFactoryPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
	}
	if (AnimationPipeline)
	{
		AnimationPipeline->ScriptedExecutePostFactoryPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
	}
}

void UInterchangeGenericAssetsPipeline::ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* InBaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport)
{
	if (MaterialPipeline)
	{
		MaterialPipeline->ScriptedExecutePostImportPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
	}
	if (MeshPipeline)
	{
		MeshPipeline->ScriptedExecutePostImportPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
	}
	if (GroomPipeline)
	{
		GroomPipeline->ScriptedExecutePostImportPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
	}
	if (AnimationPipeline)
	{
		AnimationPipeline->ScriptedExecutePostImportPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
	}

	AddMetaData(CreatedAsset, InBaseNodeContainer->GetNode(NodeKey));
}

namespace UE::InterchangeGenericAssetsPipeline::Private
{
	template<typename T>
	TOptional<FString> GetValueAsString(const UInterchangeBaseNode* Node, const FString& AttributeName)
	{
		TOptional<FString> PayloadKey;
		T Value{};
		if (UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(Node, AttributeName, Value, PayloadKey))
		{
			return UE::Interchange::AttributeValueToString(Value);
		}
		return {};
	}
}

void UInterchangeGenericAssetsPipeline::AddMetaData(UObject* CreatedAsset, const UInterchangeBaseNode* Node)
{
	if (!CreatedAsset || !Node)
	{
		return;
	}

	const FString InterchangeMetaDataPrefix = TEXT("INTERCHANGE.");

	const bool bAssetIsActor = CreatedAsset->IsA<AActor>();
#if WITH_EDITORONLY_DATA
	FMetaData& MetaData = CreatedAsset->GetPackage()->GetMetaData();
#endif

	TMap<FString, FString> MetaDataMap;
#ifndef WITH_EDITORONLY_DATA
	if (bAssetIsActor)
#endif
	{
		TArray<FInterchangeUserDefinedAttributeInfo> UserAttributeInfos;
		UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttributeInfos(Node, UserAttributeInfos);
		//We must convert all different type to String since meta data only support string
		for (const FInterchangeUserDefinedAttributeInfo& UserAttributeInfo : UserAttributeInfos)
		{
			if (UserAttributeInfo.PayloadKey.IsSet())
			{
				//Skip animated attributes
				continue;
			}
			TOptional<FString> MetaDataValue;
			TOptional<FString> PayloadKey;
			switch (UserAttributeInfo.Type)
			{
				using namespace UE::InterchangeGenericAssetsPipeline::Private;

				case UE::Interchange::EAttributeTypes::Bool:
					MetaDataValue = GetValueAsString<bool>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Color:
					MetaDataValue = GetValueAsString<FColor>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::DateTime:
					MetaDataValue = GetValueAsString<FDateTime>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Double:
					MetaDataValue = GetValueAsString<double>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Enum:
					MetaDataValue = GetValueAsString<uint8>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Float:
					MetaDataValue = GetValueAsString<float>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Guid:
					MetaDataValue = GetValueAsString<FGuid>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Int8:
					MetaDataValue = GetValueAsString<int8>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Int16:
					MetaDataValue = GetValueAsString<int16>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Int32:
					MetaDataValue = GetValueAsString<int32>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Int64:
					MetaDataValue = GetValueAsString<int64>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::IntRect:
					MetaDataValue = GetValueAsString<FIntRect>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::LinearColor:
					MetaDataValue = GetValueAsString<FLinearColor>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Name:
					MetaDataValue = GetValueAsString<FName>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::RandomStream:
					MetaDataValue = GetValueAsString<FRandomStream>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::String:
					MetaDataValue = GetValueAsString<FString>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Timespan:
					MetaDataValue = GetValueAsString<FTimespan>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::TwoVectors:
					MetaDataValue = GetValueAsString<FTwoVectors>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::UInt8:
					MetaDataValue = GetValueAsString<uint8>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::UInt16:
					MetaDataValue = GetValueAsString<uint16>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::UInt32:
					MetaDataValue = GetValueAsString<uint32>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::UInt64:
					MetaDataValue = GetValueAsString<uint64>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Vector2d:
					MetaDataValue = GetValueAsString<FVector2d>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::IntPoint:
					MetaDataValue = GetValueAsString<FIntPoint>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::IntVector:
					MetaDataValue = GetValueAsString<FIntVector>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Vector2DHalf:
					MetaDataValue = GetValueAsString<FVector2DHalf>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Float16:
					MetaDataValue = GetValueAsString<FFloat16>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::OrientedBox:
					MetaDataValue = GetValueAsString<FOrientedBox>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::FrameNumber:
					MetaDataValue = GetValueAsString<FFrameNumber>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::FrameRate:
					MetaDataValue = GetValueAsString<FFrameRate>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::FrameTime:
					MetaDataValue = GetValueAsString<FFrameTime>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::SoftObjectPath:
					MetaDataValue = GetValueAsString<FSoftObjectPath>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Matrix44f:
					MetaDataValue = GetValueAsString<FMatrix44f>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Matrix44d:
					MetaDataValue = GetValueAsString<FMatrix44d>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Plane4f:
					MetaDataValue = GetValueAsString<FPlane4f>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Plane4d:
					MetaDataValue = GetValueAsString<FPlane4d>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Quat4f:
					MetaDataValue = GetValueAsString<FQuat4f>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Quat4d:
					MetaDataValue = GetValueAsString<FQuat4d>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Rotator3f:
					MetaDataValue = GetValueAsString<FRotator3f>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Rotator3d:
					MetaDataValue = GetValueAsString<FRotator3d>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Transform3f:
					MetaDataValue = GetValueAsString<FTransform3f>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Transform3d:
					MetaDataValue = GetValueAsString<FTransform3d>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Vector3f:
					MetaDataValue = GetValueAsString<FVector3f>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Vector3d:
					MetaDataValue = GetValueAsString<FVector3d>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Vector2f:
					MetaDataValue = GetValueAsString<FVector2f>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Vector4f:
					MetaDataValue = GetValueAsString<FVector4f>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Vector4d:
					MetaDataValue = GetValueAsString<FVector4d>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Box2f:
					MetaDataValue = GetValueAsString<FBox2f>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Box2D:
					MetaDataValue = GetValueAsString<FBox2D>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Box3f:
					MetaDataValue = GetValueAsString<FBox3f>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Box3d:
					MetaDataValue = GetValueAsString<FBox3d>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::BoxSphereBounds3f:
					MetaDataValue = GetValueAsString<FBoxSphereBounds3f>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::BoxSphereBounds3d:
					MetaDataValue = GetValueAsString<FBoxSphereBounds3d>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Sphere3f:
					MetaDataValue = GetValueAsString<FSphere3f>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Sphere3d:
					MetaDataValue = GetValueAsString<FSphere3d>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::BoolArray:
					MetaDataValue = GetValueAsString<TArray<bool>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::ColorArray:
					MetaDataValue = GetValueAsString<TArray<FColor>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::DateTimeArray:
					MetaDataValue = GetValueAsString<TArray<FDateTime>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::DoubleArray:
					MetaDataValue = GetValueAsString<TArray<double>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::EnumArray:
					MetaDataValue = GetValueAsString<TArray<uint8>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::FloatArray:
					MetaDataValue = GetValueAsString<TArray<float>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::GuidArray:
					MetaDataValue = GetValueAsString<TArray<FGuid>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Int8Array:
					MetaDataValue = GetValueAsString<TArray<uint8>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Int16Array:
					MetaDataValue = GetValueAsString<TArray<int16>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Int32Array:
					MetaDataValue = GetValueAsString<TArray<int32>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Int64Array:
					MetaDataValue = GetValueAsString<TArray<int64>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::IntRectArray:
					MetaDataValue = GetValueAsString<TArray<FIntRect>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::LinearColorArray:
					MetaDataValue = GetValueAsString<TArray<FLinearColor>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::NameArray:
					MetaDataValue = GetValueAsString<TArray<FName>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::RandomStreamArray:
					MetaDataValue = GetValueAsString<TArray<FRandomStream>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::StringArray:
					MetaDataValue = GetValueAsString<TArray<FString>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::TimespanArray:
					MetaDataValue = GetValueAsString<TArray<FTimespan>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::TwoVectorsArray:
					MetaDataValue = GetValueAsString<TArray<FTwoVectors>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::ByteArray:
					MetaDataValue = GetValueAsString<TArray<uint8>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::ByteArray64:
					MetaDataValue = GetValueAsString<TArray64<uint8>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::UInt16Array:
					MetaDataValue = GetValueAsString<TArray<uint16>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::UInt32Array:
					MetaDataValue = GetValueAsString<TArray<uint32>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::UInt64Array:
					MetaDataValue = GetValueAsString<TArray<uint64>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Vector2dArray:
					MetaDataValue = GetValueAsString<TArray<FVector2d>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::IntPointArray:
					MetaDataValue = GetValueAsString<TArray<FIntPoint>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::IntVectorArray:
					MetaDataValue = GetValueAsString<TArray<FIntVector>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Vector2DHalfArray:
					MetaDataValue = GetValueAsString<TArray<FVector2DHalf>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Float16Array:
					MetaDataValue = GetValueAsString<TArray<FFloat16>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::OrientedBoxArray:
					MetaDataValue = GetValueAsString<TArray<FOrientedBox>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::FrameNumberArray:
					MetaDataValue = GetValueAsString<TArray<FFrameNumber>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::FrameRateArray:
					MetaDataValue = GetValueAsString<TArray<FFrameRate>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::FrameTimeArray:
					MetaDataValue = GetValueAsString<TArray<FFrameTime>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::SoftObjectPathArray:
					MetaDataValue = GetValueAsString<TArray<FSoftObjectPath>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Matrix44fArray:
					MetaDataValue = GetValueAsString<TArray<FMatrix44f>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Matrix44dArray:
					MetaDataValue = GetValueAsString<TArray<FMatrix44d>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Plane4fArray:
					MetaDataValue = GetValueAsString<TArray<FPlane4f>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Plane4dArray:
					MetaDataValue = GetValueAsString<TArray<FPlane4d>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Quat4fArray:
					MetaDataValue = GetValueAsString<TArray<FQuat4f>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Quat4dArray:
					MetaDataValue = GetValueAsString<TArray<FQuat4d>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Rotator3fArray:
					MetaDataValue = GetValueAsString<TArray<FRotator3f>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Rotator3dArray:
					MetaDataValue = GetValueAsString<TArray<FRotator3d>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Transform3fArray:
					MetaDataValue = GetValueAsString<TArray<FTransform3f>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Transform3dArray:
					MetaDataValue = GetValueAsString<TArray<FTransform3d>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Vector3fArray:
					MetaDataValue = GetValueAsString<TArray<FVector3f>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Vector3dArray:
					MetaDataValue = GetValueAsString<TArray<FVector3d>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Vector2fArray:
					MetaDataValue = GetValueAsString<TArray<FVector2f>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Vector4fArray:
					MetaDataValue = GetValueAsString<TArray<FVector4f>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Vector4dArray:
					MetaDataValue = GetValueAsString<TArray<FVector4d>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Box2fArray:
					MetaDataValue = GetValueAsString<TArray<FBox2f>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Box2DArray:
					MetaDataValue = GetValueAsString<TArray<FBox2D>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Box3fArray:
					MetaDataValue = GetValueAsString<TArray<FBox3f>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Box3dArray:
					MetaDataValue = GetValueAsString<TArray<FBox3d>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::BoxSphereBounds3fArray:
					MetaDataValue = GetValueAsString<TArray<FBoxSphereBounds3f>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::BoxSphereBounds3dArray:
					MetaDataValue = GetValueAsString<TArray<FBoxSphereBounds3d>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Sphere3fArray:
					MetaDataValue = GetValueAsString<TArray<FSphere3f>>(Node, UserAttributeInfo.Name);
					break;
				case UE::Interchange::EAttributeTypes::Sphere3dArray:
					MetaDataValue = GetValueAsString<TArray<FSphere3d>>(Node, UserAttributeInfo.Name);
					break;
				default:
					ensureMsgf(false, TEXT("Unsupported EAttributeTypes value!"));
					break;
			}
			if (MetaDataValue.IsSet())
			{
				FString MetaDataKeyString = UserAttributeInfo.Name;
				MetaDataMap.Add(MetaDataKeyString, MetaDataValue.GetValue());
			}
		}
	}

	if (bAssetIsActor)
	{
		if (AActor* Actor = Cast<AActor>(CreatedAsset))
		{
			if (IInterface_AssetUserData* AssetUserData = Cast<IInterface_AssetUserData>(Actor->GetRootComponent()))
			{
				UInterchangeAssetUserData* InterchangeUserData = AssetUserData->GetAssetUserData< UInterchangeAssetUserData >();

				if (!InterchangeUserData)
				{
					if (MetaDataMap.Num() > 0)
					{
						InterchangeUserData = NewObject<UInterchangeAssetUserData>(Actor->GetRootComponent(), NAME_None, RF_Public | RF_Transactional);
						AssetUserData->AddAssetUserData(InterchangeUserData);
					}
				}
				else
				{
					InterchangeUserData->MetaData.Reset();
				}

				if (InterchangeUserData)
				{
					for (const TPair<FString, FString>& MetaDataPair : MetaDataMap)
					{
						InterchangeUserData->MetaData.Add(MetaDataPair.Key, MetaDataPair.Value);
					}

					InterchangeUserData->MetaData.KeySort(TLess<FString>());
				}
			}
		}
	}

#if WITH_EDITORONLY_DATA
	//Add UObject package meta data
	//Cleanup existing INTERCHANGE_ prefix metadata name for this object (in case we re-import)
	{
		TArray<FName> InterchangeMetaDataKeys;
		if(TMap<FName, FString>* MetaDataMapPtr = MetaData.GetMapForObject(CreatedAsset))
		{
			for (const TPair<FName, FString>& ObjectMetadata : *MetaDataMapPtr)
			{
				if (ObjectMetadata.Key.ToString().StartsWith(InterchangeMetaDataPrefix))
				{
					InterchangeMetaDataKeys.Add(ObjectMetadata.Key);
				}
			}
			for (const FName& MetaDataKey : InterchangeMetaDataKeys)
			{
				MetaData.RemoveValue(CreatedAsset, MetaDataKey);
			}
		}
	}
	for (const TPair<FString, FString>& MetaDataPair : MetaDataMap)
	{
		FString MetaDataKeyStr = InterchangeMetaDataPrefix + MetaDataPair.Key;
		if (MetaDataKeyStr.Len() < NAME_SIZE)
		{
			const FName& MetaDataKey = FName(MetaDataKeyStr);
			//SetValue either add the key or set the new value
			MetaData.SetValue(CreatedAsset, MetaDataKey, *MetaDataPair.Value);
		}
		else if (!bHasNotify_MetaDataAttributeKeyNameTooLong)
		{
			bHasNotify_MetaDataAttributeKeyNameTooLong = true;
			//We cannot add this meta data, notify the user the meta attribute key name is too long
			UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
			Message->Text = FText::Format(NSLOCTEXT("UInterchangeGenericAssetsPipeline", "MetadataKeyNameTooLong", "One or more metadata key(s) cannot be added because the name exceeds the maximum length ({0}) allowed by the engine. The metadata is provided by the source file node's custom attributes."),
				FText::AsNumber(NAME_SIZE));
		}
	}
#endif
}

void UInterchangeGenericAssetsPipeline::SetReimportSourceIndex(UClass* ReimportObjectClass, const int32 SourceFileIndex)
{
	if (MaterialPipeline)
	{
		MaterialPipeline->ScriptedSetReimportSourceIndex(ReimportObjectClass, SourceFileIndex);
	}

	if (MeshPipeline)
	{
		MeshPipeline->ScriptedSetReimportSourceIndex(ReimportObjectClass, SourceFileIndex);
	}

	if (GroomPipeline)
	{
		GroomPipeline->ScriptedSetReimportSourceIndex(ReimportObjectClass, SourceFileIndex);
	}

	if (AnimationPipeline)
	{
		AnimationPipeline->ScriptedSetReimportSourceIndex(ReimportObjectClass, SourceFileIndex);
	}
}

void UInterchangeGenericAssetsPipeline::ImplementUseSourceNameForAssetOption(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas)
{
	const UClass* SkeletalMeshFactoryNodeClass = UInterchangeSkeletalMeshFactoryNode::StaticClass();
	TArray<FString> SkeletalMeshNodeUids;
	InBaseNodeContainer->GetNodes(SkeletalMeshFactoryNodeClass, SkeletalMeshNodeUids);

	const UClass* StaticMeshFactoryNodeClass = UInterchangeStaticMeshFactoryNode::StaticClass();
	TArray<FString> StaticMeshNodeUids;
	InBaseNodeContainer->GetNodes(StaticMeshFactoryNodeClass, StaticMeshNodeUids);

	const UClass* AnimSequenceFactoryNodeClass = UInterchangeAnimSequenceFactoryNode::StaticClass();
	TArray<FString> AnimSequenceNodeUids;
	InBaseNodeContainer->GetNodes(AnimSequenceFactoryNodeClass, AnimSequenceNodeUids);

	//If we import only one mesh, we want to rename the mesh using the file name.
	const int32 MeshesImportedNodeCount = SkeletalMeshNodeUids.Num() + StaticMeshNodeUids.Num();

	FString OverrideAssetName = IsStandAlonePipeline() ? DestinationName : FString();
	if(OverrideAssetName.IsEmpty() && IsStandAlonePipeline())
	{
		OverrideAssetName = AssetName;
	}

	//SkeletalMesh it must always be run even if there is no rename option, skeleton and physics asset will be rename properly
	MeshPipeline->ImplementUseSourceNameForAssetOptionSkeletalMesh(MeshesImportedNodeCount, bUseSourceNameForAsset, OverrideAssetName);
	AnimationPipeline->ImplementUseSourceNameForAssetOptionAnimSequence(bUseSourceNameForAsset, OverrideAssetName);

	if (bUseSourceNameForAsset || !OverrideAssetName.IsEmpty())
	{
		//StaticMesh
		if (MeshesImportedNodeCount == 1 && StaticMeshNodeUids.Num() > 0)
		{
			UInterchangeStaticMeshFactoryNode* StaticMeshNode = Cast<UInterchangeStaticMeshFactoryNode>(InBaseNodeContainer->GetFactoryNode(StaticMeshNodeUids[0]));
			const FString DisplayLabelName = OverrideAssetName.IsEmpty() ? FPaths::GetBaseFilename(InSourceDatas[0]->GetFilename()) : OverrideAssetName;
			StaticMeshNode->SetDisplayLabel(DisplayLabelName);
		}

		//Animation, simply look if we import only 1 animation before applying the option to animation
		if (AnimSequenceNodeUids.Num() == 1)
		{
			UInterchangeAnimSequenceFactoryNode* AnimSequenceNode = Cast<UInterchangeAnimSequenceFactoryNode>(InBaseNodeContainer->GetFactoryNode(AnimSequenceNodeUids[0]));
			FString DisplayLabelName = (OverrideAssetName.IsEmpty() ? FPaths::GetBaseFilename(InSourceDatas[0]->GetFilename()) : OverrideAssetName);
			const bool bImportOnlyAnimations = CommonSkeletalMeshesAndAnimationsProperties && CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations;
			if (!bImportOnlyAnimations)
			{
				DisplayLabelName += TEXT("_Anim");
			}
			AnimSequenceNode->SetDisplayLabel(DisplayLabelName);
		}
	}
}

class SInterchangeGenericAssetMaterialConflictListRow : public SMultiColumnTableRow<TSharedPtr<SInterchangeGenericAssetMaterialConflictWidget::FListItem>>
{
public:
	SLATE_BEGIN_ARGS(SInterchangeGenericAssetMaterialConflictListRow) {}
		SLATE_ARGUMENT(TSharedPtr<SInterchangeGenericAssetMaterialConflictWidget::FListItem>, Item)
	SLATE_END_ARGS()

	TSharedPtr<SInterchangeGenericAssetMaterialConflictWidget::FListItem> Item;

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable)
	{
		Item = InArgs._Item;

		SMultiColumnTableRow<TSharedPtr<SInterchangeGenericAssetMaterialConflictWidget::FListItem>>::Construct(
			FSuperRowType::FArguments()
			.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow"),
			OwnerTable
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		FSlateColor UnmatchedColor(FLinearColor(0.7f, 0.3f, 0.0f));
		if (ColumnName == SInterchangeGenericAssetMaterialConflictWidget::NAME_Import)
		{
			bool bConflict = Item->bMatched == INDEX_NONE;
			FSlateColor SlateColor = bConflict ? UnmatchedColor : FSlateColor::UseForeground();
			FText Tooltip;
			if (bConflict)
			{
				Tooltip = NSLOCTEXT("InterchangeGenericAssetPipeline", "SInterchangeGenericAssetMaterialConflictListRow_Conflict_unmatched", "Import material is unmatched");
			}
			else
			{
				Tooltip = FText::FromString(
					FText(NSLOCTEXT("InterchangeGenericAssetPipeline", "SInterchangeGenericAssetMaterialConflictListRow_Conflict_matched", "Import material is matched with original asset index: ")).ToString()
					+ FString::FromInt(Item->bMatched));
			}
			return SNew(SBox)
			.Padding(2.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Item->ImportName))
				.ToolTipText(Tooltip)
				.ColorAndOpacity(SlateColor)
			];
		}
		else
		{
			return SNew(SBox)
				.Padding(2.0f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(FText::FromString(Item->AssetName))
				];
		}
	}
};

void SInterchangeGenericAssetMaterialConflictWidget::Construct(const FArguments& InArgs)
{
	AssetMaterialNames = InArgs._AssetMaterialNames;
	ImportMaterialNames = InArgs._ImportMaterialNames;
	MatchMaterialIndexes = InArgs._MatchMaterialIndexes;
	ReimportObject = InArgs._ReimportObject;

	if (!ensure(ReimportObject))
	{
		return;
	}

	int32 RowNumber = FMath::Max(AssetMaterialNames.Num(), ImportMaterialNames.Num());
	for (int32 RowIndex = 0; RowIndex < RowNumber; ++RowIndex)
	{
		TSharedRef<FListItem> Item = MakeShared<FListItem>();
		Item->ImportName = ImportMaterialNames.IsValidIndex(RowIndex) ? ImportMaterialNames[RowIndex] : FString();
		Item->bMatched = INDEX_NONE;
		if (MatchMaterialIndexes.IsValidIndex(RowIndex))
		{
			Item->bMatched = MatchMaterialIndexes[RowIndex];
			if (AssetMaterialNames.IsValidIndex(Item->bMatched))
			{
				Item->AssetMatchedName = AssetMaterialNames[Item->bMatched];
			}
		}
		Item->AssetName = AssetMaterialNames.IsValidIndex(RowIndex) ? AssetMaterialNames[RowIndex] : FString();
		RowItems.Add(Item);
	}

	this->ChildSlot
	[
		SNew(SBox)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(0.0f, 0.0f, 0.0f, 8.0f)
			.FillHeight(1.0f)
			[
				SNew(SBorder)
				[
					SAssignNew(MaterialList, SListView<TSharedPtr<FListItem>>)
					.ListItemsSource(&RowItems)
					.OnGenerateRow(this, &SInterchangeGenericAssetMaterialConflictWidget::OnGenerateRow)
					.HeaderRow
					(
						SNew(SHeaderRow)

						+ SHeaderRow::Column(NAME_Import)
						.FillWidth(0.5f)
						.DefaultLabel(NSLOCTEXT("GenericAssetPipeline", "SInterchangeGenericAssetMaterialConflictWidget_ImportName", "Import"))

						+ SHeaderRow::Column(NAME_Asset)
						.FillWidth(0.5f)
						.DefaultLabel(NSLOCTEXT("GenericAssetPipeline", "SInterchangeGenericAssetMaterialConflictWidget_AssetName", "Asset"))
					)
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.Padding(2)
			[
				SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(NSLOCTEXT("SInterchangeGenericAssetMaterialConflictWidget", "SInterchangeGenericAssetMaterialConflictWidget_Done", "Done"))
					.OnClicked(this, &SInterchangeGenericAssetMaterialConflictWidget::OnDone)
			]
		]
	];
}

TSharedRef<ITableRow> SInterchangeGenericAssetMaterialConflictWidget::OnGenerateRow(TSharedPtr<SInterchangeGenericAssetMaterialConflictWidget::FListItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SInterchangeGenericAssetMaterialConflictListRow, OwnerTable)
		.Item(Item);
}

FReply SInterchangeGenericAssetMaterialConflictWidget::OnDone()
{
	if (WidgetWindow.IsValid())
	{
		WidgetWindow->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply SInterchangeGenericAssetMaterialConflictWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnDone();
	}
	return FReply::Unhandled();
}

void SInterchangeGenericAssetSkeletonConflictWidget::Construct(const FArguments& InArgs)
{
	bShowSectionFlag[(uint8)EInterchangeSkeletonCompareSection::Skeleton] = true;
	bShowSectionFlag[(uint8)EInterchangeSkeletonCompareSection::References] = true;

	AssetReferencingSkeleton = InArgs._AssetReferencingSkeleton;
	Joints = InArgs._Joints;
	ReimportObject = InArgs._ReimportObject;

	// Skeleton comparison
	TSharedPtr<SWidget> SkeletonCompareSection = ConstructSkeletonComparison();
	TSharedPtr<SWidget> SkeletonReferencesSection = ConstructSkeletonReference();

	this->ChildSlot
	[
		SNew(SBox)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					.Padding(2.0f)
					[
						SNew(SSplitter)
						.Orientation(Orient_Vertical)
						.ResizeMode(ESplitterResizeMode::Fill)
						+ SSplitter::Slot()
						.Value(0.8f)
						[
							// Skeleton Compare section
							SkeletonCompareSection.ToSharedRef()
						]
						+ SSplitter::Slot()
						.Value(0.2f)
						[
							// Skeleton Compare section
							SkeletonReferencesSection.ToSharedRef()
						]
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(2.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(NSLOCTEXT("SInterchangeGenericAssetSkeletonConflictWidget", "ConstructDoneButton", "Done"))
					.OnClicked(this, &SInterchangeGenericAssetSkeletonConflictWidget::OnDone)
				]
			]
		]
	];
}

namespace UE::Interchange::Private
{
	static bool ExpansionFilter_ConflictFilter(TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> Joint)
	{
		return Joint.IsValid() && Joint->bMatch && Joint->bChildConflict;
	}
}

FReply SInterchangeGenericAssetSkeletonConflictWidget::OnExpandToConflict()
{
	using namespace UE::Interchange::Private;
	for (TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> Joint : Joints)
	{
		OnSetExpansionRecursive(Joint, false);
	}

	for (TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> Joint : Joints)
	{
		OnSetExpansionRecursive(Joint, true, FExpansionPredicate::CreateStatic(&ExpansionFilter_ConflictFilter));
	}
	return FReply::Handled();
}

FReply SInterchangeGenericAssetSkeletonConflictWidget::SetSectionVisible(EInterchangeSkeletonCompareSection SectionIndex)
{
	bShowSectionFlag[(uint8)SectionIndex] = !bShowSectionFlag[(uint8)SectionIndex];
	return FReply::Handled();
}

EVisibility SInterchangeGenericAssetSkeletonConflictWidget::IsSectionVisible(EInterchangeSkeletonCompareSection SectionIndex)
{
	return bShowSectionFlag[(uint8)SectionIndex] ? EVisibility::Visible : EVisibility::Collapsed;
}

const FSlateBrush* SInterchangeGenericAssetSkeletonConflictWidget::GetCollapsableArrow(EInterchangeSkeletonCompareSection SectionIndex) const
{
	return bShowSectionFlag[(uint8)SectionIndex] ? FAppStyle::GetBrush("Symbols.DownArrow") : FAppStyle::GetBrush("Symbols.RightArrow");
}

void SInterchangeGenericAssetSkeletonConflictWidget::OnSetExpansionRecursive(TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> SkeletonJoint, bool bShouldBeExpanded, const FExpansionPredicate Predicate)
{
	if (!SkeletonJoint.IsValid())
	{
		return;
	}

	if (Predicate.IsBound())
	{
		if (!Predicate.Execute(SkeletonJoint))
		{
			return;
		}
	}

	CompareTree->SetItemExpansion(SkeletonJoint, bShouldBeExpanded);
	for (TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> ChildJoint : SkeletonJoint->Children)
	{
		OnSetExpansionRecursive(ChildJoint, bShouldBeExpanded, Predicate);
	}
}

TSharedPtr<SWidget> SInterchangeGenericAssetSkeletonConflictWidget::ConstructSkeletonComparison()
{
	using namespace UE::Interchange::Private;
	FText SkeletonStatus = NSLOCTEXT("SInterchangeGenericAssetSkeletonConflictWidget", "ConstructSkeletonComparison_SkeletonStatus", "The skeleton has some conflicts");
	
	CompareTree = SNew(STreeView<TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint>>)
		.SelectionMode(ESelectionMode::None)
		.TreeItemsSource(&Joints)
		.OnGenerateRow(this, &SInterchangeGenericAssetSkeletonConflictWidget::OnGenerateRowCompareTreeView)
		.OnGetChildren(this, &SInterchangeGenericAssetSkeletonConflictWidget::OnGetChildrenRowCompareTreeView)
		.OnSetExpansionRecursive(
			this,
			&SInterchangeGenericAssetSkeletonConflictWidget::OnSetExpansionRecursive, 
			FExpansionPredicate());

	for (TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> Joint : Joints)
	{
		OnSetExpansionRecursive(Joint, true, FExpansionPredicate::CreateStatic(&ExpansionFilter_ConflictFilter));
	}
	
	return SNew(SBox)
	[
		SNew(SBorder)
		.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.IsFocusable(false)
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					.OnClicked(this, &SInterchangeGenericAssetSkeletonConflictWidget::SetSectionVisible, EInterchangeSkeletonCompareSection::Skeleton)
					[
						SNew(SImage).Image(this, &SInterchangeGenericAssetSkeletonConflictWidget::GetCollapsableArrow, EInterchangeSkeletonCompareSection::Skeleton)
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
					.Text(NSLOCTEXT("SInterchangeGenericAssetSkeletonConflictWidget", "ConstructSkeletonComparison_SkeletonCompareHeader", "Skeleton"))
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(2)
			[
				SNew(SBox)
				.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SInterchangeGenericAssetSkeletonConflictWidget::IsSectionVisible, EInterchangeSkeletonCompareSection::Skeleton)))
				[
					SNew(SBorder)
					.Padding(FMargin(3))
					.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SButton)
								.HAlign(HAlign_Center)
								.Text(NSLOCTEXT("SInterchangeGenericAssetSkeletonConflictWidget", "SInterchangeGenericAssetSkeletonConflic_ExpandToConflict", "Expand To Conflict"))
								.OnClicked(this, &SInterchangeGenericAssetSkeletonConflictWidget::OnExpandToConflict)
							]
							+SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Left)
							.Padding(6.0f, 0.0f)
							[
								SNew(STextBlock)
								.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
								.Text(SkeletonStatus)
								.ColorAndOpacity(SInterchangeGenericAssetMaterialConflictWidget::SlateColorFullConflict)
							]
						]
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2)
						[
							SNew(SSeparator)
							.Orientation(EOrientation::Orient_Horizontal)
						]
						+SVerticalBox::Slot()
						.FillHeight(1.0f)
						.Padding(2)
						[
							CompareTree.ToSharedRef()
						]
					]
				]
			]
		]
	];
}

TSharedPtr<SWidget> SInterchangeGenericAssetSkeletonConflictWidget::ConstructSkeletonReference()
{
	FString SkeletonReferenceStatistic;
	if (AssetReferencingSkeleton.Num() > 0)
	{
		SkeletonReferenceStatistic += TEXT("Skeleton is referenced by ") + FString::FromInt(AssetReferencingSkeleton.Num()) + TEXT(" assets.");
	}
	
	return SNew(SBox)
	[
		SNew(SBorder)
		.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.IsFocusable(false)
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					.OnClicked(this, &SInterchangeGenericAssetSkeletonConflictWidget::SetSectionVisible, EInterchangeSkeletonCompareSection::References)
					[
						SNew(SImage).Image(this, &SInterchangeGenericAssetSkeletonConflictWidget::GetCollapsableArrow, EInterchangeSkeletonCompareSection::References)
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
					.Text(NSLOCTEXT("SInterchangeGenericAssetSkeletonConflictWidget", "SkeletonReferencesHeader", "References"))
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(2)
			[
				SNew(SBox)
				.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SInterchangeGenericAssetSkeletonConflictWidget::IsSectionVisible, EInterchangeSkeletonCompareSection::References)))
				[
					SNew(SBorder)
					.Padding(FMargin(3))
					.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2)
						[
							SNew(STextBlock)
							.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
							.Text(FText::FromString(SkeletonReferenceStatistic))
						]
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2)
						[
							SNew(SSeparator)
							.Orientation(EOrientation::Orient_Horizontal)
						]
						+SVerticalBox::Slot()
						.FillHeight(1.0f)
						.Padding(2)
						[
							//Show the asset referencing this skeleton
							SNew(SListView<TSharedPtr<FString>>)
							.ListItemsSource(&AssetReferencingSkeleton)
							.OnGenerateRow(this, &SInterchangeGenericAssetSkeletonConflictWidget::OnGenerateRowAssetReferencingSkeleton)
						]
					]
				]
			]
		]
	];
}

class SInterchangeCompareSkeletonTreeViewItem : public STableRow< TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> >
{
public:

	SLATE_BEGIN_ARGS(SInterchangeCompareSkeletonTreeViewItem)
		: _SkeletonCompareData(nullptr)
		{}

		/** The item content. */
		SLATE_ARGUMENT(TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint>, SkeletonCompareData)
	SLATE_END_ARGS()

	/**
	* Construct the widget
	*
	* @param InArgs   A declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		SkeletonCompareData = InArgs._SkeletonCompareData;

		//This is suppose to always be valid
		check(SkeletonCompareData.IsValid());

		const FSlateBrush* JointIcon = FAppStyle::GetDefaultBrush();
		FString Tooltip = NSLOCTEXT("SInterchangeCompareSkeletonTreeViewItem", "Construct_Joint_tooltip", "Re-import joint match skeletal mesh skeleton joint.").ToString();
		if(SkeletonCompareData->bAdded)
		{
			JointIcon = FAppStyle::GetBrush("FBXIcon.ReimportCompareAdd");
			Tooltip = NSLOCTEXT("SInterchangeCompareSkeletonTreeViewItem", "Construct_AddJoint_tooltip", "Re-import add this joint.").ToString();
		}
		else if(SkeletonCompareData->bRemoved)
		{
			JointIcon = FAppStyle::GetBrush("FBXIcon.ReimportCompareRemoved");
			Tooltip = NSLOCTEXT("SInterchangeCompareSkeletonTreeViewItem", "Construct_RemoveJoint_tooltip", "Re-import remove this joint.").ToString();
		}
		FSlateColor ForegroundTextColor = FSlateColor::UseForeground();
		if (SkeletonCompareData->bMatch && SkeletonCompareData->bChildConflict)
		{
			ForegroundTextColor = SInterchangeGenericAssetMaterialConflictWidget::SlateColorSubConflict;
		}
		else if (!SkeletonCompareData->bMatch)
		{
			ForegroundTextColor = SInterchangeGenericAssetMaterialConflictWidget::SlateColorFullConflict;
		}

		this->ChildSlot
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SExpanderArrow, SharedThis(this))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 2.0f, 6.0f, 2.0f)
					[
						SNew(SImage)
							.Image(JointIcon)
							.Visibility(JointIcon != FAppStyle::GetDefaultBrush() ? EVisibility::Visible : EVisibility::Collapsed)
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(0.0f, 3.0f, 6.0f, 3.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(FText::FromString(SkeletonCompareData->JointName))
							.ToolTipText(FText::FromString(Tooltip))
							.ColorAndOpacity(ForegroundTextColor)
					]
			];

		STableRow< TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(true),
			InOwnerTableView
		);
	}

private:
	/** The node info to build the tree view row from. */
	TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> SkeletonCompareData;
};


TSharedRef<ITableRow> SInterchangeGenericAssetSkeletonConflictWidget::OnGenerateRowCompareTreeView(TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> RowData, const TSharedRef<STableViewBase>& Table)
{
	TSharedRef<SInterchangeCompareSkeletonTreeViewItem> ReturnRow = SNew(SInterchangeCompareSkeletonTreeViewItem, Table)
		.SkeletonCompareData(RowData);
	return ReturnRow;
}

void SInterchangeGenericAssetSkeletonConflictWidget::OnGetChildrenRowCompareTreeView(TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> InParent, TArray< TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> >& OutChildren)
{
	for (int32 ChildIndex = 0; ChildIndex < InParent->Children.Num(); ++ChildIndex)
	{
		TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> ChildJoint = InParent->Children[ChildIndex];
		if (ChildJoint.IsValid())
		{
			OutChildren.Add(ChildJoint);
		}
	}
}

TSharedRef<ITableRow> SInterchangeGenericAssetSkeletonConflictWidget::OnGenerateRowAssetReferencingSkeleton(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	int32 AssetListIndex = AssetReferencingSkeleton.Find(InItem);
	bool LightBackgroundColor = AssetListIndex % 2 == 0;
	return SNew(STableRow<TSharedPtr<FString> >, OwnerTable)
		[
			SNew(SBorder)
			.BorderImage(LightBackgroundColor ? FAppStyle::GetBrush("ToolPanel.GroupBorder") : FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
			[
				SNew(STextBlock)
				.Text(FText::FromString(*(InItem.Get())))
			]
		];
}