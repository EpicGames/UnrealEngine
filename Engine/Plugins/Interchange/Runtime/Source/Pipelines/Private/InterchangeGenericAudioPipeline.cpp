// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeGenericAudioPipeline.h"

#include "InterchangeAudioSoundWaveFactoryNode.h"
#include "InterchangeAudioSoundWaveNode.h"

#include "InterchangePipelineHelper.h"
#include "Nodes/InterchangeSourceNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGenericAudioPipeline)

#define LOCTEXT_NAMESPACE "InterchangeGenericAudioPipeline"

FString UInterchangeGenericAudioPipeline::GetPipelineCategory(UClass* AssetClass)
{
	return TEXT("Audio");
}

void UInterchangeGenericAudioPipeline::ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath)
{
	if (!InBaseNodeContainer)
	{
		UInterchangeResultError_Generic* ErrorMessage = AddMessage<UInterchangeResultError_Generic>();
		check(ErrorMessage);
		ErrorMessage->Text = LOCTEXT("InterchangeAudio_ExecutePipeline_NullContainer", "Cannot execute pre-import pipeline because InBaseNodeContainer is null.");
		return;
	}

	BaseNodeContainer = InBaseNodeContainer;

	BaseNodeContainer->IterateNodesOfType<UInterchangeAudioSoundWaveNode>([this](const FString& NodeUid, UInterchangeAudioSoundWaveNode* Node)
	{
		SoundWaveNodes.Add(Node);
	});

	if (bImportSounds)
	{
		for (const UInterchangeAudioSoundWaveNode* SoundWaveNode : SoundWaveNodes)
		{
			CreateSoundWaveFactoryNode(SoundWaveNode);
		}
	}
}

UInterchangeAudioSoundWaveFactoryNode* UInterchangeGenericAudioPipeline::CreateSoundWaveFactoryNode(const UInterchangeAudioSoundWaveNode* SoundWaveNode)
{
	if (!SoundWaveNode)
	{
		return nullptr;
	}

	const FString DisplayLabel = SoundWaveNode->GetDisplayLabel();
	const FString NodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(SoundWaveNode->GetUniqueID());

	UInterchangeAudioSoundWaveFactoryNode* SoundWaveFactoryNode = NewObject<UInterchangeAudioSoundWaveFactoryNode>(BaseNodeContainer);
	if (!ensure(SoundWaveFactoryNode))
	{
		return nullptr;
	}

	BaseNodeContainer->SetupNode(SoundWaveFactoryNode, NodeUid, DisplayLabel, EInterchangeNodeContainerType::FactoryData);
	UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(BaseNodeContainer);
	UE::Interchange::PipelineHelper::FillSubPathFromSourceNode(SoundWaveFactoryNode, SourceNode);

	SoundWaveFactoryNodes.Add(SoundWaveFactoryNode);

	SoundWaveFactoryNode->AddTargetNodeUid(SoundWaveNode->GetUniqueID());
	SoundWaveNode->AddTargetNodeUid(SoundWaveFactoryNode->GetUniqueID());

	return SoundWaveFactoryNode;
}

void UInterchangeGenericAudioPipeline::AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams)
{
	Super::AdjustSettingsForContext(ContextParams);
#if WITH_EDITOR
	TArray<FString> HideCategories;
	const bool bIsObjectASound = ContextParams.ReimportAsset ? ContextParams.ReimportAsset.IsA(USoundWave::StaticClass()) : false;
	if ((!bIsObjectASound && ContextParams.ContextType == EInterchangePipelineContext::AssetReimport)
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomLODImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomLODReimport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetAlternateSkinningImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetAlternateSkinningReimport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomMorphTargetImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomMorphTargetReImport)
	{
		bImportSounds = false;
		HideCategories.Add(UInterchangeGenericAudioPipeline::GetPipelineCategory(nullptr));
	}

	if (UInterchangePipelineBase* OuterMostPipeline = GetMostPipelineOuter())
	{
		for (const FString& HideCategoryName : HideCategories)
		{
			HidePropertiesOfCategory(OuterMostPipeline, this, HideCategoryName);
		}
	}
#endif //WITH_EDITOR
}

#if WITH_EDITOR

bool UInterchangeGenericAudioPipeline::IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UInterchangeGenericAudioPipeline, bImportSounds))
	{
		return true;
	}
	return Super::IsPropertyChangeNeedRefresh(PropertyChangedEvent);
}

void UInterchangeGenericAudioPipeline::FilterPropertiesFromTranslatedData(UInterchangeBaseNodeContainer* InBaseNodeContainer)
{
	Super::FilterPropertiesFromTranslatedData(InBaseNodeContainer);

	
	TArray<FString> TempSoundWaveNodes;
	InBaseNodeContainer->GetNodes(UInterchangeAudioSoundWaveNode::StaticClass(), TempSoundWaveNodes);
	if (TempSoundWaveNodes.Num() == 0)
	{
		//Filter out all Audio properties
		if (UInterchangePipelineBase* OuterMostPipeline = GetMostPipelineOuter())
		{
			HidePropertiesOfCategory(OuterMostPipeline, this, UInterchangeGenericAudioPipeline::GetPipelineCategory(nullptr));
		}
	}
}

void UInterchangeGenericAudioPipeline::GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const
{
	PipelineSupportAssetClasses.Add(USoundWave::StaticClass());
}

#endif //WITH_EDITOR

#undef LOCTEXT_NAMESPACE
