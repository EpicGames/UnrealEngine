// Copyright Epic Games, Inc. All Rights Reserved. 
#include "InterchangeGenericAssetsPipelineSharedSettings.h"

#include "Animation/Skeleton.h"
#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "InterchangePipelineLog.h"
#include "InterchangeSkeletonFactoryNode.h"
#include "InterchangeSkeletonHelper.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

UInterchangeSkeletonFactoryNode* UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties::CreateSkeletonFactoryNode(UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& RootJointUid, const FString& SourceFileName/*Used for Warning generation*/)
{
	const UInterchangeBaseNode* RootJointNode = BaseNodeContainer->GetNode(RootJointUid);
	if (!RootJointNode)
	{
		return nullptr;
	}

	FString DisplayLabel = RootJointNode->GetDisplayLabel() + TEXT("_Skeleton");
	FString SkeletonUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(RootJointNode->GetUniqueID());

	UInterchangeSkeletonFactoryNode* SkeletonFactoryNode = nullptr;
	if (BaseNodeContainer->IsNodeUidValid(SkeletonUid))
	{
		//The node already exist, just return it
		SkeletonFactoryNode = Cast<UInterchangeSkeletonFactoryNode>(BaseNodeContainer->GetFactoryNode(SkeletonUid));
		if (!ensure(SkeletonFactoryNode))
		{
			//Log an error
			return nullptr;
		}
		FString ExistingSkeletonRootJointUid;
		SkeletonFactoryNode->GetCustomRootJointUid(ExistingSkeletonRootJointUid);
		if (!ensure(ExistingSkeletonRootJointUid.Equals(RootJointUid)))
		{
			//Log an error
			return nullptr;
		}
	}
	else
	{
		SkeletonFactoryNode = NewObject<UInterchangeSkeletonFactoryNode>(BaseNodeContainer, NAME_None);
		if (!ensure(SkeletonFactoryNode))
		{
			return nullptr;
		}
		SkeletonFactoryNode->InitializeSkeletonNode(SkeletonUid, DisplayLabel, USkeleton::StaticClass()->GetName(), BaseNodeContainer);
		SkeletonFactoryNode->SetCustomRootJointUid(RootJointNode->GetUniqueID());
		
		bool bUseTimeZeroAsBindPose = bUseT0AsRefPose;

#if WITH_EDITOR
		if (!bUseTimeZeroAsBindPose)
		{
			bool bHasBoneWithoutBindPose = false;
			UE::Interchange::Private::FSkeletonHelper::RecursiveBoneHasBindPose(BaseNodeContainer, RootJointUid, bHasBoneWithoutBindPose);
			if (bHasBoneWithoutBindPose)
			{
				if (!GIsAutomationTesting && Results)
				{
					UInterchangeResultWarning_Generic* Message = AddMessage<UInterchangeResultWarning_Generic>();
					Message->Text = NSLOCTEXT("InterchangeSkeletalMeshFactory", "CreatePayloadTasks_ForceRebindOfSkinWithTimeZeroPose", "Imported skeleton has some invalid bind poses. Skeletal mesh skinning has been rebind using the time zero pose.");
					Message->SourceAssetName = SourceFileName;
					Message->AssetFriendlyName = DisplayLabel;
					Message->AssetType = USkeleton::StaticClass();
				}
				bUseTimeZeroAsBindPose = true;
			}
		}
#endif

		SkeletonFactoryNode->SetCustomUseTimeZeroForBindPose(bUseTimeZeroAsBindPose);
	}

	//If we have a specified skeleton
	if (IsSkeletonValid())
	{
		if (USkeleton* LoadedSkeleton = Skeleton.LoadSynchronous())
		{
			SkeletonFactoryNode->SetEnabled(false);
			SkeletonFactoryNode->SetCustomReferenceObject(Skeleton.ToSoftObjectPath());
		}
	}
#if WITH_EDITOR
	//Iterate all joints to set the meta data value in the skeleton node
	
	//Note: We shouldn't be able to get to this point without the ChildrenCache already set, so there is no need to call ComputeChildrenCache.
	// (Also trying to avoid recreating the ChildrenCache for every SkeletonFactoryNode creation call.)

	UE::Interchange::Private::FSkeletonHelper::RecursiveAddSkeletonMetaDataValues(BaseNodeContainer, SkeletonFactoryNode, RootJointUid);
#endif //WITH_EDITOR

	return SkeletonFactoryNode;
}

bool UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties::IsSkeletonValid() const
{
	if (Skeleton.IsNull())
	{
		return false;
	}

	return Skeleton.LoadSynchronous() != nullptr;
}