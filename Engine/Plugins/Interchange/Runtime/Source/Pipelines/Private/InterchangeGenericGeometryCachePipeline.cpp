// Copyright Epic Games, Inc. All Rights Reserved. 

#include "InterchangeGenericMeshPipeline.h"

#include "InterchangeAnimationTrackSetNode.h"
#include "InterchangeGeometryCacheFactoryNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangePipelineMeshesUtilities.h"
#include "InterchangeSceneNode.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"

#include "GeometryCache.h"

void UInterchangeGenericMeshPipeline::ExecutePreImportPipelineGeometryCache()
{
	check(CommonMeshesProperties.IsValid());

	if (!bImportGeometryCaches || CommonMeshesProperties->ForceAllMeshAsType != EInterchangeForceMeshType::IFMT_None)
	{
		return;
	}

	struct FPayloadInfo
	{
		FString SceneNodeUid;
		FString PayloadKeyId;

		FPayloadInfo(const FString& InSceneNodeUid, const FString& InPayloadKeyId)
		: SceneNodeUid(InSceneNodeUid)
		, PayloadKeyId(InPayloadKeyId)
		{ }

	};

	TArray<FPayloadInfo> AnimPayloads;

	// Retrieve all raw transform animations
	TArray<UInterchangeAnimationTrackSetNode*> TrackSetNodes;
	BaseNodeContainer->IterateNodesOfType<UInterchangeAnimationTrackSetNode>([&](const FString& NodeUid, UInterchangeAnimationTrackSetNode* Node)
	{
		TrackSetNodes.Add(Node);
	});

	for (UInterchangeAnimationTrackSetNode* TrackSetNode : TrackSetNodes)
	{
		if (!TrackSetNode)
		{
			continue;
		}

		TArray<FString> AnimationTrackUids;
		TrackSetNode->GetCustomAnimationTrackUids(AnimationTrackUids);

		for (const FString& AnimationTrackUid : AnimationTrackUids)
		{
			const UInterchangeTransformAnimationTrackNode* TransformTrackNode = Cast<UInterchangeTransformAnimationTrackNode>(BaseNodeContainer->GetNode(AnimationTrackUid));
			if (!TransformTrackNode)
			{
				continue;
			}

			FString ActorNodeUid;
			if (!TransformTrackNode->GetCustomActorDependencyUid(ActorNodeUid))
			{
				continue;
			}

			const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(ActorNodeUid));
			if (!SceneNode)
			{
				continue;
			}

			FInterchangeAnimationPayLoadKey AnimationPayloadKey;
			if (TransformTrackNode->GetCustomAnimationPayloadKey(AnimationPayloadKey))
			{
				AnimPayloads.Emplace(FPayloadInfo(SceneNode->GetUniqueID(), AnimationPayloadKey.UniqueId));
			}
		}
	}

	// If there's an animated mesh, combine all meshes into a single Geometry Cache
	TArray<FString> MeshInstanceUids;
	PipelineMeshesUtilities->GetAllGeometryCacheInstance(MeshInstanceUids);

	if (MeshInstanceUids.Num() == 0)
	{
		return;
	}

	// With animated transforms, static/skeletal meshes can get baked into a geometry cache
	if (AnimPayloads.Num() > 0)
	{
		// The bConvertStaticsWithAnimatedTransformToSkeletals setting converts static meshes with animated transforms into skeletal meshes with AnimSequence
		// For the purpose of geometry cache, both are considered as just meshes
		TArray<FString> AnimMeshInstanceUids;
		if (CommonMeshesProperties->bConvertStaticsWithAnimatedTransformToSkeletals && CommonMeshesProperties->ForceAllMeshAsType == EInterchangeForceMeshType::IFMT_None)
		{
			PipelineMeshesUtilities->GetAllSkinnedMeshInstance(AnimMeshInstanceUids);
		}
		else
		{
			PipelineMeshesUtilities->GetAllStaticMeshInstance(AnimMeshInstanceUids);
		}
		MeshInstanceUids.Append(AnimMeshInstanceUids);
	}

	TSet<FString> VisitedMeshAssetUids;

	TArray<FString> MeshUids;
	for (const FString& MeshUid : MeshInstanceUids)
	{
		const FInterchangeMeshInstance& MeshInstance = PipelineMeshesUtilities->GetMeshInstanceByUid(MeshUid);

		//UInterchangeInstancedStaticMeshComponentNode is not supported and hence will be skipped.
		if (const UInterchangeSceneNode* SceneNode = Cast<const UInterchangeSceneNode>(MeshInstance.InstancingNode))
		{
			if (CommonMeshesProperties->bBakeMeshes)
			{
				// If we're baking, always add all scene nodes: We may be trying to bake multiple instances of the same mesh node
				// into a single big geometry cache asset
				MeshUids.Add(SceneNode->GetUniqueID());
			}
			else
			{
				// If not baking, only add a single scene node per each mesh asset: This prevents us from ending up with multiple copies
				// of the same mesh on top of itself (as we won't bake transforms), and it also allows us to instance geometry caches
				// by just making several scene nodes that reference the same animated mesh node: Only one of the "instances" will end
				// up on the geometry cache asset, and we'll reuse the asset for all the scene nodes
				FString AssetUid;
				if (SceneNode->GetCustomAssetInstanceUid(AssetUid) && !VisitedMeshAssetUids.Contains(AssetUid))
				{
					VisitedMeshAssetUids.Add(AssetUid);
					MeshUids.Add(SceneNode->GetUniqueID());
				}
			}
		}
	}

	if (MeshUids.Num() == 0)
	{
		return;
	}

	UInterchangeGeometryCacheFactoryNode* GeometryCacheFactoryNode = CreateGeometryCacheFactoryNode(MeshUids);
	GeometryCacheFactoryNodes.Add(GeometryCacheFactoryNode);

	for (const FPayloadInfo& PayloadInfo : AnimPayloads)
	{
		GeometryCacheFactoryNode->SetAnimationPayloadKeyForSceneNodeUid(PayloadInfo.SceneNodeUid, PayloadInfo.PayloadKeyId);
	}
}

UInterchangeGeometryCacheFactoryNode* UInterchangeGenericMeshPipeline::CreateGeometryCacheFactoryNode(const TArray<FString>& MeshUids)
{
	check(CommonMeshesProperties.IsValid());
	if (MeshUids.Num() == 0)
	{
		return nullptr;
	}

	// Name the geometry cache node according to the first non-root node
	FString GeometryCacheUid;
	FString DisplayLabel;
	{
		// Starting from the first mesh, we go up through its ancestors until we find the first node after the root
		const UInterchangeBaseNode* Node = BaseNodeContainer->GetNode(MeshUids[0]);
		if (const UInterchangeSceneNode* SceneNode = Cast<const UInterchangeSceneNode>(Node))
		{
			const UInterchangeSceneNode* ParentSceneNode = Cast<const UInterchangeSceneNode>(BaseNodeContainer->GetNode(SceneNode->GetParentUid()));
			if (!ParentSceneNode || ParentSceneNode->GetParentUid() == UInterchangeBaseNode::InvalidNodeUid())
			{
				// The SceneNode itself or its parent is the root so just get the SceneNode's info
				DisplayLabel = SceneNode->GetDisplayLabel();
				SceneNode->GetCustomAssetInstanceUid(GeometryCacheUid);
			}
			else
			{
				while (SceneNode && ParentSceneNode)
				{
					SceneNode = Cast<const UInterchangeSceneNode>(BaseNodeContainer->GetNode(SceneNode->GetParentUid()));
					if (SceneNode)
					{
						ParentSceneNode = Cast<const UInterchangeSceneNode>(BaseNodeContainer->GetNode(SceneNode->GetParentUid()));
						if (!ParentSceneNode || ParentSceneNode->GetParentUid() == UInterchangeBaseNode::InvalidNodeUid())
						{
							// The root has no parent so retrieve the needed info and break 
							DisplayLabel = SceneNode->GetDisplayLabel();
							SceneNode->GetCustomAssetInstanceUid(GeometryCacheUid);
							break;
						}
					}
				}
			}
		}
	}

	const FString StaticMeshUid = TEXT("Factory_GeometryCache_") + GeometryCacheUid;
	UInterchangeGeometryCacheFactoryNode* GeometryCacheFactoryNode = NewObject<UInterchangeGeometryCacheFactoryNode>(BaseNodeContainer, NAME_None);
	if (!ensure(GeometryCacheFactoryNode))
	{
		return nullptr;
	}

	GeometryCacheFactoryNode->InitializeGeometryCacheNode(StaticMeshUid, DisplayLabel, UGeometryCache::StaticClass()->GetName(), BaseNodeContainer);

	// For now, keep all the mesh sections separate since each section will be in its own track in the GeometryCache
	GeometryCacheFactoryNode->SetCustomKeepSectionsSeparate(true);

	AddMeshesToGeometryCache(GeometryCacheFactoryNode, MeshUids);

	// #ueent_todo: Support the common meshes build options (not all are supported)
	GeometryCacheFactoryNode->SetCustomRecomputeNormals(CommonMeshesProperties->bRecomputeNormals);
	GeometryCacheFactoryNode->SetCustomRecomputeTangents(CommonMeshesProperties->bRecomputeTangents);
	GeometryCacheFactoryNode->SetCustomUseMikkTSpace(CommonMeshesProperties->bUseMikkTSpace);
	GeometryCacheFactoryNode->SetCustomComputeWeightedNormals(CommonMeshesProperties->bComputeWeightedNormals);
	GeometryCacheFactoryNode->SetCustomUseHighPrecisionTangentBasis(CommonMeshesProperties->bUseHighPrecisionTangentBasis);
	GeometryCacheFactoryNode->SetCustomUseFullPrecisionUVs(CommonMeshesProperties->bUseFullPrecisionUVs);
	GeometryCacheFactoryNode->SetCustomUseBackwardsCompatibleF16TruncUVs(CommonMeshesProperties->bUseBackwardsCompatibleF16TruncUVs);
	GeometryCacheFactoryNode->SetCustomRemoveDegenerates(CommonMeshesProperties->bRemoveDegenerates);

	GeometryCacheFactoryNode->SetCustomFlattenTracks(bFlattenTracks);
	GeometryCacheFactoryNode->SetCustomPositionPrecision(CompressedPositionPrecision);
	GeometryCacheFactoryNode->SetCustomNumBitsForUVs(CompressedTextureCoordinatesNumberOfBits);

	if (bOverrideTimeRange)
	{
		GeometryCacheFactoryNode->SetCustomStartFrame(FrameStart);
		GeometryCacheFactoryNode->SetCustomEndFrame(FrameEnd);
	}

	GeometryCacheFactoryNode->SetCustomMotionVectorsImport(MotionVectors);
	GeometryCacheFactoryNode->SetCustomApplyConstantTopologyOptimization(bApplyConstantTopologyOptimizations);
	GeometryCacheFactoryNode->SetCustomStoreImportedVertexNumbers(bStoreImportedVertexNumbers);
	GeometryCacheFactoryNode->SetCustomOptimizeIndexBuffers(bOptimizeIndexBuffers);

	return GeometryCacheFactoryNode;
}

void UInterchangeGenericMeshPipeline::AddMeshesToGeometryCache(UInterchangeGeometryCacheFactoryNode* GeometryCacheFactoryNode, const TArray<FString>& NodeUids)
{
	TMap<FString, FString> ExistingLodSlotMaterialDependencies;
	constexpr bool bAddSourceNodeName = true;
	for (const FString& NodeUid : NodeUids)
	{
		TMap<FString, FString> SlotMaterialDependencies;
		if (const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(NodeUid)))
		{
			FString MeshDependency;
			SceneNode->GetCustomAssetInstanceUid(MeshDependency);
			if (const UInterchangeMeshNode* MeshDependencyNode = Cast<UInterchangeMeshNode>(BaseNodeContainer->GetNode(MeshDependency)))
			{
				UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(MeshDependencyNode, GeometryCacheFactoryNode, bAddSourceNodeName);

				// Note here we add the SceneNode because we'll bake its transform into the mesh
				GeometryCacheFactoryNode->AddTargetNodeUid(NodeUid);
				MeshDependencyNode->AddTargetNodeUid(GeometryCacheFactoryNode->GetUniqueID());

				MeshDependencyNode->GetSlotMaterialDependencies(SlotMaterialDependencies);
			}
			else
			{
				SceneNode->GetSlotMaterialDependencies(SlotMaterialDependencies);
			}

			UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(SceneNode, GeometryCacheFactoryNode, bAddSourceNodeName);
		}
		else if (const UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(BaseNodeContainer->GetNode(NodeUid)))
		{
			UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(MeshNode, GeometryCacheFactoryNode, bAddSourceNodeName);
			GeometryCacheFactoryNode->AddTargetNodeUid(NodeUid);
			MeshNode->AddTargetNodeUid(GeometryCacheFactoryNode->GetUniqueID());

			MeshNode->GetSlotMaterialDependencies(SlotMaterialDependencies);
		}

		UE::Interchange::MeshesUtilities::ApplySlotMaterialDependencies(*GeometryCacheFactoryNode, SlotMaterialDependencies, *BaseNodeContainer, &ExistingLodSlotMaterialDependencies);
	}
}
