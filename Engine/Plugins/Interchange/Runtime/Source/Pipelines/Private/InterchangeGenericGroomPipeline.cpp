// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeGenericGroomPipeline.h"

#include "GroomAsset.h"
#include "GroomBindingAsset.h"
#include "GroomCache.h"
#include "InterchangeGroomBindingNode.h"
#include "InterchangeGroomNode.h"
#include "InterchangeGroomBindingFactoryNode.h"
#include "InterchangeGroomCacheFactoryNode.h"
#include "InterchangeGroomComponentNode.h"
#include "InterchangeGroomFactoryNode.h"
#include "InterchangeJointNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangePipelineHelper.h"
#include "InterchangeSceneNode.h"
#include "InterchangeSkeletalMeshFactoryNode.h"
#include "InterchangeSkeletalMeshLodDataNode.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Nodes/InterchangeSourceNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"
#include "Usd/InterchangeUsdDefinitions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGenericGroomPipeline)

namespace UE::InterchangeGroomPipeline::Private
{
	UInterchangeFactoryBaseNode* CreateGroomFactoryNode(UInterchangeBaseNodeContainer* InBaseNodeContainer, const UInterchangeGroomNode* GroomNode, EInterchangeGroomCacheImportType Type)
	{
		FString NodeID = GroomNode->GetUniqueID();
		FString FactoryNodeUid;

		UInterchangeFactoryBaseNode* FactoryNode = nullptr;
		if (Type == EInterchangeGroomCacheImportType::None)
		{
			// Create groom asset node
			FactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(NodeID);
			FactoryNode = NewObject<UInterchangeGroomFactoryNode>(InBaseNodeContainer);
		}
		else
		{
			// Create groom cache node
			const FString CacheType = Type == EInterchangeGroomCacheImportType::Strands ? TEXT("_strands_") : TEXT("_guides_");
			static const FString Suffix(TEXT("cache"));

			FactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(NodeID + CacheType + Suffix);
			UInterchangeGroomCacheFactoryNode* GroomCacheFactoryNode = NewObject<UInterchangeGroomCacheFactoryNode>(InBaseNodeContainer);

			GroomCacheFactoryNode->SetCustomGroomCacheImportType(Type);
			GroomCacheFactoryNode->SetAssetName(GroomNode->GetAssetName() + CacheType + Suffix);

			int32 Value = 0;
			if (GroomNode->GetCustomNumFrames(Value))
			{
				GroomCacheFactoryNode->SetCustomNumFrames(Value);
			}

			if (GroomNode->GetCustomStartFrame(Value))
			{
				GroomCacheFactoryNode->SetCustomStartFrame(Value);
			}

			if (GroomNode->GetCustomEndFrame(Value))
			{
				GroomCacheFactoryNode->SetCustomEndFrame(Value);
			}

			double SecsPerFrames = 0;
			if (GroomNode->GetCustomFrameRate(SecsPerFrames))
			{
				GroomCacheFactoryNode->SetCustomFrameRate(SecsPerFrames);
			}

			EInterchangeGroomCacheAttributes Attributes;
			if (GroomNode->GetCustomGroomCacheAttributes(Attributes))
			{
				GroomCacheFactoryNode->SetCustomGroomCacheAttributes(Attributes);
			}

			FactoryNode = GroomCacheFactoryNode;
		}

		InBaseNodeContainer->SetupNode(FactoryNode, FactoryNodeUid, GroomNode->GetDisplayLabel(), EInterchangeNodeContainerType::FactoryData);

		UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(InBaseNodeContainer);
		UE::Interchange::PipelineHelper::FillSubPathFromSourceNode(FactoryNode, SourceNode);

		const bool bAddSourceNodeName = false;
		UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(GroomNode, FactoryNode, bAddSourceNodeName);

		FactoryNode->AddTargetNodeUid(GroomNode->GetUniqueID());
		GroomNode->AddTargetNodeUid(FactoryNode->GetUniqueID());

		return FactoryNode;
	}
}

FString UInterchangeGenericGroomPipeline::GetPipelineCategory(UClass* AssetClass)
{
	return TEXT("Grooms");
}

void UInterchangeGenericGroomPipeline::ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath)
{
	using namespace UE::InterchangeGroomPipeline::Private;

	if (!bEnableGroomTypesImport)
	{
		return;
	}

	if (!InBaseNodeContainer)
	{
		return;
	}

	// Find all the translated groom nodes we need for this pipeline
	if (bImportGrooms || bImportGroomCaches)
	{
		TArray<UInterchangeGroomNode*> GroomNodes;
		InBaseNodeContainer->GetNodesOfType<UInterchangeGroomNode>(GroomNodes);

		// Create groom factory nodes
		for (const UInterchangeGroomNode* GroomNode : GroomNodes)
		{
			const TOptional<FInterchangeGroomPayloadKey> PayloadKey = GroomNode->GetPayloadKey();
			if (!PayloadKey.IsSet())
			{
				continue;
			}

			if (bImportGrooms && PayloadKey->Type == EInterchangeGroomPayLoadType::STATIC)
			{
				UInterchangeGroomFactoryNode* GroomFactoryNode = Cast<UInterchangeGroomFactoryNode>(CreateGroomFactoryNode(InBaseNodeContainer, GroomNode, EInterchangeGroomCacheImportType::None));
				if (GroomFactoryNode)
				{
					GroomFactoryNode->SetCustomGroupInterpolationSettings(GroupInterpolationSettings);
				}
			}
			else
			{
				UInterchangeGroomFactoryNode* GroomFactoryNode = nullptr;
				if (bImportGrooms)
				{
					// A groom asset can be generated from the first frame of a groom cache
					GroomFactoryNode = Cast<UInterchangeGroomFactoryNode>(CreateGroomFactoryNode(InBaseNodeContainer, GroomNode, EInterchangeGroomCacheImportType::None));
					if (GroomFactoryNode)
					{
						GroomFactoryNode->SetCustomGroupInterpolationSettings(GroupInterpolationSettings);
					}
				}

				if (bImportGroomCaches)
				{
					// To import a groom cache, either a groom asset must be imported from its first frame or a previously imported groom asset must be specified
					// The groom asset is used to validate that the cache is compatible with the asset
					if (!bImportGrooms && !GroomAsset.IsValid())
					{
						continue;
					}

					auto InitializeGroomCacheFactoryNode = [this, InBaseNodeContainer, GroomNode, GroomFactoryNode](EInterchangeGroomCacheImportType CacheType)
					{
						if (EnumHasAnyFlags(ImportGroomCacheType, CacheType))
						{
							UInterchangeGroomCacheFactoryNode* GroomCacheFactoryNode = Cast<UInterchangeGroomCacheFactoryNode>(CreateGroomFactoryNode(InBaseNodeContainer, GroomNode, CacheType));
							if (GroomCacheFactoryNode)
							{
								if (GroomFactoryNode)
								{
									// If the groom asset is imported at the same time, set its factory node as a dependency
									GroomCacheFactoryNode->AddFactoryDependencyUid(GroomFactoryNode->GetUniqueID());
									if (bIsReimportContext && Cast<UGroomCache>(CacheContextParam.ReimportAsset))
									{
										// When reimporting a groom cache, the associated groom asset will not be reimported
										// but it's still needed as a dependency. Disabling its factory node will try to
										// fetch it instead of importing it
										GroomFactoryNode->SetEnabled(false);
									}
								}
								else
								{
									// Not importing the groom asset, so set the path to a previously imported groom asset
									GroomCacheFactoryNode->SetCustomGroomAssetPath(GroomAsset);
								}

								if (bOverrideTimeRange)
								{
									GroomCacheFactoryNode->SetCustomStartFrame(FrameStart);
									// Make sure the end is after the start and it's at least 2 frames
									FrameEnd = FMath::Max(FrameEnd, FrameStart + 1);
									GroomCacheFactoryNode->SetCustomEndFrame(FrameEnd);
									GroomCacheFactoryNode->SetCustomNumFrames(FrameEnd - FrameStart + 1);
								}
							}
						}
					};

					InitializeGroomCacheFactoryNode(EInterchangeGroomCacheImportType::Strands);
					InitializeGroomCacheFactoryNode(EInterchangeGroomCacheImportType::Guides);
				}
			}
		}
	}

	if (bImportGroomBindings)
	{
		TArray<UInterchangeGroomBindingNode*> GroomBindingNodes;
		InBaseNodeContainer->GetNodesOfType<UInterchangeGroomBindingNode>(GroomBindingNodes);

		// Create groom binding factory nodes
		for (const UInterchangeGroomBindingNode* GroomBindingNode : GroomBindingNodes)
		{
			FString GroomFactoryNodeUid;
			if (FString GroomNodeUid; GroomBindingNode->GetGroomDependencyUid(GroomNodeUid))
			{
				GroomFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(GroomNodeUid);
				if (!InBaseNodeContainer->GetFactoryNode(GroomFactoryNodeUid))
				{
					continue;
				}
			}
			else
			{
				continue;
			}

			FString MeshNodeUid;
			GroomBindingNode->GetTargetMeshDependencyUid(MeshNodeUid);

			FString AssetUid;
			const UInterchangeBaseNode* AssetNode = nullptr;
			const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(InBaseNodeContainer->GetNode(MeshNodeUid));
			if (SceneNode)
			{
				if (SceneNode->GetCustomAssetInstanceUid(AssetUid))
				{
					AssetNode = InBaseNodeContainer->GetNode(AssetUid);
				}
			}

			FString MeshFactoryNodeUid;
			if (Cast<UInterchangeGeometryCacheNode>(AssetNode))
			{
				MeshFactoryNodeUid = TEXT("Factory_GeometryCache_") + AssetUid;
			}
			else
			{
				TArray<UInterchangeSkeletalMeshFactoryNode*> FactoryNodes;
				InBaseNodeContainer->GetNodesOfType<UInterchangeSkeletalMeshFactoryNode>(FactoryNodes);
				for (UInterchangeSkeletalMeshFactoryNode* FactoryNode : FactoryNodes)
				{
					TArray<FString> LodUids;
					FactoryNode->GetLodDataUniqueIds(LodUids);
					for (const FString& LodUid : LodUids)
					{
						const UInterchangeSkeletalMeshLodDataNode* SkeletalMeshLodDataNode = Cast<const UInterchangeSkeletalMeshLodDataNode>(InBaseNodeContainer->GetFactoryNode(LodUid));
						if (SkeletalMeshLodDataNode)
						{
							TArray<FInterchangeLODMeshData> Lods;
							SkeletalMeshLodDataNode->GetLODMeshDataArray(Lods);
							for (const FInterchangeLODMeshData& Lod : Lods)
							{
								if (Lod.MeshUid == AssetUid)
								{
									MeshFactoryNodeUid = FactoryNode->GetUniqueID();
									break;
								}
							}
						}
					}
				}
			}

			if (MeshFactoryNodeUid.IsEmpty())
			{
				continue;
			}


			FString NodeID = GroomBindingNode->GetUniqueID();
			const FString FactoryNodeUid = TEXT("Factory_GroomBinding_") + NodeID;
			UInterchangeGroomBindingFactoryNode*  FactoryNode = NewObject<UInterchangeGroomBindingFactoryNode>(InBaseNodeContainer);

			InBaseNodeContainer->SetupNode(FactoryNode, FactoryNodeUid, GroomBindingNode->GetDisplayLabel(), EInterchangeNodeContainerType::FactoryData);

			UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(InBaseNodeContainer);
			UE::Interchange::PipelineHelper::FillSubPathFromSourceNode(FactoryNode, SourceNode);

			const bool bAddSourceNodeName = false;
			UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(GroomBindingNode, FactoryNode, bAddSourceNodeName);

			FactoryNode->AddTargetNodeUid(GroomBindingNode->GetUniqueID());
			GroomBindingNode->AddTargetNodeUid(FactoryNode->GetUniqueID());
			FactoryNode->AddFactoryDependencyUid(GroomFactoryNodeUid);
			FactoryNode->AddFactoryDependencyUid(MeshFactoryNodeUid);
			FactoryNode->SetNumInterpolationPoints(NumInterpolationPoints);
			
			using namespace UE::Interchange::USD;
			
			if (const UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(AssetNode))
			{
				TArray<FString> SkeletonDependencies;
				MeshNode->GetSkeletonDependencies(SkeletonDependencies);
				UInterchangeSceneNode* TargetSceneNode = nullptr;
				if (SkeletonDependencies.Num() > 0)
				{
					UInterchangeJointNode* RootJoint = Cast<UInterchangeJointNode>(const_cast<UInterchangeBaseNode*>(InBaseNodeContainer->GetNode(SkeletonDependencies[0])));
					if (RootJoint)
					{
						TargetSceneNode = RootJoint;
					}
				}
				else
				{
					TargetSceneNode = const_cast<UInterchangeSceneNode*>(SceneNode);
				}

				if (TargetSceneNode)
				{
					const FString ParentUid = TargetSceneNode->GetUniqueID();
					const FString UniqueId = ComponentPrefix + ParentUid;

					UInterchangeGroomComponentNode* GroomComponentNode = NewObject<UInterchangeGroomComponentNode>(InBaseNodeContainer);
					const FString DisplayLabel = TEXT("GroomComponent");

					InBaseNodeContainer->SetupNode(GroomComponentNode, UniqueId, DisplayLabel, EInterchangeNodeContainerType::TranslatedScene, ParentUid);
					TargetSceneNode->AddComponentUid(GroomComponentNode->GetUniqueID());

					GroomComponentNode->SetGroomDependencyUid(GroomFactoryNodeUid);
					GroomComponentNode->SetGroomBindingDependencyUid(FactoryNodeUid);
				}
			}
		}
	}
}

#if WITH_EDITOR

void UInterchangeGenericGroomPipeline::GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const
{
	PipelineSupportAssetClasses.Add(UGroomAsset::StaticClass());
	PipelineSupportAssetClasses.Add(UGroomCache::StaticClass());
	PipelineSupportAssetClasses.Add(UGroomBindingAsset::StaticClass());
}

void UInterchangeGenericGroomPipeline::FilterPropertiesFromTranslatedData(UInterchangeBaseNodeContainer* InBaseNodeContainer)
{
	int32 NumGroomNodes = 0;
	int32 NumGroomCaches = 0;
	InBaseNodeContainer->BreakableIterateNodesOfType<UInterchangeGroomNode>([&NumGroomNodes, &NumGroomCaches](const FString& NodeUid, UInterchangeGroomNode* GroomNode)
	{
		++NumGroomNodes;
		const TOptional<FInterchangeGroomPayloadKey> PayloadKey = GroomNode->GetPayloadKey();
		if (PayloadKey.IsSet() && PayloadKey->Type == EInterchangeGroomPayLoadType::ANIMATED)
		{
			++NumGroomCaches;
			return true;
		}
		return false;
	});

	UInterchangePipelineBase* OuterMostPipeline = GetMostPipelineOuter();
	if (NumGroomNodes == 0)
	{
		HidePropertiesOfCategory(OuterMostPipeline, this, GetPipelineCategory(nullptr));
	}
	else if (NumGroomCaches == 0)
	{
		HidePropertiesOfSubCategory(OuterMostPipeline, this, TEXT("Caches"));
	}
}

bool UInterchangeGenericGroomPipeline::IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const
{
	static const TSet<FName> NeedRefreshProperties =
	{
		GET_MEMBER_NAME_CHECKED(UInterchangeGenericGroomPipeline, bEnableGroomTypesImport),
		GET_MEMBER_NAME_CHECKED(UInterchangeGenericGroomPipeline, bImportGroomCaches),
		GET_MEMBER_NAME_CHECKED(UInterchangeGenericGroomPipeline, bImportGrooms),
		GET_MEMBER_NAME_CHECKED(UInterchangeGenericGroomPipeline, bImportGroomBindings),
		GET_MEMBER_NAME_CHECKED(UInterchangeGenericGroomPipeline, GroomAsset),
	};

	if (NeedRefreshProperties.Contains(PropertyChangedEvent.GetPropertyName()))
	{
		return true;
	}

	return Super::IsPropertyChangeNeedRefresh(PropertyChangedEvent);
}

#endif //WITH_EDITOR

bool UInterchangeGenericGroomPipeline::IsSettingsAreValid(TOptional<FText>& OutInvalidReason) const
{
	if (bEnableGroomTypesImport && bImportGroomCaches && !bImportGrooms && !GroomAsset.IsValid())
	{
		int32 NumGroomCaches = 0;
		CacheContextParam.BaseNodeContainer->BreakableIterateNodesOfType<UInterchangeGroomNode>([&NumGroomCaches](const FString& NodeUid, UInterchangeGroomNode* GroomNode)
		{
			const TOptional<FInterchangeGroomPayloadKey> PayloadKey = GroomNode->GetPayloadKey();
			if (PayloadKey.IsSet() && PayloadKey->Type == EInterchangeGroomPayLoadType::ANIMATED)
			{
				++NumGroomCaches;
				return true;
			}
			return false;
		});

		// If there's no groom cache, then this validation is not needed
		if (NumGroomCaches == 0)
		{
			return Super::IsSettingsAreValid(OutInvalidReason);
		}

		OutInvalidReason = NSLOCTEXT("UInterchangeGenericGroomPipeline", "GroomAssetMustBeSpecified", "When importing a groom cache without importing its associated groom asset, a previously imported and compatible groom asset must be specified.");
		return false;
	}
	return Super::IsSettingsAreValid(OutInvalidReason);
}