// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeGenericMaterialPipeline.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Import/Private/MaterialX/MaterialExpressions/MaterialExpressionLuminance.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeGenericTexturePipeline.h"
#include "InterchangeManager.h"
#include "InterchangeMaterialDefinitions.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeMaterialInstanceNode.h"
#include "InterchangeMaterialReferenceNode.h"
#include "InterchangePipelineLog.h"
#include "InterchangeSceneImportAsset.h"
#include "InterchangeShaderGraphNode.h"
#include "InterchangeSourceData.h"
#include "InterchangeSparseVolumeTexturePipeline.h"
#include "InterchangeSpecularProfileNode.h"
#include "InterchangeSpecularProfileFactoryNode.h"
#include "InterchangeTexture2DArrayNode.h"
#include "InterchangeTexture2DNode.h"
#include "InterchangeTextureBlurNode.h"
#include "InterchangeTextureCubeNode.h"
#include "InterchangeTextureFactoryNode.h"
#include "InterchangeTextureNode.h"
#include "InterchangeVolumeTextureNode.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionAdd.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionCosine.h"
#include "Materials/MaterialExpressionFresnel.h"
#include "Materials/MaterialExpressionLinearInterpolate.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialExpressionNoise.h"
#include "Materials/MaterialExpressionOneMinus.h"
#include "Materials/MaterialExpressionRotateAboutAxis.h"
#include "Materials/MaterialExpressionRotator.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionSine.h"
#include "Materials/MaterialExpressionSparseVolumeTextureSample.h"
#include "Materials/MaterialExpressionStaticBoolParameter.h"
#include "Materials/MaterialExpressionSubstrate.h"
#include "Materials/MaterialExpressionSwitch.h"
#include "Materials/MaterialExpressionTangent.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include "Materials/MaterialExpressionTextureObjectParameter.h"
#include "Materials/MaterialExpressionTextureProperty.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionTextureSampleParameter2D.h"
#include "Materials/MaterialExpressionTextureSampleParameter2DArray.h"
#include "Materials/MaterialExpressionTextureSampleParameterCube.h"
#include "Materials/MaterialExpressionTime.h"
#include "Materials/MaterialExpressionTransform.h"
#include "Materials/MaterialExpressionTransformPosition.h"
#include "Materials/MaterialExpressionVectorNoise.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MaterialX/MaterialExpressions/MaterialExpressionTextureSampleParameterBlur.h"
#include "Misc/CoreMisc.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeSourceNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"
#include "Templates/Function.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#if UE_BUILD_DEBUG
#include "HAL/PlatformFileManager.h"
#endif

// Material Hash Utils
#include "Material/InterchangeMaterialFactory.h"

#include <string_view>

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGenericMaterialPipeline)

#define LOCTEXT_NAMESPACE "InterchangeGenericMaterialPipeline"

FString LexToString(UInterchangeGenericMaterialPipeline::EMaterialInputType Value)
{
	switch(Value)
	{
	case UInterchangeGenericMaterialPipeline::EMaterialInputType::Unknown:
		return TEXT("Unknown");
	case UInterchangeGenericMaterialPipeline::EMaterialInputType::Color:
		return TEXT("Color");
	case UInterchangeGenericMaterialPipeline::EMaterialInputType::Vector:
		return TEXT("Vector");
	case UInterchangeGenericMaterialPipeline::EMaterialInputType::Scalar:
		return TEXT("Scalar");
	default:
		ensure(false);
		return FString();
	}
}

namespace UE::Interchange::InterchangeGenericMaterialPipeline::Private
{
	bool AreRequiredPackagesLoaded()
	{
		auto ArePackagesLoaded = [](const TArray<FString>& PackagePaths) -> bool
		{
			bool bAllLoaded = true;

			for (const FString& PackagePath : PackagePaths)
			{
				const FString ObjectPath(FPackageName::ExportTextPathToObjectPath(PackagePath));

				if (FPackageName::DoesPackageExist(ObjectPath))
				{
					if (FSoftObjectPath(ObjectPath).TryLoad())
					{
						continue;
					}
					else
					{
						UE_LOGF(LogInterchangePipeline, Warning, "Couldn't load %ls", *PackagePath);
					}
				}
				else
				{
					UE_LOGF(LogInterchangePipeline, Warning, "Couldn't find %ls", *PackagePath);
				}

				bAllLoaded = false;
			}

			return bAllLoaded;
		};

		TArray<FString> RequiredPackages = {
			TEXT("MaterialFunction'/Engine/Functions/Engine_MaterialFunctions01/Shading/ConvertFromDiffSpec.ConvertFromDiffSpec'"),
			TEXT("MaterialFunction'/Engine/Functions/Engine_MaterialFunctions01/Texturing/FlattenNormal.FlattenNormal'"),
			TEXT("MaterialFunction'/Engine/Functions/Engine_MaterialFunctions02/Utility/MakeFloat3.MakeFloat3'"),
			TEXT("MaterialFunction'/Engine/Functions/Engine_MaterialFunctions02/Texturing/CustomRotator.CustomRotator'"),
			TEXT("MaterialFunction'/InterchangeAssets/Functions/MF_PhongToMetalRoughness.MF_PhongToMetalRoughness'"),
		};

		static const bool bRequiredPackagesLoaded = ArePackagesLoaded(RequiredPackages);

		return bRequiredPackagesLoaded;
	}

	void UpdateBlendModeBasedOnOpacityAttributes(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode)
	{
		// Opacity Clip Value
		bool bIsMasked = false;
		{
			float OpacityClipValue;
			if (ShaderGraphNode->GetCustomOpacityMaskClipValue(OpacityClipValue))
			{
				MaterialFactoryNode->SetCustomOpacityMaskClipValue(OpacityClipValue);
				bIsMasked = true;
			}
		}

		// Don't change the blend mode if it was already set
		TEnumAsByte<EBlendMode> BlendMode = bIsMasked ? EBlendMode::BLEND_Masked : EBlendMode::BLEND_Translucent;
		if (!MaterialFactoryNode->GetCustomBlendMode(BlendMode))
		{
			MaterialFactoryNode->SetCustomBlendMode(BlendMode);
		}

		// If bland mode is masked or translucent, set lighting mode accordingly without changing it if it was already set
		if(BlendMode == EBlendMode::BLEND_Masked || BlendMode == EBlendMode::BLEND_Translucent)
		{
			TEnumAsByte<ETranslucencyLightingMode> LightingMode = ETranslucencyLightingMode::TLM_Surface;
			if (!MaterialFactoryNode->GetCustomTranslucencyLightingMode(LightingMode))
			{
				MaterialFactoryNode->SetCustomTranslucencyLightingMode(LightingMode);
			}
		}
	}

	void UpdateFunctionCallExpression(UInterchangeMaterialExpressionFactoryNode& FunctionCallExpression, const FString& MaterialFunctionPath)
	{
		const FName MaterialFunctionMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionMaterialFunctionCall, MaterialFunction);
		UClass* CustomExpressionClass = UMaterialExpressionMaterialFunctionCall::StaticClass();

		// load the material function in the Game Thread to avoid any potential crash in the Material Factory
		FSoftObjectPath{ MaterialFunctionPath }.TryLoad();

		FunctionCallExpression.SetCustomExpressionClassName(CustomExpressionClass->GetName());

		FunctionCallExpression.AddStringAttribute(MaterialFunctionMemberName.ToString(), MaterialFunctionPath);
		FunctionCallExpression.AddApplyAndFillDelegates<FString>(MaterialFunctionMemberName.ToString(), CustomExpressionClass, MaterialFunctionMemberName);
	}

	UInterchangeMaterialExpressionFactoryNode* CreateExpressionWithMaterialFunction(
		UInterchangeBaseNodeContainer* BaseNodeContainer,
		UInterchangeMaterialFactoryNode* MaterialFactoryNode,
		const FString& Label,
		const FString& MaterialFunctionPath)
	{
		const FString UniqueId = MaterialFactoryNode->GetUniqueID() + Label;

		UInterchangeMaterialExpressionFactoryNode* Expression = NewObject<UInterchangeMaterialExpressionFactoryNode>(BaseNodeContainer, NAME_None);
		if (!Expression)
		{
			return nullptr;
		}

		BaseNodeContainer->SetupNode(Expression, UniqueId, Label, EInterchangeNodeContainerType::FactoryData, MaterialFactoryNode->GetUniqueID());

		UpdateFunctionCallExpression(*Expression, MaterialFunctionPath);

		return Expression;
	}

	UMaterialInterface* FindExistingMaterial(const FString& BasePath, const FString& MaterialFullName, const bool bRecursivePaths)
	{
		UMaterialInterface* Material = nullptr;
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

		//Make sure we scan the root paths only one time per editor session.
		//There can be thousands of root path and it is a slow task even if everything is already scan.
		static bool bRootPathScanned = false;
		// Finish/update any scans
		TArray<FString> ScanPaths;
		if (!bRootPathScanned && (BasePath.IsEmpty() || BasePath == TEXT("/")))
		{
			FPackageName::QueryRootContentPaths(ScanPaths);
			bRootPathScanned = true;
		}
		else if (!BasePath.StartsWith(TEXT("/Temp"), ESearchCase::IgnoreCase)) //We must exclude Temp path to avoid asset registry scan path warnings
		{
			ScanPaths.Add(BasePath);
		}

		if (!ScanPaths.IsEmpty())
		{
			constexpr bool bForceRescan = false;
			AssetRegistry.ScanPathsSynchronous(ScanPaths, bForceRescan);
		}


		FARFilter Filter;
		Filter.bRecursiveClasses = true;
		Filter.bRecursivePaths = bRecursivePaths;
		Filter.ClassPaths.Add(UMaterialInterface::StaticClass()->GetClassPathName());
		Filter.PackagePaths.Add(FName(*BasePath));

		TArray<FAssetData> AssetData;
		AssetRegistry.GetAssets(Filter, AssetData);

		TArray<UMaterialInterface*> FoundMaterials;
		for (const FAssetData& Data : AssetData)
		{
			if (Data.AssetName == FName(*MaterialFullName))
			{
				Material = Cast<UMaterialInterface>(Data.GetAsset());
				if (Material != nullptr)
				{
					FoundMaterials.Add(Material);
				}
			}
		}

		return FoundMaterials.Num() > 0 ? FoundMaterials[0] : Material;
	}

	UMaterialInterface* FindExistingMaterialFromSearchLocation(const FString& MaterialFullName, const FString& ContentPath, EInterchangeMaterialSearchLocation SearchLocation)
	{
		//Search in memory
		UMaterialInterface* FoundMaterial = nullptr;
		//We search only in memory for search in local folder.
		if (SearchLocation == EInterchangeMaterialSearchLocation::Local)
		{
			FoundMaterial = FindObject<UMaterialInterface>(nullptr, *MaterialFullName, EFindObjectFlags::None);
			if(FoundMaterial)
			{
				//Make sure the path of the material in memory is local
				FString PackagePath = FoundMaterial->GetPackage()->GetPathName();
				if (!PackagePath.Equals(ContentPath))
				{
					FoundMaterial = nullptr;
				}
			}
		}

		if (FoundMaterial == nullptr)
		{
			FString SearchPath = ContentPath;

			// Search in asset's local folder
			FoundMaterial = FindExistingMaterial(SearchPath, MaterialFullName, false);

			// Search recursively in asset's folder
			if (FoundMaterial == nullptr &&
				(SearchLocation != EInterchangeMaterialSearchLocation::Local))
			{
				FoundMaterial = FindExistingMaterial(SearchPath, MaterialFullName, true);
			}

			if (FoundMaterial == nullptr &&
				(SearchLocation == EInterchangeMaterialSearchLocation::UnderParent ||
					SearchLocation == EInterchangeMaterialSearchLocation::UnderRoot ||
					SearchLocation == EInterchangeMaterialSearchLocation::AllAssets))
			{
				// Search recursively in parent's folder
				SearchPath = FPaths::GetPath(SearchPath);
				if (!SearchPath.IsEmpty())
				{
					FoundMaterial = FindExistingMaterial(SearchPath, MaterialFullName, true);
				}
			}
			if (FoundMaterial == nullptr &&
				(SearchLocation == EInterchangeMaterialSearchLocation::UnderRoot ||
					SearchLocation == EInterchangeMaterialSearchLocation::AllAssets))
			{
				// Search recursively in root folder of asset
				FString OutPackageRoot, OutPackagePath, OutPackageName;
				FPackageName::SplitLongPackageName(SearchPath, OutPackageRoot, OutPackagePath, OutPackageName);
				if (!SearchPath.IsEmpty())
				{
					FoundMaterial = FindExistingMaterial(OutPackageRoot, MaterialFullName, true);
				}
			}
			if (FoundMaterial == nullptr &&
				SearchLocation == EInterchangeMaterialSearchLocation::AllAssets)
			{
				// Search everywhere
				FoundMaterial = FindExistingMaterial(TEXT("/"), MaterialFullName, true);
			}
		}

		return FoundMaterial;
	}

	void AddSpecularProfileToFactoryNode(const UInterchangeBaseNode* ShaderNode, UInterchangeFactoryBaseNode* FactoryNode, UInterchangeBaseNodeContainer* BaseNodeContainer)
	{
		using namespace UE::Interchange::Materials;
		if (FString SpecularProfileUid; ShaderNode->GetStringAttribute(SubstrateMaterial::SpecularProfile.ToString(), SpecularProfileUid))
		{
			if (BaseNodeContainer->GetNode(SpecularProfileUid))
			{
				FactoryNode->AddStringAttribute(SubstrateMaterial::SpecularProfile.ToString(), SpecularProfileUid);
				FactoryNode->AddFactoryDependencyUid(UInterchangeFactoryBaseNode::BuildFactoryNodeUid(SpecularProfileUid));
			}
		}
	}
}

namespace UE::Interchange::Materials::HashUtils
{
#if UE_BUILD_DEBUG
	class FMaterialHashDebugData
	{
	public:
		FMaterialHashDebugData(const FString& InLogDirectoryPath)
			:LogDirectoryPath(InLogDirectoryPath) {}

		void Reset();

		void SaveLogsToFile(const FString& FileName);

		template<typename...TArgs>
		void LogMessage(const FString& Format, TArgs ...Args)
		{
			TArray<FStringFormatArg> FormattedArgs;
			AddFormattedArg(FormattedArgs, Args...);
			FString Message = FString::Format(*Format, FormattedArgs);
			UE_LOGF(LogInterchangePipeline, Log, "%ls", *Message);
			LogMessageContainer.Add(Message);
		}

		void LogCurrentNodeAddress()
		{
			FString NodeAddress = FString::Printf(TEXT("Current Node Address: %s\n"), *GetCurrentNodeAddress());
			UE_LOGF(LogInterchangePipeline, Log, "%ls", *NodeAddress);
			LogMessageContainer.Add(NodeAddress);
		}

		FString GetCurrentNodeAddress();
		void AddNodeAddress(const FString& NodeAddress, bool bCreatePopCheckPoint = true);
		void PopNodeAddressesToLastPopIndex();

	private:
		template<typename TArg>
		void AddFormattedArg(TArray<FStringFormatArg>& FormattedArgs, TArg Arg)
		{
			FormattedArgs.Add(Arg);
		}

		template<typename TArg, typename ...TArgs>
		void AddFormattedArg(TArray<FStringFormatArg>& FormattedArgs, TArg Arg, TArgs ...Args)
		{
			FormattedArgs.Add(Arg);
			AddFormattedArg(FormattedArgs, Args...);
		}

	private:
		FString LogDirectoryPath;
		TArray<FString> LogMessageContainer;

		TArray<FString> NodeAddressStack;
		TArray<int32> NodeAddressPopCheckPoints;
	};

	FString FMaterialHashDebugData::GetCurrentNodeAddress()
	{
		TStringBuilder<512> StringBuilder;
		for (int32 i = 0; i < NodeAddressStack.Num(); ++i)
		{
			StringBuilder.Append(NodeAddressStack[i]);
			if (i < NodeAddressStack.Num() - 1)
			{
				StringBuilder.Append(TEXT("/"));
			}
		}

		return StringBuilder.ToString();
	}

	void FMaterialHashDebugData::SaveLogsToFile(const FString& FileName)
	{
		const FString LogFileExtension = TEXT(".txt");

		if (LogMessageContainer.Num())
		{
			static FString FileDirectory = FPaths::ProjectSavedDir() + LogDirectoryPath;

			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			if (PlatformFile.CreateDirectoryTree(*FileDirectory))
			{
				FString AbsolutePath = FileDirectory + FileName + LogFileExtension;
				FFileHelper::SaveStringArrayToFile(LogMessageContainer, *AbsolutePath);
			}
		}
	}

	void FMaterialHashDebugData::Reset()
	{
		NodeAddressStack.Empty();
		LogMessageContainer.Empty();
	}

	void FMaterialHashDebugData::AddNodeAddress(const FString& NodeAddress, bool bCreatePopCheckPoint /*= true*/)
	{
		if (bCreatePopCheckPoint)
		{
			NodeAddressPopCheckPoints.Add(NodeAddressStack.Num());
		}

		NodeAddressStack.Add(NodeAddress);
	}

	void FMaterialHashDebugData::PopNodeAddressesToLastPopIndex()
	{
		int32 TargetStackSize = 0;
		if (NodeAddressPopCheckPoints.Num())
		{
			TargetStackSize = NodeAddressPopCheckPoints.Last();
		}

		if (!TargetStackSize)
		{
			NodeAddressStack.Empty();
			NodeAddressPopCheckPoints.Empty();
		}
		else
		{
			while (NodeAddressStack.Num() && NodeAddressStack.Num() > TargetStackSize)
			{
				NodeAddressStack.Pop();
			}

			NodeAddressPopCheckPoints.Pop();
		}
	}
#endif

	class FDuplicateMaterialHelper
	{
	public:
#if UE_BUILD_DEBUG
		FDuplicateMaterialHelper(UInterchangeGenericMaterialPipeline& InGenericMaterialPipeline, FMaterialHashDebugData* InHashDebugData)
			:GenericMaterialPipeline(InGenericMaterialPipeline),
			HashDebugData(InHashDebugData)
		{}
#else
		FDuplicateMaterialHelper(UInterchangeGenericMaterialPipeline& InGenericMaterialPipeline)
			:GenericMaterialPipeline(InGenericMaterialPipeline)
		{}
#endif
		void CopyLeafInputsToFactoryNode(UInterchangeBaseMaterialFactoryNode* FactoryNode);

		void ComputMaterialHash(const UInterchangeShaderGraphNode* ShaderGraphNode);

		void SetupOverridableScalarParameter(const UInterchangeShaderNode* ShaderNode, const FString& ParameterKey, const FString& OverridableParameterNameKey);
		void SetupOverridableVectorParameter(const UInterchangeShaderNode* ShaderNode, const FString& ParameterKey, const FString& OverridableParameterNameKey);
		void SetupOverridableStaticBoolParameter(const UInterchangeShaderNode* ShaderNode, const FString& ParameterKey, const FString& OverridableParameterNameKey);
		void SetupOverridableTextureParameter(const UInterchangeShaderNode* ShaderNode, const FString& InputKey, const FString& OverridableParameterNameKey);

		/**
		 * Creates a Base Material Factory Node based on if the material is a duplicate material or if it is found for the first time.
		 * If the option to create a material instance for the parent is enabled, then additional material instance factory for parent would also be created.
		 */
		UInterchangeBaseMaterialFactoryNode* CreateFactoryForDuplicateMaterials(const UInterchangeShaderGraphNode* ShaderGraphNode, bool bImportUnusedMaterial, bool bCreateInstanceForParent);

		void ResetHashData();

		bool IsDuplicate()const { return bIsDuplicate; }
		const UInterchangeBaseNode* const GetAttributeStorageNode() const { return AttributeStorageNode; }

		template<class TInterchangeResultType>
		void PostMessage(FText&& MessageText)
		{
			if (GenericMaterialPipeline.Results)
			{
				TInterchangeResultType* Result = GenericMaterialPipeline.Results->Add<TInterchangeResultType>();
				Result->Text = MoveTemp(MessageText);
			}
		}

	private:
		UInterchangeBaseMaterialFactoryNode* CreateMaterialFactory(const UInterchangeShaderGraphNode* ShaderGraphNode);
		UInterchangeMaterialInstanceFactoryNode* CreateMaterialInstanceFactoryFromReference(const UInterchangeShaderGraphNode* ShaderGraphNode);
		UInterchangeMaterialInstanceFactoryNode* CreateMaterialInstanceFactoryForParent(const UInterchangeShaderGraphNode* ShaderGraphNode);

		TEnumAsByte<EBlendMode> GetShaderGraphNodeBlendMode(const UInterchangeShaderGraphNode* ShaderGraphNode) const;
		uint8 GetShaderGraphNodeShadingModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const;

		int32 ComputeShaderGraphNodeHash(const UInterchangeShaderGraphNode* ShaderGraphNode);
		int32 ComputeShaderNodeHash(const  UInterchangeShaderNode* ShaderNode);
		int32 ComputeShaderInputHash(const UInterchangeShaderNode* ShaderNode, const FString& InputName);
		int32 HashCombineCustom(int32 Hash, int32 CombineWith);

	private:
		UInterchangeGenericMaterialPipeline& GenericMaterialPipeline;

		TMap<int32, UInterchangeBaseMaterialFactoryNode*> ParentMaterialFactoryMap;

		UInterchangeBaseNode* AttributeStorageNode = nullptr;

#if UE_BUILD_DEBUG
		FMaterialHashDebugData* HashDebugData = nullptr;
#endif

		TArray<UE::Interchange::FAttributeKey> LeafInputAttributeKeys;
		TSet<const UInterchangeShaderNode*> LeafInputShaderNodes;

		int32 AccumulatedHash = 0;
		int32 MaterialHash = 0;

		bool bIsDuplicate = false;
	};

	FString GetDefaultValueStringForShaderType(FString ShaderType)
	{
		if (*ShaderType == Standard::Nodes::ScalarParameter::Name)
		{
			return Standard::Nodes::ScalarParameter::Attributes::DefaultValue.ToString();
		}
		else if (*ShaderType == Standard::Nodes::VectorParameter::Name)
		{
			return Standard::Nodes::VectorParameter::Attributes::DefaultValue.ToString();
		}
		else if (*ShaderType == Standard::Nodes::StaticBoolParameter::Name)
		{
			return Standard::Nodes::StaticBoolParameter::Attributes::DefaultValue.ToString();
		}

		return FString();
	}
}

#if UE_BUILD_DEBUG
#define ADD_LOG_MESSAGE(...) if(HashDebugData){\
HashDebugData->LogMessage(__VA_ARGS__);\
}

#define ADD_NODE_ADDRESS_MESSAGE() if(HashDebugData){\
HashDebugData->LogCurrentNodeAddress();\
}

#define PUSH_NODE_ADDRESS(Node) if(HashDebugData){\
HashDebugData->AddNodeAddress(Node);\
}

#define PUSH_NODE_ADDRESS_WITHOUT_CHECKPOINT(Node) if(HashDebugData){\
HashDebugData->AddNodeAddress(Node, false);\
}

#define POP_NODE_ADDRESSES() if(HashDebugData){\
HashDebugData->PopNodeAddressesToLastPopIndex();\
}
#else

#define ADD_LOG_MESSAGE(...)
#define ADD_NODE_ADDRESS_MESSAGE()
#define PUSH_NODE_ADDRESS(Node)
#define PUSH_NODE_ADDRESS_WITHOUT_CHECKPOINT(Node)
#define POP_NODE_ADDRESSES()

#endif

UInterchangeGenericMaterialPipeline::UInterchangeGenericMaterialPipeline()
{
	TexturePipeline = CreateDefaultSubobject<UInterchangeGenericTexturePipeline>("TexturePipeline");
	SparseVolumeTexturePipeline = CreateDefaultSubobject<UInterchangeSparseVolumeTexturePipeline>("SparseVolumeTexturePipeline");
}

void UInterchangeGenericMaterialPipeline::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FInterchangeCustomVersion::GUID);
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FInterchangeCustomVersion::GUID) < FInterchangeCustomVersion::AdditionOfReuseExistingMaterialsBool)
		{
			UpdateSearchLocationProperty();
		}
	}

}

FString UInterchangeGenericMaterialPipeline::GetPipelineCategory(UClass* AssetClass)
{
	return TEXT("Materials");
}

void UInterchangeGenericMaterialPipeline::PreDialogCleanup(const FName PipelineStackName)
{
	if (TexturePipeline)
	{
		TexturePipeline->PreDialogCleanup(PipelineStackName);
	}

	if (SparseVolumeTexturePipeline)
	{
		SparseVolumeTexturePipeline->PreDialogCleanup(PipelineStackName);
	}

	//Save only pipeline if we are a stand alone pipeline (not a sub object of another pipeline)
	if (IsStandAlonePipeline())
	{
		SaveSettings(PipelineStackName);
	}

	UpdateSearchLocationProperty();
}

bool UInterchangeGenericMaterialPipeline::IsSettingsAreValid(TOptional<FText>& OutInvalidReason) const
{
	if (TexturePipeline && !TexturePipeline->IsSettingsAreValid(OutInvalidReason))
	{
		return false;
	}

	if (SparseVolumeTexturePipeline && !SparseVolumeTexturePipeline->IsSettingsAreValid(OutInvalidReason))
	{
		return false;
	}

	return Super::IsSettingsAreValid(OutInvalidReason);
}

void UInterchangeGenericMaterialPipeline::AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams)
{
	Super::AdjustSettingsForContext(ContextParams);

	if (TexturePipeline)
	{
		TexturePipeline->AdjustSettingsForContext(ContextParams);
	}
	if (SparseVolumeTexturePipeline)
	{
		SparseVolumeTexturePipeline->AdjustSettingsForContext(ContextParams);
	}
#if WITH_EDITOR
	TArray<FString> HideCategories;
	bool bIsObjectAMaterial = !ContextParams.ReimportAsset ? false : ContextParams.ReimportAsset->IsA(UMaterialInterface::StaticClass());
	if (ContextParams.ContextType == EInterchangePipelineContext::AssetCustomLODImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomLODReimport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetAlternateSkinningImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetAlternateSkinningReimport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomMorphTargetImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomMorphTargetReImport)
	{
		bImportMaterials = false;
		HideCategories.Add(UInterchangeGenericMaterialPipeline::GetPipelineCategory(nullptr));
		bReuseExistingMaterials = false;
	}

	if (UInterchangePipelineBase* OuterMostPipeline = GetMostPipelineOuter())
	{
		for (const FString& HideCategoryName : HideCategories)
		{
			HidePropertiesOfCategory(OuterMostPipeline, this, HideCategoryName);
		}
		if (!bIsObjectAMaterial && ContextParams.ContextType == EInterchangePipelineContext::AssetReimport)
		{
			//When we re-import we hide all setting but search location, so we can find existing materials.
			HideProperty(OuterMostPipeline, this, GET_MEMBER_NAME_CHECKED(UInterchangeGenericMaterialPipeline, bImportMaterials));
			HideProperty(OuterMostPipeline, this, GET_MEMBER_NAME_CHECKED(UInterchangeGenericMaterialPipeline, MaterialImport));
			HideProperty(OuterMostPipeline, this, GET_MEMBER_NAME_CHECKED(UInterchangeGenericMaterialPipeline, bIdentifyDuplicateMaterials));
			HideProperty(OuterMostPipeline, this, GET_MEMBER_NAME_CHECKED(UInterchangeGenericMaterialPipeline, bCreateMaterialInstanceForParent));
			HideProperty(OuterMostPipeline, this, GET_MEMBER_NAME_CHECKED(UInterchangeGenericMaterialPipeline, ParentMaterial));
			HideProperty(OuterMostPipeline, this, GET_MEMBER_NAME_CHECKED(UInterchangeGenericMaterialPipeline, AssetName));
		}
	}

#endif //WITH_EDITOR
	using namespace UE::Interchange;

	if (!InterchangeGenericMaterialPipeline::Private::AreRequiredPackagesLoaded())
	{
		UE_LOGF(LogInterchangePipeline, Warning, "UInterchangeGenericMaterialPipeline: Some required packages are missing. Material import might be wrong");
	}
}
#if WITH_EDITOR

void UInterchangeGenericMaterialPipeline::FilterPropertiesFromTranslatedData(UInterchangeBaseNodeContainer* InBaseNodeContainer)
{
	Super::FilterPropertiesFromTranslatedData(InBaseNodeContainer);

	//Filter all material pipeline properties if there is no translated material.
	TArray<FString> TmpMaterialNodes;
	InBaseNodeContainer->GetNodes(UInterchangeShaderGraphNode::StaticClass(), TmpMaterialNodes);
	uint32 MaterialCount = TmpMaterialNodes.Num();
	InBaseNodeContainer->GetNodes(UInterchangeMaterialInstanceNode::StaticClass(), TmpMaterialNodes);
	MaterialCount += TmpMaterialNodes.Num();
	if (MaterialCount == 0)
	{
		if (UInterchangePipelineBase* OuterMostPipeline = GetMostPipelineOuter())
		{
			HidePropertiesOfCategory(OuterMostPipeline, this, UInterchangeGenericMaterialPipeline::GetPipelineCategory(nullptr));
		}
	}

	if (TexturePipeline)
	{
		TexturePipeline->FilterPropertiesFromTranslatedData(InBaseNodeContainer);
	}

	if (SparseVolumeTexturePipeline)
	{
		SparseVolumeTexturePipeline->FilterPropertiesFromTranslatedData(InBaseNodeContainer);
	}
}

bool UInterchangeGenericMaterialPipeline::IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UInterchangeGenericMaterialPipeline, bImportMaterials))
	{
		return true;
	}

	if (TexturePipeline && TexturePipeline->IsPropertyChangeNeedRefresh(PropertyChangedEvent))
	{
		return true;
	}
	if (SparseVolumeTexturePipeline && SparseVolumeTexturePipeline->IsPropertyChangeNeedRefresh(PropertyChangedEvent))
	{
		return true;
	}
	return Super::IsPropertyChangeNeedRefresh(PropertyChangedEvent);
}

void UInterchangeGenericMaterialPipeline::GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const
{
	if (TexturePipeline)
	{
		TexturePipeline->GetSupportAssetClasses(PipelineSupportAssetClasses);
	}
	if (SparseVolumeTexturePipeline)
	{
		SparseVolumeTexturePipeline->GetSupportAssetClasses(PipelineSupportAssetClasses);
	}

	PipelineSupportAssetClasses.Add(UMaterial::StaticClass());
	PipelineSupportAssetClasses.Add(UMaterialInstance::StaticClass());
}

#endif //WITH_EDITOR

void UInterchangeGenericMaterialPipeline::ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath)
{
#if UE_BUILD_DEBUG
	UE::Interchange::Materials::HashUtils::FMaterialHashDebugData HashDebugData(TEXT("InterchangeDebug/MaterialHashLogs/"));
	UE::Interchange::Materials::HashUtils::FDuplicateMaterialHelper HashHelper(*this, &HashDebugData);
#else
	UE::Interchange::Materials::HashUtils::FDuplicateMaterialHelper HashHelper(*this);
#endif

	if (!InBaseNodeContainer)
	{
		UE_LOGF(LogInterchangePipeline, Warning, "UInterchangeGenericMaterialPipeline: Cannot execute pre-import pipeline because InBaseNodeContrainer is null");
		return;
	}

	//Set the result container to allow error message
	//The parent Results container should be set at this point
	ensure(Results);
	{
		if (TexturePipeline)
		{
			TexturePipeline->SetResultsContainer(Results);
		}
		if (SparseVolumeTexturePipeline)
		{
			SparseVolumeTexturePipeline->SetResultsContainer(Results);
		}
	}

	BaseNodeContainer = InBaseNodeContainer;
	SourceDatas.Empty(InSourceDatas.Num());
	for (const UInterchangeSourceData* SourceData : InSourceDatas)
	{
		SourceDatas.Add(SourceData);
	}

	if (TexturePipeline)
	{
		TexturePipeline->ScriptedExecutePipeline(InBaseNodeContainer, InSourceDatas, ContentBasePath);
	}
	if (SparseVolumeTexturePipeline)
	{
		SparseVolumeTexturePipeline->ScriptedExecutePipeline(InBaseNodeContainer, InSourceDatas, ContentBasePath);
	}

	if (!bImportMaterials)
	{
		// Skip import of materials as a whole
		return;
	}

	TSet<UInterchangeShaderGraphNode*> ShaderGraphNodes;
	TSet<UInterchangeMaterialInstanceNode*> MaterialInstanceNodes;
	TSet<UInterchangeMaterialReferenceNode*> MaterialReferenceNodes;
	TSet<UInterchangeBaseMaterialFactoryNode*> BaseMaterialFactoryNodes;

	auto FixMaterialName = [](UInterchangeBaseNode* Node)
		{
			FString MaterialName = Node->GetDisplayLabel();

			bool bUpdateMaterialName = false;
			if (MaterialName.Len() > 6)
			{
				int32 Offset = MaterialName.Find(TEXT("_SKIN"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
				if (Offset != INDEX_NONE)
				{
					// Chop off the material name so we are left with the number in _SKINXX
					FString SkinXXNumber = MaterialName.Right(MaterialName.Len() - (Offset + 1)).RightChop(4);

					if (SkinXXNumber.IsNumeric())
					{
						// remove the '_skinXX' suffix from the material name					
						MaterialName.LeftChopInline(MaterialName.Len() - Offset, EAllowShrinking::No);
						bUpdateMaterialName = true;
					}
				}
			}
			else if (MaterialName.IsEmpty())
			{
				MaterialName = TEXT("UnnamedMaterial");
				bUpdateMaterialName = true;
			}

			if (bUpdateMaterialName)
			{
				Node->SetDisplayLabel(MaterialName);
			}
		};

	// Find all nodes we need for this pipeline
	BaseNodeContainer->IterateNodes(
		[&ShaderGraphNodes,
		 &MaterialInstanceNodes,
		 &MaterialReferenceNodes,
		 &BaseMaterialFactoryNodes,
		 &FixMaterialName](const FString& NodeUid, UInterchangeBaseNode* Node)
		{
			switch (Node->GetNodeContainerType())
			{
				case EInterchangeNodeContainerType::TranslatedAsset:
				{
					if (UInterchangeShaderGraphNode* ShaderGraphNode = Cast<UInterchangeShaderGraphNode>(Node))
					{
						FixMaterialName(ShaderGraphNode);
						ShaderGraphNodes.Add(ShaderGraphNode);
					}
					else if (UInterchangeMaterialInstanceNode* InstanceNode = Cast<UInterchangeMaterialInstanceNode>(Node))
					{
						FixMaterialName(InstanceNode);
						MaterialInstanceNodes.Add(InstanceNode);
					}
					else if (UInterchangeMaterialReferenceNode* ReferenceNode = Cast<UInterchangeMaterialReferenceNode>(Node))
					{
						FixMaterialName(ReferenceNode);
						MaterialReferenceNodes.Add(ReferenceNode);
					}
				}
				case EInterchangeNodeContainerType::FactoryData:
				{
					if (UInterchangeBaseMaterialFactoryNode* BaseMaterialFactoryNode = Cast<UInterchangeBaseMaterialFactoryNode>(Node))
					{
						FixMaterialName(BaseMaterialFactoryNode);
						BaseMaterialFactoryNodes.Add(BaseMaterialFactoryNode);
					}
				}
				break;
			}
		}
	);

	// Check to see whether materials should be created even if unused
	// By default we let the setting of the pipeline to decide if we create the materials, every node with mesh attribute can enable/disable them, depending on the pipeline stack chosen.
	bool bImportUnusedMaterial = bImportMaterials;
	if (const UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::GetUniqueInstance(BaseNodeContainer))
	{
		SourceNode->GetCustomImportUnusedMaterial(bImportUnusedMaterial);
		bImportUnusedMaterial |= bImportMaterials ;
	}

	// Can't import materials at runtime, fall back to instances
	UInterchangeEditorUtilitiesBase* EditorUtilities = UInterchangeManager::GetInterchangeManager().GetEditorUtilities();
	if ((EditorUtilities && EditorUtilities->IsRuntimeOrPIE()) && MaterialImport == EInterchangeMaterialImportOption::ImportAsMaterials)
	{
		MaterialImport = EInterchangeMaterialImportOption::ImportAsMaterialInstances;
	}

	if (MaterialImport == EInterchangeMaterialImportOption::ImportAsMaterials)
	{
		for (const UInterchangeShaderGraphNode* ShaderGraphNode : ShaderGraphNodes)
		{
			UInterchangeBaseMaterialFactoryNode* MaterialBaseFactoryNode = nullptr;

			bool bIsAShaderFunction;
			if (ShaderGraphNode->GetCustomIsAShaderFunction(bIsAShaderFunction) && bIsAShaderFunction)
			{
				MaterialBaseFactoryNode = CreateMaterialFunctionFactoryNode(ShaderGraphNode);
			}
			else if (!bIdentifyDuplicateMaterials)
			{
				MaterialBaseFactoryNode = CreateMaterialFactoryNode(ShaderGraphNode);
			}
			else
			{
				HashHelper.ResetHashData();
				HashHelper.ComputMaterialHash(ShaderGraphNode);
				AttributeStorageNode = HashHelper.GetAttributeStorageNode();

				/* Creates Material Instance Factory if duplicate material is found. */
				MaterialBaseFactoryNode = HashHelper.CreateFactoryForDuplicateMaterials(ShaderGraphNode, bImportUnusedMaterial, bCreateMaterialInstanceForParent);

#if UE_BUILD_DEBUG
				HashDebugData.SaveLogsToFile(ShaderGraphNode->GetUniqueID());
#endif
				/* Clearing the AttributeStorageNode as it might affect how the MaterialFunctionsFactories are created. */
				AttributeStorageNode = nullptr;
			}

			if (MaterialBaseFactoryNode)
			{
				BaseMaterialFactoryNodes.Add(MaterialBaseFactoryNode);
			}
		}
	}
	else if (MaterialImport == EInterchangeMaterialImportOption::ImportAsMaterialInstances)
	{
		for (const UInterchangeShaderGraphNode* ShaderGraphNode : ShaderGraphNodes)
		{
			if (UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode = CreateMaterialInstanceFactoryNode(ShaderGraphNode))
			{
				BaseMaterialFactoryNodes.Add(MaterialInstanceFactoryNode);
				MaterialInstanceFactoryNode->SetEnabled(bImportUnusedMaterial);
			}
		}
	}

	for (UInterchangeMaterialInstanceNode* MaterialNode : MaterialInstanceNodes)
	{
		FString ParentPathOrMaterialUid;
		// Override any existing parent material if one has been specified and bCreateMaterialInstanceForParent is true
		UMaterialInterface* ParentMaterialObj = bCreateMaterialInstanceForParent ? Cast<UMaterialInterface>(ParentMaterial.TryLoad()) : nullptr;
		if (ParentMaterialObj)
		{
			ParentPathOrMaterialUid = ParentMaterialObj->GetPathName();
			MaterialNode->SetCustomParent(ParentPathOrMaterialUid);
		}
        else if (!MaterialNode->GetCustomParent(ParentPathOrMaterialUid) || ParentPathOrMaterialUid.IsEmpty())
        {
            continue;
        }

        // the parent path can either be the path to an existing Material or the UID of a ShaderGraphNode
        const FString MaterialFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(ParentPathOrMaterialUid);
		const bool bIsMaterialFactoryUid = Cast<UInterchangeMaterialFactoryNode>(BaseNodeContainer->GetFactoryNode(MaterialFactoryNodeUid)) != nullptr;
        if (bIsMaterialFactoryUid)
        {
            ParentPathOrMaterialUid = MaterialFactoryNodeUid;
        }

		UInterchangeMaterialInstanceFactoryNode* MaterialFactoryNode = nullptr;
		FString DisplayLabel = MaterialNode->GetDisplayLabel();
		const FString NodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(MaterialNode->GetUniqueID());
		if (BaseNodeContainer->IsNodeUidValid(NodeUid))
		{
			//The node already exist, just return it
			MaterialFactoryNode = Cast<UInterchangeMaterialInstanceFactoryNode>(BaseNodeContainer->GetFactoryNode(NodeUid));
			if (!MaterialFactoryNode)
			{
				continue;
			}
		}
		else
		{
			MaterialFactoryNode = NewObject<UInterchangeMaterialInstanceFactoryNode>(BaseNodeContainer);
			if (!ensure(MaterialFactoryNode))
			{
				continue;
			}
			//Creating a Material
			BaseNodeContainer->SetupNode(MaterialFactoryNode, NodeUid, DisplayLabel, EInterchangeNodeContainerType::FactoryData);

			MaterialFactoryNode->AddTargetNodeUid(MaterialNode->GetUniqueID());
			MaterialNode->AddTargetNodeUid(MaterialFactoryNode->GetUniqueID());

			UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(MaterialNode, MaterialFactoryNode, false);
		}

		BaseMaterialFactoryNodes.Add(MaterialFactoryNode);

		// Set MaterialFactoryNode's display label to MaterialNode's uniqueID
		// to reconcile mesh's slot names and material assets
		MaterialFactoryNode->SetDisplayLabel(MaterialNode->GetAssetName());
		MaterialFactoryNode->SetCustomParent(ParentPathOrMaterialUid);
		if (bIsMaterialFactoryUid)
		{
			MaterialFactoryNode->AddFactoryDependencyUid(ParentPathOrMaterialUid);
		}

		const UClass* MaterialClass = (EditorUtilities && EditorUtilities->IsRuntimeOrPIE()) ? UMaterialInstanceDynamic::StaticClass() : UMaterialInstanceConstant::StaticClass();
		MaterialFactoryNode->SetCustomInstanceClassName(MaterialClass->GetPathName());

		if (int BlendMode; MaterialNode->GetCustomBlendMode(BlendMode))
		{
			MaterialFactoryNode->SetCustomBlendMode(EBlendMode{ BlendMode });
		}

		if (bool bTwoSided; MaterialNode->GetCustomTwoSided(bTwoSided))
		{
			MaterialFactoryNode->SetCustomTwoSided(bTwoSided);
		}

		if (bool bEnableTessellation; MaterialNode->GetCustomEnableTessellation(bEnableTessellation))
		{
			MaterialFactoryNode->SetCustomEnableTessellation(bEnableTessellation);
		}

		UE::Interchange::InterchangeGenericMaterialPipeline::Private::AddSpecularProfileToFactoryNode(MaterialNode, MaterialFactoryNode, BaseNodeContainer);

		TArray<FString> Inputs;
		UInterchangeShaderPortsAPI::GatherInputs(MaterialNode, Inputs);

		for (const FString& InputName : Inputs)
		{
			const bool bIsAParameter = UInterchangeShaderPortsAPI::HasParameter(MaterialNode, FName(InputName));
			FString InputValueKey = CreateInputKey(InputName,bIsAParameter);

			switch (UInterchangeShaderPortsAPI::GetInputType(MaterialNode, InputName, bIsAParameter))
			{
			case UE::Interchange::EAttributeTypes::Bool:
			{
				bool AttributeValue = false;
				MaterialNode->GetBooleanAttribute(InputValueKey, AttributeValue);
				MaterialFactoryNode->AddBooleanAttribute(InputValueKey, AttributeValue);
			}
			break;
			case UE::Interchange::EAttributeTypes::Int32:
			{
				int32 AttributeValue = 0;
				MaterialNode->GetInt32Attribute(InputValueKey, AttributeValue);
				MaterialFactoryNode->AddInt32Attribute(InputValueKey, AttributeValue);
			}
			break;
			case UE::Interchange::EAttributeTypes::Float:
			{
				float AttributeValue = 0.f;
				MaterialNode->GetFloatAttribute(InputValueKey, AttributeValue);
				MaterialFactoryNode->AddFloatAttribute(InputValueKey, AttributeValue);
			}
			break;
			case UE::Interchange::EAttributeTypes::LinearColor:
			{
				FLinearColor AttributeValue = FLinearColor::White;
				MaterialNode->GetLinearColorAttribute(InputValueKey, AttributeValue);
				MaterialFactoryNode->AddLinearColorAttribute(InputValueKey, AttributeValue);
			}
			break;
			case UE::Interchange::EAttributeTypes::String:
			{
				FString TextureUid;
				MaterialNode->GetStringAttribute(InputValueKey, TextureUid);

				FString FactoryTextureUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TextureUid);

				MaterialFactoryNode->AddStringAttribute(InputValueKey, FactoryTextureUid);
				MaterialFactoryNode->AddFactoryDependencyUid(FactoryTextureUid);
			}
			break;
			}
		}
	}

	for (UInterchangeMaterialReferenceNode* MaterialReferenceNode : MaterialReferenceNodes)
	{
		FString MaterialContentPath;
		if (MaterialReferenceNode->GetCustomContentPath(MaterialContentPath))
		{
			UInterchangeMaterialReferenceFactoryNode* FactoryNode = NewObject<UInterchangeMaterialReferenceFactoryNode>(BaseNodeContainer);

			const FString DisplayLabel = MaterialReferenceNode->GetDisplayLabel();
			const FString NodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(MaterialReferenceNode->GetUniqueID());
			BaseNodeContainer->SetupNode(FactoryNode, NodeUid, DisplayLabel, EInterchangeNodeContainerType::FactoryData);

			FactoryNode->SetCustomReferenceObject(FSoftObjectPath{MaterialContentPath});

			BaseMaterialFactoryNodes.Add(FactoryNode);

			FactoryNode->AddTargetNodeUid(MaterialReferenceNode->GetUniqueID());
			MaterialReferenceNode->AddTargetNodeUid(FactoryNode->GetUniqueID());
		}
	}

	//If we have a valid override name
	FString OverrideAssetName = IsStandAlonePipeline() ? DestinationName : FString();
	if (OverrideAssetName.IsEmpty() && IsStandAlonePipeline())
	{
		OverrideAssetName = AssetName;
	}

	// Enable or disable factory nodes according to import/reimport context
	const bool bCombinedImportMaterials = bImportMaterials || bImportUnusedMaterial;
	for (UInterchangeBaseMaterialFactoryNode* BaseMaterialFactoryNode : BaseMaterialFactoryNodes)
	{
		using namespace UE::Interchange::InterchangeGenericMaterialPipeline::Private;

		bool bEnableFactoryNode = bCombinedImportMaterials;

		if (bReuseExistingMaterials)
		{
			FString MaterialName = BaseMaterialFactoryNode->GetDisplayLabel();
			if (UMaterialInterface* ExistingMaterial = FindExistingMaterialFromSearchLocation(MaterialName, ContentBasePath, SearchLocation))
			{
				// Add the search location to notify other pipelines what to do about assets.
				BaseMaterialFactoryNode->AddInt32Attribute(TEXT("SearchLocation"), int32(SearchLocation));
				BaseMaterialFactoryNode->SetCustomReferenceObject(ExistingMaterial);

				const bool bIsReimport = CacheContextParam.ReimportAsset != nullptr;

				// We're reimporting the existing material we found: Enable the factory node if we can.
				// TODO: Can this potentially lead to issues, since the existing material to reimport is discovered by name and search location?
				if (bIsReimport && CacheContextParam.ReimportAsset == ExistingMaterial)
				{
					const bool bIsMaterial = BaseMaterialFactoryNode->IsA<UInterchangeMaterialFactoryNode>()
						&& ExistingMaterial->IsA<UMaterial>();

					const bool bIsMaterialInstance = BaseMaterialFactoryNode->IsA<UInterchangeMaterialInstanceFactoryNode>()
						&& ExistingMaterial->IsA<UMaterialInstance>();


					// Reimport can only be done on material instances
					bEnableFactoryNode = !bIsMaterial && bIsMaterialInstance;
				}
#if WITH_EDITORONLY_DATA
				// This is a scene reimport of a material instance
				// check whether or not it is a reimport of the current material instance factory node 
				else if (bIsReimport && CacheContextParam.ReimportAsset->IsA<UInterchangeSceneImportAsset>() && ExistingMaterial->IsA<UMaterialInstance>())
				{
					// Skip it if it is not a reimport of the current material instance factory node
					bEnableFactoryNode = false;

					if (const UInterchangeAssetImportData* AssetImportData = Cast<UInterchangeAssetImportData>(ExistingMaterial->AssetImportData))
					{
						const UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(CacheContextParam.ReimportAsset);
						if (SceneImportAsset->AssetImportData)
						{
							bEnableFactoryNode = AssetImportData->GetFirstFilename().Equals(SceneImportAsset->AssetImportData->GetFirstFilename());
						}
					}
				}
#endif
				else
				{
					// Disabling the factory node here is important so that we don't overwrite the existing asset.
					// The existing material will still actually be used for the import's material assignments and etc.
					bEnableFactoryNode = false;
				}

				BaseMaterialFactoryNode->SetCustomIsMaterialImportEnabled(bEnableFactoryNode);
				BaseMaterialFactoryNode->SetEnabled(bEnableFactoryNode);

				if (!bEnableFactoryNode)
				{
					continue;
				}
			}
		}

		bEnableFactoryNode &= bCreateNewMaterials;

		BaseMaterialFactoryNode->SetCustomIsMaterialImportEnabled(bEnableFactoryNode);
		BaseMaterialFactoryNode->SetEnabled(bEnableFactoryNode);
	}

	if (IsStandAlonePipeline() && !OverrideAssetName.IsEmpty() && BaseMaterialFactoryNodes.Num() == 1)
	{
		if (UInterchangeBaseMaterialFactoryNode* Node = *BaseMaterialFactoryNodes.CreateIterator())
		{
			Node->SetAssetName(OverrideAssetName);
			Node->SetDisplayLabel(OverrideAssetName);
		}
	}


	TArray<UInterchangeSpecularProfileNode*> SpecularProfileNodes;
	BaseNodeContainer->IterateNodesOfType<UInterchangeSpecularProfileNode>([&](const FString& NodeUid, UInterchangeSpecularProfileNode* Node)
																			 {
																			   SpecularProfileNodes.Add(Node);
																			 });

	for (const UInterchangeSpecularProfileNode* SpecularProfileNode : SpecularProfileNodes)
	{
		CreateSpecularProfileFactoryNode(SpecularProfileNode);
	}
}

void UInterchangeGenericMaterialPipeline::ExecutePostFactoryPipeline(const UInterchangeBaseNodeContainer* InBaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport)
{
	if (TexturePipeline)
	{
		TexturePipeline->ScriptedExecutePostFactoryPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
	}
	if (SparseVolumeTexturePipeline)
	{
		SparseVolumeTexturePipeline->ScriptedExecutePostFactoryPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
	}
}

void UInterchangeGenericMaterialPipeline::ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* InBaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport)
{
	if (TexturePipeline)
	{
		TexturePipeline->ScriptedExecutePostImportPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
	}
	if (SparseVolumeTexturePipeline)
	{
		SparseVolumeTexturePipeline->ScriptedExecutePostImportPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
	}
}

void UInterchangeGenericMaterialPipeline::SetReimportSourceIndex(UClass* ReimportObjectClass, const int32 SourceFileIndex)
{
	if (TexturePipeline)
	{
		TexturePipeline->ScriptedSetReimportSourceIndex(ReimportObjectClass, SourceFileIndex);
	}
	if (SparseVolumeTexturePipeline)
	{
		SparseVolumeTexturePipeline->ScriptedSetReimportSourceIndex(ReimportObjectClass, SourceFileIndex);
	}
}

UInterchangeBaseMaterialFactoryNode* UInterchangeGenericMaterialPipeline::CreateBaseMaterialFactoryNode(const UInterchangeBaseNode* MaterialNode, TSubclassOf<UInterchangeBaseMaterialFactoryNode> NodeType, bool bAddMaterialInstanceSuffix /*= false*/)
{
	const FString MaterialInstanceSuffix = TEXT("_MI");

	FString DisplayLabel = MaterialNode->GetDisplayLabel();
	FString NodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(MaterialNode->GetUniqueID());
	if (bAddMaterialInstanceSuffix)
	{
		NodeUid += MaterialInstanceSuffix;
	}

	UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode = nullptr;
	if (BaseNodeContainer->IsNodeUidValid(NodeUid))
	{
		//The node already exist, just return it
		MaterialFactoryNode = Cast<UInterchangeBaseMaterialFactoryNode>(BaseNodeContainer->GetFactoryNode(NodeUid));
		if (!ensure(MaterialFactoryNode))
		{
			//Log an error
		}
	}
	else
	{
		MaterialFactoryNode = NewObject<UInterchangeBaseMaterialFactoryNode>(BaseNodeContainer, NodeType.Get(), NAME_None);
		if (!ensure(MaterialFactoryNode))
		{
			return nullptr;
		}

		BaseNodeContainer->SetupNode(MaterialFactoryNode, NodeUid, DisplayLabel, EInterchangeNodeContainerType::FactoryData);

		MaterialFactoryNode->AddTargetNodeUid(MaterialNode->GetUniqueID());
		MaterialNode->AddTargetNodeUid(MaterialFactoryNode->GetUniqueID());
	}
	return MaterialFactoryNode;
}

bool UInterchangeGenericMaterialPipeline::HasClearCoat(const UInterchangeShaderGraphNode* ShaderGraphNode) const
{
	using namespace UE::Interchange::Materials::ClearCoat;

	const bool bHasClearCoatInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::ClearCoat);

	return bHasClearCoatInput;
}

bool UInterchangeGenericMaterialPipeline::HasSheen(const UInterchangeShaderGraphNode* ShaderGraphNode) const
{
	using namespace UE::Interchange::Materials::Sheen;

	const bool bHasSheenColorInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::SheenColor);

	return bHasSheenColorInput;
}

bool UInterchangeGenericMaterialPipeline::HasSubsurface(const UInterchangeShaderGraphNode* ShaderGraphNode) const
{
	using namespace UE::Interchange::Materials::Subsurface;

	const bool bHasSubsurfaceColorInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::SubsurfaceColor);

	return bHasSubsurfaceColorInput;
}

bool UInterchangeGenericMaterialPipeline::HasThinTranslucency(const UInterchangeShaderGraphNode* ShaderGraphNode) const
{
	using namespace UE::Interchange::Materials::ThinTranslucent;

	const bool bHasTransmissionColorInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::TransmissionColor);
	const bool bHasSurfaceCoverageInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::SurfaceCoverage);

	return bHasTransmissionColorInput || bHasSurfaceCoverageInput;
}

bool UInterchangeGenericMaterialPipeline::IsMetalRoughModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const
{
	using namespace UE::Interchange::Materials::PBRMR;

	const bool bHasBaseColorInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::BaseColor);

	return bHasBaseColorInput;
}

bool UInterchangeGenericMaterialPipeline::IsSpecGlossModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const
{
	using namespace UE::Interchange::Materials::PBRSG;

	const bool bHasSpecularInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::SpecularColor);
	const bool bHasGlossinessInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::Glossiness);

	return bHasSpecularInput && bHasGlossinessInput;
}

bool UInterchangeGenericMaterialPipeline::IsPhongModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const
{
	using namespace UE::Interchange::Materials::Phong;

	const bool bHasDiffuseInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::DiffuseColor);
	const bool bHasSpecularInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::SpecularColor);

	return bHasDiffuseInput && bHasSpecularInput;
}

bool UInterchangeGenericMaterialPipeline::IsLambertModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const
{
	using namespace UE::Interchange::Materials::Lambert;

	const bool bHasDiffuseInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::DiffuseColor);

	return bHasDiffuseInput;
}


bool UInterchangeGenericMaterialPipeline::IsSurfaceUnlitModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const
{
	using namespace UE::Interchange::Materials;
	FString ShaderType;
	ShaderGraphNode->GetCustomShaderType(ShaderType);

	if(ShaderType == SurfaceUnlit::Name.ToString())
	{
		return true;
	}

	return false;
}


bool UInterchangeGenericMaterialPipeline::HandleSpecGlossModel(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode)
{
	using namespace UE::Interchange::Materials;
	using namespace UE::Interchange::InterchangeGenericMaterialPipeline::Private;

	if (IsSpecGlossModel(ShaderGraphNode))
	{
		// ConvertFromDiffSpec function call
		const FString MaterialFunctionPath = TEXT("MaterialFunction'/Engine/Functions/Engine_MaterialFunctions01/Shading/ConvertFromDiffSpec.ConvertFromDiffSpec'");
		UInterchangeMaterialExpressionFactoryNode* FunctionCallExpression = CreateExpressionWithMaterialFunction(BaseNodeContainer, MaterialFactoryNode, TEXT("DiffSpecFunc"), MaterialFunctionPath);

		const FString FunctionCallExpressionUid = FunctionCallExpression->GetUniqueID();
		MaterialFactoryNode->ConnectOutputToBaseColor(FunctionCallExpressionUid, PBRMR::Parameters::BaseColor.ToString());
		MaterialFactoryNode->ConnectOutputToMetallic(FunctionCallExpressionUid, PBRMR::Parameters::Metallic.ToString());
		MaterialFactoryNode->ConnectOutputToSpecular(FunctionCallExpressionUid, PBRMR::Parameters::Specular.ToString());

		// DiffuseColor
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> DiffuseExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, PBRSG::Parameters::DiffuseColor.ToString(), FunctionCallExpressionUid);

			if (DiffuseExpressionFactoryNode.Get<0>())
			{
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(FunctionCallExpression, PBRSG::Parameters::DiffuseColor.ToString(),
					DiffuseExpressionFactoryNode.Get<0>()->GetUniqueID(), DiffuseExpressionFactoryNode.Get<1>());
			}
		}

		// Specular Color
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> SpecularExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, PBRSG::Parameters::SpecularColor.ToString(), FunctionCallExpressionUid);

			if (SpecularExpressionFactoryNode.Get<0>())
			{
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(FunctionCallExpression, PBRSG::Parameters::SpecularColor.ToString(),
					SpecularExpressionFactoryNode.Get<0>()->GetUniqueID(), SpecularExpressionFactoryNode.Get<1>());
			}
		}

		// Glossiness
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> GlossinessExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, PBRSG::Parameters::Glossiness.ToString(), MaterialFactoryNode->GetUniqueID());

			if (UInterchangeMaterialExpressionFactoryNode* GlossinessFactoryNode = GlossinessExpressionFactoryNode.Get<0>())
			{
				UInterchangeMaterialExpressionFactoryNode* OneMinusNode =
					CreateExpressionNode(TEXT("InverseGlossiness"), MaterialFactoryNode->GetUniqueID(), UMaterialExpressionOneMinus::StaticClass());

				const FString OneMinusNodeInput = GET_MEMBER_NAME_CHECKED(UMaterialExpressionOneMinus, Input).ToString();
				const FString OutputName = GlossinessExpressionFactoryNode.Get<1>();
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(OneMinusNode, OneMinusNodeInput, GlossinessFactoryNode->GetUniqueID(), OutputName);

				MaterialFactoryNode->ConnectOutputToRoughness(OneMinusNode->GetUniqueID(), PBRMR::Parameters::Roughness.ToString());
			}
		}

		return true;
	}

	return false;
}

bool UInterchangeGenericMaterialPipeline::HandlePhongModel(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode)
{
	using namespace UE::Interchange::Materials::Phong;
	using namespace UE::Interchange::InterchangeGenericMaterialPipeline::Private;

	if (IsPhongModel(ShaderGraphNode))
	{
		// ConvertFromDiffSpec function call
		const FString MaterialFunctionPath = TEXT("MaterialFunction'/InterchangeAssets/Functions/MF_PhongToMetalRoughness.MF_PhongToMetalRoughness'");
		UInterchangeMaterialExpressionFactoryNode* FunctionCallExpression = CreateExpressionWithMaterialFunction(BaseNodeContainer, MaterialFactoryNode, TEXT("DiffSpecFunc"), MaterialFunctionPath);

		const FString FunctionCallExpressionUid = FunctionCallExpression->GetUniqueID();
		MaterialFactoryNode->ConnectOutputToBaseColor(FunctionCallExpressionUid, UE::Interchange::Materials::PBRMR::Parameters::BaseColor.ToString());
		MaterialFactoryNode->ConnectOutputToMetallic(FunctionCallExpressionUid, UE::Interchange::Materials::PBRMR::Parameters::Metallic.ToString());
		MaterialFactoryNode->ConnectOutputToSpecular(FunctionCallExpressionUid, UE::Interchange::Materials::PBRMR::Parameters::Specular.ToString());
		MaterialFactoryNode->ConnectOutputToRoughness(FunctionCallExpressionUid, UE::Interchange::Materials::PBRMR::Parameters::Roughness.ToString());

		{
			const FString UniqueID = FunctionCallExpression->GetUniqueID();

			TFunction<void(const FString&)> ConnectInput;
			ConnectInput = [&](const FString& InputName) -> void
			{
				TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
					this->CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, InputName, UniqueID);

				if (ExpressionFactoryNode.Get<0>())
				{
					UInterchangeShaderPortsAPI::ConnectOuputToInputByName(FunctionCallExpression, InputName,
						ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
				}
			};

			ConnectInput(Parameters::AmbientColor.ToString());
			ConnectInput(Parameters::DiffuseColor.ToString());
			ConnectInput(Parameters::Shininess.ToString());
			ConnectInput(Parameters::SpecularColor.ToString());
		}

		return true;
	}

	return false;
}

bool UInterchangeGenericMaterialPipeline::HandleLambertModel(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode)
{
	using namespace UE::Interchange::Materials::Lambert;

	if (IsLambertModel(ShaderGraphNode))
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> DiffuseExpressionFactoryNode =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::DiffuseColor.ToString(), MaterialFactoryNode->GetUniqueID());

		if (DiffuseExpressionFactoryNode.Get<0>())
		{
			MaterialFactoryNode->ConnectOutputToBaseColor(DiffuseExpressionFactoryNode.Get<0>()->GetUniqueID(), DiffuseExpressionFactoryNode.Get<1>());
		}

		return true;
	}

	return false;
}

bool UInterchangeGenericMaterialPipeline::HandleMetalRoughnessModel(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode)
{
	using namespace UE::Interchange::Materials::PBRMR;

	bool bShadingModelHandled = false;

	// BaseColor
	{
		const bool bHasInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::BaseColor);

		if (bHasInput)
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::BaseColor.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToBaseColor(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}

			bShadingModelHandled = true;
		}
	}

	// Metallic
	{
		const bool bHasInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::Metallic);

		if (bHasInput)
		{
			TGuardValue<EMaterialInputType> InputTypeBeingProcessedGuard(MaterialCreationContext.InputTypeBeingProcessed, EMaterialInputType::Scalar);

			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::Metallic.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToMetallic(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}

			bShadingModelHandled = true;
		}
	}

	// Specular
	{
		const bool bHasInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::Specular);

		if (bHasInput)
		{
			TGuardValue<EMaterialInputType> InputTypeBeingProcessedGuard(MaterialCreationContext.InputTypeBeingProcessed, EMaterialInputType::Scalar);

			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::Specular.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToSpecular(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}

			bShadingModelHandled = true;
		}
	}

	// Roughness
	{
		const bool bHasInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::Roughness);

		if (bHasInput)
		{
			TGuardValue<EMaterialInputType> InputTypeBeingProcessedGuard(MaterialCreationContext.InputTypeBeingProcessed, EMaterialInputType::Scalar);

			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::Roughness.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToRoughness(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}

			bShadingModelHandled = true;
		}
	}

	return bShadingModelHandled;
}

bool UInterchangeGenericMaterialPipeline::HandleClearCoat(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode)
{
	using namespace UE::Interchange::Materials::ClearCoat;

	bool bShadingModelHandled = false;

	// Clear Coat
	{
		const bool bHasInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::ClearCoat);

		if (bHasInput)
		{
			TGuardValue<EMaterialInputType> InputTypeBeingProcessedGuard(MaterialCreationContext.InputTypeBeingProcessed, EMaterialInputType::Scalar);

			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::ClearCoat.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToClearCoat(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}

			bShadingModelHandled = true;
		}
	}

	// Clear Coat Roughness
	{
		const bool bHasInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::ClearCoatRoughness);

		if (bHasInput)
		{
			TGuardValue<EMaterialInputType> InputTypeBeingProcessedGuard(MaterialCreationContext.InputTypeBeingProcessed, EMaterialInputType::Scalar);

			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::ClearCoatRoughness.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToClearCoatRoughness(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}

			bShadingModelHandled = true;
		}
	}

	// Clear Coat Normal
	{
		const bool bHasInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::ClearCoatNormal);

		if (bHasInput)
		{
			TGuardValue<EMaterialInputType> InputTypeBeingProcessedGuard(MaterialCreationContext.InputTypeBeingProcessed, EMaterialInputType::Vector);

			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::ClearCoatNormal.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToClearCoatNormal(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}

			bShadingModelHandled = true;
		}
	}

	if (bShadingModelHandled)
	{
		MaterialFactoryNode->SetCustomShadingModel(EMaterialShadingModel::MSM_ClearCoat);
	}

	return bShadingModelHandled;
}

bool UInterchangeGenericMaterialPipeline::HandleSubsurface(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode)
{
	using namespace UE::Interchange::Materials::Subsurface;

	bool bShadingModelHandled = false;

	// Subsurface Color
	{
		const bool bHasInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::SubsurfaceColor);

		if(bHasInput)
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::SubsurfaceColor.ToString(), MaterialFactoryNode->GetUniqueID());

			if(ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToSubsurface(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}

			bShadingModelHandled = true;
		}
	}

	if(bShadingModelHandled)
	{
		MaterialFactoryNode->SetCustomShadingModel(EMaterialShadingModel::MSM_Subsurface);
		MaterialFactoryNode->SetCustomBlendMode(EBlendMode::BLEND_Opaque); // Opacity in Subsurface doesn't mean Translucency, according to UE doc
	}

	return bShadingModelHandled;
}

bool UInterchangeGenericMaterialPipeline::HandleSheen(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode)
{
	using namespace UE::Interchange::Materials::Sheen;

	bool bShadingModelHandled = false;

	// Sheen Color
	{
		const bool bHasInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::SheenColor);

		if (bHasInput)
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::SheenColor.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToFuzzColor(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}

			bShadingModelHandled = true;
		}
	}

	// Sheen Roughness
	{
		const bool bHasInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::SheenRoughness);

		if (bHasInput)
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::SheenRoughness.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToCloth(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}

			bShadingModelHandled = true;
		}
	}

	if (bShadingModelHandled)
	{
		MaterialFactoryNode->SetCustomShadingModel(EMaterialShadingModel::MSM_Cloth);
	}

	return bShadingModelHandled;
}

bool UInterchangeGenericMaterialPipeline::HandleThinTranslucent(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode)
{
	using namespace UE::Interchange::Materials::ThinTranslucent;

	bool bShadingModelHandled = false;

	// Transmission Color
	{
		const bool bHasInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::TransmissionColor);

		if (bHasInput)
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::TransmissionColor.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToTransmissionColor(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}

			bShadingModelHandled = true;
		}
	}

	// Surface Coverage
	{
		const bool bHasInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::SurfaceCoverage);

		if(bHasInput)
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::SurfaceCoverage.ToString(), MaterialFactoryNode->GetUniqueID());

			if(ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToSurfaceCoverage(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}

			bShadingModelHandled = true;
		}
	}

	if (bShadingModelHandled)
	{
		MaterialFactoryNode->SetCustomBlendMode(EBlendMode::BLEND_Translucent);
		MaterialFactoryNode->SetCustomShadingModel(EMaterialShadingModel::MSM_ThinTranslucent);
		MaterialFactoryNode->SetCustomTranslucencyLightingMode(ETranslucencyLightingMode::TLM_SurfacePerPixelLighting);
	}

	return bShadingModelHandled;
}

void UInterchangeGenericMaterialPipeline::HandleCommonParameters(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode)
{
	using namespace UE::Interchange::Materials::Common;

	if (bool bScreenSpaceReflections; ShaderGraphNode->GetCustomScreenSpaceReflections(bScreenSpaceReflections))
	{
		MaterialFactoryNode->SetCustomScreenSpaceReflections(bScreenSpaceReflections);
	}

	bool bTwoSidedTransmission = false;
	ShaderGraphNode->GetCustomTwoSidedTransmission(bTwoSidedTransmission);
	// Two sidedness (ignored for thin translucency as it looks wrong)
	if (bTwoSidedTransmission || !HasThinTranslucency(ShaderGraphNode))
	{
		bool bTwoSided = false;
		ShaderGraphNode->GetCustomTwoSided(bTwoSided);
		MaterialFactoryNode->SetCustomTwoSided(bTwoSided);
	}

	// Anisotropy
	{
		const bool bHasInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::Anisotropy);

		if (bHasInput)
		{
			TGuardValue<EMaterialInputType> InputTypeBeingProcessedGuard(MaterialCreationContext.InputTypeBeingProcessed, EMaterialInputType::Scalar);

			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::Anisotropy.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToAnisotropy(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}
		}
	}

	// Emissive
	{
		const bool bHasEmissiveInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::EmissiveColor);

		if (bHasEmissiveInput)
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> EmissiveExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::EmissiveColor.ToString(), MaterialFactoryNode->GetUniqueID());

			if (EmissiveExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToEmissiveColor(EmissiveExpressionFactoryNode.Get<0>()->GetUniqueID(), EmissiveExpressionFactoryNode.Get<1>());
			}
		}
	}

	// Normal
	{
		const bool bHasNormalInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::Normal);

		if (bHasNormalInput)
		{
			TGuardValue<EMaterialInputType> InputTypeBeingProcessedGuard(MaterialCreationContext.InputTypeBeingProcessed, EMaterialInputType::Vector);

			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::Normal.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToNormal(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}
		}
	}

	// Tangent
	{
		const bool bHasNormalInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::Tangent);

		if(bHasNormalInput)
		{
			TGuardValue<EMaterialInputType> InputTypeBeingProcessedGuard(MaterialCreationContext.InputTypeBeingProcessed, EMaterialInputType::Vector);

			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::Tangent.ToString(), MaterialFactoryNode->GetUniqueID());

			if(ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToTangent(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}
		}
	}

	// Opacity / OpacityMask
	{
		TGuardValue<EMaterialInputType> InputTypeBeingProcessedGuard(MaterialCreationContext.InputTypeBeingProcessed, EMaterialInputType::Scalar);

		bool bUpdateBlendMode = false;
		const bool bHasOpacityMaskInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::OpacityMask);
		if (bHasOpacityMaskInput)
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> OpacityMaskExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::OpacityMask.ToString(), MaterialFactoryNode->GetUniqueID());

			if (OpacityMaskExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToOpacity(OpacityMaskExpressionFactoryNode.Get<0>()->GetUniqueID(), OpacityMaskExpressionFactoryNode.Get<1>());
			}
			bUpdateBlendMode = true;
		}
		else
		{
			const bool bHasOpacityInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::Opacity);
			if (bHasOpacityInput)
			{
				bool bHasSomeTransparency = true;

				float OpacityValue;
				if (ShaderGraphNode->GetFloatAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(Parameters::Opacity.ToString()), OpacityValue))
				{
					bHasSomeTransparency = !FMath::IsNearlyEqual(OpacityValue, 1.f);
				}

				if (bHasSomeTransparency)
				{
					TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> OpacityExpressionFactoryNode =
						CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::Opacity.ToString(), MaterialFactoryNode->GetUniqueID());

					if (OpacityExpressionFactoryNode.Get<0>())
					{
						MaterialFactoryNode->ConnectOutputToOpacity(OpacityExpressionFactoryNode.Get<0>()->GetUniqueID(), OpacityExpressionFactoryNode.Get<1>());
					}

					bUpdateBlendMode = true;
				}
			}
		}

		if (bUpdateBlendMode)
		{
			UE::Interchange::InterchangeGenericMaterialPipeline::Private::UpdateBlendModeBasedOnOpacityAttributes(ShaderGraphNode, MaterialFactoryNode);
		}
	}

	// Ambient Occlusion
	{
		TGuardValue<EMaterialInputType> InputTypeBeingProcessedGuard(MaterialCreationContext.InputTypeBeingProcessed, EMaterialInputType::Scalar);

		const bool bHasOcclusionInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::Occlusion);

		if (bHasOcclusionInput)
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::Occlusion.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToOcclusion(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}
		}
	}

	// Refraction
	// probably unlikely that someone will use both at same time but to keep backwards compability IndexOfRefraction will override this one
	{
		TGuardValue<EMaterialInputType> InputTypeBeingProcessedGuard(MaterialCreationContext.InputTypeBeingProcessed, EMaterialInputType::Scalar);

		const bool bHasIorInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::Refraction);

		if(bHasIorInput)
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::Refraction.ToString(), MaterialFactoryNode->GetUniqueID());

			if(ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->SetCustomRefractionMethod(ERefractionMode::RM_IndexOfRefraction);
				MaterialFactoryNode->ConnectOutputToRefraction(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}
		}
	}

	// Index of Refraction (IOR)
	// We'll lerp between Air IOR (1) and the IOR from the shader graph based on a fresnel, as per UE doc on refraction.
	{
		TGuardValue<EMaterialInputType> InputTypeBeingProcessedGuard(MaterialCreationContext.InputTypeBeingProcessed, EMaterialInputType::Scalar);

		const bool bHasIorInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::IndexOfRefraction);

		if (bHasIorInput)
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::IndexOfRefraction.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->SetCustomRefractionMethod(ERefractionMode::RM_IndexOfRefraction);
				UInterchangeMaterialExpressionFactoryNode* IORLerp = CreateExpressionNode(TEXT("IORLerp"), ShaderGraphNode->GetUniqueID(), UMaterialExpressionLinearInterpolate::StaticClass());

				const float AirIOR = 1.f;
				const FName ConstAMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionLinearInterpolate, ConstA);
				IORLerp->AddFloatAttribute(ConstAMemberName.ToString(), AirIOR);
				IORLerp->AddApplyAndFillDelegates<float>(ConstAMemberName.ToString(), UMaterialExpressionLinearInterpolate::StaticClass(), ConstAMemberName);

				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(IORLerp, GET_MEMBER_NAME_CHECKED(UMaterialExpressionLinearInterpolate, B).ToString(),
					ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());

				UInterchangeMaterialExpressionFactoryNode* IORFresnel = CreateExpressionNode(TEXT("IORFresnel"), ShaderGraphNode->GetUniqueID(), UMaterialExpressionFresnel::StaticClass());

				UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(IORLerp, GET_MEMBER_NAME_CHECKED(UMaterialExpressionLinearInterpolate, Alpha).ToString(), IORFresnel->GetUniqueID());

				MaterialFactoryNode->ConnectToRefraction(IORLerp->GetUniqueID());
			}
		}
	}

	// Displacement
	{
		const bool bHasDisplacementInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::Displacement);

		if(bHasDisplacementInput)
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> DisplacementExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Parameters::Displacement.ToString(), MaterialFactoryNode->GetUniqueID());

			if(DisplacementExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToDisplacement(DisplacementExpressionFactoryNode.Get<0>()->GetUniqueID(), DisplacementExpressionFactoryNode.Get<1>());
			}

			if (float DisplacementCenter; !bOverrideDisplacement && ShaderGraphNode->GetCustomDisplacementCenterMode(DisplacementCenter))
			{
				MaterialFactoryNode->SetCustomDisplacementCenter(DisplacementCenter);
			}

			if (float DisplacementMagnitude; ShaderGraphNode->GetCustomDisplacementMagnitude(DisplacementMagnitude))
			{
				MaterialFactoryNode->SetCustomDisplacementMagnitude(DisplacementMagnitude);
			}
		}
	}
}

void UInterchangeGenericMaterialPipeline::HandleFlattenNormalNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode,
	UInterchangeMaterialExpressionFactoryNode* FlattenNormalFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes::FlattenNormal;
	using namespace UE::Interchange::InterchangeGenericMaterialPipeline::Private;

	if (!FlattenNormalFactoryNode)
	{
		return;
	}

	const FString MaterialFunctionPath = TEXT("/Engine/Functions/Engine_MaterialFunctions01/Texturing/FlattenNormal.FlattenNormal");
	UpdateFunctionCallExpression(*FlattenNormalFactoryNode, MaterialFunctionPath);

	// Normal
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> NormalExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Inputs::Normal.ToString(), FlattenNormalFactoryNode->GetUniqueID());

		if (NormalExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(FlattenNormalFactoryNode, TEXT("Normal"),
				NormalExpression.Get<0>()->GetUniqueID(), NormalExpression.Get<1>());
		}
	}

	// Flatness
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> FlatnessExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Inputs::Flatness.ToString(), FlattenNormalFactoryNode->GetUniqueID());

		if (FlatnessExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(FlattenNormalFactoryNode, TEXT("Flatness"),
				FlatnessExpression.Get<0>()->GetUniqueID(), FlatnessExpression.Get<1>());
		}
	}
}

void UInterchangeGenericMaterialPipeline::HandleNormalFromHeightMapNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* NormalFromHeightMapFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes::NormalFromHeightMap;
	using namespace UE::Interchange::InterchangeGenericMaterialPipeline::Private;

	if (!NormalFromHeightMapFactoryNode)
	{
		return;
	}

	const FString MaterialFunctionPath = TEXT("/Engine/Functions/Engine_MaterialFunctions03/Procedurals/NormalFromHeightmap.NormalFromHeightmap");
	UpdateFunctionCallExpression(*NormalFromHeightMapFactoryNode, MaterialFunctionPath);

	// Heightmap
	{
		const FString HeightMapInput = Inputs::HeightMap.ToString();
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> HeightMapExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, HeightMapInput, NormalFromHeightMapFactoryNode->GetUniqueID());

		if(HeightMapExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(NormalFromHeightMapFactoryNode, HeightMapInput, HeightMapExpression.Get<0>()->GetUniqueID(), HeightMapExpression.Get<1>());
		}
	}

	// Intensity
	{
		const FString Intensity = Inputs::Intensity.ToString();
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> IntensityExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Intensity, NormalFromHeightMapFactoryNode->GetUniqueID());

		if(IntensityExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(NormalFromHeightMapFactoryNode, Intensity, IntensityExpression.Get<0>()->GetUniqueID(), IntensityExpression.Get<1>());
		}
	}

	// Offset
	{
		const FString Offset = Inputs::Offset.ToString();
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> OffsetExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Offset, NormalFromHeightMapFactoryNode->GetUniqueID());

		if(OffsetExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(NormalFromHeightMapFactoryNode, Offset, OffsetExpression.Get<0>()->GetUniqueID(), OffsetExpression.Get<1>());
		}
	}

	// Coordinates
	{
		const FString Coordinates = Inputs::Coordinates.ToString();
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> CoordinatesExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Coordinates, NormalFromHeightMapFactoryNode->GetUniqueID());

		if(CoordinatesExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(NormalFromHeightMapFactoryNode, Coordinates, CoordinatesExpression.Get<0>()->GetUniqueID(), CoordinatesExpression.Get<1>());
		}
	}

	// Channel
	{
		const FString Channel = Inputs::Channel.ToString();
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ChannelExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Channel, NormalFromHeightMapFactoryNode->GetUniqueID());

		if(ChannelExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(NormalFromHeightMapFactoryNode, Channel, ChannelExpression.Get<0>()->GetUniqueID(), ChannelExpression.Get<1>());
		}
	}
}

void UInterchangeGenericMaterialPipeline::HandleMakeFloat3Node(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode,
	UInterchangeMaterialExpressionFactoryNode* MakeFloat3FactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes::MakeFloat3;
	using namespace UE::Interchange::InterchangeGenericMaterialPipeline::Private;

	if (!MakeFloat3FactoryNode)
	{
		return;
	}

	const FString MaterialFunctionPath = TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/MakeFloat3.MakeFloat3");
	UpdateFunctionCallExpression(*MakeFloat3FactoryNode, MaterialFunctionPath);

	TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> RedChannelExpression =
		CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Inputs::X.ToString(), MakeFloat3FactoryNode->GetUniqueID());
	if (RedChannelExpression.Get<0>())
	{
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(MakeFloat3FactoryNode, TEXT("X"),
			RedChannelExpression.Get<0>()->GetUniqueID(), RedChannelExpression.Get<1>());
	}

	TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> GreenChannelExpression =
		CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Inputs::Y.ToString(), MakeFloat3FactoryNode->GetUniqueID());
	if (GreenChannelExpression.Get<0>())
	{
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(MakeFloat3FactoryNode, TEXT("Y"),
			GreenChannelExpression.Get<0>()->GetUniqueID(), GreenChannelExpression.Get<1>());
	}

	TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> BlueChannelExpression =
		CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Inputs::Z.ToString(), MakeFloat3FactoryNode->GetUniqueID());
	if (BlueChannelExpression.Get<0>())
	{
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(MakeFloat3FactoryNode, TEXT("Z"),
			BlueChannelExpression.Get<0>()->GetUniqueID(), BlueChannelExpression.Get<1>());
	}
}

void UInterchangeGenericMaterialPipeline::HandleTextureNode(
	const UInterchangeTextureNode* TextureNode,
	UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode,
	UInterchangeMaterialExpressionFactoryNode* TextureBaseFactoryNode,
	const FString & ExpressionClassName,
	bool bIsAParameter)
{
	using namespace UE::Interchange::Materials::Standard::Nodes::TextureSample;

	FString TextureFactoryUid;
	TArray<FString> TextureTargetNodes;
	TextureNode->GetTargetNodeUids(TextureTargetNodes);

	if(TextureTargetNodes.Num() > 0)
	{
		TextureFactoryUid = TextureTargetNodes[0];
	}

	TextureBaseFactoryNode->SetCustomExpressionClassName(ExpressionClassName);
	TextureBaseFactoryNode->AddStringAttribute(CreateInputKey(Inputs::Texture.ToString(), bIsAParameter), TextureFactoryUid);

	if(UInterchangeTextureFactoryNode* TextureFactoryNode = Cast<UInterchangeTextureFactoryNode>(BaseNodeContainer->GetFactoryNode(TextureFactoryUid)))
	{
		EMaterialInputType TextureUsage = EMaterialInputType::Unknown;
		TextureFactoryNode->GetAttribute(TEXT("TextureUsage"), TextureUsage);

		const bool bIsOutputLinear = MaterialExpressionCreationContextStack.Top().OutputName.Equals(Outputs::A.ToString());
		const EMaterialInputType DesiredTextureUsage = MaterialCreationContext.InputTypeBeingProcessed == EMaterialInputType::Scalar && bIsOutputLinear ?
			EMaterialInputType::Unknown : // Alpha channels are always in linear space so ignore them when determining texture usage
			MaterialCreationContext.InputTypeBeingProcessed;

		if(TextureUsage == EMaterialInputType::Unknown)
		{
			if(DesiredTextureUsage == EMaterialInputType::Vector)
			{
				TextureFactoryNode->SetCustomCompressionSettings(TextureCompressionSettings::TC_Normalmap);
				TextureFactoryNode->SetCustomLODGroup(TextureGroup::TEXTUREGROUP_WorldNormalMap);
			}
			else if(DesiredTextureUsage == EMaterialInputType::Scalar)
			{
				bool bSRGB;
				if(!TextureNode->GetCustomSRGB(bSRGB))
				{
					// Only set CustomSRGB if it wasn't set by the InterchangeGenericTexturePipeline before
					TextureFactoryNode->SetCustomSRGB(false);
				}
			}

			TextureFactoryNode->SetAttribute(TEXT("TextureUsage"), DesiredTextureUsage);
		}
		else if(TextureUsage != DesiredTextureUsage && DesiredTextureUsage != EMaterialInputType::Unknown)
		{
			UInterchangeResultWarning_Generic* TextureUsageWarning = AddMessage<UInterchangeResultWarning_Generic>();
			TextureUsageWarning->DestinationAssetName = TextureFactoryNode->GetAssetName();
			TextureUsageWarning->AssetType = TextureFactoryNode->GetObjectClass();

			TextureUsageWarning->Text = FText::Format(LOCTEXT("TextureUsageMismatch", "{0} is being used as both {1} and {2} which aren't compatible."),
													  FText::FromString(TextureFactoryNode->GetAssetName()), FText::FromString(LexToString(TextureUsage)), FText::FromString(LexToString(DesiredTextureUsage)));

			// Flipping the green channel only makes sense for vector data as it's used to compensate for different handedness.
			// Clear it if we're not gonna be used only as a vector map. This normally happens when a normal map is also used as a color map.
			bool bFlipGreenChannel;
			if(TextureFactoryNode->GetCustombFlipGreenChannel(bFlipGreenChannel))
			{
				TextureFactoryNode->SetCustombFlipGreenChannel(false);
			}
		}
	}
}

void UInterchangeGenericMaterialPipeline::HandleTextureObjectNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TextureObjectFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes::TextureObject;

	bool bIsAParameter;
	FString TextureUid = GetTextureUidAttributeFromShaderNode(ShaderNode, Inputs::Texture, bIsAParameter);
	FString ExpressionClassName;
	FString TextureFactoryUid;
	FString TextureObjectClassName = bIsAParameter ? UMaterialExpressionTextureObjectParameter::StaticClass()->GetName() : UMaterialExpressionTextureObject::StaticClass()->GetName();

	if(const UInterchangeTextureNode* TextureNode = Cast<const UInterchangeTextureNode>(BaseNodeContainer->GetNode(TextureUid)))
	{
		HandleTextureNode(TextureNode, MaterialFactoryNode, TextureObjectFactoryNode, TextureObjectClassName, bIsAParameter);
	}
	else
	{
		TextureObjectFactoryNode->SetCustomExpressionClassName(TextureObjectClassName);
		TextureObjectFactoryNode->AddStringAttribute(CreateInputKey(Inputs::Texture.ToString(), bIsAParameter), TextureFactoryUid);
	}
}

void UInterchangeGenericMaterialPipeline::HandleTextureSampleNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TextureSampleFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes::TextureSample;

	bool bIsAParameter;
	FString TextureUid = GetTextureUidAttributeFromShaderNode(ShaderNode, Inputs::Texture, bIsAParameter);
	FString ExpressionClassName;
	FString TextureFactoryUid;

	if (const UInterchangeTextureNode* TextureNode = Cast<const UInterchangeTextureNode>(BaseNodeContainer->GetNode(TextureUid)))
	{
		if (TextureNode->IsA<UInterchangeTextureCubeNode>())
		{
			ExpressionClassName = UMaterialExpressionTextureSampleParameterCube::StaticClass()->GetName();
		}
		else if (TextureNode->IsA<UInterchangeTexture2DArrayNode>())
		{
			ExpressionClassName = UMaterialExpressionTextureSampleParameter2DArray::StaticClass()->GetName();
		}
		else if (TextureNode->IsA<UInterchangeTextureBlurNode>())
		{
			ExpressionClassName = UMaterialExpressionMaterialXTextureSampleParameterBlur::StaticClass()->GetName();
		}
		else if (TextureNode->IsA<UInterchangeTexture2DNode>())
		{
			ExpressionClassName = UMaterialExpressionTextureSampleParameter2D::StaticClass()->GetName();
		}
		else if (TextureNode->IsA<UInterchangeVolumeTextureNode>())
		{
			ExpressionClassName = UMaterialExpressionSparseVolumeTextureSampleParameter::StaticClass()->GetName();
		}
		else
		{
			ExpressionClassName = UMaterialExpressionTextureSampleParameter2D::StaticClass()->GetName();
		}

		HandleTextureNode(TextureNode, MaterialFactoryNode, TextureSampleFactoryNode, ExpressionClassName, bIsAParameter);
	}
	else
	{
		TextureSampleFactoryNode->SetCustomExpressionClassName(UMaterialExpressionTextureSampleParameter2D::StaticClass()->GetName());
		TextureSampleFactoryNode->AddStringAttribute(CreateInputKey(Inputs::Texture.ToString(), bIsAParameter), TextureFactoryUid);
	}

	// Coordinates
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> CoordinatesExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Inputs::Coordinates.ToString(), TextureSampleFactoryNode->GetUniqueID());

		if (CoordinatesExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(TextureSampleFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionTextureSample, Coordinates).ToString(),
				CoordinatesExpression.Get<0>()->GetUniqueID(), CoordinatesExpression.Get<1>());
		}
	}

	if(ExpressionClassName == UMaterialExpressionMaterialXTextureSampleParameterBlur::StaticClass()->GetName())
	{
		HandleTextureSampleBlurNode(ShaderNode, MaterialFactoryNode, TextureSampleFactoryNode);
	}
}

void UInterchangeGenericMaterialPipeline::HandleTextureSampleBlurNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TextureSampleFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	// KernelSize
	if(int32 KernelSize; ShaderNode->GetInt32Attribute(TextureSampleBlur::Attributes::KernelSize.ToString(), KernelSize))
	{
		const FName KernelSizeMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionMaterialXTextureSampleParameterBlur, KernelSize);
		TextureSampleFactoryNode->AddInt32Attribute(KernelSizeMemberName.ToString(), KernelSize);
		TextureSampleFactoryNode->AddApplyAndFillDelegates<int32>(KernelSizeMemberName.ToString(), UMaterialExpressionMaterialXTextureSampleParameterBlur::StaticClass(), KernelSizeMemberName);
	}

	// FilterSize
	if(float FilterSize; ShaderNode->GetFloatAttribute(TextureSampleBlur::Attributes::FilterSize.ToString(), FilterSize))
	{
		const FName FilterSizeMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionMaterialXTextureSampleParameterBlur, FilterSize);
		TextureSampleFactoryNode->AddFloatAttribute(FilterSizeMemberName.ToString(), FilterSize);
		TextureSampleFactoryNode->AddApplyAndFillDelegates<float>(FilterSizeMemberName.ToString(), UMaterialExpressionMaterialXTextureSampleParameterBlur::StaticClass(), FilterSizeMemberName);
	}

	// FilterOffset
	if(float FilterOffset; ShaderNode->GetFloatAttribute(TextureSampleBlur::Attributes::FilterOffset.ToString(), FilterOffset))
	{
		const FName FilterOffsetMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionMaterialXTextureSampleParameterBlur, FilterOffset);
		TextureSampleFactoryNode->AddFloatAttribute(FilterOffsetMemberName.ToString(), FilterOffset);
		TextureSampleFactoryNode->AddApplyAndFillDelegates<float>(FilterOffsetMemberName.ToString(), UMaterialExpressionMaterialXTextureSampleParameterBlur::StaticClass(), FilterOffsetMemberName);
	}

	// Filter
	if(int32 Filter; ShaderNode->GetInt32Attribute(TextureSampleBlur::Attributes::Filter.ToString(), Filter))
	{
		const FName FilterMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionMaterialXTextureSampleParameterBlur, Filter);
		TextureSampleFactoryNode->AddInt32Attribute(FilterMemberName.ToString(), Filter);
		TextureSampleFactoryNode->AddApplyAndFillDelegates<int32>(FilterMemberName.ToString(), UMaterialExpressionMaterialXTextureSampleParameterBlur::StaticClass(), FilterMemberName);
	}
}

void UInterchangeGenericMaterialPipeline::HandleTextureCoordinateNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode,
	UInterchangeMaterialExpressionFactoryNode*& TexCoordFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard;
	using namespace UE::Interchange::InterchangeGenericMaterialPipeline::Private;

	TexCoordFactoryNode->SetCustomExpressionClassName(UMaterialExpressionTextureCoordinate::StaticClass()->GetName());

	// Index
	{
		int32 CoordIndex;
		if (ShaderNode->GetInt32Attribute(UInterchangeShaderPortsAPI::MakeInputValueKey(Nodes::TextureCoordinate::Inputs::Index.ToString()), CoordIndex))
		{
			const FName CoordinateIndexMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionTextureCoordinate, CoordinateIndex);
			TexCoordFactoryNode->AddInt32Attribute(CoordinateIndexMemberName.ToString(), CoordIndex);
			TexCoordFactoryNode->AddApplyAndFillDelegates<int32>(CoordinateIndexMemberName.ToString(), UMaterialExpressionTextureCoordinate::StaticClass(), CoordinateIndexMemberName);
		}
	}

	// U tiling
	{
		if (float UTilingValue; ShaderNode->GetFloatAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(Nodes::TextureCoordinate::Inputs::UTiling.ToString()), UTilingValue))
		{
			const FName UTilingMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionTextureCoordinate, UTiling);
			TexCoordFactoryNode->AddFloatAttribute(UTilingMemberName.ToString(), UTilingValue);
			TexCoordFactoryNode->AddApplyAndFillDelegates<float>(UTilingMemberName.ToString(), UMaterialExpressionTextureCoordinate::StaticClass(), UTilingMemberName);
		}
	}

	// V tiling
	{
		if(float VTilingValue; ShaderNode->GetFloatAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(Nodes::TextureCoordinate::Inputs::UTiling.ToString()), VTilingValue))
		{
			const FName VTilingMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionTextureCoordinate, VTiling);
			TexCoordFactoryNode->AddFloatAttribute(VTilingMemberName.ToString(), VTilingValue);
			TexCoordFactoryNode->AddApplyAndFillDelegates<float>(VTilingMemberName.ToString(), UMaterialExpressionTextureCoordinate::StaticClass(), VTilingMemberName);
		}
	}

	// Scale
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ScaleExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Nodes::TextureCoordinate::Inputs::Scale.ToString(), TexCoordFactoryNode->GetUniqueID());

		if (ScaleExpression.Get<0>())
		{
			UInterchangeMaterialExpressionFactoryNode* MultiplyExpression =
				CreateExpressionNode(ScaleExpression.Get<0>()->GetDisplayLabel() + TEXT("_Multiply"), TexCoordFactoryNode->GetUniqueID(), UMaterialExpressionMultiply::StaticClass());

			UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(MultiplyExpression, GET_MEMBER_NAME_CHECKED(UMaterialExpressionMultiply, A).ToString(),
				TexCoordFactoryNode->GetUniqueID());
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(MultiplyExpression, GET_MEMBER_NAME_CHECKED(UMaterialExpressionMultiply, B).ToString(),
				ScaleExpression.Get<0>()->GetUniqueID(), ScaleExpression.Get<1>());

			TexCoordFactoryNode = MultiplyExpression;
		}
	}

	// Rotate
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> RotateExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Nodes::TextureCoordinate::Inputs::Rotate.ToString(), TexCoordFactoryNode->GetUniqueID());

		if (RotateExpression.Get<0>())
		{
			UInterchangeMaterialExpressionFactoryNode* CallRotatorExpression =
				CreateExpressionNode(RotateExpression.Get<0>()->GetDisplayLabel() + TEXT("_Rotator"), TexCoordFactoryNode->GetUniqueID(), UMaterialExpressionMaterialFunctionCall::StaticClass());

			const FString MaterialFunctionPath = TEXT("/Engine/Functions/Engine_MaterialFunctions02/Texturing/CustomRotator.CustomRotator");
			UpdateFunctionCallExpression(*CallRotatorExpression, MaterialFunctionPath);

			UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(CallRotatorExpression, TEXT("UVs"), TexCoordFactoryNode->GetUniqueID());
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(CallRotatorExpression, TEXT("Rotation Angle (0-1)"), RotateExpression.Get<0>()->GetUniqueID(), RotateExpression.Get<1>());

			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> RotationCenterExpression =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Nodes::TextureCoordinate::Inputs::RotationCenter.ToString(), TexCoordFactoryNode->GetUniqueID());

			if (RotationCenterExpression.Get<0>())
			{
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(CallRotatorExpression, TEXT("Rotation Center"), RotationCenterExpression.Get<0>()->GetUniqueID(), RotationCenterExpression.Get<1>());
			}

			TexCoordFactoryNode = CallRotatorExpression;
		}
	}

	// Offset
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> OffsetExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Nodes::TextureCoordinate::Inputs::Offset.ToString(), TexCoordFactoryNode->GetUniqueID());

		if (OffsetExpression.Get<0>())
		{
			UInterchangeMaterialExpressionFactoryNode* AddExpression =
				CreateExpressionNode(OffsetExpression.Get<0>()->GetDisplayLabel() + TEXT("_Add"), TexCoordFactoryNode->GetUniqueID(), UMaterialExpressionAdd::StaticClass());

			UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(AddExpression, GET_MEMBER_NAME_CHECKED(UMaterialExpressionAdd, A).ToString(),
				TexCoordFactoryNode->GetUniqueID());
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(AddExpression, GET_MEMBER_NAME_CHECKED(UMaterialExpressionAdd, B).ToString(),
				OffsetExpression.Get<0>()->GetUniqueID(), OffsetExpression.Get<1>());

			TexCoordFactoryNode = AddExpression;
		}
	}
}

void UInterchangeGenericMaterialPipeline::HandleLerpNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* LerpFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard;

	LerpFactoryNode->SetCustomExpressionClassName(UMaterialExpressionLinearInterpolate::StaticClass()->GetName());

	// A
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ColorAExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Nodes::Lerp::Inputs::A.ToString(), LerpFactoryNode->GetUniqueID());

		if (ColorAExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(LerpFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionLinearInterpolate, A).ToString(),
				ColorAExpression.Get<0>()->GetUniqueID(), ColorAExpression.Get<1>());
		}
	}

	// B
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ColorBExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Nodes::Lerp::Inputs::B.ToString(), LerpFactoryNode->GetUniqueID());

		if (ColorBExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(LerpFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionLinearInterpolate, B).ToString(),
				ColorBExpression.Get<0>()->GetUniqueID(), ColorBExpression.Get<1>());
		}
	}

	// Factor
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> FactorExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Nodes::Lerp::Inputs::Factor.ToString(), LerpFactoryNode->GetUniqueID());

		if (FactorExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(LerpFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionLinearInterpolate, Alpha).ToString(),
				FactorExpression.Get<0>()->GetUniqueID(), FactorExpression.Get<1>());
		}
	}
}

void UInterchangeGenericMaterialPipeline::HandleMaskNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* MaskFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	MaskFactoryNode->SetCustomExpressionClassName(UMaterialExpressionComponentMask::StaticClass()->GetName());

	bool bRChannel = false;
	ShaderNode->GetBooleanAttribute(Mask::Attributes::R.ToString(), bRChannel);
	bool bGChannel = false;
	ShaderNode->GetBooleanAttribute(Mask::Attributes::G.ToString(), bGChannel);
	bool bBChannel = false;
	ShaderNode->GetBooleanAttribute(Mask::Attributes::B.ToString(), bBChannel);
	bool bAChannel = false;
	ShaderNode->GetBooleanAttribute(Mask::Attributes::A.ToString(), bAChannel);
	bool bIsAnyMaskChannelSet = bRChannel || bGChannel || bBChannel || bAChannel;

	if(bIsAnyMaskChannelSet)
	{
		// R
		{
			const FName RMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionComponentMask, R);
			MaskFactoryNode->AddBooleanAttribute(RMemberName.ToString(), bRChannel);
			MaskFactoryNode->AddApplyAndFillDelegates<bool>(RMemberName.ToString(), UMaterialExpressionComponentMask::StaticClass(), RMemberName);
		}

		// G
		{
			const FName GMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionComponentMask, G);
			MaskFactoryNode->AddBooleanAttribute(GMemberName.ToString(), bGChannel);
			MaskFactoryNode->AddApplyAndFillDelegates<bool>(GMemberName.ToString(), UMaterialExpressionComponentMask::StaticClass(), GMemberName);
		}

		// B
		{
			const FName BMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionComponentMask, B);
			MaskFactoryNode->AddBooleanAttribute(BMemberName.ToString(), bBChannel);
			MaskFactoryNode->AddApplyAndFillDelegates<bool>(BMemberName.ToString(), UMaterialExpressionComponentMask::StaticClass(), BMemberName);
		}

		// A
		{
			const FName AMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionComponentMask, A);
			MaskFactoryNode->AddBooleanAttribute(AMemberName.ToString(), bAChannel);
			MaskFactoryNode->AddApplyAndFillDelegates<bool>(AMemberName.ToString(), UMaterialExpressionComponentMask::StaticClass(), AMemberName);
		}
	}

	// Input
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> InputExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Mask::Inputs::Input.ToString(), MaskFactoryNode->GetUniqueID());

		if(InputExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(MaskFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionComponentMask, Input).ToString(),
				InputExpression.Get<0>()->GetUniqueID(), InputExpression.Get<1>());
		}
	}
}

void UInterchangeGenericMaterialPipeline::HandleRotatorNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* RotatorFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	RotatorFactoryNode->SetCustomExpressionClassName(UMaterialExpressionRotator::StaticClass()->GetName());

	// Coordinate
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> CoordinateExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Rotator::Inputs::Coordinate.ToString(), RotatorFactoryNode->GetUniqueID());
		if(CoordinateExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(RotatorFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionRotator, Coordinate).ToString(),
																  CoordinateExpression.Get<0>()->GetUniqueID(), CoordinateExpression.Get<1>());
		}
	}

	// Time
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> TimeExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Rotator::Inputs::Time.ToString(), RotatorFactoryNode->GetUniqueID());
		if(TimeExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(RotatorFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionRotator, Time).ToString(),
																  TimeExpression.Get<0>()->GetUniqueID(), TimeExpression.Get<1>());
		}
	}

	// CenterX
	if(float CenterX; ShaderNode->GetFloatAttribute(Rotator::Attributes::CenterX.ToString(), CenterX))
	{
		const FName CenterXMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionRotator, CenterX);
		RotatorFactoryNode->AddFloatAttribute(CenterXMemberName.ToString(), CenterX);
		RotatorFactoryNode->AddApplyAndFillDelegates<float>(CenterXMemberName.ToString(), UMaterialExpressionRotator::StaticClass(), CenterXMemberName);
	}

	// CenterY
	if(float CenterY; ShaderNode->GetFloatAttribute(Rotator::Attributes::CenterY.ToString(), CenterY))
	{
		const FName CenterYMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionRotator, CenterY);
		RotatorFactoryNode->AddFloatAttribute(CenterYMemberName.ToString(), CenterY);
		RotatorFactoryNode->AddApplyAndFillDelegates<float>(CenterYMemberName.ToString(), UMaterialExpressionRotator::StaticClass(), CenterYMemberName);
	}

	// Speed
	if(float Speed; ShaderNode->GetFloatAttribute(Rotator::Attributes::Speed.ToString(), Speed))
	{
		const FName SpeedMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionRotator, Speed);
		RotatorFactoryNode->AddFloatAttribute(SpeedMemberName.ToString(), Speed);
		RotatorFactoryNode->AddApplyAndFillDelegates<float>(SpeedMemberName.ToString(), UMaterialExpressionRotator::StaticClass(), SpeedMemberName);
	}
}

void UInterchangeGenericMaterialPipeline::HandleRotateAboutAxisNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* RotateAboutAxisFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	RotateAboutAxisFactoryNode->SetCustomExpressionClassName(UMaterialExpressionRotateAboutAxis::StaticClass()->GetName());

	// NormalizedRotationAxis
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> NormalizedRotationAxisExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, RotateAboutAxis::Inputs::NormalizedRotationAxis.ToString(), RotateAboutAxisFactoryNode->GetUniqueID());
		if (NormalizedRotationAxisExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(RotateAboutAxisFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionRotateAboutAxis, NormalizedRotationAxis).ToString(),
																  NormalizedRotationAxisExpression.Get<0>()->GetUniqueID(), NormalizedRotationAxisExpression.Get<1>());
		}
	}

	// PivotPoint
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> RotateAboutAxisExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, RotateAboutAxis::Inputs::PivotPoint.ToString(), RotateAboutAxisFactoryNode->GetUniqueID());
		if (RotateAboutAxisExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(RotateAboutAxisFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionRotateAboutAxis, PivotPoint).ToString(),
																  RotateAboutAxisExpression.Get<0>()->GetUniqueID(), RotateAboutAxisExpression.Get<1>());
		}
	}

	// Position
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> PositionExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, RotateAboutAxis::Inputs::Position.ToString(), RotateAboutAxisFactoryNode->GetUniqueID());
		if (PositionExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(RotateAboutAxisFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionRotateAboutAxis, Position).ToString(),
																  PositionExpression.Get<0>()->GetUniqueID(), PositionExpression.Get<1>());
		}
	}

	// RotationAngle
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> RotationAngleExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, RotateAboutAxis::Inputs::RotationAngle.ToString(), RotateAboutAxisFactoryNode->GetUniqueID());
		if (RotationAngleExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(RotateAboutAxisFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionRotateAboutAxis, RotationAngle).ToString(),
																  RotationAngleExpression.Get<0>()->GetUniqueID(), RotationAngleExpression.Get<1>());
		}
	}

	// Period (By default the period is 1, but the angle is in radians, let's default it to 2Pi
	{
		float Period = 2.f * UE_PI;
		ShaderNode->GetFloatAttribute(RotateAboutAxis::Attributes::Period.ToString(), Period);
		const FName PeriodMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionRotateAboutAxis, Period);
		RotateAboutAxisFactoryNode->AddFloatAttribute(PeriodMemberName.ToString(), Period);
		RotateAboutAxisFactoryNode->AddApplyAndFillDelegates<float>(PeriodMemberName.ToString(), UMaterialExpressionRotateAboutAxis::StaticClass(), PeriodMemberName);
	}
}

void UInterchangeGenericMaterialPipeline::HandleTimeNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TimeFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	TimeFactoryNode->SetCustomExpressionClassName(UMaterialExpressionTime::StaticClass()->GetName());

	// IgnorePause
	if(bool bIgnorePause; ShaderNode->GetBooleanAttribute(Time::Attributes::IgnorePause.ToString(), bIgnorePause))
	{
		const FName IgnorePauseMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionTime, bIgnorePause);
		TimeFactoryNode->AddBooleanAttribute(IgnorePauseMemberName.ToString(), bIgnorePause);
		TimeFactoryNode->AddApplyAndFillDelegates<bool>(IgnorePauseMemberName.ToString(), UMaterialExpressionTime::StaticClass(), IgnorePauseMemberName);
	}

	// OverridePeriod
	if(bool bOverridePeriod; ShaderNode->GetBooleanAttribute(Time::Attributes::OverridePeriod.ToString(), bOverridePeriod))
	{
		const FName OverridePeriodMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionTime, bOverride_Period);
		TimeFactoryNode->AddBooleanAttribute(OverridePeriodMemberName.ToString(), bOverridePeriod);
		TimeFactoryNode->AddApplyAndFillDelegates<bool>(OverridePeriodMemberName.ToString(), UMaterialExpressionTime::StaticClass(), OverridePeriodMemberName);
	}

	// Period
	if(float Period; ShaderNode->GetFloatAttribute(Time::Attributes::Period.ToString(), Period))
	{
		const FName PeriodMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionTime, Period);
		TimeFactoryNode->AddFloatAttribute(PeriodMemberName.ToString(), Period);
		TimeFactoryNode->AddApplyAndFillDelegates<float>(PeriodMemberName.ToString(), UMaterialExpressionTime::StaticClass(), PeriodMemberName);
	}
}

void UInterchangeGenericMaterialPipeline::HandleTransformPositionNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TransformPositionFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	TransformPositionFactoryNode->SetCustomExpressionClassName(UMaterialExpressionTransformPosition::StaticClass()->GetName());

	// Input
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> InputExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, TransformPosition::Inputs::Input.ToString(), TransformPositionFactoryNode->GetUniqueID());

		if(InputExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(TransformPositionFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionTransformPosition, Input).ToString(),
															InputExpression.Get<0>()->GetUniqueID(), InputExpression.Get<1>());
		}
	}

	// TransformSourceType
	if(int32 TransformSourceType; ShaderNode->GetInt32Attribute(TransformPosition::Attributes::TransformSourceType.ToString(), TransformSourceType))
	{
		const FName TransformSourceTypeMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionTransformPosition, TransformSourceType);
		TransformPositionFactoryNode->AddInt32Attribute(TransformSourceTypeMemberName.ToString(),TransformSourceType);
		TransformPositionFactoryNode->AddApplyAndFillDelegates<int32>(TransformSourceTypeMemberName.ToString(), UMaterialExpressionTransformPosition::StaticClass(), TransformSourceTypeMemberName);
	}

	// TransformType
	if(int32 TransformType; ShaderNode->GetInt32Attribute(TransformPosition::Attributes::TransformType.ToString(), TransformType))
	{
		const FName TransformTypeMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionTransformPosition, TransformType);
		TransformPositionFactoryNode->AddInt32Attribute(TransformTypeMemberName.ToString(), TransformType);
		TransformPositionFactoryNode->AddApplyAndFillDelegates<int32>(TransformTypeMemberName.ToString(), UMaterialExpressionTransformPosition::StaticClass(), TransformTypeMemberName);
	}
}

void UInterchangeGenericMaterialPipeline::HandleTransformVectorNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TransformVectorFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	TransformVectorFactoryNode->SetCustomExpressionClassName(UMaterialExpressionTransform::StaticClass()->GetName());

	// Input
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> InputExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, TransformVector::Inputs::Input.ToString(), TransformVectorFactoryNode->GetUniqueID());

		if(InputExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(TransformVectorFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionTransform, Input).ToString(),
															InputExpression.Get<0>()->GetUniqueID(), InputExpression.Get<1>());
		}
	}

	// TransformSourceType
	if(int32 TransformSourceType; ShaderNode->GetInt32Attribute(TransformVector::Attributes::TransformSourceType.ToString(), TransformSourceType))
	{
		const FName TransformSourceTypeMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionTransform, TransformSourceType);
		TransformVectorFactoryNode->AddInt32Attribute(TransformSourceTypeMemberName.ToString(), TransformSourceType);
		TransformVectorFactoryNode->AddApplyAndFillDelegates<int32>(TransformSourceTypeMemberName.ToString(), UMaterialExpressionTransform::StaticClass(), TransformSourceTypeMemberName);
	}

	// TransformType
	if(int32 TransformType; ShaderNode->GetInt32Attribute(TransformVector::Attributes::TransformType.ToString(), TransformType))
	{
		const FName TransformTypeMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionTransform, TransformType);
		TransformVectorFactoryNode->AddInt32Attribute(TransformTypeMemberName.ToString(), TransformType);
		TransformVectorFactoryNode->AddApplyAndFillDelegates<int32>(TransformTypeMemberName.ToString(), UMaterialExpressionTransform::StaticClass(), TransformTypeMemberName);
	}
}

void UInterchangeGenericMaterialPipeline::HandleNoiseNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* NoiseFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	NoiseFactoryNode->SetCustomExpressionClassName(UMaterialExpressionNoise::StaticClass()->GetName());

	// Position
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> PositionExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Noise::Inputs::Position.ToString(), NoiseFactoryNode->GetUniqueID());

		if(PositionExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(NoiseFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionNoise, Position).ToString(),
															PositionExpression.Get<0>()->GetUniqueID(), PositionExpression.Get<1>());
		}
	}

	// FilterWidth
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> FilterWidthExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Noise::Inputs::FilterWidth.ToString(), NoiseFactoryNode->GetUniqueID());

		if(FilterWidthExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(NoiseFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionNoise, FilterWidth).ToString(),
															FilterWidthExpression.Get<0>()->GetUniqueID(), FilterWidthExpression.Get<1>());
		}
	}

	// Scale
	if(float Scale; ShaderNode->GetFloatAttribute(Noise::Attributes::Scale.ToString(), Scale))
	{
		const FName ScaleMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionNoise, Scale);
		NoiseFactoryNode->AddFloatAttribute(ScaleMemberName.ToString(), Scale);
		NoiseFactoryNode->AddApplyAndFillDelegates<float>(ScaleMemberName.ToString(), UMaterialExpressionNoise::StaticClass(), ScaleMemberName);
	}

	// Quality
	if(int32 Quality; ShaderNode->GetInt32Attribute(Noise::Attributes::Quality.ToString(), Quality))
	{
		const FName QualityMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionNoise, Quality);
		NoiseFactoryNode->AddInt32Attribute(QualityMemberName.ToString(), Quality);
		NoiseFactoryNode->AddApplyAndFillDelegates<int32>(QualityMemberName.ToString(), UMaterialExpressionNoise::StaticClass(), QualityMemberName);
	}

	// Noise Function
	if(int32 NoiseFunction; ShaderNode->GetInt32Attribute(Noise::Attributes::Function.ToString(), NoiseFunction))
	{
		const FName NoiseFunctionMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionNoise, NoiseFunction);
		NoiseFactoryNode->AddInt32Attribute(NoiseFunctionMemberName.ToString(), NoiseFunction);
		NoiseFactoryNode->AddApplyAndFillDelegates<int32>(NoiseFunctionMemberName.ToString(), UMaterialExpressionNoise::StaticClass(), NoiseFunctionMemberName);
	}

	// Turbulence
	if(bool bTurbulence; ShaderNode->GetBooleanAttribute(Noise::Attributes::Turbulence.ToString(), bTurbulence))
	{
		const FName TurbulenceMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionNoise, bTurbulence);
		NoiseFactoryNode->AddBooleanAttribute(TurbulenceMemberName.ToString(), bTurbulence);
		NoiseFactoryNode->AddApplyAndFillDelegates<bool>(TurbulenceMemberName.ToString(), UMaterialExpressionNoise::StaticClass(), TurbulenceMemberName);
	}

	// Levels
	if(int32 Levels; ShaderNode->GetInt32Attribute(Noise::Attributes::Levels.ToString(), Levels))
	{
		const FName LevelsMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionNoise, Levels);
		NoiseFactoryNode->AddInt32Attribute(LevelsMemberName.ToString(), Levels);
		NoiseFactoryNode->AddApplyAndFillDelegates<int32>(LevelsMemberName.ToString(), UMaterialExpressionNoise::StaticClass(), LevelsMemberName);
	}

	// Output Min
	if(float OutputMin; ShaderNode->GetFloatAttribute(Noise::Attributes::OutputMin.ToString(), OutputMin))
	{
		const FName OutputMinMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionNoise, OutputMin);
		NoiseFactoryNode->AddFloatAttribute(OutputMinMemberName.ToString(), OutputMin);
		NoiseFactoryNode->AddApplyAndFillDelegates<float>(OutputMinMemberName.ToString(), UMaterialExpressionNoise::StaticClass(), OutputMinMemberName);
	}

	// Output Max
	if(float OutputMax; ShaderNode->GetFloatAttribute(Noise::Attributes::OutputMax.ToString(), OutputMax))
	{
		const FName OutputMaxMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionNoise, OutputMax);
		NoiseFactoryNode->AddFloatAttribute(OutputMaxMemberName.ToString(), OutputMax);
		NoiseFactoryNode->AddApplyAndFillDelegates<float>(OutputMaxMemberName.ToString(), UMaterialExpressionNoise::StaticClass(), OutputMaxMemberName);
	}

	// Level Scale
	if(float LevelScale; ShaderNode->GetFloatAttribute(Noise::Attributes::LevelScale.ToString(), LevelScale))
	{
		const FName LevelScaleMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionNoise, LevelScale);
		NoiseFactoryNode->AddFloatAttribute(LevelScaleMemberName.ToString(), LevelScale);
		NoiseFactoryNode->AddApplyAndFillDelegates<float>(LevelScaleMemberName.ToString(), UMaterialExpressionNoise::StaticClass(), LevelScaleMemberName);
	}

	// Tiling
	if(bool bTiling; ShaderNode->GetBooleanAttribute(Noise::Attributes::Tiling.ToString(), bTiling))
	{
		const FName TilingMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionNoise, bTiling);
		NoiseFactoryNode->AddBooleanAttribute(TilingMemberName.ToString(), bTiling);
		NoiseFactoryNode->AddApplyAndFillDelegates<bool>(TilingMemberName.ToString(), UMaterialExpressionNoise::StaticClass(), TilingMemberName);
	}

	// Levels
	if(int32 RepeatSize; ShaderNode->GetInt32Attribute(Noise::Attributes::RepeatSize.ToString(), RepeatSize))
	{
		const FName RepeatSizeMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionNoise, RepeatSize);
		NoiseFactoryNode->AddInt32Attribute(RepeatSizeMemberName.ToString(), RepeatSize);
		NoiseFactoryNode->AddApplyAndFillDelegates<int32>(RepeatSizeMemberName.ToString(), UMaterialExpressionNoise::StaticClass(), RepeatSizeMemberName);
	}
}

void UInterchangeGenericMaterialPipeline::HandleVectorNoiseNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* NoiseFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	NoiseFactoryNode->SetCustomExpressionClassName(UMaterialExpressionVectorNoise::StaticClass()->GetName());

	// Position
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> PositionExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, VectorNoise::Inputs::Position.ToString(), NoiseFactoryNode->GetUniqueID());

		if(PositionExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(NoiseFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionVectorNoise, Position).ToString(),
															PositionExpression.Get<0>()->GetUniqueID(), PositionExpression.Get<1>());
		}
	}

	// Noise Function
	if(int32 NoiseFunction; ShaderNode->GetInt32Attribute(VectorNoise::Attributes::Function.ToString(), NoiseFunction))
	{
		const FName NoiseFunctionMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionVectorNoise, NoiseFunction);
		NoiseFactoryNode->AddInt32Attribute(NoiseFunctionMemberName.ToString(), NoiseFunction);
		NoiseFactoryNode->AddApplyAndFillDelegates<int32>(NoiseFunctionMemberName.ToString(), UMaterialExpressionVectorNoise::StaticClass(), NoiseFunctionMemberName);
	}

	// Quality
	if(int32 Quality; ShaderNode->GetInt32Attribute(VectorNoise::Attributes::Quality.ToString(), Quality))
	{
		const FName QualityMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionVectorNoise, Quality);
		NoiseFactoryNode->AddInt32Attribute(QualityMemberName.ToString(), Quality);
		NoiseFactoryNode->AddApplyAndFillDelegates<int32>(QualityMemberName.ToString(), UMaterialExpressionVectorNoise::StaticClass(), QualityMemberName);
	}

	// Tiling
	if(bool bTiling; ShaderNode->GetBooleanAttribute(VectorNoise::Attributes::Tiling.ToString(), bTiling))
	{
		const FName TilingMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionVectorNoise, bTiling);
		NoiseFactoryNode->AddBooleanAttribute(TilingMemberName.ToString(), bTiling);
		NoiseFactoryNode->AddApplyAndFillDelegates<bool>(TilingMemberName.ToString(), UMaterialExpressionVectorNoise::StaticClass(), TilingMemberName);
	}

	// Tile Size
	if(int32 TileSize; ShaderNode->GetInt32Attribute(VectorNoise::Attributes::Function.ToString(), TileSize))
	{
		const FName TileSizeMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionVectorNoise, TileSize);
		NoiseFactoryNode->AddInt32Attribute(TileSizeMemberName.ToString(), TileSize);
		NoiseFactoryNode->AddApplyAndFillDelegates<int32>(TileSizeMemberName.ToString(), UMaterialExpressionVectorNoise::StaticClass(), TileSizeMemberName);
	}
}

void UInterchangeGenericMaterialPipeline::HandleSwitchNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* SwitchFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	SwitchFactoryNode->SetCustomExpressionClassName(UMaterialExpressionSwitch::StaticClass()->GetName());

	// SwitchValue
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> SwitchValueExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Switch::Inputs::Value.ToString(), SwitchFactoryNode->GetUniqueID());
		if (SwitchValueExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(SwitchFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionSwitch, SwitchValue).ToString(),
																  SwitchValueExpression.Get<0>()->GetUniqueID(), SwitchValueExpression.Get<1>());
		}
	}

	// Default
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> DefaultExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Switch::Inputs::Default.ToString(), SwitchFactoryNode->GetUniqueID());
		if (DefaultExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(SwitchFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionSwitch, Default).ToString(),
																  DefaultExpression.Get<0>()->GetUniqueID(), DefaultExpression.Get<1>());
		}
	}

	// Inputs
	if (int32 InputCount; ShaderNode->GetInt32Attribute(Switch::Attributes::InputCount.ToString(), InputCount))
	{
		SwitchFactoryNode->AddInt32Attribute(Switch::Attributes::InputCount.ToString(), InputCount);

		for (int32 Index = 0; Index < InputCount; ++Index)
		{
			if (FString InputName; ShaderNode->GetStringAttribute(Switch::Inputs::InputName.ToString() + FString::FromInt(Index), InputName))
			{
				SwitchFactoryNode->AddStringAttribute(Switch::Inputs::InputName.ToString() + FString::FromInt(Index), InputName);

				TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> InputExpression =
					CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, InputName, SwitchFactoryNode->GetUniqueID());
				if (InputExpression.Get<0>())
				{
					UInterchangeShaderPortsAPI::ConnectOuputToInputByName(SwitchFactoryNode, InputName, InputExpression.Get<0>()->GetUniqueID(), InputExpression.Get<1>());
				}
			}
		}
	}
}

void UInterchangeGenericMaterialPipeline::HandleSlabBSDFNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* SlabBSDFFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	using namespace UE::Interchange::InterchangeGenericMaterialPipeline::Private;

	SlabBSDFFactoryNode->SetCustomExpressionClassName(UMaterialExpressionSubstrateSlabBSDF::StaticClass()->GetName());

	// ensure that SlabBSDF expression have the same members define as Interchange;
#define SubstrateSlabBSDF_MEMBER_CHECKED(X) GET_MEMBER_NAME_STRING_CHECKED(UMaterialExpressionSubstrateSlabBSDF, X), SlabBSDF::Inputs::X.ToString()
	const TSet<FString> SlabInputs{
		SubstrateSlabBSDF_MEMBER_CHECKED(Anisotropy),
		SubstrateSlabBSDF_MEMBER_CHECKED(DiffuseAlbedo),
		SubstrateSlabBSDF_MEMBER_CHECKED(EmissiveColor),
		SubstrateSlabBSDF_MEMBER_CHECKED(F0),
		SubstrateSlabBSDF_MEMBER_CHECKED(F90),
		SubstrateSlabBSDF_MEMBER_CHECKED(FuzzAmount),
		SubstrateSlabBSDF_MEMBER_CHECKED(FuzzColor),
		SubstrateSlabBSDF_MEMBER_CHECKED(FuzzRoughness),
		SubstrateSlabBSDF_MEMBER_CHECKED(GlintUV),
		SubstrateSlabBSDF_MEMBER_CHECKED(GlintValue),
		SubstrateSlabBSDF_MEMBER_CHECKED(Normal),
		SubstrateSlabBSDF_MEMBER_CHECKED(Roughness),
		SubstrateSlabBSDF_MEMBER_CHECKED(SecondRoughness),
		SubstrateSlabBSDF_MEMBER_CHECKED(SecondRoughnessWeight),
		SubstrateSlabBSDF_MEMBER_CHECKED(SSSMFP),
		SubstrateSlabBSDF_MEMBER_CHECKED(SSSMFPScale),
		SubstrateSlabBSDF_MEMBER_CHECKED(SSSPhaseAnisotropy),
		SubstrateSlabBSDF_MEMBER_CHECKED(Tangent),
	};
#undef SubstrateSlabBSDF_MEMBER_CHECKED

	TArray<FString> Inputs;
	UInterchangeShaderPortsAPI::GatherInputs(ShaderNode, Inputs);

	for (const FString& InputName : Inputs)
	{
		if(SlabInputs.Contains(InputName))
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> InputExpression =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, InputName, SlabBSDFFactoryNode->GetUniqueID());

			if (InputExpression.Get<0>())
			{
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(SlabBSDFFactoryNode, InputName, InputExpression.Get<0>()->GetUniqueID(), InputExpression.Get<1>());
			}
		}
	}

	// Specular Profile
	AddSpecularProfileToFactoryNode(ShaderNode, SlabBSDFFactoryNode, BaseNodeContainer);
}

void UInterchangeGenericMaterialPipeline::HandleLuminanceNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* LuminanceFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	LuminanceFactoryNode->SetCustomExpressionClassName(UMaterialExpressionMaterialXLuminance::StaticClass()->GetName());

	// Input
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> InputExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Luminance::Inputs::Input.ToString(), LuminanceFactoryNode->GetUniqueID());

		if (InputExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(LuminanceFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionMaterialXLuminance, Input).ToString(),
																  InputExpression.Get<0>()->GetUniqueID(), InputExpression.Get<1>());
		}
	}

	// LuminanceFactors
	if (FLinearColor LuminanceFactors; ShaderNode->GetLinearColorAttribute(Luminance::Attributes::LuminanceFactors.ToString(), LuminanceFactors))
	{
		const FName LuminanceFactorsMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionMaterialXLuminance, LuminanceFactors);
		LuminanceFactoryNode->AddLinearColorAttribute(LuminanceFactorsMemberName.ToString(), LuminanceFactors);
		LuminanceFactoryNode->AddApplyAndFillDelegates<FLinearColor>(LuminanceFactorsMemberName.ToString(), UMaterialExpressionMaterialXLuminance::StaticClass(), LuminanceFactorsMemberName);
	}
}

void UInterchangeGenericMaterialPipeline::HandleTexturePropertyNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TexturePropertyFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	TexturePropertyFactoryNode->SetCustomExpressionClassName(UMaterialExpressionTextureProperty::StaticClass()->GetName());

	// TextureObject
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> TextureObjectExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, TextureProperty::Inputs::Texture.ToString(), TexturePropertyFactoryNode->GetUniqueID());

		if (TextureObjectExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(TexturePropertyFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionTextureProperty, TextureObject).ToString(),
				TextureObjectExpression.Get<0>()->GetUniqueID(), TextureObjectExpression.Get<1>());
		}
	}

	// Property
	{
		if (int32 Property; ShaderNode->GetInt32Attribute(TextureProperty::Attributes::Property.ToString(), Property))
		{
			const FName PropertyMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionTextureProperty, Property);
			TexturePropertyFactoryNode->AddInt32Attribute(PropertyMemberName.ToString(), Property);
			TexturePropertyFactoryNode->AddApplyAndFillDelegates<int32>(PropertyMemberName.ToString(), UMaterialExpressionTextureProperty::StaticClass(), PropertyMemberName);
		}
	}
}

/** Trigonometry expressions are unitless and sets the period to 1 by default to let the user sets their period*/
void UInterchangeGenericMaterialPipeline::HandleTrigonometryNode(const UInterchangeShaderNode* ShaderNode, UClass* StaticClass, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TrigonometryFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	using StringView =
#if PLATFORM_WIDECHAR_IS_CHAR16
		std::u16string_view;
#else
		std::wstring_view;
#endif

	// ensure that trig expressions have the Input and Period members;
	static_assert(StringView(GET_MEMBER_NAME_STRING_CHECKED(UMaterialExpressionSine, Input)) == StringView(GET_MEMBER_NAME_STRING_CHECKED(UMaterialExpressionCosine, Input)));
	static_assert(StringView(GET_MEMBER_NAME_STRING_CHECKED(UMaterialExpressionSine, Input)) == StringView(GET_MEMBER_NAME_STRING_CHECKED(UMaterialExpressionTangent, Input)));
	static_assert(StringView(GET_MEMBER_NAME_STRING_CHECKED(UMaterialExpressionSine, Period)) == StringView(GET_MEMBER_NAME_STRING_CHECKED(UMaterialExpressionCosine, Period)));
	static_assert(StringView(GET_MEMBER_NAME_STRING_CHECKED(UMaterialExpressionSine, Period)) == StringView(GET_MEMBER_NAME_STRING_CHECKED(UMaterialExpressionTangent, Period)));

	TrigonometryFactoryNode->SetCustomExpressionClassName(StaticClass->GetName());

	// Input
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> InputExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, Trigonometry::Inputs::Input.ToString(), TrigonometryFactoryNode->GetUniqueID());
		if (InputExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(TrigonometryFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionSine, Input).ToString(),
																  InputExpression.Get<0>()->GetUniqueID(), InputExpression.Get<1>());
		}
	}

	// Period (should be 2Pi, unlike sine, a period of 0 always return cos(0) / tan(0) instead of cos(Input)/tan(Input), to be consistent over all 3, let's just set by default 2Pi
	{
		float Period = 2.f * UE_PI;
		ShaderNode->GetFloatAttribute(Trigonometry::Attributes::Period.ToString(), Period);
		const FName PeriodMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionSine, Period);
		TrigonometryFactoryNode->AddFloatAttribute(PeriodMemberName.ToString(), Period);
		TrigonometryFactoryNode->AddApplyAndFillDelegates<float>(PeriodMemberName.ToString(), StaticClass, PeriodMemberName);
	}
}

void UInterchangeGenericMaterialPipeline::HandleScalarParameterNode(const UInterchangeShaderNode* ShaderNode, UInterchangeMaterialExpressionFactoryNode* ScalarParameterFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	const FString ParameterKey = UInterchangeShaderPortsAPI::MakeInputParameterKey(ScalarParameter::Attributes::DefaultValue.ToString());
	float InputValue;
	if (ShaderNode->GetFloatAttribute(ParameterKey, InputValue))
	{
		ScalarParameterFactoryNode->SetCustomExpressionClassName(UMaterialExpressionScalarParameter::StaticClass()->GetName());
		const FName DefaultValueMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionScalarParameter, DefaultValue);
		ScalarParameterFactoryNode->AddFloatAttribute(DefaultValueMemberName.ToString(), InputValue);
		ScalarParameterFactoryNode->AddApplyAndFillDelegates<float>(DefaultValueMemberName.ToString(), UMaterialExpressionScalarParameter::StaticClass(), DefaultValueMemberName);
	}

	ScalarParameterFactoryNode->SetDisplayLabel(ShaderNode->GetDisplayLabel());
}
void UInterchangeGenericMaterialPipeline::HandleVectorParameterNode(const UInterchangeShaderNode* ShaderNode, UInterchangeMaterialExpressionFactoryNode* VectorParameterFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	const FString ParameterKey = UInterchangeShaderPortsAPI::MakeInputParameterKey(VectorParameter::Attributes::DefaultValue.ToString());
	FLinearColor InputValue;
	if (ShaderNode->GetLinearColorAttribute(ParameterKey, InputValue))
	{
		VectorParameterFactoryNode->SetCustomExpressionClassName(UMaterialExpressionVectorParameter::StaticClass()->GetName());
		const FName DefaultValueMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionVectorParameter, DefaultValue);
		VectorParameterFactoryNode->AddLinearColorAttribute(DefaultValueMemberName.ToString(), InputValue);
		VectorParameterFactoryNode->AddApplyAndFillDelegates<FLinearColor>(DefaultValueMemberName.ToString(), UMaterialExpressionVectorParameter::StaticClass(), DefaultValueMemberName);
	}

	VectorParameterFactoryNode->SetDisplayLabel(ShaderNode->GetDisplayLabel());
}
void UInterchangeGenericMaterialPipeline::HandleStaticBooleanParameterNode(const UInterchangeShaderNode* ShaderNode, UInterchangeMaterialExpressionFactoryNode* StaticBoolParameterFactoryNode)
{
	using namespace UE::Interchange::Materials::Standard::Nodes;
	const FString ParameterKey = UInterchangeShaderPortsAPI::MakeInputParameterKey(StaticBoolParameter::Attributes::DefaultValue.ToString());
	bool InputValue;
	if (ShaderNode->GetBooleanAttribute(ParameterKey, InputValue))
	{
		StaticBoolParameterFactoryNode->SetCustomExpressionClassName(UMaterialExpressionStaticBoolParameter::StaticClass()->GetName());
		const FName DefaultValueMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionStaticBoolParameter, DefaultValue);
		StaticBoolParameterFactoryNode->AddBooleanAttribute(DefaultValueMemberName.ToString(), InputValue);
		StaticBoolParameterFactoryNode->AddApplyAndFillDelegates<bool>(DefaultValueMemberName.ToString(), UMaterialExpressionStaticBoolParameter::StaticClass(), DefaultValueMemberName);
	}

	StaticBoolParameterFactoryNode->SetDisplayLabel(ShaderNode->GetDisplayLabel());
}

UInterchangeMaterialExpressionFactoryNode* UInterchangeGenericMaterialPipeline::CreateMaterialExpressionForShaderNode(UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode,
	const UInterchangeShaderNode* ShaderNode, const FString& ParentUid)
{
	using namespace UE::Interchange::Materials::Standard;

	// If we recognize the shader node type
	// - Create material expression for specific node type
	//
	// If we don't recognize the shader node type
	// - Create material expression by trying to match the node type to a material expression class name

	const FString MaterialExpressionUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(ShaderNode->GetUniqueID());

	UInterchangeMaterialExpressionFactoryNode* MaterialExpression = Cast<UInterchangeMaterialExpressionFactoryNode>(BaseNodeContainer->GetFactoryNode(MaterialExpressionUid));
	if (MaterialExpression != nullptr)
	{
		return MaterialExpression;
	}

	// Create function call expression if applicable
	MaterialExpression = CreateFunctionCallExpression(ShaderNode, MaterialExpressionUid, MaterialFactoryNode);
	if (MaterialExpression)
	{
		return MaterialExpression;
	}

	MaterialExpression = NewObject<UInterchangeMaterialExpressionFactoryNode>(BaseNodeContainer);
	if (!MaterialExpression)
	{
		return nullptr;
	}

	FString ShaderType;
	ShaderNode->GetCustomShaderType(ShaderType);

	BaseNodeContainer->SetupNode(MaterialExpression, MaterialExpressionUid, ShaderNode->GetDisplayLabel(), EInterchangeNodeContainerType::FactoryData, ParentUid);

	if (*ShaderType == Nodes::FlattenNormal::Name)
	{
		HandleFlattenNormalNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if (*ShaderType == Nodes::MakeFloat3::Name)
	{
		HandleMakeFloat3Node(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if (*ShaderType == Nodes::Lerp::Name)
	{
		HandleLerpNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if (*ShaderType == Nodes::Luminance::Name)
	{
		HandleLuminanceNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if(*ShaderType == Nodes::Mask::Name)
	{
		HandleMaskNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if(*ShaderType == Nodes::Noise::Name)
	{
		HandleNoiseNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if(*ShaderType == Nodes::NormalFromHeightMap::Name)
	{
		HandleNormalFromHeightMapNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if(*ShaderType == Nodes::RotateAboutAxis::Name)
	{
		HandleRotateAboutAxisNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if(*ShaderType == Nodes::Rotator::Name)
	{
		HandleRotatorNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if (*ShaderType == Nodes::Switch::Name)
	{
		HandleSwitchNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if (*ShaderType == Nodes::TextureCoordinate::Name)
	{
		HandleTextureCoordinateNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if(*ShaderType == Nodes::TextureObject::Name)
	{
		HandleTextureObjectNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if(*ShaderType == Nodes::TextureProperty::Name)
	{
		HandleTexturePropertyNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if (*ShaderType == Nodes::TextureSample::Name)
	{
		HandleTextureSampleNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if(*ShaderType == Nodes::Time::Name)
	{
		HandleTimeNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if(*ShaderType == Nodes::TransformPosition::Name)
	{
		HandleTransformPositionNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if(*ShaderType == Nodes::TransformVector::Name)
	{
		HandleTransformVectorNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if(*ShaderType == Nodes::VectorNoise::Name)
	{
		HandleVectorNoiseNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if (*ShaderType == Nodes::SlabBSDF::Name)
	{
		HandleSlabBSDFNode(ShaderNode, MaterialFactoryNode, MaterialExpression);
	}
	else if(*ShaderType == Nodes::Cosine::Name)
	{
		HandleTrigonometryNode(ShaderNode, UMaterialExpressionCosine::StaticClass(), MaterialFactoryNode, MaterialExpression);
	}
	else if(*ShaderType == Nodes::Sine::Name)
	{
		HandleTrigonometryNode(ShaderNode, UMaterialExpressionSine::StaticClass(), MaterialFactoryNode, MaterialExpression);
	}
	else if(*ShaderType == Nodes::Tangent::Name)
	{
		HandleTrigonometryNode(ShaderNode, UMaterialExpressionTangent::StaticClass(), MaterialFactoryNode, MaterialExpression);
	}
	else if (*ShaderType == Nodes::ScalarParameter::Name)
	{
		HandleScalarParameterNode(ShaderNode, MaterialExpression);
	}
	else if (*ShaderType == Nodes::VectorParameter::Name)
	{
		HandleVectorParameterNode(ShaderNode, MaterialExpression);
	}
	else if (*ShaderType == Nodes::StaticBoolParameter::Name)
	{
		HandleStaticBooleanParameterNode(ShaderNode, MaterialExpression);
	}
	else if (ensure(!ShaderType.IsEmpty()))
	{
		const FString ExpressionClassName = TEXT("MaterialExpression") + ShaderType;
		MaterialExpression->SetCustomExpressionClassName(ExpressionClassName);

		TArray<FString> Inputs;
		UInterchangeShaderPortsAPI::GatherInputs(ShaderNode, Inputs);

		for (const FString& InputName : Inputs)
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> InputExpression =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, InputName, MaterialExpressionUid);

			if (InputExpression.Get<0>())
			{
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(MaterialExpression, InputName, InputExpression.Get<0>()->GetUniqueID(), InputExpression.Get<1>());
			}
		}
	}

	MaterialExpression->AddTargetNodeUid(ShaderNode->GetUniqueID());

	if (*ShaderType == Nodes::TextureSample::Name
		|| *ShaderType == Nodes::TextureObject::Name)
	{
		FString TextureUid;

		if (*ShaderType == Nodes::TextureSample::Name)
		{
			const bool bIsAParameter = UInterchangeShaderPortsAPI::HasParameter(ShaderNode, Nodes::TextureSample::Inputs::Texture);
			ShaderNode->GetStringAttribute(CreateInputKey(Nodes::TextureSample::Inputs::Texture.ToString(), bIsAParameter), TextureUid);
		}
		else if (*ShaderType == Nodes::TextureObject::Name)
		{
			const bool bIsAParameter = UInterchangeShaderPortsAPI::HasParameter(ShaderNode, Nodes::TextureObject::Inputs::Texture);
			ShaderNode->GetStringAttribute(CreateInputKey(Nodes::TextureObject::Inputs::Texture.ToString(),bIsAParameter), TextureUid);
		}

		// Make the material factory node have a dependency on the texture factory node so that the texture asset gets created first
		if (const UInterchangeTextureNode* TextureNode = Cast<const UInterchangeTextureNode>(BaseNodeContainer->GetNode(TextureUid)))
		{
			TArray<FString> TextureNodeTargets;
			TextureNode->GetTargetNodeUids(TextureNodeTargets);

			if (TextureNodeTargets.Num() > 0)
			{
				FString TextureFactoryNodeUid = TextureNodeTargets[0];

				if (BaseNodeContainer->IsNodeUidValid(TextureFactoryNodeUid))
				{
					MaterialFactoryNode->AddFactoryDependencyUid(TextureFactoryNodeUid);
				}
			}
		}
	}

	return MaterialExpression;
}

UInterchangeMaterialExpressionFactoryNode* UInterchangeGenericMaterialPipeline::CreateExpressionNode(const FString& ExpressionName, const FString& ParentUid, UClass* MaterialExpressionClass)
{
	const FString MaterialExpressionUid = ParentUid + TEXT("\\") + ExpressionName;

	UInterchangeMaterialExpressionFactoryNode* MaterialExpressionFactoryNode = NewObject<UInterchangeMaterialExpressionFactoryNode>(BaseNodeContainer);
	MaterialExpressionFactoryNode->SetCustomExpressionClassName(MaterialExpressionClass->GetName());
	BaseNodeContainer->SetupNode(MaterialExpressionFactoryNode, MaterialExpressionUid, ExpressionName, EInterchangeNodeContainerType::FactoryData, ParentUid);

	return MaterialExpressionFactoryNode;
}

UInterchangeMaterialExpressionFactoryNode* UInterchangeGenericMaterialPipeline::HandleFloatInput(const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid, bool bIsAParameter)
{
	if (bIsAParameter)
	{
		return CreateScalarParameterExpression(ShaderNode, InputName, ParentUid);
	}
	else
	{
		return CreateConstantExpression(ShaderNode, InputName, ParentUid);
	}
}

UInterchangeMaterialExpressionFactoryNode* UInterchangeGenericMaterialPipeline::CreateConstantExpression(const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid)
{
	UInterchangeMaterialExpressionFactoryNode* MaterialExpressionFactoryNode = CreateExpressionNode(InputName, ParentUid, UMaterialExpressionConstant::StaticClass());

	float InputValue;
	if (ShaderNode->GetFloatAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(InputName), InputValue))
	{
		const FName RMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionConstant, R);
		MaterialExpressionFactoryNode->AddFloatAttribute(RMemberName.ToString(), InputValue);
		MaterialExpressionFactoryNode->AddApplyAndFillDelegates<float>(RMemberName.ToString(), UMaterialExpressionConstant::StaticClass(), RMemberName);
	}

	return MaterialExpressionFactoryNode;
}

UInterchangeMaterialExpressionFactoryNode* UInterchangeGenericMaterialPipeline::CreateScalarParameterExpression(const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid)
{
	using namespace UE::Interchange::Materials::Standard;

	UInterchangeMaterialExpressionFactoryNode* MaterialExpressionFactoryNode = CreateExpressionNode(InputName, ParentUid, UMaterialExpressionScalarParameter::StaticClass());

	float InputValue;
	if (ShaderNode->GetFloatAttribute(UInterchangeShaderPortsAPI::MakeInputParameterKey(InputName), InputValue))
	{
		const FName DefaultValueMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionScalarParameter, DefaultValue);
		MaterialExpressionFactoryNode->AddFloatAttribute(DefaultValueMemberName.ToString(), InputValue);
		MaterialExpressionFactoryNode->AddApplyAndFillDelegates<float>(DefaultValueMemberName.ToString(), UMaterialExpressionScalarParameter::StaticClass(), DefaultValueMemberName);
	}

	if(FString DisplayLabel = ShaderNode->GetDisplayLabel(); DisplayLabel.IsEmpty())
	{
		MaterialExpressionFactoryNode->SetDisplayLabel(InputName);
	}
	else
	{
		MaterialExpressionFactoryNode->SetDisplayLabel(DisplayLabel);
	}

	return MaterialExpressionFactoryNode;
}

UInterchangeMaterialExpressionFactoryNode* UInterchangeGenericMaterialPipeline::HandleLinearColorInput(const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid, bool bIsAParameter)
{
	if (bIsAParameter)
	{
		return CreateVectorParameterExpression(ShaderNode, InputName, ParentUid);
	}
	else
	{
		return CreateConstant3VectorExpression(ShaderNode, InputName, ParentUid);
	}
}

UInterchangeMaterialExpressionFactoryNode* UInterchangeGenericMaterialPipeline::CreateConstant3VectorExpression(const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid)
{
	UInterchangeMaterialExpressionFactoryNode* MaterialExpressionFactoryNode = CreateExpressionNode(InputName, ParentUid, UMaterialExpressionConstant3Vector::StaticClass());

	FLinearColor InputValue;
	if (ShaderNode->GetLinearColorAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(InputName), InputValue))
	{
		const FName ConstantMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionConstant3Vector, Constant);
		MaterialExpressionFactoryNode->AddLinearColorAttribute(ConstantMemberName.ToString(), InputValue);
		MaterialExpressionFactoryNode->AddApplyAndFillDelegates<FLinearColor>(ConstantMemberName.ToString(), UMaterialExpressionConstant3Vector::StaticClass(), ConstantMemberName);
	}

	return MaterialExpressionFactoryNode;
}

UInterchangeMaterialExpressionFactoryNode* UInterchangeGenericMaterialPipeline::CreateVectorParameterExpression(const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid)
{
	UInterchangeMaterialExpressionFactoryNode* MaterialExpressionFactoryNode = CreateExpressionNode(InputName, ParentUid, UMaterialExpressionVectorParameter::StaticClass());

	FLinearColor InputValue;
	if (ShaderNode->GetLinearColorAttribute(UInterchangeShaderPortsAPI::MakeInputParameterKey(InputName), InputValue))
	{
		const FName DefaultValueName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionVectorParameter, DefaultValue);
		MaterialExpressionFactoryNode->AddLinearColorAttribute(DefaultValueName.ToString(), InputValue);
		MaterialExpressionFactoryNode->AddApplyAndFillDelegates<FLinearColor>(DefaultValueName.ToString(), UMaterialExpressionVectorParameter::StaticClass(), DefaultValueName);
	}

	if(FString DisplayLabel = ShaderNode->GetDisplayLabel(); DisplayLabel.IsEmpty())
	{
		MaterialExpressionFactoryNode->SetDisplayLabel(InputName);
	}
	else
	{
		MaterialExpressionFactoryNode->SetDisplayLabel(DisplayLabel);
	}

	return MaterialExpressionFactoryNode;
}

UInterchangeMaterialExpressionFactoryNode* UInterchangeGenericMaterialPipeline::CreateStaticBooleanParameterExpression(const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid)
{
	UInterchangeMaterialExpressionFactoryNode* MaterialExpressionFactoryNode = CreateExpressionNode(InputName, ParentUid, UMaterialExpressionStaticBoolParameter::StaticClass());

	bool InputValue;
	if (ShaderNode->GetBooleanAttribute(InputName, InputValue))
	{
		const FName DefaultValueMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionStaticBoolParameter, DefaultValue);
		MaterialExpressionFactoryNode->AddBooleanAttribute(DefaultValueMemberName.ToString(), InputValue);
		MaterialExpressionFactoryNode->AddApplyAndFillDelegates<bool>(DefaultValueMemberName.ToString(), UMaterialExpressionStaticBoolParameter::StaticClass(), DefaultValueMemberName);
	}

	MaterialExpressionFactoryNode->SetDisplayLabel(InputName);

	return MaterialExpressionFactoryNode;
}

UInterchangeMaterialExpressionFactoryNode* UInterchangeGenericMaterialPipeline::CreateVector2ParameterExpression(const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid)
{
	FVector2f InputValue;
	if (ShaderNode->GetAttribute<FVector2f>(UInterchangeShaderPortsAPI::MakeInputValueKey(InputName), InputValue))
	{
		UInterchangeMaterialExpressionFactoryNode* VectorParameterFactoryNode = CreateExpressionNode(InputName, ParentUid, UMaterialExpressionVectorParameter::StaticClass());

		const FName DefaultValueMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionVectorParameter, DefaultValue);
		VectorParameterFactoryNode->AddLinearColorAttribute(DefaultValueMemberName.ToString(), FLinearColor(InputValue.X, InputValue.Y, 0.f));
		VectorParameterFactoryNode->AddApplyAndFillDelegates<FLinearColor>(DefaultValueMemberName.ToString(), UMaterialExpressionVectorParameter::StaticClass(), DefaultValueMemberName);

		// Defaults to R&G
		UInterchangeMaterialExpressionFactoryNode* ComponentMaskFactoryNode = CreateExpressionNode(InputName + TEXT("_Mask"), ParentUid, UMaterialExpressionComponentMask::StaticClass());

		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(ComponentMaskFactoryNode, GET_MEMBER_NAME_CHECKED(UMaterialExpressionComponentMask, Input).ToString(),
			VectorParameterFactoryNode->GetUniqueID() );

		return ComponentMaskFactoryNode;
	}

	return nullptr;
}

TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> UInterchangeGenericMaterialPipeline::CreateMaterialExpressionForInput(UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid)
{
	// Make sure we don't create an expression for an input if it already has one
	if (UInterchangeShaderPortsAPI::HasInput(MaterialFactoryNode, *InputName))
	{
		return TTuple<UInterchangeMaterialExpressionFactoryNode*, FString>{};
	}

	// If we have a connection
	// - Create material expression for the connected shader node
	//
	// If we don't have a connection
	// - Create material expression for the input value

	UInterchangeMaterialExpressionFactoryNode* MaterialExpressionFactoryNode = nullptr;

	int32 ExpressionContextIndex = MaterialExpressionCreationContextStack.AddDefaulted();

	FString ConnectedShaderNodeUid;
	if (UInterchangeShaderPortsAPI::GetInputConnection(ShaderNode, InputName, ConnectedShaderNodeUid, MaterialExpressionCreationContextStack[ExpressionContextIndex].OutputName))
	{
		if (const UInterchangeShaderNode* ConnectedShaderNode = Cast<const UInterchangeShaderNode>(BaseNodeContainer->GetNode(ConnectedShaderNodeUid)))
		{
			MaterialExpressionFactoryNode = CreateMaterialExpressionForShaderNode(MaterialFactoryNode, ConnectedShaderNode, ParentUid);
		}
	}
	else
	{
		const bool bIsAParameter = UInterchangeShaderPortsAPI::HasParameter(ShaderNode, FName(InputName));
		switch(UInterchangeShaderPortsAPI::GetInputType(ShaderNode, InputName, bIsAParameter))
		{
		case UE::Interchange::EAttributeTypes::Float:
			MaterialExpressionFactoryNode = HandleFloatInput(ShaderNode, InputName, ParentUid, bIsAParameter);
			break;
		case UE::Interchange::EAttributeTypes::LinearColor:
			MaterialExpressionFactoryNode = HandleLinearColorInput(ShaderNode, InputName, ParentUid, bIsAParameter);
			break;
		case UE::Interchange::EAttributeTypes::Vector2f:
			MaterialExpressionFactoryNode = CreateVector2ParameterExpression(ShaderNode, InputName, ParentUid);
			break;
		case UE::Interchange::EAttributeTypes::Bool:
			MaterialExpressionFactoryNode = CreateStaticBooleanParameterExpression(ShaderNode, InputName, ParentUid);
			break;
		}

		if (MaterialExpressionFactoryNode)
		{
			FString MaterialExpressionName;
			if (AttributeStorageNode && AttributeStorageNode->GetStringAttribute(ShaderNode->GetUniqueID(), MaterialExpressionName))
			{
				MaterialExpressionFactoryNode->SetDisplayLabel(MaterialExpressionName);
			}
		}
	}


	TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> Result {MaterialExpressionFactoryNode, MaterialExpressionCreationContextStack[ExpressionContextIndex].OutputName};
	MaterialExpressionCreationContextStack.Pop();

	return Result;
}

UInterchangeMaterialFactoryNode* UInterchangeGenericMaterialPipeline::CreateMaterialFactoryNode(const UInterchangeShaderGraphNode* ShaderGraphNode)
{
	UInterchangeMaterialFactoryNode* MaterialFactoryNode = Cast<UInterchangeMaterialFactoryNode>( CreateBaseMaterialFactoryNode(ShaderGraphNode, UInterchangeMaterialFactoryNode::StaticClass()) );

	UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(ShaderGraphNode, MaterialFactoryNode, false);
	
	if (bOverrideDisplacement)
	{
		MaterialFactoryNode->SetCustomDisplacementCenter(OverrideDisplacementCenter);
	}

	if(HandleSubstrate(ShaderGraphNode, MaterialFactoryNode))
	{
		return MaterialFactoryNode;
	}

	// Handle the case where the material will be connected through the material attributes input
	if (HandleBxDFInput(ShaderGraphNode, MaterialFactoryNode))
	{
		// No need to proceed any further
		return MaterialFactoryNode;
	}

	if (HandleUnlitModel(ShaderGraphNode, MaterialFactoryNode))
	{
		// No need to proceed any further
		return MaterialFactoryNode;
	}

	if (!HandleMetalRoughnessModel(ShaderGraphNode, MaterialFactoryNode))
	{
		if (!HandleSpecGlossModel(ShaderGraphNode, MaterialFactoryNode))
		{
			if (!HandlePhongModel(ShaderGraphNode, MaterialFactoryNode))
			{
				HandleLambertModel(ShaderGraphNode, MaterialFactoryNode);
			}
		}
	}

	// Can't have different shading models
	// Favor translucency over coats (clear coat, sheen, etc.) since it tends to have a bigger impact visually
	if (!HandleThinTranslucent(ShaderGraphNode, MaterialFactoryNode))
	{
		if (!HandleClearCoat(ShaderGraphNode, MaterialFactoryNode))
		{
			if(!HandleSheen(ShaderGraphNode, MaterialFactoryNode))
			{
				HandleSubsurface(ShaderGraphNode, MaterialFactoryNode);
			}
		}
	}

	HandleCommonParameters(ShaderGraphNode, MaterialFactoryNode);

	return MaterialFactoryNode;
}

UInterchangeMaterialInstanceFactoryNode* UInterchangeGenericMaterialPipeline::CreateMaterialInstanceFactoryNode(const UInterchangeShaderGraphNode* ShaderGraphNode)
{
	using namespace UE::Interchange::InterchangeGenericMaterialPipeline::Private;

	UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode =
		Cast<UInterchangeMaterialInstanceFactoryNode>( CreateBaseMaterialFactoryNode(ShaderGraphNode, UInterchangeMaterialInstanceFactoryNode::StaticClass()) );

	UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(ShaderGraphNode, MaterialInstanceFactoryNode, false);

	TFunction<void(const FString&)> ChooseParent = [this, MaterialInstanceFactoryNode, ShaderGraphNode](const FString& Model) -> void
	{
		FString ParentRootName;

		if (HasThinTranslucency(ShaderGraphNode))
		{
			ParentRootName = TEXT("ThinTranslucentMaterial_");
		}
		else if (HasClearCoat(ShaderGraphNode))
		{
			ParentRootName = TEXT("ClearCoatMaterial_");
		}
		else if (HasSheen(ShaderGraphNode))
		{
			ParentRootName = TEXT("SheenMaterial_");
		}
		else if (HasSubsurface(ShaderGraphNode))
		{
			ParentRootName = TEXT("SubsurfaceMaterial_");
		}
		else
		{
			ParentRootName = TEXT("PBRSurfaceMaterial_");
		}

		const FString ParentAssetPath = TEXT("/InterchangeAssets/Materials/") + ParentRootName + Model + TEXT(".") + ParentRootName + Model;
		MaterialInstanceFactoryNode->SetCustomParent(ParentAssetPath);
	};

	if (UMaterialInterface* ParentMaterialObj = Cast<UMaterialInterface>(ParentMaterial.TryLoad()))
	{
		MaterialInstanceFactoryNode->SetCustomParent(ParentMaterialObj->GetPathName());
	}
	else if (IsSpecGlossModel(ShaderGraphNode))
	{
		ChooseParent(TEXT("SG"));
	}
	else if (IsMetalRoughModel(ShaderGraphNode))
	{
		ChooseParent(TEXT("MR"));
	}
	else if (IsPhongModel(ShaderGraphNode))
	{
		MaterialInstanceFactoryNode->SetCustomParent(TEXT("/InterchangeAssets/Materials/PhongSurfaceMaterial.PhongSurfaceMaterial"));
	}
	else if (IsLambertModel(ShaderGraphNode))
	{
		MaterialInstanceFactoryNode->SetCustomParent(TEXT("/InterchangeAssets/Materials/LambertSurfaceMaterial.LambertSurfaceMaterial"));
	}
	else if (IsUnlitModel(ShaderGraphNode))
	{
		MaterialInstanceFactoryNode->SetCustomParent(TEXT("/InterchangeAssets/Materials/UnlitMaterial.UnlitMaterial"));
	}
	else
	{
		// Default to PBR
		MaterialInstanceFactoryNode->SetCustomParent(TEXT("/InterchangeAssets/Materials/PBRSurfaceMaterial.PBRSurfaceMaterial"));
	}

#if WITH_EDITOR
	const UClass* MaterialClass = IsRunningGame() ? UMaterialInstanceDynamic::StaticClass() : UMaterialInstanceConstant::StaticClass();
	MaterialInstanceFactoryNode->SetCustomInstanceClassName(MaterialClass->GetPathName());
#else
	MaterialInstanceFactoryNode->SetCustomInstanceClassName(UMaterialInstanceDynamic::StaticClass()->GetPathName());
#endif

	VisitShaderGraphNode(ShaderGraphNode, MaterialInstanceFactoryNode);

	AddSpecularProfileToFactoryNode(ShaderGraphNode, MaterialInstanceFactoryNode, BaseNodeContainer);

	return MaterialInstanceFactoryNode;
}

void UInterchangeGenericMaterialPipeline::CreateSpecularProfileFactoryNode(const UInterchangeSpecularProfileNode* SpecularProfileNode)
{
	const FString FactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(SpecularProfileNode->GetUniqueID());

	UInterchangeSpecularProfileFactoryNode* FactoryNode = NewObject<UInterchangeSpecularProfileFactoryNode>(BaseNodeContainer, NAME_None);

	BaseNodeContainer->SetupNode(FactoryNode, FactoryNodeUid, SpecularProfileNode->GetDisplayLabel(), EInterchangeNodeContainerType::FactoryData);

	FactoryNode->SetEnabled(true);

	if (uint8 Format; SpecularProfileNode->GetCustomFormat(Format))
	{
		FactoryNode->SetCustomFormat(ESpecularProfileFormat{ Format });
	}

	if (FString TextureUid; SpecularProfileNode->GetCustomTexture(TextureUid))
	{
		if(BaseNodeContainer->GetNode(TextureUid))
		{
			FactoryNode->SetCustomTexture(TextureUid);
			FactoryNode->AddFactoryDependencyUid(UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TextureUid));
		}
	}

	FactoryNode->AddTargetNodeUid(SpecularProfileNode->GetUniqueID());
	SpecularProfileNode->AddTargetNodeUid(FactoryNode->GetUniqueID());
}

void UInterchangeGenericMaterialPipeline::VisitShaderGraphNode(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode) const
{
	TArray<FString> Inputs;
	UInterchangeShaderPortsAPI::GatherInputs(ShaderGraphNode, Inputs);

	// We don't want to visit the whole shader graph for every input, for example with a StandardSurface with 31 inputs, the MaterialFunction is connected to all inputs of the Material but should be visited only once
	TSet<const UInterchangeShaderNode*> VisitedNodes;
	for(const FString& InputName : Inputs)
	{
		VisitShaderInput(ShaderGraphNode, MaterialInstanceFactoryNode, InputName, VisitedNodes);
	}
}

void UInterchangeGenericMaterialPipeline::VisitShaderNode(const UInterchangeShaderNode* ShaderNode, UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode, TSet<const UInterchangeShaderNode*>& VisitedNodes) const
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	if(FString ShaderType; ShaderNode->GetCustomShaderType(ShaderType))
	{
		if(*ShaderType == ScalarParameter::Name)
		{
			return VisitScalarParameterNode(ShaderNode, MaterialInstanceFactoryNode);
		}
		else if (*ShaderType == TextureSample::Name)
		{
			return VisitTextureSampleNode(ShaderNode, MaterialInstanceFactoryNode);
		}
		else if(*ShaderType == VectorParameter::Name)
		{
			return VisitVectorParameterNode(ShaderNode, MaterialInstanceFactoryNode);
		}
	}

	{
		TArray<FString> Inputs;
		UInterchangeShaderPortsAPI::GatherInputs(ShaderNode, Inputs);

		for(const FString & InputName: Inputs)
		{
			VisitShaderInput(ShaderNode, MaterialInstanceFactoryNode, InputName, VisitedNodes);
		}
	}
}

void UInterchangeGenericMaterialPipeline::VisitShaderInput(const UInterchangeShaderNode* ShaderNode, UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode, const FString& InputName, TSet<const UInterchangeShaderNode*>& VisitedNodes) const
{
	if(VisitedNodes.Find(ShaderNode))
	{
		return;
	}

	const bool bIsAParameter = UInterchangeShaderPortsAPI::HasParameter(ShaderNode, FName(InputName));

	FString ConnectedShaderNodeUid;
	FString OutputName;
	if (UInterchangeShaderPortsAPI::GetInputConnection(ShaderNode, InputName, ConnectedShaderNodeUid, OutputName))
	{
		const UInterchangeShaderNode* ConnectedShaderNode = Cast<const UInterchangeShaderNode>(BaseNodeContainer->GetNode(ConnectedShaderNodeUid));
		if (ConnectedShaderNode && !VisitedNodes.Find(ConnectedShaderNode))
		{
			VisitShaderNode(ConnectedShaderNode, MaterialInstanceFactoryNode, VisitedNodes);
			VisitedNodes.Emplace(ConnectedShaderNode);
		}
	}
	else
	{
		switch(UInterchangeShaderPortsAPI::GetInputType(ShaderNode, InputName, bIsAParameter))
		{
		case UE::Interchange::EAttributeTypes::Float:
		{
			if(float InputValue; ShaderNode->GetFloatAttribute(CreateInputKey(InputName, bIsAParameter), InputValue))
			{
				MaterialInstanceFactoryNode->AddFloatAttribute(CreateInputKey(InputName, bIsAParameter), InputValue);
			}
		}
		break;
		case UE::Interchange::EAttributeTypes::LinearColor:
		{
			if(FLinearColor InputValue;	ShaderNode->GetLinearColorAttribute(CreateInputKey(InputName, bIsAParameter), InputValue))
			{
				MaterialInstanceFactoryNode->AddLinearColorAttribute(CreateInputKey(InputName, bIsAParameter), InputValue);
			}
		}
		break;
		}
	}
}

void UInterchangeGenericMaterialPipeline::VisitScalarParameterNode(const UInterchangeShaderNode* ShaderNode, UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode) const
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	const bool bIsAParameter = UInterchangeShaderPortsAPI::HasParameter(ShaderNode, ScalarParameter::Attributes::DefaultValue);

	if(float DefaultValue; ShaderNode->GetFloatAttribute(CreateInputKey(ScalarParameter::Attributes::DefaultValue.ToString(), bIsAParameter), DefaultValue))
	{
		MaterialInstanceFactoryNode->AddFloatAttribute(CreateInputKey(ShaderNode->GetDisplayLabel(), bIsAParameter), DefaultValue);
	}
}

void UInterchangeGenericMaterialPipeline::VisitTextureSampleNode(const UInterchangeShaderNode* ShaderNode, UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode) const
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	const bool bIsAParameter = UInterchangeShaderPortsAPI::HasParameter(ShaderNode, TextureSample::Inputs::Texture);

	FString TextureUid;
	if (ShaderNode->GetStringAttribute(CreateInputKey(TextureSample::Inputs::Texture.ToString(),bIsAParameter), TextureUid))
	{
		if (!TextureUid.IsEmpty())
		{
			FString TextureFactoryUid;
			if (const UInterchangeTextureNode* TextureNode = Cast<const UInterchangeTextureNode>(BaseNodeContainer->GetNode(TextureUid)))
			{
				TArray<FString> TextureTargetNodes;
				TextureNode->GetTargetNodeUids(TextureTargetNodes);

				if (TextureTargetNodes.Num() > 0)
				{
					TextureFactoryUid = TextureTargetNodes[0];
					MaterialInstanceFactoryNode->AddStringAttribute(CreateInputKey(ShaderNode->GetDisplayLabel(), bIsAParameter), TextureFactoryUid);
					MaterialInstanceFactoryNode->AddFactoryDependencyUid(TextureFactoryUid);
				}
			}
		}
	}
}

void UInterchangeGenericMaterialPipeline::VisitVectorParameterNode(const UInterchangeShaderNode* ShaderNode, UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode) const
{
	using namespace UE::Interchange::Materials::Standard::Nodes;

	const bool bIsAParameter = UInterchangeShaderPortsAPI::HasParameter(ShaderNode, VectorParameter::Attributes::DefaultValue);

	if(FLinearColor DefaultValue; ShaderNode->GetLinearColorAttribute(CreateInputKey(VectorParameter::Attributes::DefaultValue.ToString(), bIsAParameter), DefaultValue))
	{
		MaterialInstanceFactoryNode->AddLinearColorAttribute(CreateInputKey(ShaderNode->GetDisplayLabel(), true), DefaultValue);
	}
}

FString UInterchangeGenericMaterialPipeline::GetTextureUidAttributeFromShaderNode(const UInterchangeShaderNode* ShaderNode, FName ParameterName, bool& OutIsAParameter) const
{
	OutIsAParameter = UInterchangeShaderPortsAPI::HasParameter(ShaderNode, ParameterName);
	FString TextureUid;
	ShaderNode->GetStringAttribute(CreateInputKey(ParameterName.ToString(), OutIsAParameter), TextureUid);
	return TextureUid;
}

FString UInterchangeGenericMaterialPipeline::CreateInputKey(const FString& InputName, bool bIsAParameter) const
{
	if (bIsAParameter)
	{
		return UInterchangeShaderPortsAPI::MakeInputParameterKey(InputName);
	}
	else
	{
		return UInterchangeShaderPortsAPI::MakeInputValueKey(InputName);
	}
}

bool UInterchangeGenericMaterialPipeline::HandleBxDFInput(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode)
{
	using namespace UE::Interchange::Materials;

	if (!ShaderGraphNode || !UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Common::Parameters::BxDF))
	{
		return false;
	}

	TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
		CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Common::Parameters::BxDF.ToString(), MaterialFactoryNode->GetUniqueID());
	ensure(ExpressionFactoryNode.Get<0>());

	if (ExpressionFactoryNode.Get<0>())
	{
		UInterchangeShaderPortsAPI::ConnectOuputToInputByName(MaterialFactoryNode, Common::Parameters::BxDF.ToString(), ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
	}

	// Make sure the bUseMaterialAttributes property of the material is set to true
	static const FName UseMaterialAttributesMemberName = GET_MEMBER_NAME_CHECKED(UMaterial, bUseMaterialAttributes);

	MaterialFactoryNode->AddBooleanAttribute(UseMaterialAttributesMemberName.ToString(), true);
	MaterialFactoryNode->AddApplyAndFillDelegates<FString>(UseMaterialAttributesMemberName.ToString(), UMaterialExpressionMaterialFunctionCall::StaticClass(), UseMaterialAttributesMemberName);

	return true;
}

UInterchangeMaterialFunctionFactoryNode* UInterchangeGenericMaterialPipeline::CreateMaterialFunctionFactoryNode(const UInterchangeShaderGraphNode* ShaderGraphNode)
{
	UInterchangeMaterialFunctionFactoryNode* FactoryNode = Cast<UInterchangeMaterialFunctionFactoryNode>(CreateBaseMaterialFactoryNode(ShaderGraphNode, UInterchangeMaterialFunctionFactoryNode::StaticClass()));

	TArray<FString> InputNames;
	UInterchangeShaderPortsAPI::GatherInputs(ShaderGraphNode, InputNames);

	for (const FString& InputName : InputNames)
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
			CreateMaterialExpressionForInput(FactoryNode, ShaderGraphNode, InputName, FactoryNode->GetUniqueID());

		if (ExpressionFactoryNode.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(FactoryNode, InputName, ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
		}
	}

	return FactoryNode;
}

bool UInterchangeGenericMaterialPipeline::IsUnlitModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const
{
	using namespace UE::Interchange::Materials::Unlit;

	const bool bHasUnlitColorInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Parameters::UnlitColor);

	return bHasUnlitColorInput;
}

bool UInterchangeGenericMaterialPipeline::HandleUnlitModel(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode)
{
	using namespace UE::Interchange::Materials;

	bool bShadingModelHandled = false;

	// Unlit Color
	{
		const bool bHasInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Unlit::Parameters::UnlitColor);

		if (bHasInput)
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> ExpressionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Unlit::Parameters::UnlitColor.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ExpressionFactoryNode.Get<0>())
			{
				MaterialFactoryNode->ConnectOutputToEmissiveColor(ExpressionFactoryNode.Get<0>()->GetUniqueID(), ExpressionFactoryNode.Get<1>());
			}

			//gltf allows unlit color to be also translucent:
			{
				const bool bHasOpacityInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Common::Parameters::Opacity);

				if (bHasOpacityInput)
				{
					TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> OpacityExpressionFactoryNode =
						CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Common::Parameters::Opacity.ToString(), MaterialFactoryNode->GetUniqueID());

					if (OpacityExpressionFactoryNode.Get<0>())
					{
						MaterialFactoryNode->ConnectOutputToOpacity(OpacityExpressionFactoryNode.Get<0>()->GetUniqueID(), OpacityExpressionFactoryNode.Get<1>());
					}

					using namespace UE::Interchange::InterchangeGenericMaterialPipeline;

					Private::UpdateBlendModeBasedOnOpacityAttributes(ShaderGraphNode, MaterialFactoryNode);
				}
			}

			bShadingModelHandled = true;
		}
	}

	if (bShadingModelHandled)
	{
		MaterialFactoryNode->SetCustomShadingModel(EMaterialShadingModel::MSM_Unlit);
	}

	return bShadingModelHandled;
}

bool UInterchangeGenericMaterialPipeline::HandleSubstrate(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode)
{
	using namespace UE::Interchange::Materials;
	bool bShadingModelHandled = false;

	if (UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, SubstrateMaterial::Parameters::FrontMaterial))
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> FrontMaterialFactoryNode =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, SubstrateMaterial::Parameters::FrontMaterial.ToString(), MaterialFactoryNode->GetUniqueID());
		
		if (ensure(FrontMaterialFactoryNode.Get<0>()))
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(MaterialFactoryNode, SubstrateMaterial::Parameters::FrontMaterial.ToString(), FrontMaterialFactoryNode.Get<0>()->GetUniqueID(), FrontMaterialFactoryNode.Get<1>());
		}

		if (UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, SubstrateMaterial::Parameters::OpacityMask))
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> OpacityMaskFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, SubstrateMaterial::Parameters::OpacityMask.ToString(), MaterialFactoryNode->GetUniqueID());			

			if (ensure(OpacityMaskFactoryNode.Get<0>()))
			{
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(MaterialFactoryNode, SubstrateMaterial::Parameters::OpacityMask.ToString(), OpacityMaskFactoryNode.Get<0>()->GetUniqueID(), OpacityMaskFactoryNode.Get<1>());
			}
		}

		if (UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, SubstrateMaterial::Parameters::Displacement))
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> DisplacementFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, SubstrateMaterial::Parameters::Displacement.ToString(), MaterialFactoryNode->GetUniqueID());			

			if (ensure(DisplacementFactoryNode.Get<0>()))
			{
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(MaterialFactoryNode, SubstrateMaterial::Parameters::Displacement.ToString(), DisplacementFactoryNode.Get<0>()->GetUniqueID(), DisplacementFactoryNode.Get<1>());
			}

			if (float DisplacementCenter; !bOverrideDisplacement && ShaderGraphNode->GetCustomDisplacementCenterMode(DisplacementCenter))
			{
				MaterialFactoryNode->SetCustomDisplacementCenter(DisplacementCenter);
			}

			if (float DisplacementMagnitude; ShaderGraphNode->GetCustomDisplacementMagnitude(DisplacementMagnitude))
			{
				MaterialFactoryNode->SetCustomDisplacementMagnitude(DisplacementMagnitude);
			}
		}

		if (UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, SubstrateMaterial::Parameters::Occlusion))
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> OcclusionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, SubstrateMaterial::Parameters::Occlusion.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ensure(OcclusionFactoryNode.Get<0>()))
			{
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(MaterialFactoryNode, SubstrateMaterial::Parameters::Occlusion.ToString(), OcclusionFactoryNode.Get<0>()->GetUniqueID(), OcclusionFactoryNode.Get<1>());
			}
		}

		if (UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, Common::Parameters::Refraction))
		{
			TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> RefractionFactoryNode =
				CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderGraphNode, Common::Parameters::Refraction.ToString(), MaterialFactoryNode->GetUniqueID());

			if (ensure(RefractionFactoryNode.Get<0>()))
			{
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(MaterialFactoryNode, Common::Parameters::Refraction.ToString(), RefractionFactoryNode.Get<0>()->GetUniqueID(), RefractionFactoryNode.Get<1>());
			}
		}

		if (EBlendMode BlendMode; ShaderGraphNode->GetCustomBlendMode(reinterpret_cast<int&>(BlendMode)))
		{
			MaterialFactoryNode->SetCustomBlendMode(BlendMode);
			if (BlendMode == BLEND_TranslucentColoredTransmittance)
			{
				MaterialFactoryNode->SetCustomTranslucencyLightingMode(ETranslucencyLightingMode::TLM_SurfacePerPixelLighting);
				MaterialFactoryNode->SetCustomRefractionMethod(ERefractionMode::RM_IndexOfRefraction);
			}
		}

		bShadingModelHandled = true;
	}

	return bShadingModelHandled;
}

UInterchangeMaterialExpressionFactoryNode* UInterchangeGenericMaterialPipeline::CreateFunctionCallExpression(const UInterchangeShaderNode* ShaderNode, const FString& MaterialExpressionUid, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode)
{
	using namespace UE::Interchange::InterchangeGenericMaterialPipeline::Private;

	const UInterchangeFunctionCallShaderNode* FunctionCallShaderNode = Cast<UInterchangeFunctionCallShaderNode>(ShaderNode);
	if (!FunctionCallShaderNode)
	{
		return nullptr;
	}

	UInterchangeMaterialFunctionCallExpressionFactoryNode* FunctionCallFactoryNode = NewObject<UInterchangeMaterialFunctionCallExpressionFactoryNode>(BaseNodeContainer);
	if (!FunctionCallFactoryNode)
	{
		return nullptr;
	}

	// Check whether the MaterialFunction attribute is valid
	FString MaterialFunctionAttribute;
	if (FunctionCallShaderNode->GetCustomMaterialFunction(MaterialFunctionAttribute))
	{
		if (!BaseNodeContainer->GetNode(MaterialFunctionAttribute))
		{
			if (!FPackageName::IsValidObjectPath(MaterialFunctionAttribute))
			{
				MaterialFunctionAttribute.Empty();
			}
		}
	}

	// Nothing to do if the MaterialFunction attribute is not valid
	if (MaterialFunctionAttribute.IsEmpty())
	{
		// TODO: Log a warning
		return nullptr;
	}

	BaseNodeContainer->SetupNode(FunctionCallFactoryNode, MaterialExpressionUid, ShaderNode->GetDisplayLabel(), EInterchangeNodeContainerType::FactoryData);
	
	if (BaseNodeContainer->GetNode(MaterialFunctionAttribute))
	{
		const FString MaterialFunctionFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(MaterialFunctionAttribute);
		FunctionCallFactoryNode->SetCustomMaterialFunctionDependency(MaterialFunctionFactoryNodeUid);

		UClass* CustomExpressionClass = UMaterialExpressionMaterialFunctionCall::StaticClass();
		FunctionCallFactoryNode->SetCustomExpressionClassName(CustomExpressionClass->GetName());
	}
	else if (FPackageName::IsValidObjectPath(MaterialFunctionAttribute))
	{
		FunctionCallFactoryNode->SetCustomMaterialFunctionDependency(MaterialFunctionAttribute);
		UpdateFunctionCallExpression(*FunctionCallFactoryNode, MaterialFunctionAttribute);
	}

	TArray<FString> Inputs;
	UInterchangeShaderPortsAPI::GatherInputs(ShaderNode, Inputs);

	for (const FString& InputName : Inputs)
	{
		TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> InputExpression =
			CreateMaterialExpressionForInput(MaterialFactoryNode, ShaderNode, InputName, MaterialExpressionUid);

		if (InputExpression.Get<0>())
		{
			UInterchangeShaderPortsAPI::ConnectOuputToInputByName(FunctionCallFactoryNode, InputName, InputExpression.Get<0>()->GetUniqueID(), InputExpression.Get<1>());
		}
	}

	return FunctionCallFactoryNode;
}

namespace UE::Interchange::Materials::HashUtils
{
	void FDuplicateMaterialHelper::ResetHashData()
	{
		AccumulatedHash = 0;
		MaterialHash = 0;
		bIsDuplicate = false;

		if (AttributeStorageNode)
		{
			AttributeStorageNode->MarkAsGarbage();
			AttributeStorageNode = nullptr;
		}
		AttributeStorageNode = NewObject<UInterchangeBaseNode>();

		LeafInputAttributeKeys.Empty();
		LeafInputShaderNodes.Empty();

#if UE_BUILD_DEBUG
		if (HashDebugData)
		{
			HashDebugData->Reset();
		}
#endif
	}

	void FDuplicateMaterialHelper::ComputMaterialHash(const UInterchangeShaderGraphNode* ShaderGraphNode)
	{
		MaterialHash = ComputeShaderGraphNodeHash(ShaderGraphNode);
		if (ParentMaterialFactoryMap.Contains(MaterialHash))
		{
			bIsDuplicate = true;
		}
	}

	int32 FDuplicateMaterialHelper::ComputeShaderGraphNodeHash(const UInterchangeShaderGraphNode* ShaderGraphNode)
	{
		using namespace UE::Interchange::Materials::HashUtils;

		/* Two Sided*/
		bool bTwoSided;
		ShaderGraphNode->GetCustomTwoSided(bTwoSided);
		int32 Hash = GetTypeHash(bTwoSided);
		ADD_LOG_MESSAGE(TEXT("TwoSided: {0}, Hash: {1}"), bTwoSided, Hash);

		/* Use Material Attributes*/
		bool bUseMaterialAttributes = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, UE::Interchange::Materials::Common::Parameters::BxDF);
		Hash = HashCombine(Hash, GetTypeHash(bUseMaterialAttributes));
		ADD_LOG_MESSAGE(TEXT("Use Material Attributes: {0}, Hash: {1}"), bUseMaterialAttributes, Hash);

		/* Blend Mode */
		TEnumAsByte<EBlendMode> BlendMode = GetShaderGraphNodeBlendMode(ShaderGraphNode);
		Hash = HashCombine(Hash, GetTypeHash(BlendMode));
		ADD_LOG_MESSAGE(TEXT("Blend Mode: {0}, Hash: {1}"), (uint8)BlendMode, Hash);

		/* Is Thin Surface */
		Hash = HashCombine(Hash, GetTypeHash(BlendMode == EBlendMode::BLEND_Translucent));
		ADD_LOG_MESSAGE(TEXT("Is Thin Surface: {0}, Hash: {1}"), (BlendMode == EBlendMode::BLEND_Translucent), Hash);

		/* EDatasmithShadingModel: uint8 */
		Hash = HashCombine(Hash, GetTypeHash(GetShaderGraphNodeShadingModel(ShaderGraphNode)));
		ADD_LOG_MESSAGE(TEXT("Shading Model: {0}, Hash: {1}"), GetShaderGraphNodeShadingModel(ShaderGraphNode), Hash);

		Hash = HashCombine(Hash, ComputeShaderNodeHash(ShaderGraphNode));
		ADD_LOG_MESSAGE(TEXT("ShaderHash: {0}"), Hash);
		return Hash;
	}

	uint8 FDuplicateMaterialHelper::GetShaderGraphNodeShadingModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const
	{
		if (GenericMaterialPipeline.HasThinTranslucency(ShaderGraphNode))
		{
			return 1;
		}
		else if (GenericMaterialPipeline.HasSubsurface(ShaderGraphNode))
		{
			return 2;
		}
		else if (GenericMaterialPipeline.HasClearCoat(ShaderGraphNode))
		{
			return 3;
		}
		else if (GenericMaterialPipeline.IsUnlitModel(ShaderGraphNode))
		{
			return 4;
		}

		return 0;
	}

	TEnumAsByte<EBlendMode> FDuplicateMaterialHelper::GetShaderGraphNodeBlendMode(const UInterchangeShaderGraphNode* ShaderGraphNode) const
	{
		using namespace UE::Interchange::Materials;

		TEnumAsByte<EBlendMode> BlendMode = EBlendMode::BLEND_Opaque;

		if (GenericMaterialPipeline.HasThinTranslucency(ShaderGraphNode))
		{
			BlendMode = EBlendMode::BLEND_Translucent;

		}
		else if (GenericMaterialPipeline.HasSubsurface(ShaderGraphNode))
		{
			BlendMode = EBlendMode::BLEND_Opaque;
		}
		else
		{
			const bool bHasOpacityInput = UInterchangeShaderPortsAPI::HasInput(ShaderGraphNode, UE::Interchange::Materials::Common::Parameters::Opacity);
			if (bHasOpacityInput && GenericMaterialPipeline.IsUnlitModel(ShaderGraphNode))
			{
				float OpacityClipValue;
				if (ShaderGraphNode->GetCustomOpacityMaskClipValue(OpacityClipValue))
				{
					BlendMode = EBlendMode::BLEND_Masked;
				}
				else
				{
					BlendMode = EBlendMode::BLEND_Translucent;
				}
			}
		}

		return BlendMode;
	}

	int32 FDuplicateMaterialHelper::ComputeShaderNodeHash(const UInterchangeShaderNode* ShaderNode)
	{
		int32 Hash = 0;

		FString ShaderTypeName;
		ShaderNode->GetCustomShaderType(ShaderTypeName);

		TArray<FString> Inputs;
		UInterchangeShaderPortsAPI::GatherInputs(ShaderNode, Inputs);

		if (!ShaderTypeName.IsEmpty())
		{
			using namespace UE::Interchange::Materials;

			PUSH_NODE_ADDRESS_WITHOUT_CHECKPOINT(ShaderTypeName)

			Hash = HashCombineCustom(Hash, GetTypeHash(ShaderTypeName));
			ADD_LOG_MESSAGE(TEXT("{0}, Accumulated Hash: {1}"), ShaderTypeName, AccumulatedHash)
			TArray<FInterchangeUserDefinedAttributeInfo> UserDefinedAttributes = UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttributeInfos(ShaderNode);
			if (UserDefinedAttributes.Num())
			{
				for (const auto& UserDefinedAttribute : UserDefinedAttributes)
				{
					Hash = HashCombineCustom(Hash, GetTypeHash(UserDefinedAttribute.Type));
					Hash = HashCombineCustom(Hash, GetTypeHash(UserDefinedAttribute.Name));

					ADD_LOG_MESSAGE(TEXT("UDA[Type: {0}, Name: {1}], Accumulated Hash: {2}"), (int32)UserDefinedAttribute.Type, UserDefinedAttribute.Name, AccumulatedHash)

					const FString UserDefinedAttributeType = AttributeTypeToString(UserDefinedAttribute.Type);

					if (UserDefinedAttribute.Type == UE::Interchange::EAttributeTypes::String)
					{
						const FString InputValueKey = (UInterchangeUserDefinedAttributesAPI::MakeUserDefinedPropertyValueKey(UserDefinedAttribute.Name, UserDefinedAttribute.RequiresDelegate)).Key;
						const FString OverrideParameterNameAttributeKey = FInterchangeMaterialInstanceOverridesAPI::MakeOverrideParameterName(ShaderNode->GetDisplayLabel());
						SetupOverridableTextureParameter(ShaderNode, InputValueKey, OverrideParameterNameAttributeKey);
					}
				}
			}
		}
		else
		{
			if (const UInterchangeFunctionCallShaderNode* FunctionCallNode = Cast<UInterchangeFunctionCallShaderNode>(ShaderNode))
			{
				FString MaterialFunction;
				if (FunctionCallNode->GetCustomMaterialFunction(MaterialFunction) && !MaterialFunction.IsEmpty())
				{
					Hash = HashCombineCustom(Hash, GetTypeHash(MaterialFunction));
					ADD_LOG_MESSAGE(TEXT("MF[{0}], Accumulate Hash: {1}"), MaterialFunction, AccumulatedHash);
#if UE_BUILD_DEBUG
					FString MaterialFunctionName;
					FString Discard;
					if (MaterialFunction.Split(TEXT("."), &Discard, &MaterialFunctionName))
					{
						PUSH_NODE_ADDRESS_WITHOUT_CHECKPOINT(FString::Printf(TEXT("MaterialFunction[%s]"), *MaterialFunctionName));
					}
					else
					{
						PUSH_NODE_ADDRESS_WITHOUT_CHECKPOINT(TEXT("MaterialFunction"));
					}
#endif
				}
			}
		}

		if (!Inputs.IsEmpty())
		{
			for (const FString& InputName : Inputs)
			{
				PUSH_NODE_ADDRESS(FString::Printf(TEXT("[%s]"), *InputName))
				ADD_NODE_ADDRESS_MESSAGE();
				Hash = HashCombineCustom(Hash, ComputeShaderInputHash(ShaderNode, InputName));
				POP_NODE_ADDRESSES()
			}
		}

		return Hash;
	}

	int32 FDuplicateMaterialHelper::ComputeShaderInputHash(const UInterchangeShaderNode* ShaderNode, const FString& InputName)
	{
		int32 Hash = 0;
		FString ConnectedShaderNodeUid;
		FString OutputName;
		if (UInterchangeShaderPortsAPI::GetInputConnection(ShaderNode, InputName, ConnectedShaderNodeUid, OutputName))
		{
			if (const UInterchangeShaderNode* ConnectedShaderNode = Cast<const UInterchangeShaderNode>(GenericMaterialPipeline.BaseNodeContainer->GetNode(ConnectedShaderNodeUid)))
			{
				Hash = HashCombineCustom(Hash, ComputeShaderNodeHash(ConnectedShaderNode));
			}

			if (!OutputName.IsEmpty())
			{
				Hash = HashCombineCustom(Hash, GetTypeHash(OutputName));
			}
		}
		else
		{
			const bool bIsAParameter = UInterchangeShaderPortsAPI::HasParameter(ShaderNode, FName(InputName));
			const UE::Interchange::EAttributeTypes InputType = UInterchangeShaderPortsAPI::GetInputType(ShaderNode, InputName, bIsAParameter);
			Hash = HashCombineCustom(Hash, GetTypeHash(InputType));
			ADD_LOG_MESSAGE(TEXT("{0}, Accumulated Hash: {1}"), AttributeTypeToString(InputType), AccumulatedHash);

			// Just setup all the Parameters as overridable parameters. Do not include the values in the Hash
			if (bIsAParameter)
			{
				using namespace UE::Interchange::Materials::HashUtils;
				const FString ParameterKey = UInterchangeShaderPortsAPI::MakeInputParameterKey(InputName);
				const FString OverridableParameterNameKey = FInterchangeMaterialInstanceOverridesAPI::MakeOverrideParameterName(ShaderNode->GetDisplayLabel());

				switch (InputType)
				{
				case UE::Interchange::EAttributeTypes::Float:
				{
					SetupOverridableScalarParameter(ShaderNode, ParameterKey, OverridableParameterNameKey);
					break;
				}
				case UE::Interchange::EAttributeTypes::LinearColor:
				{
					SetupOverridableVectorParameter(ShaderNode, ParameterKey, OverridableParameterNameKey);
					break;
				}
				case UE::Interchange::EAttributeTypes::Bool:
				{
					SetupOverridableStaticBoolParameter(ShaderNode, ParameterKey, OverridableParameterNameKey);
					break;
				}
				case UE::Interchange::EAttributeTypes::String:
				{
					SetupOverridableTextureParameter(ShaderNode, ParameterKey, OverridableParameterNameKey);
					break;
				}
				}
			}
			else
			{
				const FString InputValueKey = UInterchangeShaderPortsAPI::MakeInputValueKey(InputName);
				switch (InputType)
				{
				case UE::Interchange::EAttributeTypes::Float:
				{
					float InputValue;
					if (ShaderNode->GetFloatAttribute(InputValueKey, InputValue))
					{
						Hash = HashCombineCustom(Hash, GetTypeHash(InputValue));
						ADD_LOG_MESSAGE(TEXT("Unnamed Float({0}), Accumulated Hash: {1}"), FString::SanitizeFloat(InputValue), AccumulatedHash);
					}
					break;
				}
				case UE::Interchange::EAttributeTypes::LinearColor:
				{
					FLinearColor InputValue;
					if (ShaderNode->GetLinearColorAttribute(InputValueKey, InputValue))
					{
						Hash = HashCombineCustom(Hash, GetTypeHash(InputValue));
						ADD_LOG_MESSAGE(TEXT("Unnamed LinearColor({0}), Accumulated Hash: {1}"), InputValue.ToString(), AccumulatedHash)
					}
					break;
				}
				case UE::Interchange::EAttributeTypes::String:
				{
					FString InputValue;
					if (ShaderNode->GetStringAttribute(InputValueKey, InputValue))
					{
						Hash = HashCombineCustom(Hash, GetTypeHash(InputValue));
						ADD_LOG_MESSAGE(TEXT("Unnamed String({0}), Accumulated Hash: {1}"), InputValue, AccumulatedHash)
					}
					break;
				}
				}
			}
		}
		return Hash;
	}

	void FDuplicateMaterialHelper::SetupOverridableScalarParameter(const UInterchangeShaderNode* ShaderNode, const FString& ParameterKey, const FString& OverridableParameterNameKey)
	{
		float InputValue;
		if (ShaderNode->GetFloatAttribute(ParameterKey, InputValue))
		{
			const UE::Interchange::FAttributeKey AttributeKey(OverridableParameterNameKey);
			if (!AttributeStorageNode->HasAttribute(AttributeKey))
			{
				AttributeStorageNode->AddFloatAttribute(OverridableParameterNameKey, InputValue);
				LeafInputAttributeKeys.Add(AttributeKey);
				LeafInputShaderNodes.Emplace(ShaderNode);
				ADD_LOG_MESSAGE(TEXT("Scalar Parameter: {0}({1})"), ShaderNode->GetDisplayLabel(), FString::SanitizeFloat(InputValue));
			}
		}
	}

	void FDuplicateMaterialHelper::SetupOverridableVectorParameter(const UInterchangeShaderNode* ShaderNode, const FString& ParameterKey, const FString& OverridableParameterNameKey)
	{
		FLinearColor InputValue;
		if (ShaderNode->GetLinearColorAttribute(ParameterKey, InputValue))
		{
			const UE::Interchange::FAttributeKey AttributeKey(OverridableParameterNameKey);
			if (!AttributeStorageNode->HasAttribute(AttributeKey))
			{
				AttributeStorageNode->AddLinearColorAttribute(OverridableParameterNameKey, InputValue);
				LeafInputAttributeKeys.Add(AttributeKey);
				LeafInputShaderNodes.Emplace(ShaderNode);
				ADD_LOG_MESSAGE(TEXT("Vector Parameter: {0}({1})"), ShaderNode->GetDisplayLabel(), InputValue.ToString());
			}
		}
	}

	void FDuplicateMaterialHelper::SetupOverridableStaticBoolParameter(const UInterchangeShaderNode* ShaderNode, const FString& ParameterKey, const FString& OverridableParameterNameKey)
	{
		bool InputValue;
		if (ShaderNode->GetBooleanAttribute(ParameterKey, InputValue))
		{
			const UE::Interchange::FAttributeKey AttributeKey(OverridableParameterNameKey);
			if (!AttributeStorageNode->HasAttribute(AttributeKey))
			{
				AttributeStorageNode->AddBooleanAttribute(OverridableParameterNameKey, InputValue);
				LeafInputAttributeKeys.Add(AttributeKey);
				LeafInputShaderNodes.Emplace(ShaderNode);
				ADD_LOG_MESSAGE(TEXT("Bool Parameter: {0}({1})"), ShaderNode->GetDisplayLabel(), InputValue);
			}
		}
	}

	void FDuplicateMaterialHelper::SetupOverridableTextureParameter(const UInterchangeShaderNode* ShaderNode, const FString& InputKey, const FString& OverridableParameterNameKey)
	{
		FString InputValue;
		if (ShaderNode->GetStringAttribute(InputKey, InputValue))
		{
			const UE::Interchange::FAttributeKey AttributeKey(OverridableParameterNameKey);
			if (!AttributeStorageNode->HasAttribute(AttributeKey))
			{
				if (!FPackageName::IsValidObjectPath(InputValue))
				{
					// Material Factory expects Texture Factory Uid as opposed to Texture Uid
					const FString TextureFactoryUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(InputValue);
					AttributeStorageNode->AddStringAttribute(OverridableParameterNameKey, TextureFactoryUid);
				}
				else
				{
					AttributeStorageNode->AddStringAttribute(OverridableParameterNameKey, InputValue);
				}

				LeafInputAttributeKeys.Add(AttributeKey);
				LeafInputShaderNodes.Emplace(ShaderNode);

				ADD_LOG_MESSAGE(TEXT("Texture Parameter: {0}({1})"), ShaderNode->GetDisplayLabel(), InputValue);
			}
		}
	}

	int32 FDuplicateMaterialHelper::HashCombineCustom(int32 Hash, int32 CombineWith)
	{
		Hash = HashCombine(Hash, CombineWith);
		AccumulatedHash = HashCombine(AccumulatedHash, CombineWith);
		return Hash;
	}

	void FDuplicateMaterialHelper::CopyLeafInputsToFactoryNode(UInterchangeBaseMaterialFactoryNode* FactoryNode)
	{
		UInterchangeBaseNode::CopyStorageAttributes(AttributeStorageNode, FactoryNode, LeafInputAttributeKeys);
	}

	UInterchangeBaseMaterialFactoryNode* FDuplicateMaterialHelper::CreateFactoryForDuplicateMaterials(const UInterchangeShaderGraphNode* ShaderGraphNode, bool bImportUnusedMaterial, bool bCreateMaterialInstanceForParent)
	{
		UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode = nullptr;
		if (IsDuplicate())
		{
			MaterialFactoryNode = CreateMaterialInstanceFactoryFromReference(ShaderGraphNode);
		}
		else
		{
			MaterialFactoryNode = CreateMaterialFactory(ShaderGraphNode);
			MaterialFactoryNode->SetEnabled(bImportUnusedMaterial);

			if (bCreateMaterialInstanceForParent)
			{
				MaterialFactoryNode = CreateMaterialInstanceFactoryForParent(ShaderGraphNode);
			}
		}

		return MaterialFactoryNode;
	}

	UInterchangeBaseMaterialFactoryNode* FDuplicateMaterialHelper::CreateMaterialFactory(const UInterchangeShaderGraphNode* ShaderGraphNode)
	{
		UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode = GenericMaterialPipeline.CreateMaterialFactoryNode(ShaderGraphNode);
		ParentMaterialFactoryMap.Emplace(MaterialHash, MaterialFactoryNode);
		CopyLeafInputsToFactoryNode(MaterialFactoryNode);
		return MaterialFactoryNode;
	}

	UInterchangeMaterialInstanceFactoryNode* FDuplicateMaterialHelper::CreateMaterialInstanceFactoryFromReference(const UInterchangeShaderGraphNode* ShaderGraphNode)
	{
		const UInterchangeBaseMaterialFactoryNode* ParentMaterialFactory = nullptr;
		if (UInterchangeBaseMaterialFactoryNode** ParentMaterialFactoryEntry = ParentMaterialFactoryMap.Find(MaterialHash))
		{
			ParentMaterialFactory = *ParentMaterialFactoryEntry;
		}

		ensure(ParentMaterialFactory);

		if (!ParentMaterialFactory)
		{
			return nullptr;
		}

		UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode =
			Cast<UInterchangeMaterialInstanceFactoryNode>(GenericMaterialPipeline.CreateBaseMaterialFactoryNode(ShaderGraphNode, UInterchangeMaterialInstanceFactoryNode::StaticClass()));

		ensure(MaterialInstanceFactoryNode);

		if (ParentMaterialFactory)
		{
			MaterialInstanceFactoryNode->SetCustomParent(ParentMaterialFactory->GetUniqueID());
			MaterialInstanceFactoryNode->AddFactoryDependencyUid(ParentMaterialFactory->GetUniqueID());
		}

		for (const auto& LeafInputKey : LeafInputAttributeKeys)
		{
			UE::Interchange::EAttributeTypes AttributeType = AttributeStorageNode->GetAttributeType(LeafInputKey);
			switch (AttributeType)
			{
			case UE::Interchange::EAttributeTypes::Float:
			{
				float ParentValue;
				float CurrentValue;
				if (!AttributeStorageNode->GetFloatAttribute(LeafInputKey.Key, CurrentValue))
				{
					continue;
				}

				if (!ParentMaterialFactory->GetFloatAttribute(LeafInputKey.Key, ParentValue))
				{
					continue;
				}

				if (ParentValue != CurrentValue)
				{
					MaterialInstanceFactoryNode->AddFloatAttribute(LeafInputKey.Key, CurrentValue);
				}
			}
			break;
			case UE::Interchange::EAttributeTypes::LinearColor:
			{
				FLinearColor ParentValue;
				FLinearColor CurrentValue;
				if (!AttributeStorageNode->GetLinearColorAttribute(LeafInputKey.Key, CurrentValue))
				{
					continue;
				}

				if (!ParentMaterialFactory->GetLinearColorAttribute(LeafInputKey.Key, ParentValue))
				{
					continue;
				}

				if (ParentValue != CurrentValue)
				{
					MaterialInstanceFactoryNode->AddLinearColorAttribute(LeafInputKey.Key, CurrentValue);
				}
			}
			break;
			case UE::Interchange::EAttributeTypes::String:
			{
				FString ParentValue;
				FString CurrentValue;
				if (!AttributeStorageNode->GetStringAttribute(LeafInputKey.Key, CurrentValue))
				{
					continue;
				}

				if (!ParentMaterialFactory->GetStringAttribute(LeafInputKey.Key, ParentValue))
				{
					continue;
				}

				if (ParentValue != CurrentValue)
				{
					MaterialInstanceFactoryNode->AddStringAttribute(LeafInputKey.Key, CurrentValue);
				}
			}
			break;
			}
		}

		return MaterialInstanceFactoryNode;
	}

	UInterchangeMaterialInstanceFactoryNode* FDuplicateMaterialHelper::CreateMaterialInstanceFactoryForParent(const UInterchangeShaderGraphNode* ShaderGraphNode)
	{
		const UInterchangeBaseMaterialFactoryNode* ParentMaterialFactory = ParentMaterialFactoryMap[MaterialHash];

		UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode =
			Cast<UInterchangeMaterialInstanceFactoryNode>(GenericMaterialPipeline.CreateBaseMaterialFactoryNode(ShaderGraphNode, UInterchangeMaterialInstanceFactoryNode::StaticClass(), true));

		if (ParentMaterialFactory)
		{
			MaterialInstanceFactoryNode->SetCustomParent(ParentMaterialFactory->GetUniqueID());
			MaterialInstanceFactoryNode->AddFactoryDependencyUid(ParentMaterialFactory->GetUniqueID());
		}

		return MaterialInstanceFactoryNode;
	}
}

#undef ADD_LOG_MESSAGE
#undef ADD_NODE_ADDRESS_MESSAGE
#undef PUSH_NODE_ADDRESS
#undef PUSH_NODE_ADDRESS_WITHOUT_CHECKPOINT
#undef POP_NODE_ADDRESSES

#undef LOCTEXT_NAMESPACE