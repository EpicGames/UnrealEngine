// Copyright Epic Games, Inc. All Rights Reserved. 
#include "InterchangeGenericMeshPipeline.h"

#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "GeometryCache.h"
#include "InterchangeAnimationTrackSetNode.h"
#include "InterchangeGenericAssetsPipeline.h"
#include "InterchangeMeshNode.h"
#include "InterchangePipelineLog.h"
#include "InterchangePipelineMeshesUtilities.h"
#include "InterchangeSceneNode.h"
#include "InterchangeSkeletalMeshFactoryNode.h"
#include "InterchangeStaticMeshFactoryNode.h"
#include "InterchangeSourceData.h"
#include "Misc/Paths.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "InterchangeCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGenericMeshPipeline)

//////////////////////////////////////////// COMMON SKELETAL MESH AND ANIMATION PROPERTIES //////////////////////////////////////////////////

#if WITH_EDITOR
void UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties, bTryAutoSelectSkeleton))
	{
		if (!IsSkeletonValid() && bTryAutoSelectSkeleton)
		{

			if (UInterchangePipelineBase* MostOuterPipeline = GetMostPipelineOuter())
			{
				//Search if there is a valid skeleton in this path
				FARFilter Filter;
				Filter.PackagePaths.Add(*(MostOuterPipeline->ContentImportPath));
				Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());

				IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
				TArray<FAssetData> SkeletonAssets;
				AssetRegistry.GetAssets(Filter, SkeletonAssets);
				if (SkeletonAssets.Num() > 0)
				{
					Skeleton = CastChecked<USkeleton>(SkeletonAssets[0].GetAsset());
				}
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif


//////////////////////////////////////////// GENERIC MESH PIPELINE //////////////////////////////////////////////////

FString UInterchangeGenericMeshPipeline::GetPipelineCategory(UClass* AssetClass)
{
	if (ensure(AssetClass))
	{
		if (AssetClass->IsChildOf(UStaticMesh::StaticClass()))
		{
			return TEXT("Static Meshes");
		}
		else if (AssetClass->IsChildOf(USkeletalMesh::StaticClass()))
		{
			return TEXT("Skeletal Meshes");
		}
		else if (AssetClass->IsChildOf(UGeometryCache::StaticClass()))
		{
			return TEXT("Geometry Caches");
		}
	}
	return TEXT("Static Meshes");
}

void UInterchangeGenericMeshPipeline::AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams)
{
	Super::AdjustSettingsForContext(ContextParams);

#if WITH_EDITOR

	check(CommonSkeletalMeshesAndAnimationsProperties.IsValid());
	if (ContextParams.ContextType == EInterchangePipelineContext::None)
	{
		//We do not change the setting if we are in editing context
		return;
	}

	//Avoid creating physics asset when importing a LOD or the alternate skinning
	if (ContextParams.ContextType == EInterchangePipelineContext::AssetCustomLODImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomLODReimport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetAlternateSkinningImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetAlternateSkinningReimport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomMorphTargetImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomMorphTargetReImport)
	{
		bCreatePhysicsAsset = false;
		PhysicsAsset = nullptr;
		LodGroup = NAME_None;
		
		if (ContextParams.ContextType == EInterchangePipelineContext::AssetAlternateSkinningImport
			|| ContextParams.ContextType == EInterchangePipelineContext::AssetAlternateSkinningReimport)
		{
			CommonMeshesProperties->ForceAllMeshAsType = EInterchangeForceMeshType::IFMT_SkeletalMesh;
			CommonMeshesProperties->bBakeMeshes = true;
			CommonMeshesProperties->bBakePivotMeshes = false;
			CommonMeshesProperties->bImportLods = false;
			CommonMeshesProperties->bKeepSectionsSeparate = false;
			CommonMeshesProperties->VertexColorImportOption = EInterchangeVertexColorImportOption::IVCIO_Ignore;
			bImportSkeletalMeshes = true;
			bImportStaticMeshes = false;
			bBuildNanite = false;
			bImportMorphTargets = false;
			bImportVertexAttributes = false;
			bUpdateSkeletonReferencePose = false;
			SkeletalMeshImportContentType = EInterchangeSkeletalMeshContentType::All;
			CommonSkeletalMeshesAndAnimationsProperties->Skeleton.Reset();
			CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations = false;
			CommonMeshesProperties->bConvertStaticsWithAnimatedTransformToSkeletals = false;
		}
		else if (ContextParams.ContextType == EInterchangePipelineContext::AssetCustomMorphTargetImport
			|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomMorphTargetReImport)
		{
			//Custom morph target are imported has a combined static mesh
			CommonMeshesProperties->ForceAllMeshAsType = EInterchangeForceMeshType::IFMT_StaticMesh;
			CommonMeshesProperties->bBakeMeshes = true;
			CommonMeshesProperties->bBakePivotMeshes = false;
			CommonMeshesProperties->bImportLods = true;
			CommonMeshesProperties->bKeepSectionsSeparate = false;
			CommonMeshesProperties->VertexColorImportOption = EInterchangeVertexColorImportOption::IVCIO_Ignore;
			CommonMeshesProperties->bConvertStaticsWithAnimatedTransformToSkeletals = false;
			bImportSkeletalMeshes = false;
			bImportStaticMeshes = true;
			CombineStaticMeshesBehavior = EInterchangeCombineStaticMeshesBehavior::All;
			bBuildNanite = false;
			LodGroup = NAME_None;
			bCollision = false;
			Collision = EInterchangeMeshCollision::None;
			bImportCollisionAccordingToMeshName = false;
			bGenerateLightmapUVs = false;
			bGenerateDistanceFieldAsIfTwoSided = false;
			bSupportFaceRemap = false;
		}
		else if (ContextParams.ContextType == EInterchangePipelineContext::AssetCustomLODImport
			|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomLODReimport)
		{
			//We are importing custom LODs
			if (ContextParams.ImportObjectType)
			{
				//If we have a provided import object type we can make sure we import the correct type
				if (ContextParams.ImportObjectType->IsChildOf<UStaticMesh>())
				{
					bImportStaticMeshes = true;
					CommonMeshesProperties->ForceAllMeshAsType = EInterchangeForceMeshType::IFMT_StaticMesh;
					CommonMeshesProperties->bConvertStaticsWithAnimatedTransformToSkeletals = false;
					bImportSkeletalMeshes = false;
					CombineStaticMeshesBehavior = EInterchangeCombineStaticMeshesBehavior::All;
					LodGroup = NAME_None;
					bSupportFaceRemap = false;
					bCollision = false;
					Collision = EInterchangeMeshCollision::None;
					bImportCollisionAccordingToMeshName = false;
					bGenerateLightmapUVs = false;
					bGenerateDistanceFieldAsIfTwoSided = false;
				}
				else if (ContextParams.ImportObjectType->IsChildOf<USkeletalMesh>())
				{
					bImportSkeletalMeshes = true;
					CommonMeshesProperties->ForceAllMeshAsType = EInterchangeForceMeshType::IFMT_SkeletalMesh;
					CommonMeshesProperties->bConvertStaticsWithAnimatedTransformToSkeletals = false;
					bCreatePhysicsAsset = false;
					bImportStaticMeshes = false;
				}
				else
				{
					CommonMeshesProperties->bConvertStaticsWithAnimatedTransformToSkeletals = true;
				}
			}
		}
	}
	const FString CommonMeshesCategory = UInterchangeGenericCommonMeshesProperties::GetPipelineCategory(nullptr);
	const FString StaticMeshesCategory = UInterchangeGenericMeshPipeline::GetPipelineCategory(UStaticMesh::StaticClass());
	const FString SkeletalMeshesCategory = UInterchangeGenericMeshPipeline::GetPipelineCategory(USkeletalMesh::StaticClass());
	const FString GeometryCachesCategory = UInterchangeGenericMeshPipeline::GetPipelineCategory(UGeometryCache::StaticClass());
	const FString CommonSkeletalMeshesAndAnimationCategory = UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties::GetPipelineCategory(nullptr);

	TArray<FString> HideCategories;
	TArray<FString> HideSubCategories;
	if (ContextParams.ContextType == EInterchangePipelineContext::AssetReimport)
	{
		bool bContainStaticMesh = false;
		bool bContainSkeletalMesh = false;
		bool bContainGeometryCache = false;

		GetMeshesInformationFromTranslatedData(ContextParams.BaseNodeContainer, bContainStaticMesh, bContainSkeletalMesh, bContainGeometryCache);

		CommonMeshesProperties->bConvertStaticsWithAnimatedTransformToSkeletals = false;

		HideSubCategories.Add(TEXT("Build"));
		if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(ContextParams.ReimportAsset))
		{
			//Set the skeleton to the current asset skeleton
			CommonSkeletalMeshesAndAnimationsProperties->Skeleton = SkeletalMesh->GetSkeleton();
			PhysicsAsset = SkeletalMesh->GetPhysicsAsset();
			if (PhysicsAsset.IsValid())
			{
				bCreatePhysicsAsset = false;
			}
			bImportStaticMeshes = false;
			HideCategories.Add(StaticMeshesCategory);
			HideCategories.Add(GeometryCachesCategory);

			//Handle the case of reimporting a Skeletal Mesh using a model that contains only Static Meshes
			if (!bContainSkeletalMesh
				|| SkeletalMeshImportContentType == EInterchangeSkeletalMeshContentType::Geometry
				|| CommonMeshesProperties->ForceAllMeshAsType == EInterchangeForceMeshType::IFMT_StaticMesh)
			{
				CommonMeshesProperties->ForceAllMeshAsType = EInterchangeForceMeshType::IFMT_SkeletalMesh;
			}
		}
		else if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(ContextParams.ReimportAsset))
		{
			HideCategories.Add(SkeletalMeshesCategory);
			HideCategories.Add(GeometryCachesCategory);
			HideCategories.Add(CommonSkeletalMeshesAndAnimationCategory);
			bImportSkeletalMeshes = false;

			//Handle the case of reimporting a Static Mesh using a model that contains only Skeletal Meshes
			if (!bContainStaticMesh
				|| CommonMeshesProperties->ForceAllMeshAsType == EInterchangeForceMeshType::IFMT_SkeletalMesh)
			{
				CommonMeshesProperties->ForceAllMeshAsType = EInterchangeForceMeshType::IFMT_StaticMesh;
			}
		}
		else if (UAnimSequence* AnimSequence = Cast<UAnimSequence>(ContextParams.ReimportAsset))
		{
			HideCategories.Add(StaticMeshesCategory);
			HideCategories.Add(SkeletalMeshesCategory);
			HideCategories.Add(GeometryCachesCategory);
			HideCategories.Add(CommonMeshesCategory);
		}
		else if (UGeometryCache* GeometryCache = Cast<UGeometryCache>(ContextParams.ReimportAsset))
		{
			HideCategories.Add(StaticMeshesCategory);
			HideCategories.Add(SkeletalMeshesCategory);
			HideCategories.Add(CommonMeshesCategory);
			HideCategories.Add(CommonSkeletalMeshesAndAnimationCategory);
		}
		else if (ContextParams.ReimportAsset)
		{
			HideCategories.Add(StaticMeshesCategory);
			HideCategories.Add(SkeletalMeshesCategory);
			HideCategories.Add(GeometryCachesCategory);
			HideCategories.Add(CommonMeshesCategory);
			HideCategories.Add(CommonSkeletalMeshesAndAnimationCategory);
		}
	}

	if (UInterchangePipelineBase* OuterMostPipeline = GetMostPipelineOuter())
	{
		constexpr bool bDoTransientSubPipeline = true;
		if (UInterchangeGenericAssetsPipeline* ParentPipeline = Cast<UInterchangeGenericAssetsPipeline>(OuterMostPipeline))
		{
			if (ParentPipeline->ReimportStrategy == EReimportStrategyFlags::ApplyNoProperties)
			{
				for (const FString& HideSubCategoryName : HideSubCategories)
				{
					HidePropertiesOfSubCategory(OuterMostPipeline, this, HideSubCategoryName, bDoTransientSubPipeline);
				}
			}
		}

		for (const FString& HideCategoryName : HideCategories)
		{
			HidePropertiesOfCategory(OuterMostPipeline, this, HideCategoryName, bDoTransientSubPipeline);
		}
	}
#endif //WITH_EDITOR
}

#if WITH_EDITOR

bool UInterchangeGenericMeshPipeline::IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const
{
	static const TSet<FName> NeedRefreshProperties =
	{
		GET_MEMBER_NAME_CHECKED(UInterchangeGenericMeshPipeline, SkeletalMeshImportContentType),
		GET_MEMBER_NAME_CHECKED(UInterchangeGenericMeshPipeline, bImportStaticMeshes),
		GET_MEMBER_NAME_CHECKED(UInterchangeGenericMeshPipeline, bImportSkeletalMeshes),
		GET_MEMBER_NAME_CHECKED(UInterchangeGenericMeshPipeline, bCreatePhysicsAsset),
		GET_MEMBER_NAME_CHECKED(UInterchangeGenericMeshPipeline, CombineStaticMeshesBehavior),
		GET_MEMBER_NAME_CHECKED(UInterchangeGenericMeshPipeline, CombineSkeletalMeshesBehavior),
	};
	
	if (NeedRefreshProperties.Contains(PropertyChangedEvent.GetPropertyName()))
	{
		return true;
	}
	return Super::IsPropertyChangeNeedRefresh(PropertyChangedEvent);
}

#endif //WITH_EDITOR

void UInterchangeGenericMeshPipeline::PreDialogCleanup(const FName PipelineStackName)
{
	//Do not change the physics asset if this pipeline is a re-import or an override pipeline
	if (!IsFromReimportOrOverride())
	{
		PhysicsAsset = nullptr;
	}
}

#if WITH_EDITOR

bool UInterchangeGenericMeshPipeline::GetPropertyPossibleValues(const FName PropertyPath, TArray<FString>& PossibleValues)
{
	FString PropertyPathString = PropertyPath.ToString();
	int32 PropertyNameIndex = INDEX_NONE;
	if (PropertyPathString.FindLastChar(':', PropertyNameIndex))
	{
		PropertyPathString = PropertyPathString.RightChop(PropertyNameIndex+1);
	}
	if (PropertyPathString.Equals(GET_MEMBER_NAME_STRING_CHECKED(UInterchangeGenericMeshPipeline, LodGroup)))
	{
		TArray<FName> LODGroupNames;
		UStaticMesh::GetLODGroups(LODGroupNames);
		for (int32 GroupIndex = 0; GroupIndex < LODGroupNames.Num(); ++GroupIndex)
		{
			PossibleValues.Add(LODGroupNames[GroupIndex].GetPlainNameString());
		}
		return true;
	}
	//If we did not find any property call the super implementation
	return Super::GetPropertyPossibleValues(PropertyPath, PossibleValues);
}

void UInterchangeGenericMeshPipeline::GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const
{
	PipelineSupportAssetClasses.Add(UStaticMesh::StaticClass());
	PipelineSupportAssetClasses.Add(USkeletalMesh::StaticClass());
	if (bCreatePhysicsAsset && !PhysicsAsset.IsValid())
	{
		PipelineSupportAssetClasses.Add(UPhysicsAsset::StaticClass());
	}

	if (CommonSkeletalMeshesAndAnimationsProperties.IsValid() && !CommonSkeletalMeshesAndAnimationsProperties->IsSkeletonValid())
	{
		PipelineSupportAssetClasses.Add(USkeleton::StaticClass());
	}
}

#endif

void UInterchangeGenericMeshPipeline::GetMeshesInformationFromTranslatedData(const UInterchangeBaseNodeContainer* InBaseNodeContainer
	, bool& bContainStaticMesh
	, bool& bContainSkeletalMesh
	, bool& bContainGeometryCache) const
{
	bContainStaticMesh = false;
	bContainSkeletalMesh = false;
	bContainGeometryCache = false;

	//Its valid to call GetMeshesInformationFromTranslatedData with a null container
	if (!InBaseNodeContainer)
	{
		return;
	}
	
	InBaseNodeContainer->BreakableIterateNodesOfType<UInterchangeMeshNode>([&bContainStaticMesh, &bContainSkeletalMesh, &bContainGeometryCache](const FString& NodeUid, UInterchangeMeshNode* MeshNode)
		{
			if (!MeshNode->IsMorphTarget())
			{
				if (Cast<UInterchangeGeometryCacheNode>(MeshNode))
				{
					bContainGeometryCache = true;
				}
				else
				{
					if (MeshNode->IsSkinnedMesh())
					{
						bContainSkeletalMesh = true;
					}
					else
					{
						bContainStaticMesh = true;
					}
				}
			}

			return bContainStaticMesh && bContainSkeletalMesh && bContainGeometryCache; //If we found one of every sort we can stop the iteration.
		});
}

void UInterchangeGenericMeshPipeline::PostLoad()
{
	Super::PostLoad();

	if(!bImportCollision_DEPRECATED)
	{
		bCollision = bImportCollision_DEPRECATED;
	}
	
	if (bCombineStaticMeshes_DEPRECATED && CombineStaticMeshesBehavior == EInterchangeCombineStaticMeshesBehavior::DoNotCombine)
	{
		CombineStaticMeshesBehavior = EInterchangeCombineStaticMeshesBehavior::All;
	}
}

UInterchangePipelineMeshesUtilities* UInterchangeGenericMeshPipeline::CreateMeshPipelineUtilities(UInterchangeBaseNodeContainer* InBaseNodeContainer
	, const UInterchangeGenericMeshPipeline* Pipeline)
{
	FInterchangePipelineMeshesUtilitiesContext DataContext;

	//Set the context option to use when querying the pipeline mesh utilities
	DataContext.ForceMeshType = Pipeline->CommonMeshesProperties->ForceAllMeshAsType;
	DataContext.bConvertStaticsWithAnimatedTransformToSkeletals = Pipeline->CommonMeshesProperties->bConvertStaticsWithAnimatedTransformToSkeletals;
	DataContext.bConvertStaticsWithMorphTargetsToSkeletals = Pipeline->CommonMeshesProperties->bConvertStaticsWithMorphTargetsToSkeletals;
	DataContext.bConvertStaticsInBoneHierarchyToSkeletals = Pipeline->CommonMeshesProperties->bConvertStaticsInBoneHierarchyToSkeletals;
	DataContext.bQueryGeometryOnlyIfNoInstance = Pipeline->CommonMeshesProperties->bBakeMeshes || Pipeline->CommonMeshesProperties->bBakePivotMeshes;

	UInterchangePipelineMeshesUtilities* CreatedPipelineMeshesUtilities = UInterchangePipelineMeshesUtilities::CreateInterchangePipelineMeshesUtilities(InBaseNodeContainer);
	CreatedPipelineMeshesUtilities->SetContext(DataContext);
	return CreatedPipelineMeshesUtilities;
}

void UInterchangeGenericMeshPipeline::ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath)
{
	if (!InBaseNodeContainer)
	{
		UE_LOGF(LogInterchangePipeline, Warning, "UInterchangeGenericMeshPipeline: Cannot execute pre-import pipeline because InBaseNodeContrainer is null.");
		return;
	}

	if (!CommonSkeletalMeshesAndAnimationsProperties.IsValid())
	{
		return;
	}
	CommonSkeletalMeshesAndAnimationsProperties->SetResultsContainer(Results);

	BaseNodeContainer = InBaseNodeContainer;
	SourceDatas.Empty(InSourceDatas.Num());
	for (const UInterchangeSourceData* SourceData : InSourceDatas)
	{
		SourceDatas.Add(SourceData);
	}
	PipelineMeshesUtilities = CreateMeshPipelineUtilities(BaseNodeContainer, this);

	//Create skeletalmesh factory nodes
	ExecutePreImportPipelineSkeletalMesh();

	//Create staticmesh factory nodes
	ExecutePreImportPipelineStaticMesh();

	//Create geometry cache factory nodes
	ExecutePreImportPipelineGeometryCache();
}

void UInterchangeGenericMeshPipeline::ExecutePostFactoryPipeline(const UInterchangeBaseNodeContainer* InBaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport)
{
	ExecutePostFactoryPipelineStaticMesh(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
}

void UInterchangeGenericMeshPipeline::ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* InBaseNodeContainer, const FString& FactoryNodeKey, UObject* CreatedAsset, bool bIsAReimport)
{
	//We do not use the provided base container since ExecutePreImportPipeline cache it
	//We just make sure the same one is pass in parameter
	if (!InBaseNodeContainer || !ensure(BaseNodeContainer == InBaseNodeContainer) || !CreatedAsset)
	{
		return;
	}

	const UInterchangeFactoryBaseNode* FactoryNode = BaseNodeContainer->GetFactoryNode(FactoryNodeKey);
	if (!FactoryNode)
	{
		return;
	}

	//Set the last content type import
	LastSkeletalMeshImportContentType = SkeletalMeshImportContentType;

	PostImportSkeletalMesh(CreatedAsset, FactoryNode);

	//Finish the physics asset import, it need the skeletal mesh render data to create the physics collision geometry
	PostImportPhysicsAssetImport(CreatedAsset, FactoryNode);
}

void UInterchangeGenericMeshPipeline::SetReimportSourceIndex(UClass* ReimportObjectClass, const int32 SourceFileIndex)
{
	if (ReimportObjectClass == USkeletalMesh::StaticClass())
	{
		switch (SourceFileIndex)
		{
			case 0:
			{
				//Geo and skinning
				SkeletalMeshImportContentType = EInterchangeSkeletalMeshContentType::All;
			}
			break;

			case 1:
			{
				//Geo only
				SkeletalMeshImportContentType = EInterchangeSkeletalMeshContentType::Geometry;
			}
			break;

			case 2:
			{
				//Skinning only
				SkeletalMeshImportContentType = EInterchangeSkeletalMeshContentType::SkinningWeights;
			}
			break;

			default:
			{
				//In case SourceFileIndex == INDEX_NONE //No specified options, we use the last imported content type
				SkeletalMeshImportContentType = LastSkeletalMeshImportContentType;
			}
		};
	}
}

#if WITH_EDITOR
bool UInterchangeGenericMeshPipeline::DoClassesIncludeAllEditableStructProperties(const TArray<const UClass*>& Classes, const UStruct* Struct)
{
	check(IsInGameThread());

	bool bResult = true;
	const FName CategoryKey("Category");
	for (const FProperty* Property = Struct->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		//skip (transient, deprecated, const) property
		if (Property->HasAnyPropertyFlags(CPF_Transient | CPF_Deprecated | CPF_EditConst))
		{
			continue;
		}
		//skip property that is not editable
		if (!Property->HasAnyPropertyFlags(CPF_Edit))
		{
			continue;
		}
		const FObjectProperty* SubObject = CastField<FObjectProperty>(Property);
		if (SubObject)
		{
			continue;
		}
		else if (const FString* PropertyCategoryString = Property->FindMetaData(CategoryKey))
		{
			FName PropertyName = Property->GetFName();
			bool bFindProperty = false;
			for (const UClass* Class : Classes)
			{
				if (Class->FindPropertyByName(PropertyName) != nullptr)
				{
					bFindProperty = true;
					break;
				}
			}
			//Ensure to notify
			if (!bFindProperty)
			{
				UE_LOGF(LogInterchangePipeline, Log, "The Interchange mesh pipeline does not include build property %ls.", *PropertyName.ToString());
				bResult = false;
			}
		}
	}
	return bResult;
}
#endif

void UInterchangeGenericMeshPipeline::UpdateAssemblyPartDependencyTable(UInterchangeMeshFactoryNode* MeshFactoryNode, const TMap<int32, TArray<FString>>& NodeUidsPerLodIndex)
{
	// Early exit for the common case of no assemblies
	if (!PipelineMeshesUtilities || !PipelineMeshesUtilities->HasAssemblyMeshDependencies() || !ensure(MeshFactoryNode))
	{
		return;
	}

	for (const TPair<int32, TArray<FString>>& LodIndexAndNodeUids : NodeUidsPerLodIndex)
	{
		for (const FString& NodeUid : LodIndexAndNodeUids.Value)
		{
			const UInterchangeBaseNode* BaseNode = BaseNodeContainer->GetNode(NodeUid);
			if (const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(BaseNode))
			{
				FString MeshDependency;
				SceneNode->GetCustomAssetInstanceUid(MeshDependency);
				if (BaseNodeContainer->IsNodeUidValid(MeshDependency))
				{
					if (const UInterchangeMeshNode* MeshDependencyNode = Cast<UInterchangeMeshNode>(BaseNodeContainer->GetNode(MeshDependency)))
					{
						if (PipelineMeshesUtilities->IsAssemblyMeshUid(MeshDependencyNode->GetUniqueID()))
						{
							AssemblyPartMeshUidToFactoryNodeTable.Add(MeshDependencyNode->GetUniqueID(), MeshFactoryNode);
						}
					}
				}
			}
			else if (const UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(BaseNode))
			{
				if (PipelineMeshesUtilities->IsAssemblyMeshUid(MeshNode->GetUniqueID()))
				{
					AssemblyPartMeshUidToFactoryNodeTable.Add(MeshNode->GetUniqueID(), MeshFactoryNode);
				}
			}
		}
	}
}

void UInterchangeGenericMeshPipeline::CreateAssemblyPartDependencies()
{
	if (!PipelineMeshesUtilities)
	{
		return;
	}

	// Table to help identify invalid part dependencies. Nested assemblies - parts that themselves have assemblies that
	// use more parts and so are both a base and part mesh at the same time are not supported right now.
	TMap<FString, FString> GlobalPartUidToBaseUidTable;

	for (const TPair<FString, UInterchangeMeshFactoryNode*>& MeshUidAndFactoryNode : AssemblyPartMeshUidToFactoryNodeTable)
	{
		UInterchangeMeshFactoryNode* CurrentFactoryNode = MeshUidAndFactoryNode.Value;
		if (!CurrentFactoryNode)
		{
			continue;
		}

		const FString& CurrentMeshUid = MeshUidAndFactoryNode.Key;
		if (!PipelineMeshesUtilities->IsValidMeshGeometryUid(CurrentMeshUid))
		{
			UE_LOGF(LogInterchangePipeline, Warning
				, "Failed to find mesh geometry '%ls' while creating mesh-to-mesh factory node dependencies."
				, *CurrentMeshUid
			);
			continue;
		}

		const FInterchangeMeshGeometry& MeshGeometry = PipelineMeshesUtilities->GetMeshGeometryByUid(CurrentMeshUid);
		if (!MeshGeometry.BaseNode || !MeshGeometry.HasAssemblyPartDependencies())
		{
			continue;
		}
		
		TArray<FString> AssemblyPartDependencies;
		MeshGeometry.GetAssemblyPartDependencies(AssemblyPartDependencies);

		// Check that the current mesh hasn't already been used as a part itself
		if (const FString* BaseUid = GlobalPartUidToBaseUidTable.Find(CurrentMeshUid))
		{
			FString Message = FString::Printf(
				TEXT("Mesh '%s' has already been used as an assembly part by mesh '%s' and therefore must not ")
				TEXT("specify its own dependencies as well. The (%d) dependencies that will be ignored are : \n")
				, *CurrentMeshUid
				, **BaseUid
				, AssemblyPartDependencies.Num()
			);
			for (const FString& AssemblyPartUid : AssemblyPartDependencies)
			{
				Message.Append(FString::Printf(TEXT("... %s"), *AssemblyPartUid));
			}

			UE_LOGF(LogInterchangePipeline, Warning, "%ls", *Message);

			continue;
		}

		// Assuming we can find the part, set up the required factory node dependency so that 
		// the part mesh asset is imported and built before the base mesh asset.

		for (const FString& AssemblyPartUid : AssemblyPartDependencies)
		{
			if (AssemblyPartUid == CurrentMeshUid)
			{
				continue;
			}

			if (!AssemblyPartMeshUidToFactoryNodeTable.Contains(AssemblyPartUid))
			{
				UE_LOGF(LogInterchangePipeline, Warning
					, "Failed to create mesh-to-mesh factory node dependency '%ls' -> '%ls' - target not found."
					, *CurrentMeshUid
					, *AssemblyPartUid
				);
				continue;
			}
			
			const UInterchangeMeshFactoryNode* AssemblyPartFactoryNode = AssemblyPartMeshUidToFactoryNodeTable.FindChecked(AssemblyPartUid);
			if (AssemblyPartFactoryNode)
			{
				const FString AssemblyPartFactoryNodeUid = AssemblyPartFactoryNode->GetUniqueID();
				CurrentFactoryNode->SetAssemblyPartDependencyUid(AssemblyPartUid, AssemblyPartFactoryNodeUid);
				// Create a factory dependency so the part asset is imported before the mesh asset
				CurrentFactoryNode->AddFactoryDependencyUid(AssemblyPartFactoryNodeUid);

				GlobalPartUidToBaseUidTable.Add(AssemblyPartUid, CurrentMeshUid);
			}
		}
	}
}

void UInterchangeGenericMeshPipeline::AddToAssemblyPartMeshUidToFactoryNodeTable(const FString& MeshNodeUid, UInterchangeMeshFactoryNode* MeshFactoryNode)
{
	AssemblyPartMeshUidToFactoryNodeTable.Add(MeshNodeUid, MeshFactoryNode);
}