// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeGenericScenesPipeline.h"

#include "InterchangeActorFactoryNode.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeAssetUserData.h"
#include "InterchangeCameraFactoryNode.h"
#include "InterchangeCameraNode.h"
#include "InterchangeCommonPipelineDataFactoryNode.h"
#include "InterchangeDecalActorFactoryNode.h"
#include "InterchangeDecalNode.h"
#include "InterchangeEditorUtilitiesBase.h"
#include "InterchangeGroomComponentFactoryNode.h"
#include "InterchangeGroomComponentNode.h"
#include "InterchangeHeterogeneousVolumeActorFactoryNode.h"
#include "InterchangeImportReset.h"
#include "InterchangeLevelFactoryNode.h"
#include "InterchangeLevelInstanceActorFactoryNode.h"
#include "InterchangeLightFactoryNode.h"
#include "InterchangeLightNode.h"
#include "InterchangeManager.h"
#include "InterchangeMeshActorFactoryNode.h"
#include "InterchangeMeshBundleNode.h"
#include "InterchangeMeshLODContainerNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangePipelineHelper.h"
#include "InterchangePipelineLog.h"
#include "InterchangePipelineMeshesUtilities.h"
#include "InterchangeSceneComponentFactoryNodes.h"
#include "InterchangeSceneComponentNodes.h"
#include "InterchangeSceneImportAsset.h"
#include "InterchangeSceneImportAssetFactoryNode.h"
#include "InterchangeJointNode.h"
#include "InterchangeSceneVariantSetsFactoryNode.h"
#include "InterchangeSkeletalMeshFactoryNode.h"
#include "InterchangeSkeletalMeshLodDataNode.h"
#include "InterchangeSkeletonFactoryNode.h"
#include "InterchangeStaticMeshFactoryNode.h"
#include "InterchangeStaticMeshLodDataNode.h"
#include "InterchangeVariantSetNode.h"
#include "InterchangeVolumeNode.h"
#include "Nodes/InterchangeSourceNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"
#include "Scene/InterchangeActorHelper.h"
#include "Volume/InterchangeVolumeDefinitions.h"

#include "Animation/SkeletalMeshActor.h"
#include "CineCameraActor.h"
#include "Engine/Blueprint.h"
#include "Engine/DirectionalLight.h"
#include "Engine/Level.h"
#include "Engine/PointLight.h"
#include "Engine/RectLight.h"
#include "Engine/SkyLight.h"
#include "Engine/SpotLight.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "GeometryCacheActor.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "Misc/PackageName.h"
#include "PackedLevelActor/PackedLevelActor.h"
#include "PackedLevelActor/PackedLevelActorBuilder.h"
#include "PreviewScene.h"
#include "UObject/UObjectIterator.h"
#include "WorldPartition/WorldPartition.h"

#if WITH_EDITOR
#include "ObjectTools.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGenericScenesPipeline)

#define COPY_FROM_TRANSLATED_TO_FACTORY(TranslatedNode, FactoryNode, AttributeName, AttributeType) \
if(AttributeType AttributeName; TranslatedNode->GetCustom##AttributeName(AttributeName)) \
{ \
	FactoryNode->SetCustom##AttributeName(AttributeName); \
}

namespace UE::Interchange::Private
{
	//Either a (!Joint || RootJoint) can be a parent (only those get FactoryNodes) :
	FString FindFactoryParentSceneNodeUid(UInterchangeBaseNodeContainer* BaseNodeContainer, TSet<FString>& ActiveSkeletonUids, const UInterchangeSceneNode* SceneNode)
	{
		FString ParentUid = SceneNode->GetParentUid();

		if (!ActiveSkeletonUids.Contains(ParentUid))
		{
			return ParentUid;
		}

		const UInterchangeSceneNode* ParentSceneNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(ParentUid));
		if (!ParentSceneNode)
		{
			//Should never happen in theory since we always have the RootNode that cannot be a Joint and will never be in ActiveSkeletonUids
			return UInterchangeBaseNode::InvalidNodeUid();
		}

		return FindFactoryParentSceneNodeUid(BaseNodeContainer, ActiveSkeletonUids, ParentSceneNode);
	}

#if WITH_EDITOR
	void DeleteAssets(const TArray<UObject*>& AssetsToDelete)
	{
		if (AssetsToDelete.IsEmpty())
		{
			return;
		}

		TArray<UObject*> ObjectsToForceDelete;
		ObjectsToForceDelete.Reserve(AssetsToDelete.Num());

		for (UObject* Asset : AssetsToDelete)
		{
			if (Asset)
			{
				ObjectsToForceDelete.Add(Asset);
			}
		}

		if (ObjectsToForceDelete.IsEmpty())
		{
			return;
		}

		constexpr bool bShowConfirmation = true;
		constexpr ObjectTools::EAllowCancelDuringDelete AllowCancelDuringDelete = ObjectTools::EAllowCancelDuringDelete::CancelNotAllowed;
		ObjectTools::DeleteObjects(ObjectsToForceDelete, bShowConfirmation, AllowCancelDuringDelete);
	}
#else
	void DeleteAssets(const TArray<UObject*>& AssetsToDelete)
	{
		if (AssetsToDelete.IsEmpty())
		{
			return;
		}

		bool bForceGarbageCollection = false;
		for (UObject* Asset : AssetsToDelete)
		{
			if (Asset)
			{
				Asset->Rename(nullptr, GetTransientPackage(), REN_NonTransactional | REN_DontCreateRedirectors);

				if (Asset->IsRooted())
				{
					Asset->RemoveFromRoot();
				}

				Asset->ClearFlags(RF_Public | RF_Standalone);
				Asset->MarkAsGarbage();

				bForceGarbageCollection = true;
			}
		}

		if (bForceGarbageCollection)
		{
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}
	}
#endif // WITH_EDITOR

	void DeleteActors(const TArray<AActor*>& ActorsToDelete)
	{
		if (ActorsToDelete.IsEmpty())
		{
			return;
		}

		for (AActor* Actor : ActorsToDelete)
		{
			if (!Actor)
			{
				continue;
			}

			if (UWorld* OwningWorld = Actor->GetWorld())
			{
				OwningWorld->EditorDestroyActor(Actor, true);
				// Since deletion can be delayed, rename to avoid future name collision
				// Call UObject::Rename directly on actor to avoid AActor::Rename which unnecessarily sunregister and re-register components
				Actor->UObject::Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
			}
		}
	}
}

UInterchangeGenericLevelPipeline::~UInterchangeGenericLevelPipeline()
{
	if (PreviewScene)
	{
		delete PreviewScene;
		PreviewScene = nullptr;
	}
}

void UInterchangeGenericLevelPipeline::AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams)
{
	Super::AdjustSettingsForContext(ContextParams);

	bIsReimportContext |= ContextParams.ReimportAsset != nullptr;
}

void UInterchangeGenericLevelPipeline::ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath)
{
	if (!InBaseNodeContainer)
	{
		UE_LOGF(LogInterchangePipeline, Warning, "UInterchangeGenericLevelPipeline: Cannot execute pre-import pipeline because InBaseNodeContrainer is null");
		return;
	}

	//Clear any editor selection to avoid crash if selected actor change
	if (UInterchangeEditorUtilitiesBase* EditorUtilities = UInterchangeManager::GetInterchangeManager().GetEditorUtilities())
	{
		EditorUtilities->ClearEditorSelection();
	}

	BaseNodeContainer = InBaseNodeContainer;

	// Make sure all factory nodes created for assets have the chosen policy strategy
	InBaseNodeContainer->IterateNodesOfType<UInterchangeFactoryBaseNode>([this](const FString& NodeUid, UInterchangeFactoryBaseNode* FactoryNode)
		{
			if (this->bForceReimportDeletedAssets)
			{
				FactoryNode->SetForceNodeReimport();
			}
		});

	FTransform GlobalOffsetTransform = FTransform::Identity;
	if (UInterchangeCommonPipelineDataFactoryNode* CommonPipelineDataFactoryNode = UInterchangeCommonPipelineDataFactoryNode::GetUniqueInstance(BaseNodeContainer))
	{
		CommonPipelineDataFactoryNode->GetCustomGlobalOffsetTransform(GlobalOffsetTransform);
	}

	TArray<UInterchangeSceneNode*> SceneNodes;
	TArray<UInterchangeSceneComponentNode*> SceneComponentNodes;

	// Find all translated node we need for this pipeline
	PopulateSceneNodesPreImport(SceneNodes, SceneComponentNodes);

	for (UInterchangeSceneComponentNode* SceneComponentNode : SceneComponentNodes)
	{
		CreateComponentFactoryNode(SceneComponentNode);
	}

#if WITH_EDITORONLY_DATA
	const FString FilePath = FPaths::ConvertRelativePathToFull(InSourceDatas.Num() ? InSourceDatas[0]->GetFilename() : TEXT("Empty"));

	UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(InBaseNodeContainer);

	if (SceneHierarchyType != EInterchangeSceneHierarchyType::CreateLevelActors)
	{
		ensure(!LevelFactoryNode);
		const bool bCreatePackedActor = SceneHierarchyType == EInterchangeSceneHierarchyType::CreatePackedActor;
		const FString DisplayLabel = ("Level_") + FPaths::GetBaseFilename(FilePath);
		const FString NodeUid = TEXT("Level_") + FilePath;
		ensure(!BaseNodeContainer->IsNodeUidValid(NodeUid));

		LevelFactoryNode = NewObject<UInterchangeLevelFactoryNode>(BaseNodeContainer, NAME_None);
		BaseNodeContainer->SetupNode(LevelFactoryNode, NodeUid, DisplayLabel, EInterchangeNodeContainerType::FactoryData);
		
		LevelFactoryNode->SetCustomCreateWorldPartitionLevel(false);
		LevelFactoryNode->SetCustomShouldCreateLevel(!bIsReimportContext);
		UE::Interchange::PipelineHelper::FillSubPathFromSourceNode(LevelFactoryNode, SourceNode);
			
		if (bIsReimportContext)
		{
			//When we re-import we want the FinalizeObject_GameThread to be call on the level factory for this level node
			LevelFactoryNode->SetForceNodeReimport();
		}

		//Create a level instance or packed actor
		{
			LevelInstanceActorFactoryNode = NewObject<UInterchangeLevelInstanceActorFactoryNode>(BaseNodeContainer, NAME_None);

			if (ensure(LevelInstanceActorFactoryNode))
			{
				const UClass* LevelInstanceClass = bCreatePackedActor ? APackedLevelActor::StaticClass() : ALevelInstance::StaticClass();
				LevelInstanceActorFactoryNode->SetCustomActorClassName(LevelInstanceClass->GetPathName());

				const FString ActorNodeUid = TEXT("LevelInstance_") + FilePath;
				BaseNodeContainer->SetupNode(LevelInstanceActorFactoryNode, ActorNodeUid, DisplayLabel, EInterchangeNodeContainerType::FactoryData);
				
				//Set the level this actor is referring
				LevelInstanceActorFactoryNode->SetCustomLevelReference(LevelFactoryNode->GetUniqueID());
				//We ensure the actor will be created after the parent and reference world are create or ready
				LevelInstanceActorFactoryNode->AddFactoryDependencyUid(LevelFactoryNode->GetUniqueID());
			}
		}
	}

	// Add the SceneImportData factory node
	{
		ensure(!SceneImportFactoryNode);
		const FString DisplayLabel = TEXT("SceneImport_") + FPaths::GetBaseFilename(FilePath);
		const FString NodeUid = TEXT("SceneImport_") + FilePath;
		const FString FactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(NodeUid);
		ensure(!BaseNodeContainer->IsNodeUidValid(FactoryNodeUid));

		TArray<FString> FactoryNodeUIDs;
		BaseNodeContainer->GetNodeUIDsOfType<UInterchangeFactoryBaseNode>(FactoryNodeUIDs);

		SceneImportFactoryNode = NewObject<UInterchangeSceneImportAssetFactoryNode>(BaseNodeContainer, NAME_None);
		BaseNodeContainer->SetupNode(SceneImportFactoryNode, FactoryNodeUid, DisplayLabel, EInterchangeNodeContainerType::FactoryData);

		UE::Interchange::PipelineHelper::FillSubPathFromSourceNode(SceneImportFactoryNode, SourceNode);

		// Add dependency to all the factory nodes created so far
		for (const FString& FactoryNodeUID : FactoryNodeUIDs)
		{
			this->SceneImportFactoryNode->AddFactoryDependencyUid(FactoryNodeUID);
		}
	}
#endif

	/* Find all scene node that are active joint. Non active joint should be convert to actor if they are in a static mesh hierarchy */
	CacheActiveJointUids();

	for (const UInterchangeSceneNode* SceneNode : SceneNodes)
	{
		if (SceneNode)
		{
			TArray<FString> SpecializeTypes;
			SceneNode->GetSpecializedTypes(SpecializeTypes);

			if (SpecializeTypes.Num() > 0 || SceneNode->IsA(UInterchangeJointNode::StaticClass()))
			{
				bool bSkipNode = true;
				if (SceneNode->IsA(UInterchangeJointNode::StaticClass()))
				{
					if (!Cached_ActiveJointUids.Contains(SceneNode->GetUniqueID()))
					{
						bSkipNode = false;
					}
					else
					{
						//check if its the root joint (we want to create an actor for the root joint)
						FString CurrentNodesParentUid = SceneNode->GetParentUid();
						const UInterchangeBaseNode* ParentNode = BaseNodeContainer->GetNode(CurrentNodesParentUid);
						if (const UInterchangeSceneNode* ParentSceneNode = Cast<UInterchangeSceneNode>(ParentNode))
						{
							if (!ParentSceneNode->IsA(UInterchangeJointNode::StaticClass()))
							{
								bSkipNode = false;
							}
						}
					}
				}

				if (bSkipNode)
				{
					//Skip any scene node that have specialized types but not the "Transform" type.
					continue;
				}
			}

			ExecuteSceneNodePreImport(GlobalOffsetTransform, SceneNode);
		}
	}

	//Find all translated scene variant sets
	TArray<UInterchangeSceneVariantSetsNode*> SceneVariantSetNodes;

	InBaseNodeContainer->IterateNodesOfType<UInterchangeSceneVariantSetsNode>([&SceneVariantSetNodes](const FString& NodeUid, UInterchangeSceneVariantSetsNode* Node)
		{
			SceneVariantSetNodes.Add(Node);
		});

	for (const UInterchangeSceneVariantSetsNode* SceneVariantSetNode : SceneVariantSetNodes)
	{
		if (SceneVariantSetNode)
		{
			ExecuteSceneVariantSetNodePreImport(*SceneVariantSetNode);
		}
	}
}

void UInterchangeGenericLevelPipeline::PopulateSceneNodesPreImport(TArray<UInterchangeSceneNode*>& SceneNodes,
	TArray<UInterchangeSceneComponentNode*>& SceneComponentNodes) const
{
	BaseNodeContainer->IterateNodes([this, &SceneNodes, &SceneComponentNodes](const FString& NodeUid, UInterchangeBaseNode* Node)
	{
		switch (Node->GetNodeContainerType())
		{
			case EInterchangeNodeContainerType::TranslatedScene:
			{
				if (UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(Node))
				{
					SceneNodes.Add(SceneNode);
				}
				else if (UInterchangeSceneComponentNode* SceneComponentNode = Cast<UInterchangeSceneComponentNode>(Node))
				{
					SceneComponentNodes.Add(SceneComponentNode);
				}
			}
			break;
		}
	});
}

void UInterchangeGenericLevelPipeline::SetUpComponents(UInterchangeActorFactoryNode* ActorFactoryNode, const UInterchangeSceneNode* SceneNode) const
{
	TArray<FString> SceneComponentUids;
	SceneNode->GetComponentUids(SceneComponentUids);
	SetUpComponentDependencies(ActorFactoryNode, SceneComponentUids);
	for (const FString& SceneComponentUid : SceneComponentUids)
	{
		FString SceneComponentFactoryUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(SceneComponentUid);
		ActorFactoryNode->AddComponentUid(SceneComponentFactoryUid);
	}
}

void UInterchangeGenericLevelPipeline::SetUpComponentDependencies(UInterchangeActorFactoryNode* ActorFactoryNode, const TArray<FString>& SceneComponentUids) const
{
	for (const FString& SceneComponentUid : SceneComponentUids)
	{
		if (const UInterchangeSceneComponentNode* SceneComponentNode = Cast<const UInterchangeSceneComponentNode>(BaseNodeContainer->GetNode(SceneComponentUid)))
		{
			if (const UInterchangeInstancedStaticMeshComponentNode* ISMComponentNode = Cast<const UInterchangeInstancedStaticMeshComponentNode>(SceneComponentNode))
			{
				FString MeshNodeUid;
				if (ISMComponentNode->GetCustomInstancedAssetUid(MeshNodeUid))
				{
					ActorFactoryNode->AddFactoryDependencyUid(UInterchangeFactoryBaseNode::BuildFactoryNodeUid(MeshNodeUid));
				}
			}
			else if (const UInterchangeGroomComponentNode* GroomComponentNode = Cast<const UInterchangeGroomComponentNode>(SceneComponentNode))
			{
				FString AssetUid;
				if (GroomComponentNode->GetGroomDependencyUid(AssetUid))
				{
					ActorFactoryNode->AddFactoryDependencyUid(AssetUid);
				}
				if (GroomComponentNode->GetGroomBindingDependencyUid(AssetUid))
				{
					ActorFactoryNode->AddFactoryDependencyUid(AssetUid);
				}
			}
			TArray<FString> SubSceneComponentUids;
			SceneComponentNode->GetComponentUids(SubSceneComponentUids);
			SetUpComponentDependencies(ActorFactoryNode, SubSceneComponentUids);
		}
	}
}

void UInterchangeGenericLevelPipeline::ExecuteSceneNodePreImport(const FTransform& GlobalOffsetTransform, const UInterchangeSceneNode* SceneNode)
{
	using namespace UE::Interchange;

	if (!BaseNodeContainer || !SceneNode)
	{
		return;
	}

	bool IsPartOfNamedLODGroup = false;
	if (SceneNode->GetCustomDoNotInstantiateInLevel(IsPartOfNamedLODGroup) && IsPartOfNamedLODGroup)
	{
		//Note: this scenarios impose a possible issue where children of these skipped Actors we would still want to create.
		return;
	}

	FString SkeletonFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(SceneNode->GetUniqueID());
	const UInterchangeSkeletonFactoryNode* SkeletonFactoryNode = Cast<UInterchangeSkeletonFactoryNode>(BaseNodeContainer->GetFactoryNode(SkeletonFactoryNodeUid));

	bool bRootJointNode = false;

	//If this SceneNode has a SkeletonFactoryNode associated with it, then we need to create a Skeletal actors
	if (SkeletonFactoryNode)
	{
		//Create one Skeletal actor for each SkeletalMeshFactoryNode referenced by this SkeletonFactoryNode.
		const int32 Count = SkeletonFactoryNode->GetCustomSkeletalMeshFactoryNodeCount();
		for (int32 Index = 0; Index < Count; ++Index)
		{
			FString SkeletalMeshFactoryNodeUid;
			SkeletonFactoryNode->GetCustomSkeletalMeshFactoryNodeUidAtIndex(Index, SkeletalMeshFactoryNodeUid);

			CreateActorFactoryNode(GlobalOffsetTransform, SceneNode, SkeletalMeshFactoryNodeUid);
			bRootJointNode = true;
		}
	}

	if (!bRootJointNode)
	{
		CreateActorFactoryNode(GlobalOffsetTransform, SceneNode, {});
	}
}

void UInterchangeGenericLevelPipeline::CreateActorFactoryNode(const FTransform& GlobalOffsetTransform, const UInterchangeSceneNode* SceneNode, const FString& SkeletalMeshFactoryNodeUid)
{	
	const UInterchangeBaseNode* TranslatedAssetNode = nullptr;
	bool bRootJointNode = false;

	FString ActorName = SceneNode->GetDisplayLabel();

	if (!SkeletalMeshFactoryNodeUid.IsEmpty())
	{
		if (const UInterchangeFactoryBaseNode* SkeletalMeshFactoryNode = BaseNodeContainer->GetFactoryNode(SkeletalMeshFactoryNodeUid))
		{
			ActorName = SkeletalMeshFactoryNode->GetDisplayLabel();

			TArray<FString> NodeUids;
			SkeletalMeshFactoryNode->GetTargetNodeUids(NodeUids);

			if (NodeUids.Num() > 0)
			{
				TranslatedAssetNode = BaseNodeContainer->GetNode(NodeUids[0]);
				bRootJointNode = true;
			}
		}
	}

	if (!bRootJointNode)
	{
		FString AssetInstanceUid;
		if (SceneNode->GetCustomAssetInstanceUid(AssetInstanceUid))
		{
			TranslatedAssetNode = BaseNodeContainer->GetNode(AssetInstanceUid);
		}

		if (const UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(TranslatedAssetNode))
		{
			bool bIsSkinnedMesh = MeshNode->IsSkinnedMesh();
			
			// Skinned mesh are added when the bRootJointNode is true.
			// In this case we don't want to add an empty static mesh actor.
			if (bIsSkinnedMesh)
			{
				return;
			}
			else if (!Cast<UInterchangeGeometryCacheNode>(TranslatedAssetNode))
			{
				// Don't generate actor factory nodes for static mesh scene nodes that haven't produced any static mesh factory nodes, unless it is a rigid mesh,
				// as we're not going to have any static mesh to assign to the component anyway!
				//
				// We currently use this to prevent the generation of static mesh components for collision mesh nodes, when importing into level.
				// In that case we won't produce a factory node for the collider mesh node, which we can detect here and early out
				const FString MeshFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(MeshNode->GetUniqueID());
				const UInterchangeStaticMeshFactoryNode* MeshFactoryNode = Cast<UInterchangeStaticMeshFactoryNode>(BaseNodeContainer->GetFactoryNode(MeshFactoryNodeUid));
				if (!MeshFactoryNode)
				{
					return;
				}
			}
		}

		// Don't spawn actor factory nodes if our target asset hasn't produced a factory node itself for us to assign (as we likely don't want
		// e.g. a static mesh component if we don't have a static mesh for it). Currently this is mainly used to prevent us from spawning actor
		// factory nodes for mesh nodes that were ultimately consumed as collision meshes for other static meshes. We filter for meshes only
		// to avoid false positives like for cameras/lights, which have translated asset nodes, but no asset factory nodes.
		//
		// Note that without this snippet it may seem like these actors don't get assigned any StaticMesh anyway, so is this really that bad an issue?
		// Well, that is only due to the fact that UInterchangeGenericLevelPipeline::SetUpFactoryNode() can't specify the factory dependency correctly
		// in the collision mesh case, as the produced static mesh factory node won't have the UID based on the collision mesh translated node, but
		// instead the render mesh translated node. So randomly depending on your scene/setup, if the spawned actor gets finalized after the static mesh
		// you'd get the render mesh assigned to the collision mesh spawned actor (really wrong), or if it gets finalized before the static mesh you'd
		// get the actor without anything assigned (slightly wrong / useless)
		if (TranslatedAssetNode &&
			(TranslatedAssetNode->IsA<const UInterchangeMeshNode>() 
				|| TranslatedAssetNode->IsA<const UInterchangeMeshBundleNode>()
				|| TranslatedAssetNode->IsA<const UInterchangeMeshLODContainerNode>()))
		{
			//Note: these scenarios impose a possible issue where children of these skipped Actors we would still want to create.
			const bool bIsGeometryCache = TranslatedAssetNode->IsA<const UInterchangeGeometryCacheNode>();

			FString AssetFactoryNodeUid = bIsGeometryCache ? TEXT("Factory_GeometryCache_") + TranslatedAssetNode->GetUniqueID() :
				UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TranslatedAssetNode->GetUniqueID());
			UInterchangeFactoryBaseNode* AssetFactoryNode = BaseNodeContainer->GetFactoryNode(AssetFactoryNodeUid);
			if (!AssetFactoryNode)
			{
				return;
			}
		}
	}

	UInterchangeActorFactoryNode* ActorFactoryNode = CreateActorFactoryNode(TranslatedAssetNode);

	if (!ensure(ActorFactoryNode))
	{
		return;
	}

	UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(SceneNode, ActorFactoryNode, false);

	TArray<FString> LayerNames;
	SceneNode->GetLayerNames(LayerNames);
	ActorFactoryNode->AddLayerNames(LayerNames);

	TArray<FString> Tags;
	SceneNode->GetTags(Tags);
	ActorFactoryNode->AddTags(Tags);

	FString NodeUid;

	if (bRootJointNode)
	{
		NodeUid = SkeletalMeshFactoryNodeUid + TEXT("_SkeletonNode");
	}
	else
	{
		NodeUid = SceneNode->GetUniqueID();
	}

	const FString ActorFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(NodeUid);
	FString ParentFactoryNodeUid;
	if (!SceneNode->GetParentUid().IsEmpty())
	{
		/* Find all scene node that are active joint. Non active joint should be convert to actor if they are in a static mesh hierarchy */
		FString ParentNodeUid = UE::Interchange::Private::FindFactoryParentSceneNodeUid(BaseNodeContainer, Cached_ActiveJointUids, SceneNode);
		if (ParentNodeUid != UInterchangeBaseNode::InvalidNodeUid())
		{
			ParentFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(ParentNodeUid);
		}
	}
	BaseNodeContainer->SetupNode(ActorFactoryNode, ActorFactoryNodeUid, ActorName, EInterchangeNodeContainerType::FactoryData, ParentFactoryNodeUid);
#if WITH_EDITORONLY_DATA
	//The level must be create before any actor asset since all actors will be created in the specified level
	if (LevelFactoryNode)
	{
		ActorFactoryNode->AddFactoryDependencyUid(LevelFactoryNode->GetUniqueID());
		ActorFactoryNode->SetCustomLevelUid(LevelFactoryNode->GetUniqueID());
		LevelFactoryNode->AddCustomActorFactoryNodeUid(ActorFactoryNodeUid);
		//The level instance actor must be create after the actor on the reference level are created
		if (LevelInstanceActorFactoryNode)
		{
			LevelInstanceActorFactoryNode->AddFactoryDependencyUid(ActorFactoryNode->GetUniqueID());
		}
	}
#endif
	// The translator is responsible to provide a unique name
	ActorFactoryNode->SetAssetName(SceneNode->GetAssetName());

	if (ParentFactoryNodeUid.Len())
	{
		ActorFactoryNode->AddFactoryDependencyUid(ParentFactoryNodeUid);
	}

	if (bRootJointNode)
	{
		ActorFactoryNode->AddTargetNodeUid(SkeletalMeshFactoryNodeUid);
	}
	else
	{
		ActorFactoryNode->AddTargetNodeUid(SceneNode->GetUniqueID());
		SceneNode->AddTargetNodeUid(ActorFactoryNode->GetUniqueID());
	}

	//TODO move this code to the factory, a stack over pipeline can change the global offset transform which will affect this value.
	FTransform LocalTransform;
	if (SceneNode->GetCustomLocalTransform(LocalTransform))
	{
		if (bRootJointNode)
		{
			//LocalTransform of RootjointNode is already baked into the Skeletal and animation.
			LocalTransform = FTransform::Identity;
		}

		if (SceneNode->GetParentUid().IsEmpty())
		{
			LocalTransform = LocalTransform * GlobalOffsetTransform;
		}

		ActorFactoryNode->SetCustomLocalTransform(LocalTransform);
	}

	bool bCustomVisibility;
	// Copy component's visibility from SceneNode or set component visible by default
	if (SceneNode->GetCustomComponentVisibility(bCustomVisibility))
	{
		ActorFactoryNode->SetCustomComponentVisibility(bCustomVisibility);
	}
	else
	{
		ActorFactoryNode->SetCustomComponentVisibility(true);
	}
	
	// Copy actor's visibility from SceneNode or set actor visible by default
	if (SceneNode->GetCustomActorVisibility(bCustomVisibility))
	{
		ActorFactoryNode->SetCustomActorVisibility(bCustomVisibility);
	}
	else
	{
		ActorFactoryNode->SetCustomActorVisibility(true);
	}

	ActorFactoryNode->SetCustomMobility(EComponentMobility::Static);

	if (TranslatedAssetNode)
	{
		SetUpFactoryNode(ActorFactoryNode, SceneNode, TranslatedAssetNode, bRootJointNode);
	}

	SetUpComponents(ActorFactoryNode, SceneNode);
	
	if (CacheContextParam.ContextType == EInterchangePipelineContext::SceneReimport)
	{
		ActorFactoryNode->SetReimportStrategyFlags(ReimportPropertyStrategy);
		if (bForceReimportDeletedActors)
		{
			ActorFactoryNode->SetForceNodeReimport();
		}
	}

#if WITH_EDITORONLY_DATA
	// Add dependency to newly created factory node
	SceneImportFactoryNode->AddFactoryDependencyUid(ActorFactoryNode->GetUniqueID());
#endif
}

UInterchangeActorFactoryNode* UInterchangeGenericLevelPipeline::CreateActorFactoryNode(const UInterchangeBaseNode* TranslatedAssetNode) const
{
	if (!ensure(BaseNodeContainer))
	{
		return nullptr;
	}

	if(TranslatedAssetNode)
	{
		if(TranslatedAssetNode->IsA<UInterchangePhysicalCameraNode>())
		{
			return NewObject<UInterchangePhysicalCameraFactoryNode>(BaseNodeContainer, NAME_None);
		}
		if (TranslatedAssetNode->IsA<UInterchangeStandardCameraNode>())
		{
			if (bUsePhysicalInsteadOfStandardPerspectiveCamera)
			{
				//in case it has perspective projection we want to use PhysicalCamera (CineCamera) instead:
				if (const UInterchangeStandardCameraNode* CameraNode = Cast<UInterchangeStandardCameraNode>(TranslatedAssetNode))
				{
					EInterchangeCameraProjectionType ProjectionType = EInterchangeCameraProjectionType::Perspective;
					if (CameraNode->GetCustomProjectionMode(ProjectionType) && ProjectionType == EInterchangeCameraProjectionType::Perspective)
					{
						return NewObject<UInterchangePhysicalCameraFactoryNode>(BaseNodeContainer, NAME_None);
					}
				}
			}

			return NewObject<UInterchangeStandardCameraFactoryNode>(BaseNodeContainer, NAME_None);
		}
		else if(TranslatedAssetNode->IsA<UInterchangeMeshNode>() || TranslatedAssetNode->IsA<UInterchangeMeshBundleNode>() || TranslatedAssetNode->IsA<UInterchangeMeshLODContainerNode>())
		{
			return NewObject<UInterchangeMeshActorFactoryNode>(BaseNodeContainer, NAME_None);
		}
		else if(TranslatedAssetNode->IsA<UInterchangeSpotLightNode>())
		{
			return NewObject<UInterchangeSpotLightFactoryNode>(BaseNodeContainer, NAME_None);
		}
		else if(TranslatedAssetNode->IsA<UInterchangePointLightNode>())
		{
			return NewObject<UInterchangePointLightFactoryNode>(BaseNodeContainer, NAME_None);
		}
		else if(TranslatedAssetNode->IsA<UInterchangeRectLightNode>())
		{
			return NewObject<UInterchangeRectLightFactoryNode>(BaseNodeContainer, NAME_None);
		}
		else if(TranslatedAssetNode->IsA<UInterchangeDirectionalLightNode>())
		{
			return NewObject<UInterchangeDirectionalLightFactoryNode>(BaseNodeContainer, NAME_None);
		}
		else if(TranslatedAssetNode->IsA<UInterchangeSkyLightNode>())
		{
			return NewObject<UInterchangeSkyLightFactoryNode>(BaseNodeContainer, NAME_None);
		}
		else if (TranslatedAssetNode && TranslatedAssetNode->IsA<UInterchangeDecalNode>())
		{
			return NewObject<UInterchangeDecalActorFactoryNode>(BaseNodeContainer, NAME_None);
		}
		else if (TranslatedAssetNode->IsA<UInterchangeVolumeNode>())
		{
			return NewObject<UInterchangeHeterogeneousVolumeActorFactoryNode>(BaseNodeContainer, NAME_None);
		}
	}

	return NewObject<UInterchangeActorFactoryNode>(BaseNodeContainer, NAME_None);
}

void UInterchangeGenericLevelPipeline::CreateComponentFactoryNode(const UInterchangeSceneComponentNode* SceneComponentNode) const
{
	auto GetParentFactoryNodeUid = [&SceneComponentNode]() -> FString
		{
			return SceneComponentNode->GetParentUid().IsEmpty() ? TEXT("") : UInterchangeFactoryBaseNode::BuildFactoryNodeUid(SceneComponentNode->GetParentUid());
		};
	UInterchangeSceneComponentFactoryNode* SceneComponentFactoryNode = nullptr;
	const FString ComponentFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(SceneComponentNode->GetUniqueID());

	if (const UInterchangeInstancedStaticMeshComponentNode* ISMComponentNode = Cast<const UInterchangeInstancedStaticMeshComponentNode>(SceneComponentNode))
	{
		UInterchangeInstancedStaticMeshComponentFactoryNode * ISMComponentFactoryNode = NewObject<UInterchangeInstancedStaticMeshComponentFactoryNode >(BaseNodeContainer, NAME_None);
		BaseNodeContainer->SetupNode(ISMComponentFactoryNode, ComponentFactoryNodeUid, SceneComponentNode->GetDisplayLabel(), EInterchangeNodeContainerType::FactoryData, GetParentFactoryNodeUid());
		UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(ISMComponentNode, ISMComponentFactoryNode, false);
		if (FString AssetInstanceUid; ISMComponentNode->GetCustomInstancedAssetUid(AssetInstanceUid))
		{
			ISMComponentFactoryNode->SetCustomInstancedAssetUid(UInterchangeFactoryBaseNode::BuildFactoryNodeUid(AssetInstanceUid));
		}
		TArray<FTransform> InstanceTransforms;
		ISMComponentNode->GetInstanceTransforms(InstanceTransforms);
		ISMComponentFactoryNode->AddInstanceTransforms(InstanceTransforms);
		ISMComponentFactoryNode->SetCustomHierarchicalISM(bUseHierarchicalISMComponents);

		SceneComponentFactoryNode = ISMComponentFactoryNode;
	}
	else if (const UInterchangeGroomComponentNode* GroomComponentNode = Cast<const UInterchangeGroomComponentNode>(SceneComponentNode))
	{
		UInterchangeGroomComponentFactoryNode* GroomComponentFactoryNode = NewObject<UInterchangeGroomComponentFactoryNode>(BaseNodeContainer, NAME_None);
		BaseNodeContainer->SetupNode(GroomComponentFactoryNode, ComponentFactoryNodeUid, SceneComponentNode->GetDisplayLabel(), EInterchangeNodeContainerType::FactoryData, GetParentFactoryNodeUid());
		UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(GroomComponentNode, GroomComponentFactoryNode, false);

		FString AssetUid;
		if (GroomComponentNode->GetGroomDependencyUid(AssetUid))
		{
			GroomComponentFactoryNode->SetGroomDependencyUid(AssetUid);
		}
		if (GroomComponentNode->GetGroomBindingDependencyUid(AssetUid))
		{
			GroomComponentFactoryNode->SetGroomBindingDependencyUid(AssetUid);
		}

		SceneComponentFactoryNode = GroomComponentFactoryNode;
	}
	else
	{
		SceneComponentFactoryNode = NewObject<UInterchangeSceneComponentFactoryNode>(BaseNodeContainer, NAME_None);
		BaseNodeContainer->SetupNode(SceneComponentFactoryNode, ComponentFactoryNodeUid, SceneComponentNode->GetDisplayLabel(), EInterchangeNodeContainerType::FactoryData, GetParentFactoryNodeUid());
		UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(SceneComponentNode, SceneComponentFactoryNode, false);
	}
	//General setups:
	{
		FTransform LocalTransform;
		if (SceneComponentNode->GetCustomLocalTransform(LocalTransform))
		{
			SceneComponentFactoryNode->SetCustomLocalTransform(LocalTransform);
		}
		TArray<FString> ComponentUids;
		SceneComponentNode->GetComponentUids(ComponentUids);
		for (const FString& ComponentUid : ComponentUids)
		{
			FString ComponentFactoryUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(ComponentUid);
			SceneComponentFactoryNode->AddComponentUid(ComponentFactoryUid);
		}
		bool bComponentVisibility = true;
		if (SceneComponentNode->GetCustomComponentVisibility(bComponentVisibility))
		{
			SceneComponentFactoryNode->SetCustomComponentVisibility(bComponentVisibility);
		}
	}
}


void UInterchangeGenericLevelPipeline::SetUpFactoryNode(UInterchangeActorFactoryNode* ActorFactoryNode, const UInterchangeSceneNode* SceneNode, const UInterchangeBaseNode* TranslatedAssetNode, bool bIsRootJoint /* false */) const
{
	if (!ensure(BaseNodeContainer && ActorFactoryNode && SceneNode && TranslatedAssetNode))
	{
		return;
	}

	if (TranslatedAssetNode->IsA<UInterchangeMeshNode>() || TranslatedAssetNode->IsA<UInterchangeMeshBundleNode>() || TranslatedAssetNode->IsA<UInterchangeMeshLODContainerNode>())
	{
		TArray<FString> TargetNodeUids;
		ActorFactoryNode->GetTargetNodeUids(TargetNodeUids);

		if (bIsRootJoint)
		{
			ActorFactoryNode->SetCustomActorClassName(ASkeletalMeshActor::StaticClass()->GetPathName());
			ActorFactoryNode->SetCustomMobility(EComponentMobility::Movable);
		}
		else
		{
			if (TranslatedAssetNode->IsA<UInterchangeGeometryCacheNode>())
			{
				ActorFactoryNode->SetCustomActorClassName(AGeometryCacheActor::StaticClass()->GetPathName());
				ActorFactoryNode->SetCustomMobility(EComponentMobility::Movable);
			}
			else
			{
				ActorFactoryNode->SetCustomActorClassName(AStaticMeshActor::StaticClass()->GetPathName());
			}
		}

		if (UInterchangeMeshActorFactoryNode* MeshActorFactoryNode = Cast<UInterchangeMeshActorFactoryNode>(ActorFactoryNode))
		{
			TMap<FString, FString> SlotMaterialDependencies;
			SceneNode->GetSlotMaterialDependencies(SlotMaterialDependencies);

			UE::Interchange::MeshesUtilities::ApplySlotMaterialDependencies(*MeshActorFactoryNode, SlotMaterialDependencies, *BaseNodeContainer, nullptr);

			FString AnimationAssetUidToPlay;
			if (SceneNode->GetCustomAnimationAssetUidToPlay(AnimationAssetUidToPlay))
			{
				MeshActorFactoryNode->SetCustomAnimationAssetUidToPlay(AnimationAssetUidToPlay);
			}

			MeshActorFactoryNode->AddFactoryDependencyUid(UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TranslatedAssetNode->GetUniqueID()));
			if (bIsRootJoint)
			{
				// Directly points to the SkeletalMeshActorFactory that holds reference to SkeletalMesh Asset in the CustomReferencedObject.
				MeshActorFactoryNode->SetCustomInstancedAssetFactoryNodeUid(TargetNodeUids[0]);
			}
			else
			{
				MeshActorFactoryNode->SetCustomInstancedAssetFactoryNodeUid(UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TranslatedAssetNode->GetUniqueID()));
			}

			FTransform GeometricTransform;
			if (SceneNode->GetCustomGeometricTransform(GeometricTransform))
			{
				MeshActorFactoryNode->SetCustomGeometricTransform(GeometricTransform);
			}
		}
	}
	else if (const UInterchangeBaseLightNode* BaseLightNode = Cast<UInterchangeBaseLightNode>(TranslatedAssetNode))
	{
		// Default all lights to stationary as that's also what you get when placing UE lights onto the level manually
		ActorFactoryNode->SetCustomMobility(EComponentMobility::Stationary);

		if (UInterchangeBaseLightFactoryNode* BaseLightFactoryNode = Cast<UInterchangeBaseLightFactoryNode>(ActorFactoryNode))
		{
			if (FLinearColor LightColor; BaseLightNode->GetCustomLightColor(LightColor))
			{
				BaseLightFactoryNode->SetCustomLightColor(LightColor.ToFColor(true));
			}

			if (float Intensity; BaseLightNode->GetCustomIntensity(Intensity))
			{
				BaseLightFactoryNode->SetCustomIntensity(Intensity);
			}

			if(bool bUseTemperature; BaseLightNode->GetCustomUseTemperature(bUseTemperature))
			{
				BaseLightFactoryNode->SetCustomUseTemperature(bUseTemperature);

				if(float Temperature; BaseLightNode->GetCustomTemperature(Temperature))
				{
					BaseLightFactoryNode->SetCustomTemperature(Temperature);
				}
			}

			using FLightUnits = std::underlying_type_t<ELightUnits>;
			using FInterchangeLightUnits = std::underlying_type_t<EInterchangeLightUnits>;
			using FCommonLightUnits = std::common_type_t<FLightUnits, FInterchangeLightUnits>;

			static_assert(FCommonLightUnits(EInterchangeLightUnits::Unitless) == FCommonLightUnits(ELightUnits::Unitless), "EInterchangeLightUnits::Unitless differs from ELightUnits::Unitless");
			static_assert(FCommonLightUnits(EInterchangeLightUnits::Lumens) == FCommonLightUnits(ELightUnits::Lumens), "EInterchangeLightUnits::Lumens differs from ELightUnits::Lumens");
			static_assert(FCommonLightUnits(EInterchangeLightUnits::Candelas) == FCommonLightUnits(ELightUnits::Candelas), "EInterchangeLightUnits::Candelas differs from ELightUnits::Candelas");
			static_assert(FCommonLightUnits(EInterchangeLightUnits::EV) == FCommonLightUnits(ELightUnits::EV), "EInterchangeLightUnits::EV differs from ELightUnits::EV");
			static_assert(FCommonLightUnits(EInterchangeLightUnits::Nits) == FCommonLightUnits(ELightUnits::Nits), "EInterchangeLightUnits::Nits differs from ELightUnits::Nits");

			if (const UInterchangeLightNode* LightNode = Cast<UInterchangeLightNode>(BaseLightNode))
			{
				if (UInterchangeLightFactoryNode* LightFactoryNode = Cast<UInterchangeLightFactoryNode>(BaseLightFactoryNode))
				{
					if (FString IESTextureUid; LightNode->GetCustomIESTexture(IESTextureUid))
					{
						if (BaseNodeContainer->GetNode(IESTextureUid))
						{
							LightFactoryNode->SetCustomIESTexture(IESTextureUid);
							LightFactoryNode->AddFactoryDependencyUid(UInterchangeFactoryBaseNode::BuildFactoryNodeUid(IESTextureUid));

							COPY_FROM_TRANSLATED_TO_FACTORY(LightNode, LightFactoryNode, UseIESBrightness, bool)
							COPY_FROM_TRANSLATED_TO_FACTORY(LightNode, LightFactoryNode, IESBrightnessScale, float)
							COPY_FROM_TRANSLATED_TO_FACTORY(LightNode, LightFactoryNode, Rotation, FRotator)
						}
					}

					if (EInterchangeLightUnits IntensityUnits; LightNode->GetCustomIntensityUnits(IntensityUnits))
					{
						LightFactoryNode->SetCustomIntensityUnits(ELightUnits(IntensityUnits));
					}

					COPY_FROM_TRANSLATED_TO_FACTORY(LightNode, LightFactoryNode, AttenuationRadius, float)

					// RectLight
					if(const UInterchangeRectLightNode* RectLightNode = Cast<UInterchangeRectLightNode>(LightNode))
					{
						if(UInterchangeRectLightFactoryNode* RectLightFactoryNode = Cast<UInterchangeRectLightFactoryNode>(LightFactoryNode))
						{
							COPY_FROM_TRANSLATED_TO_FACTORY(RectLightNode, RectLightFactoryNode, SourceWidth, float)
							COPY_FROM_TRANSLATED_TO_FACTORY(RectLightNode, RectLightFactoryNode, SourceHeight, float)
						}
					}

					// Point Light
					if (const UInterchangePointLightNode* PointLightNode = Cast<UInterchangePointLightNode>(LightNode))
					{
						if (UInterchangePointLightFactoryNode* PointLightFactoryNode = Cast<UInterchangePointLightFactoryNode>(LightFactoryNode))
						{
							COPY_FROM_TRANSLATED_TO_FACTORY(PointLightNode, PointLightFactoryNode, SourceRadius, float)

							if (bool bUseInverseSquaredFalloff; PointLightNode->GetCustomUseInverseSquaredFalloff(bUseInverseSquaredFalloff))
							{
								PointLightFactoryNode->SetCustomUseInverseSquaredFalloff(bUseInverseSquaredFalloff);

								COPY_FROM_TRANSLATED_TO_FACTORY(PointLightNode, PointLightFactoryNode, LightFalloffExponent, float)
							}


							// Spot Light
							if (const UInterchangeSpotLightNode* SpotLightNode = Cast<UInterchangeSpotLightNode>(PointLightNode))
							{
								if (UInterchangeSpotLightFactoryNode* SpotLightFactoryNode = Cast<UInterchangeSpotLightFactoryNode>(PointLightFactoryNode))
								{
									COPY_FROM_TRANSLATED_TO_FACTORY(SpotLightNode, SpotLightFactoryNode, InnerConeAngle, float)
									COPY_FROM_TRANSLATED_TO_FACTORY(SpotLightNode, SpotLightFactoryNode, OuterConeAngle, float)
								}
							}
						}
					}
				}
			}

			// DirectionalLight
			if (const UInterchangeDirectionalLightNode* DirectionalLightNode = Cast<UInterchangeDirectionalLightNode>(BaseLightNode))
			{
				if (UInterchangeDirectionalLightFactoryNode* DirectionalLightFactoryNode = Cast<UInterchangeDirectionalLightFactoryNode>(BaseLightFactoryNode))
				{
					COPY_FROM_TRANSLATED_TO_FACTORY(DirectionalLightNode, DirectionalLightFactoryNode, LightSourceAngle, float)
				}
			}

			// Sky Light
			if (const UInterchangeSkyLightNode* SkyLightNode = Cast<UInterchangeSkyLightNode>(BaseLightNode))
			{
				if (UInterchangeSkyLightFactoryNode* SkyLightFactoryNode = Cast<UInterchangeSkyLightFactoryNode>(BaseLightFactoryNode))
				{
					if (FString TranslatedTextureUid; SkyLightNode->GetCustomCubemapDependency(TranslatedTextureUid) && !TranslatedTextureUid.IsEmpty())
					{
						const FString FactoryTextureUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TranslatedTextureUid);
						SkyLightFactoryNode->AddFactoryDependencyUid(FactoryTextureUid);
						SkyLightFactoryNode->SetCustomCubemapDependency(FactoryTextureUid);
					}

					COPY_FROM_TRANSLATED_TO_FACTORY(SkyLightNode, SkyLightFactoryNode, SourceType, EInterchangeSkyLightSourceType)
				}
			}
		}

		//Test for spot before point since a spot light is a point light
		if (BaseLightNode->IsA<UInterchangeSpotLightNode>())
		{
			ActorFactoryNode->SetCustomActorClassName(ASpotLight::StaticClass()->GetPathName());
		}
		else if (BaseLightNode->IsA<UInterchangePointLightNode>())
		{
			ActorFactoryNode->SetCustomActorClassName(APointLight::StaticClass()->GetPathName());
		}
		else if (BaseLightNode->IsA<UInterchangeRectLightNode>())
		{
			ActorFactoryNode->SetCustomActorClassName(ARectLight::StaticClass()->GetPathName());
		}
		else if (BaseLightNode->IsA<UInterchangeDirectionalLightNode>())
		{
			ActorFactoryNode->SetCustomActorClassName(ADirectionalLight::StaticClass()->GetPathName());
		}
		else if (BaseLightNode->IsA<UInterchangeSkyLightNode>())
		{
			ActorFactoryNode->SetCustomActorClassName(ASkyLight::StaticClass()->GetPathName());
		}
		else
		{
			ActorFactoryNode->SetCustomActorClassName(APointLight::StaticClass()->GetPathName());
		}
	}
	else if (const UInterchangeBaseCameraNode* CameraNode = Cast<UInterchangeBaseCameraNode>(TranslatedAssetNode))
	{
		if (UInterchangeBaseCameraFactoryNode* CameraFactoryNode = Cast<UInterchangeBaseCameraFactoryNode>(ActorFactoryNode))
		{
			ActorFactoryNode->SetCustomMobility(EComponentMobility::Movable);

			static_assert(int(EInterchangeAutoExposureMethod::Histogram) == int(EAutoExposureMethod::AEM_Histogram), "EInterchangeAutoExposureMethod::Histogram differs from EAutoExposureMethod::AEM_Histogram");
			static_assert(int(EInterchangeAutoExposureMethod::Basic) == int(EAutoExposureMethod::AEM_Basic), "EInterchangeAutoExposureMethod::Basic differs from EAutoExposureMethod::AEM_Basic");
			static_assert(int(EInterchangeAutoExposureMethod::Manual) == int(EAutoExposureMethod::AEM_Manual), "EInterchangeAutoExposureMethod::Manual differs from EAutoExposureMethod::AEM_Manual");
			static_assert(int(EInterchangeAutoExposureMethod::MAX) == int(EAutoExposureMethod::AEM_MAX), "EInterchangeAutoExposureMethod::MAX differs from EAutoExposureMethod::AEM_MAX");

			float NearClipPlane;
			if (CameraNode->GetCustomNearClipPlane(NearClipPlane))
			{
				CameraFactoryNode->SetCustomNearClipPlane(NearClipPlane);
			}

			EInterchangeAutoExposureMethod AutoExposureMethod = EInterchangeAutoExposureMethod::Histogram;
			if (CameraNode->GetCustomAutoExposureMethod(AutoExposureMethod))
			{
				CameraFactoryNode->SetCustomAutoExposureMethod(AutoExposureMethod);
			}

			float ExposureCompensation;
			if (CameraNode->GetCustomExposureCompensation(ExposureCompensation))
			{
				CameraFactoryNode->SetCustomExposureCompensation(ExposureCompensation);
			}

			float ExposureFstop;
			if (CameraNode->GetCustomExposureFstop(ExposureFstop))
			{
				CameraFactoryNode->SetCustomExposureFstop(ExposureFstop);
			}

			float ExposureISO;
			if (CameraNode->GetCustomExposureISO(ExposureISO))
			{
				CameraFactoryNode->SetCustomExposureISO(ExposureISO);
			}

			float ExposureShutterSpeed;
			if (CameraNode->GetCustomExposureShutterSpeed(ExposureShutterSpeed))
			{
				CameraFactoryNode->SetCustomExposureShutterSpeed(ExposureShutterSpeed);
			}
		}

		if (const UInterchangePhysicalCameraNode* PhysicalCameraNode = Cast<UInterchangePhysicalCameraNode>(CameraNode))
		{
			ActorFactoryNode->SetCustomActorClassName(ACineCameraActor::StaticClass()->GetPathName());

			if (UInterchangePhysicalCameraFactoryNode* PhysicalCameraFactoryNode = Cast<UInterchangePhysicalCameraFactoryNode>(ActorFactoryNode))
			{
				float FocalLength;
				if (PhysicalCameraNode->GetCustomFocalLength(FocalLength))
				{
					PhysicalCameraFactoryNode->SetCustomFocalLength(FocalLength);
				}

				float FocusDistance;
				if (PhysicalCameraNode->GetCustomFocusDistance(FocusDistance))
				{
					PhysicalCameraFactoryNode->SetCustomFocusDistance(FocusDistance);
				}

				float SensorHeight;
				if (PhysicalCameraNode->GetCustomSensorHeight(SensorHeight))
				{
					PhysicalCameraFactoryNode->SetCustomSensorHeight(SensorHeight);
				}

				float SensorWidth;
				if (PhysicalCameraNode->GetCustomSensorWidth(SensorWidth))
				{
					PhysicalCameraFactoryNode->SetCustomSensorWidth(SensorWidth);
				}

				float SensorVerticalOffset;
				if (PhysicalCameraNode->GetCustomSensorVerticalOffset(SensorVerticalOffset))
				{
					PhysicalCameraFactoryNode->SetCustomSensorVerticalOffset(SensorVerticalOffset);
				}

				float SensorHorizontalOffset;
				if (PhysicalCameraNode->GetCustomSensorHorizontalOffset(SensorHorizontalOffset))
				{
					PhysicalCameraFactoryNode->SetCustomSensorHorizontalOffset(SensorHorizontalOffset);
				}

				bool bEnableDepthOfField;
				if (PhysicalCameraNode->GetCustomEnableDepthOfField(bEnableDepthOfField))
				{
					PhysicalCameraFactoryNode->SetCustomFocusMethod(bEnableDepthOfField ? ECameraFocusMethod::Manual : ECameraFocusMethod::DoNotOverride);
				}

				float DepthOfFieldFstop;
				if (PhysicalCameraNode->GetCustomDepthOfFieldFstop(DepthOfFieldFstop))
				{
					PhysicalCameraFactoryNode->SetCustomDepthOfFieldFstop(DepthOfFieldFstop);
				}
			}
		}
		else if (const UInterchangeStandardCameraNode* StandardCameraNode = Cast<UInterchangeStandardCameraNode>(TranslatedAssetNode))
		{
			EInterchangeCameraProjectionType ProjectionType = EInterchangeCameraProjectionType::Perspective;
			if (StandardCameraNode->GetCustomProjectionMode(ProjectionType) && bUsePhysicalInsteadOfStandardPerspectiveCamera && ProjectionType == EInterchangeCameraProjectionType::Perspective)
			{
				float AspectRatio = 1.0f;
				StandardCameraNode->GetCustomAspectRatio(AspectRatio);

				const float SensorWidth = 36.f;  // mm
				float SensorHeight = SensorWidth / AspectRatio;

				float FieldOfView = 90;
				StandardCameraNode->GetCustomFieldOfView(FieldOfView); //Degrees

				float FocalLength = (SensorHeight) / (2.0 * tan(FMath::DegreesToRadians(FieldOfView) / 2.0));

				ActorFactoryNode->SetCustomActorClassName(ACineCameraActor::StaticClass()->GetPathName());

				if (UInterchangePhysicalCameraFactoryNode* PhysicalCameraFactoryNode = Cast<UInterchangePhysicalCameraFactoryNode>(ActorFactoryNode))
				{
					PhysicalCameraFactoryNode->SetCustomFocalLength(FocalLength);
					PhysicalCameraFactoryNode->SetCustomSensorHeight(SensorHeight);
					PhysicalCameraFactoryNode->SetCustomSensorWidth(SensorWidth);
					PhysicalCameraFactoryNode->SetCustomFocusMethod(ECameraFocusMethod::DoNotOverride);
				}
			}
			else
			{
				ActorFactoryNode->SetCustomActorClassName(ACameraActor::StaticClass()->GetPathName());

				if (UInterchangeStandardCameraFactoryNode* CameraFactoryNode = Cast<UInterchangeStandardCameraFactoryNode>(ActorFactoryNode))
				{
					if (StandardCameraNode->GetCustomProjectionMode(ProjectionType))
					{
						CameraFactoryNode->SetCustomProjectionMode((ECameraProjectionMode::Type)ProjectionType);
					}

					float OrthoWidth;
					if (StandardCameraNode->GetCustomWidth(OrthoWidth))
					{
						CameraFactoryNode->SetCustomWidth(OrthoWidth);
					}

					float OrthoFarClipPlane;
					if (StandardCameraNode->GetCustomFarClipPlane(OrthoFarClipPlane))
					{
						CameraFactoryNode->SetCustomFarClipPlane(OrthoFarClipPlane);
					}

					float AspectRatio;
					if (StandardCameraNode->GetCustomAspectRatio(AspectRatio))
					{
						CameraFactoryNode->SetCustomAspectRatio(AspectRatio);
					}

					float FieldOfView;
					if (StandardCameraNode->GetCustomFieldOfView(FieldOfView))
					{
						CameraFactoryNode->SetCustomFieldOfView(FieldOfView);
					}
				}
			}
		}
	}
	else if (const UInterchangeDecalNode* DecalNode = Cast<UInterchangeDecalNode>(TranslatedAssetNode))
	{
		UInterchangeDecalActorFactoryNode* DecalActorFactory = Cast<UInterchangeDecalActorFactoryNode>(ActorFactoryNode);
		ensure(DecalActorFactory);

		if (FVector DecalSize; DecalNode->GetCustomDecalSize(DecalSize))
		{
			DecalActorFactory->SetCustomDecalSize(DecalSize);
		}

		if (int32 SortOrder; DecalNode->GetCustomSortOrder(SortOrder))
		{
			DecalActorFactory->SetCustomSortOrder(SortOrder);
		}

		bool bHasMaterialPathName = false;
		FString DecalMaterialPathName;
		if (DecalNode->GetCustomDecalMaterialPathName(DecalMaterialPathName))
		{
			DecalActorFactory->SetCustomDecalMaterialPathName(DecalMaterialPathName);
			bHasMaterialPathName = true;
		}

		// If the path is not a valid object path then it is an Interchange Node UID (Decal Material Node to be specific).
		if (bHasMaterialPathName && !FPackageName::IsValidObjectPath(DecalMaterialPathName))
		{
			const FString MaterialFactoryUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(DecalMaterialPathName);
			DecalActorFactory->SetCustomDecalMaterialPathName(MaterialFactoryUid);
			DecalActorFactory->AddFactoryDependencyUid(MaterialFactoryUid);
		}
	}
	else if (const UInterchangeVolumeNode* VolumeNode = Cast<UInterchangeVolumeNode>(TranslatedAssetNode))
	{
		UInterchangeHeterogeneousVolumeActorFactoryNode* FactoryNode = Cast<UInterchangeHeterogeneousVolumeActorFactoryNode>(ActorFactoryNode);
		ensure(FactoryNode);

		// Setup a material binding if we have one (we always should, but let's not ensure here as the user
		// may have custom pipelines, etc.)
		{
			TMap<FString, FString> SlotNameToMaterialPath;
			SceneNode->GetSlotMaterialDependencies(SlotNameToMaterialPath);

			FString MaterialFactoryNodeUid;
			if (FString* FoundUid = SlotNameToMaterialPath.Find(UE::Interchange::Volume::VolumetricMaterial))
			{
				MaterialFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(*FoundUid);
				if (!MaterialFactoryNodeUid.IsEmpty())
				{
					FactoryNode->SetCustomVolumetricMaterialUid(MaterialFactoryNodeUid);
				}
			}
		}
	}
}

void UInterchangeGenericLevelPipeline::ExecuteSceneVariantSetNodePreImport(const UInterchangeSceneVariantSetsNode& SceneVariantSetNode)
{
	if (!ensure(BaseNodeContainer))
	{
		return;
	}

	// We may eventually want to optionally import variants
	static bool bEnableSceneVariantSet = true;

	const FString FactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(SceneVariantSetNode.GetUniqueID());

	UInterchangeSceneVariantSetsFactoryNode* FactoryNode = NewObject<UInterchangeSceneVariantSetsFactoryNode>(BaseNodeContainer, NAME_None);

	BaseNodeContainer->SetupNode(FactoryNode, FactoryNodeUid, SceneVariantSetNode.GetDisplayLabel(), EInterchangeNodeContainerType::FactoryData);

	FactoryNode->SetEnabled(bEnableSceneVariantSet);

	UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(BaseNodeContainer);
	UE::Interchange::PipelineHelper::FillSubPathFromSourceNode(FactoryNode, SourceNode);

	// The reimport strategy should be the one from the InterchangeAssetsPipeline not the Level
	if (EReimportStrategyFlags ReimportStrategy; SourceNode->GetCustomReimportStrategyFlags(reinterpret_cast<uint8&>(ReimportStrategy)))
	{
		FactoryNode->SetReimportStrategyFlags(ReimportStrategy);
	}

	TArray<FString> VariantSetUids;
	SceneVariantSetNode.GetCustomVariantSetUids(VariantSetUids);

	for (const FString& VariantSetUid : VariantSetUids)
	{
		FactoryNode->AddCustomVariantSetUid(VariantSetUid);

		// Update factory's dependencies
		if (const UInterchangeVariantSetNode* TrackNode = Cast<UInterchangeVariantSetNode>(BaseNodeContainer->GetNode(VariantSetUid)))
		{
			TArray<FString> DependencyNodeUids;
			TrackNode->GetCustomDependencyUids(DependencyNodeUids);

			for (const FString& DependencyNodeUid : DependencyNodeUids)
			{

				const FString DependencyFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(DependencyNodeUid);
				FactoryNode->AddFactoryDependencyUid(DependencyFactoryNodeUid);

				if (UInterchangeFactoryBaseNode* DependencyFactoryNode = BaseNodeContainer->GetFactoryNode(DependencyFactoryNodeUid))
				{
					if (bEnableSceneVariantSet && !DependencyFactoryNode->IsEnabled())
					{
						DependencyFactoryNode->SetEnabled(true);
					}
				}
			}
		}
	}

	UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(&SceneVariantSetNode, FactoryNode, false);

	FactoryNode->AddTargetNodeUid(SceneVariantSetNode.GetUniqueID());
	SceneVariantSetNode.AddTargetNodeUid(FactoryNode->GetUniqueID());
}

void UInterchangeGenericLevelPipeline::ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* InBaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport)
{
	Super::ExecutePostImportPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);

#if WITH_EDITORONLY_DATA
	using namespace UE::Interchange;

	//We do not use the provided base container since ExecutePreImportPipeline cache it
	//We just make sure the same one is pass in parameter
	if (!InBaseNodeContainer || !ensure(BaseNodeContainer == InBaseNodeContainer) || !CreatedAsset || !ensure(IsInGameThread()))
	{
		return;
	}

	if (UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(CreatedAsset))
	{
		const FSoftObjectPath SceneImportAssetPath(SceneImportAsset);

		auto AddAssetUserDataToObjects = [&]()
		{
			if (SceneHierarchyType == EInterchangeSceneHierarchyType::CreateLevelActors)
			{
				InBaseNodeContainer->IterateNodesOfType<UInterchangeActorFactoryNode>(
					[&SceneImportAssetPath, &SceneImportAsset](const FString& NodeUid, UInterchangeActorFactoryNode* FactoryNode)
					{
						FSoftObjectPath ObjectPath;
						if (FactoryNode->GetCustomReferenceObject(ObjectPath))
						{
							if (AActor* ReferencedObject = Cast<AActor>(ObjectPath.TryLoad()))
							{
								if (UWorld* World = ReferencedObject->GetWorld())
								{
									UE::Interchange::ActorHelper::AddInterchangeAssetUserDataToActor(ReferencedObject, SceneImportAsset, FactoryNode);
								}
							}
						}
					}
				);
			}
			else
			{
				InBaseNodeContainer->IterateNodesOfType<UInterchangeLevelFactoryNode>(
					[&SceneImportAssetPath, &SceneImportAsset](const FString& NodeUid, UInterchangeLevelFactoryNode* FactoryNode)
					{
						bool bWasLevelCreatedForImport = false;
						if (FactoryNode->GetCustomShouldCreateLevel(bWasLevelCreatedForImport) && bWasLevelCreatedForImport)
						{
							FSoftObjectPath ObjectPath;
							if (FactoryNode->GetCustomReferenceObject(ObjectPath))
							{
								if (UWorld* World = Cast<UWorld>(ObjectPath.TryLoad()))
								{
									UE::Interchange::ActorHelper::AddInterchangeLevelAssetUserDataToWorld(World, SceneImportAsset);
								}
							}
						}
					}
				);
			}
		};

		if (!bIsReimportContext || !(bDeleteMissingActors || bDeleteMissingAssets))
		{
			SceneImportAsset->UpdateSceneObjects();
			AddAssetUserDataToObjects();
			return;
		}

		const UInterchangeSceneImportAssetFactoryNode* FactoryNode = Cast<UInterchangeSceneImportAssetFactoryNode>(BaseNodeContainer->GetFactoryNode(NodeKey));
		if (!ensure(FactoryNode))
		{
			SceneImportAsset->UpdateSceneObjects();
			AddAssetUserDataToObjects();
			return;
		}

		// Cache list of objects previously imported in case of a re-import
		TArray<FSoftObjectPath> PrevSoftObjectPaths;
		SceneImportAsset->GetSceneSoftObjectPaths(PrevSoftObjectPaths);

		SceneImportAsset->UpdateSceneObjects();
		AddAssetUserDataToObjects();

		const bool bIsCreateLevelActor = (SceneHierarchyType == EInterchangeSceneHierarchyType::CreateLevelActors);
		// Nothing to take care of
		if (PrevSoftObjectPaths.IsEmpty())
		{
			return;
		}

		TArray<FSoftObjectPath> NewSoftObjectPaths;
		SceneImportAsset->GetSceneSoftObjectPaths(NewSoftObjectPaths);

		TSet<FSoftObjectPath> SoftObjectPathSet(NewSoftObjectPaths);
		TArray<AActor*> ActorsToDelete;
		TArray<UObject*> AssetsToForceDelete;

		ActorsToDelete.Reserve(PrevSoftObjectPaths.Num());
		AssetsToForceDelete.Reserve(PrevSoftObjectPaths.Num());

		for (const FSoftObjectPath& ObjectPath : PrevSoftObjectPaths)
		{
			if (!SoftObjectPathSet.Contains(ObjectPath))
			{
				if (UObject* Object = ObjectPath.TryLoad())
				{
					if (IsValid(Object))
					{
						if (Object->GetClass()->IsChildOf<AActor>())
						{
							//Do not delete ALevelInstance or APackedLevelActor
							if (bIsCreateLevelActor || !(Object->GetClass()->IsChildOf<ALevelInstance>() || Object->GetClass()->IsChildOf<APackedLevelActor>()))
							{
								ActorsToDelete.Add(Cast<AActor>(Object));
							}
						}
						else if(!Object->IsA<UWorld>()) //Avoid deleting UWorld asset
						{
							AssetsToForceDelete.Add(Object);
						}
					}
				}
			}
		}

		if (bDeleteMissingActors)
		{
			Private::DeleteActors(ActorsToDelete);
		}

		if (bDeleteMissingAssets && !AssetsToForceDelete.IsEmpty())
		{
			Private::DeleteAssets(AssetsToForceDelete);
		}

		// Update newly imported objects with a soft reference to the UInterchangeSceneImportAsset
		for (const FSoftObjectPath& ObjectPath : NewSoftObjectPaths)
		{
			UObject* Object = ObjectPath.TryLoad();
			if (IsValid(Object) && Object != CreatedAsset)
			{
				if (UInterchangeAssetImportData* AssetImportData = UInterchangeAssetImportData::GetFromObject(Object))
				{
					AssetImportData->SceneImportAsset = CreatedAsset;
				}
			}
		}
	}

	if (ALevelInstance* LevelInstanceActor = Cast<ALevelInstance>(CreatedAsset))
	{
		const UInterchangeLevelInstanceActorFactoryNode* LevelInstanceFactoryNode = Cast<UInterchangeLevelInstanceActorFactoryNode>(InBaseNodeContainer->GetFactoryNode(NodeKey));
		FString LevelFactoryNodeUid;
		if (LevelInstanceFactoryNode->GetCustomLevelReference(LevelFactoryNodeUid))
		{
			if (const UInterchangeFactoryBaseNode* ReferenceLevelFactoryNode = InBaseNodeContainer->GetFactoryNode(LevelFactoryNodeUid))
			{
				FSoftObjectPath ReferenceLevelPath;
				if (ReferenceLevelFactoryNode->GetCustomReferenceObject(ReferenceLevelPath))
				{
					if (UWorld* ReferenceWorld = Cast<UWorld>(ReferenceLevelPath.TryLoad()))
					{
						PostPipelineImportData.AddLevelInstanceActor(LevelInstanceActor, ReferenceWorld);
					}
				}
			}
		}
	}
#endif
}

void UInterchangeGenericLevelPipeline::ExecutePostBroadcastPipeline(const UInterchangeBaseNodeContainer* InBaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport)
{
	Super::ExecutePostBroadcastPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);

	if (!CreatedAsset || !ensure(IsInGameThread()))
	{
		return;
	}

#if WITH_EDITORONLY_DATA
	if (ALevelInstance* LevelInstanceActor = Cast<ALevelInstance>(CreatedAsset))
	{
		if (UWorld* WorldAsset = LevelInstanceActor->GetWorldAsset().Get())
		{
			//We cannot call EnterEdit on a dirty world.
			if (WorldAsset->GetPackage()->IsDirty())
			{
				if (UInterchangeEditorUtilitiesBase* EditorUtilities = UInterchangeManager::GetInterchangeManager().GetEditorUtilities())
				{
					if (!EditorUtilities->SaveAsset(WorldAsset))
					{
						UE_LOGF(LogInterchangePipeline, Warning, "UInterchangeGenericLevelPipeline: Cannot save the level instance actor (%ls) referenced world (%ls)", *LevelInstanceActor->GetName(), *WorldAsset->GetName());
					}
				}
			}

			//Update all levelinstance actor that are referencing this reference world
			for (FThreadSafeObjectIterator It; It; ++It)
			{
				if (ALevelInstance* CurrentLevelInstanceActor = Cast<ALevelInstance>(*It))
				{
					if (CurrentLevelInstanceActor->GetWorldAsset().Get() == WorldAsset)
					{
						if (CurrentLevelInstanceActor->HasAnyFlags(RF_MirroredGarbage))
						{
							continue;
						}

						const UWorld* CurrentLevelInstanceActorWorld = CurrentLevelInstanceActor->GetWorld();
						if (!CurrentLevelInstanceActorWorld || CurrentLevelInstanceActorWorld->IsCleanedUp())
						{
							continue;
						}

						CurrentLevelInstanceActor->UpdateLevelInstanceFromWorldAsset();
						if (!CurrentLevelInstanceActor->IsA<APackedLevelActor>())
						{
							CurrentLevelInstanceActor->LoadLevelInstance();
						}
					}
				}
			}

			//Reference world must be cleanup since they are not the main world. It was loaded by the Update of the level instance
			//This remove all the world managers and prevent GC issue when unloading the main world referencing this world.
			WorldAsset->ClearFlags(RF_Standalone);
			WorldAsset->ClearInternalFlags(EInternalObjectFlags::Async);
			if (WorldAsset->bIsWorldInitialized)
			{
				WorldAsset->CleanupWorld();
			}
		}
	}
#endif
}

#if WITH_EDITORONLY_DATA

void UInterchangeGenericLevelPipeline::FPostPipelineImportData::AddLevelInstanceActor(ALevelInstance* LevelInstanceActor, UWorld* ReferencedWorld)
{
	bool bReferencedWorldAlreadyProcess = Worlds.Contains(ReferencedWorld);

	UWorld* ParentWorld = LevelInstanceActor->GetWorld();

	if (!bReferencedWorldAlreadyProcess)
	{
		if (UInterchangeEditorUtilitiesBase* EditorUtilities = UInterchangeManager::GetInterchangeManager().GetEditorUtilities())
		{
			if (!EditorUtilities->SaveAsset(ReferencedWorld))
			{
				UE_LOGF(LogInterchangePipeline, Warning, "UInterchangeGenericScenesPipeline: Cannot save the level instance actor (%ls) referenced world (%ls)", *LevelInstanceActor->GetName(), *ReferencedWorld->GetName());
			}
		}

		// Make sure newly created level asset gets scanned
		ULevel::ScanLevelAssets(ReferencedWorld->GetPackage()->GetName());
	}

	ParentWorld->PreEditChange(nullptr);

	LevelInstanceActor->SetWorldAsset(ReferencedWorld);
	if (APackedLevelActor* PackedLevelActor = Cast<APackedLevelActor>(LevelInstanceActor))
	{
		constexpr bool bCheckoutAndSaveFalse = false;
		constexpr bool bPromptForSaveFalse = false;
		//Get the blueprint
		UBlueprint* Blueprint = PackedLevelActor->GetRootBlueprint();
		ULevel::ScanLevelAssets(Blueprint->GetPackage()->GetName());
		//Update the blueprint with the reference level content
		FPackedLevelActorBuilder::CreateDefaultBuilder()->CreateOrUpdateBlueprint(ReferencedWorld, Blueprint, bCheckoutAndSaveFalse, bPromptForSaveFalse);
	}
	else
	{
		LevelInstanceActor->UpdateLevelInstanceFromWorldAsset();
		LevelInstanceActor->LoadLevelInstance();
	}

	if (!bReferencedWorldAlreadyProcess)
	{
		//Reference world must be cleanup since they are not the main world.
		//This remove all the world managers and prevent GC issue when unloading the main world referencing this world.
		if (ReferencedWorld->bIsWorldInitialized)
		{
			ReferencedWorld->CleanupWorld();
		}

		Worlds.Add(ReferencedWorld);
	}

	ParentWorld->PostEditChange();
}

#endif //WITH_EDITORONLY_DATA

void UInterchangeGenericLevelPipeline::CacheActiveJointUids()
{
	Cached_ActiveJointUids.Reset();
	BaseNodeContainer->IterateNodesOfType<UInterchangeSkeletonFactoryNode>([this](const FString& NodeUid, UInterchangeSkeletonFactoryNode* Node)
		{
			FString RootNodeUid;
			if (Node->GetCustomRootJointUid(RootNodeUid))
			{
				Cached_ActiveJointUids.Add(RootNodeUid);
				BaseNodeContainer->IterateNodeChildren(RootNodeUid, [this](const UInterchangeBaseNode* Node)
					{
						if (const UInterchangeJointNode* SceneNode = Cast<UInterchangeJointNode>(Node))
						{
							Cached_ActiveJointUids.Add(Node->GetUniqueID());
						}
					});
			}
		});
}