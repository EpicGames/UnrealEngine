// Copyright Epic Games, Inc. All Rights Reserved. 
#include "InterchangeGenericMeshPipeline.h"

#include "Animation/Skeleton.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/Async.h"
#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "InterchangeCommonPipelineDataFactoryNode.h"
#include "InterchangeGenericAssetsPipeline.h"
#include "InterchangeLODDataParser.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeMeshBundleNode.h"
#include "InterchangeMeshLODContainerNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangePhysicsAssetFactoryNode.h"
#include "InterchangePipelineLog.h"
#include "InterchangePipelineMeshesUtilities.h"
#include "InterchangeSceneComponentNodes.h"
#include "InterchangeSceneNode.h"
#include "InterchangeSkeletalMeshFactoryNode.h"
#include "InterchangeSkeletalMeshLodDataNode.h"
#include "InterchangeSkeletonFactoryNode.h"
#include "InterchangeSkeletonHelper.h"
#include "InterchangeSourceData.h"
#include "Mesh/InterchangeMeshHelper.h"
#include "Misc/Paths.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"
#if WITH_EDITOR
#include "PhysicsAssetUtils.h"
#endif //WITH_EDITOR
#include "PhysicsEngine/PhysicsAsset.h"
#include "ReferenceSkeleton.h"
#include "Tasks/Task.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "InterchangeManager.h"

namespace UE::Interchange::SkeletalMeshGenericPipeline
{
	bool RecursiveFindChildUid(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& ParentUid, const FString& SearchUid)
	{
		if (ParentUid == SearchUid)
		{
			return true;
		}
		const int32 ChildCount = BaseNodeContainer->GetNodeChildrenCount(ParentUid);
		TArray<FString> Childrens = BaseNodeContainer->GetNodeChildrenUids(ParentUid);
		for (int32 ChildIndex = 0; ChildIndex < ChildCount; ++ChildIndex)
		{
			if (RecursiveFindChildUid(BaseNodeContainer, Childrens[ChildIndex], SearchUid))
			{
				return true;
			}
		}
		return false;
	}

	void RemoveNestedMeshNodes(const UInterchangeBaseNodeContainer* BaseNodeContainer, const UInterchangeSkeletonFactoryNode* SkeletonFactoryNode, TArray<FString>& NodeUids)
	{
		if (!SkeletonFactoryNode)
		{
			return;
		}
		FString SkeletonRootJointUid;
		SkeletonFactoryNode->GetCustomRootJointUid(SkeletonRootJointUid);
		for (int32 NodeIndex = NodeUids.Num()-1; NodeIndex >= 0; NodeIndex--)
		{
			const FString& NodeUid = NodeUids[NodeIndex];
			if (RecursiveFindChildUid(BaseNodeContainer, SkeletonRootJointUid, NodeUid))
			{
				NodeUids.RemoveAt(NodeIndex);
			}
		}
	}
}

void UInterchangeGenericMeshPipeline::ExecutePreImportPipelineSkeletalMesh()
{
	LLM_SCOPE_BYNAME(TEXT("Interchange"));
	check(CommonMeshesProperties.IsValid());
	if (!bImportSkeletalMeshes)
	{
		//Nothing to import
		return;
	}

	if (CommonMeshesProperties->ForceAllMeshAsType != EInterchangeForceMeshType::IFMT_None && CommonMeshesProperties->ForceAllMeshAsType != EInterchangeForceMeshType::IFMT_SkeletalMesh)
	{
		//Nothing to import
		return;
	}

#if WITH_EDITOR
	//Make sure the generic pipeline we cover all skeletal mesh build settings by asserting when we import
	Async(EAsyncExecution::TaskGraphMainThread, []()
		{
			static bool bVerifyBuildProperties = false;
			if (!bVerifyBuildProperties)
			{
				bVerifyBuildProperties = true;
				TArray<const UClass*> Classes;
				Classes.Add(UInterchangeGenericCommonMeshesProperties::StaticClass());
				Classes.Add(UInterchangeGenericMeshPipeline::StaticClass());
				if (!DoClassesIncludeAllEditableStructProperties(Classes, FSkeletalMeshBuildSettings::StaticStruct()))
				{
					UE_LOGF(LogInterchangePipeline, Log, "UInterchangeGenericMeshPipeline: The generic pipeline does not cover all skeletal mesh build options.");
				}
			}
		});
#endif

	TMap<FString, TArray<FString>> SkeletalMeshFactoryDependencyOrderPerSkeletonRootNodeUid;

	auto CreateSkeletalMeshFactoryForSkeletonRootUid = [this, &SkeletalMeshFactoryDependencyOrderPerSkeletonRootNodeUid](const FString& RootJointUid, const TArray<FString>& NodeUids)
		{
			UInterchangeSkeletalMeshFactoryNode* SkeletalMeshFactoryNode = CreateSkeletalMeshFactoryNode(RootJointUid, NodeUids);
			if (SkeletalMeshFactoryNode != nullptr)
			{
				TArray<FString>& SkeletalMeshFactoryDependencyOrder = SkeletalMeshFactoryDependencyOrderPerSkeletonRootNodeUid.FindOrAdd(RootJointUid);
				//Skeletal Mesh Factory are responsible of populating the USkeleton, and updating the skeleton is not multi thread safe, so we add dependency between skeletal mesh altering the same skeleton
				//TODO make the skeletalMesh ReferenceSkeleton thread safe to allow multiple parallel skeletal mesh factory on the same skeleton asset.
				int32 DependencyIndex = SkeletalMeshFactoryDependencyOrder.AddUnique(SkeletalMeshFactoryNode->GetUniqueID());
				if (DependencyIndex > 0)
				{
					const FString SkeletalMeshFactoryNodeDependencyUid = SkeletalMeshFactoryDependencyOrder[DependencyIndex - 1];
					SkeletalMeshFactoryNode->AddFactoryDependencyUid(SkeletalMeshFactoryNodeDependencyUid);
				}
			}
		};

	TMap<FString, TArray<FString>> MeshUidsPerSkeletonRootUid;
	const bool bUseSingleBoneForConvertedMeshes = CommonMeshesProperties->ForceAllMeshAsType == EInterchangeForceMeshType::IFMT_SkeletalMesh && CommonMeshesProperties->bSingleBoneSkeleton;
	const bool bVisibleOnly = CombineSkeletalMeshesBehavior == EInterchangeCombineSkeletalMeshesBehavior::BySkeletonVisibleOnly;
	PipelineMeshesUtilities->GetCombinedSkinnedMeshInstances(MeshUidsPerSkeletonRootUid, bUseSingleBoneForConvertedMeshes, bVisibleOnly);

	if (MeshUidsPerSkeletonRootUid.Num() > 0)
	{
#if !WITH_EDITOR || !WITH_EDITORONLY_DATA
		UE_LOGF(LogInterchangePipeline, Warning, "Cannot import skeletalMesh asset in runtime, this is an editor only feature.");
#else
		const FString SourceFileName = SourceDatas.Num() ? SourceDatas[0]->GetFilename() : TEXT("No Source File Specified");

		for (const TPair<FString, TArray<FString>>& SkeletonRootUidAndMeshUids : MeshUidsPerSkeletonRootUid)
		{
			const FString& SkeletonRootUid = SkeletonRootUidAndMeshUids.Key;
			//Every iteration is a skeletal mesh asset that combine all MeshInstances sharing the same skeleton root node
			CommonSkeletalMeshesAndAnimationsProperties->CreateSkeletonFactoryNode(BaseNodeContainer, SkeletonRootUid, SourceFileName);
			
			//The MeshUids can represent a SceneNode pointing on a MeshNode or directly a MeshNode;
			const TArray<FString>& MeshNodeOrMeshInstanceNodeUids = SkeletonRootUidAndMeshUids.Value;

			if (CombineSkeletalMeshesBehavior == EInterchangeCombineSkeletalMeshesBehavior::DoNotCombine)
			{
				for (const FString& MeshUid : MeshNodeOrMeshInstanceNodeUids)
				{
					CreateSkeletalMeshFactoryForSkeletonRootUid(SkeletonRootUid, { MeshUid });
				}
			}
			else
			{
				CreateSkeletalMeshFactoryForSkeletonRootUid(SkeletonRootUid, MeshNodeOrMeshInstanceNodeUids);
			}
		}
#endif
	}

	CreateAssemblyPartDependencies();
}

UInterchangeSkeletalMeshFactoryNode* UInterchangeGenericMeshPipeline::CreateSkeletalMeshFactoryNode(const FString& RootJointUid, const TArray<FString>& NodeUids)
{
	check(CommonMeshesProperties.IsValid());
	check(CommonSkeletalMeshesAndAnimationsProperties.IsValid());
	//Get the skeleton factory node
	const FString SkeletonFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(RootJointUid);
	UInterchangeSkeletonFactoryNode* SkeletonFactoryNode = Cast<UInterchangeSkeletonFactoryNode>(BaseNodeContainer->GetFactoryNode(SkeletonFactoryNodeUid));
	if (!ensure(SkeletonFactoryNode))
	{
		//Log an error
		return nullptr;
	}
	
	if (NodeUids.Num() == 0)
	{
		return nullptr;
	}
	
	// Note: UInterchangeMeshBundleNode does not need a separate case here. When collapsing produces a
	// MeshBundle, a scene node is created with AssetInstanceUid pointing to the bundle. The pipeline sees
	// this scene node (via the UInterchangeSceneNode branch below), and the LOD data parser handles
	// MeshBundle expansion when building the actual skeletal mesh geometry.
	auto GetFirstNodeInfo = [this, &NodeUids](FString& OutFirstMeshNodeUid)->const UInterchangeBaseNode*
	{
		FString FirstNodeUid = NodeUids.Num() > 0 ? (NodeUids)[0] : FString();
		for (int32 Index = 1; Index < NodeUids.Num(); ++Index)
		{
			if (NodeUids[Index] < FirstNodeUid)
			{
				FirstNodeUid = NodeUids[Index];
			}
		}

		if (!FirstNodeUid.IsEmpty())
		{
			if (const UInterchangeMeshNode* MeshNode = Cast<const UInterchangeMeshNode>(BaseNodeContainer->GetNode(FirstNodeUid)))
			{
				OutFirstMeshNodeUid = FirstNodeUid;
				
				return MeshNode;
			}

			if (const UInterchangeSceneNode* SceneNode = Cast<const UInterchangeSceneNode>(BaseNodeContainer->GetNode(FirstNodeUid)))
			{
				FString MeshNodeUid;
				if (SceneNode->GetCustomAssetInstanceUid(MeshNodeUid))
				{
					OutFirstMeshNodeUid = MeshNodeUid;
					
					return SceneNode;
				}
			}
			if (const UInterchangeInstancedStaticMeshComponentNode* ISMComponentNode = Cast<const UInterchangeInstancedStaticMeshComponentNode>(BaseNodeContainer->GetNode(FirstNodeUid)))
			{
				FString AssetInstanceUid;
				if (ISMComponentNode->GetCustomInstancedAssetUid(AssetInstanceUid))
				{
					OutFirstMeshNodeUid = AssetInstanceUid;

					return ISMComponentNode;
				}
			}
		}
		
		return nullptr;
	};

	FString AssetNodeUid;
	const UInterchangeBaseNode* InterchangeBaseNode = GetFirstNodeInfo(AssetNodeUid);
	if (!InterchangeBaseNode)
	{
		//TODO: Log an error
		return nullptr;
	}
	const UInterchangeSceneNode* FirstSceneNode = Cast<UInterchangeSceneNode>(InterchangeBaseNode);
	const UInterchangeBaseNode* AssetNode/*Mesh or LODContainer or MeshBundle etc*/ = BaseNodeContainer->GetNode(AssetNodeUid);

	if (!AssetNode)
	{
		//TODO: Log an error
		return nullptr;
	}

	//Create the skeletal mesh factory node
	FString DisplayLabel = AssetNode->GetDisplayLabel();
	FString SkeletalMeshUid_MeshNamePart = AssetNodeUid;
	if(FirstSceneNode)
	{
		//use the scene node to name the skeletal mesh
		DisplayLabel = FirstSceneNode->GetDisplayLabel();
		//Use the first scene node uid this skeletalmesh reference, add backslash since this uid is not asset typed (\\Mesh\\) like FirstMeshNodeUid
		SkeletalMeshUid_MeshNamePart = TEXT("\\") + FirstSceneNode->GetUniqueID();
	}

	const FString SkeletalMeshUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(SkeletalMeshUid_MeshNamePart + SkeletonFactoryNodeUid);
	UInterchangeSkeletalMeshFactoryNode* SkeletalMeshFactoryNode = NewObject<UInterchangeSkeletalMeshFactoryNode>(BaseNodeContainer, NAME_None);
	if (!ensure(SkeletalMeshFactoryNode))
	{
		return nullptr;
	}
	SkeletalMeshFactoryNode->InitializeSkeletalMeshNode(SkeletalMeshUid, DisplayLabel, USkeletalMesh::StaticClass()->GetName(), BaseNodeContainer);
	SkeletalMeshFactoryNode->AddFactoryDependencyUid(SkeletonFactoryNodeUid);

	if (CommonMeshesProperties->bKeepSectionsSeparate)
	{
		SkeletalMeshFactoryNode->SetCustomKeepSectionsSeparate(CommonMeshesProperties->bKeepSectionsSeparate);
	}

	SkeletonFactoryNode->AddCustomSkeletalMeshFactoryNodeUid(SkeletalMeshFactoryNode->GetUniqueID());

	bool bUseTimeZeroAsBindPose = false;
	SkeletonFactoryNode->GetCustomUseTimeZeroForBindPose(bUseTimeZeroAsBindPose);

	FInterchangeMeshLODDataParser MeshLODDataParser(*BaseNodeContainer, SkeletalMeshFactoryNode, PipelineMeshesUtilities, this, SourceDatas.Num() ? SourceDatas[0]->GetFilename() : TEXT("No Source File Specified"), RootJointUid, SkeletonFactoryNodeUid, bUseTimeZeroAsBindPose);
	MeshLODDataParser.ProcessNodeUids(NodeUids);

	SkeletalMeshFactoryNode->SetCustomImportMorphTarget(bImportMorphTargets);
	SkeletalMeshFactoryNode->SetCustomImportVertexAttributes(bImportVertexAttributes);

	SkeletalMeshFactoryNode->SetCustomImportContentType(SkeletalMeshImportContentType);
	SkeletalMeshFactoryNode->SetCustomImportSockets(CommonMeshesProperties->bImportSockets);

	SkeletalMeshFactoryNode->SetCustomAddCurveMetadataToSkeleton(CommonSkeletalMeshesAndAnimationsProperties->bAddCurveMetadataToSkeleton);
	//If we have a specified skeleton
	if (CommonSkeletalMeshesAndAnimationsProperties->IsSkeletonValid())
	{
		if (USkeleton* Skeleton = CommonSkeletalMeshesAndAnimationsProperties->Skeleton.LoadSynchronous())
		{
			bool bSkeletonCompatible = false;

			//TODO: support skeleton helper in runtime
#if WITH_EDITOR
			bSkeletonCompatible = UE::Interchange::Private::FSkeletonHelper::IsCompatibleSkeleton(Skeleton
				, RootJointUid
				, BaseNodeContainer
				, CommonMeshesProperties->bConvertStaticsWithMorphTargetsToSkeletals || CommonMeshesProperties->ForceAllMeshAsType == EInterchangeForceMeshType::IFMT_SkeletalMesh
				, false /*bCheckForIdenticalSkeleton*/
				, CommonMeshesProperties->bImportSockets
			);
#endif
			if (bSkeletonCompatible)
			{
				SkeletalMeshFactoryNode->SetCustomSkeletonSoftObjectPath(CommonSkeletalMeshesAndAnimationsProperties->Skeleton.ToSoftObjectPath());
			}
			else
			{
				UInterchangeResultDisplay_Generic* Message = AddMessage<UInterchangeResultDisplay_Generic>();
				Message->Text = FText::Format(NSLOCTEXT("UInterchangeGenericMeshPipeline", "IncompatibleSkeleton", "Incompatible skeleton {0} when importing skeletalmesh {1}."),
					FText::FromString(Skeleton->GetName()),
					FText::FromString(DisplayLabel));
			}
		}
		else
		{
			UInterchangeResultWarning_Generic* Warning = AddMessage<UInterchangeResultWarning_Generic>();
			Warning->Text = FText::Format(NSLOCTEXT("UInterchangeGenericMeshPipeline", "InvalidSkeleton", "Failed to load skeleton at path {0} to test compatibility when importing skeletalmesh {1}."),
				FText::FromString(CommonSkeletalMeshesAndAnimationsProperties->Skeleton.ToString()),
				FText::FromString(DisplayLabel));
		}
	}

	//Physic asset dependency, if we must create or use a specialize physic asset let create
	//a PhysicsAsset factory node, so the asset will exist when we will setup the skeletalmesh
	if (bCreatePhysicsAsset)
	{
		UInterchangePhysicsAssetFactoryNode* PhysicsAssetFactoryNode = NewObject<UInterchangePhysicsAssetFactoryNode>(BaseNodeContainer, NAME_None);
		if (ensure(SkeletalMeshFactoryNode))
		{
			const FString PhysicsAssetUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(SkeletalMeshUid_MeshNamePart + SkeletonFactoryNodeUid + TEXT("_PhysicsAsset"));
			const FString PhysicsAssetDisplayLabel = DisplayLabel + TEXT("_PhysicsAsset");
			PhysicsAssetFactoryNode->InitializePhysicsAssetNode(PhysicsAssetUid, PhysicsAssetDisplayLabel, UPhysicsAsset::StaticClass()->GetName(), BaseNodeContainer);
			PhysicsAssetFactoryNode->SetCustomSkeletalMeshUid(SkeletalMeshUid);
			PhysicsAssetFactoryNode->AddFactoryDependencyUid(SkeletalMeshFactoryNode->GetUniqueID());
		}
	}
	SkeletalMeshFactoryNode->SetCustomCreatePhysicsAsset(bCreatePhysicsAsset);
	if (!bCreatePhysicsAsset && PhysicsAsset.IsValid())
	{
		FSoftObjectPath PhysicSoftObjectPath(PhysicsAsset.Get());
		SkeletalMeshFactoryNode->SetCustomPhysicAssetSoftObjectPath(PhysicSoftObjectPath);
	}

	const bool bTrueValue = true;
	switch (CommonMeshesProperties->VertexColorImportOption)
	{
		case EInterchangeVertexColorImportOption::IVCIO_Replace:
		{
			SkeletalMeshFactoryNode->SetCustomVertexColorReplace(bTrueValue);
		}
		break;
		case EInterchangeVertexColorImportOption::IVCIO_Ignore:
		{
			SkeletalMeshFactoryNode->SetCustomVertexColorIgnore(bTrueValue);
		}
		break;
		case EInterchangeVertexColorImportOption::IVCIO_Override:
		{
			SkeletalMeshFactoryNode->SetCustomVertexColorOverride(CommonMeshesProperties->VertexOverrideColor);
		}
		break;
	}

	//Avoid importing skeletalmesh if we want to only import animation
	if (CommonSkeletalMeshesAndAnimationsProperties->bImportOnlyAnimations)
	{
		SkeletonFactoryNode->SetEnabled(false);
		SkeletalMeshFactoryNode->SetEnabled(false);
	}

	// When Nanite is enabled, recomputed tangents are unused. Nanite generates its own tangents
	// unless explicit tangent data is provided, so we can safely skip recomputation here.
	auto HasNaniteAssemblyDependencies = [this, MeshUids = MeshLODDataParser.GetUniqueMeshUids()]()
		{
			if (!PipelineMeshesUtilities || !PipelineMeshesUtilities->HasAssemblyMeshDependencies())
			{
				return false;
			}

			for (const FString& MeshUid : MeshUids)
			{
				if (const UInterchangeMeshNode* MeshNode = Cast<const UInterchangeMeshNode>(BaseNodeContainer->GetNode(MeshUid)))
				{
					if (MeshNode->GetAssemblyPartDependenciesCount() > 0)
					{
						return true;
					}
				}
			}

			return false;
		};
	bool bRecomputeTangents = CommonMeshesProperties->bRecomputeTangents;
	if (bRecomputeTangents && HasNaniteAssemblyDependencies())
	{
		UE_LOGF(LogInterchangePipeline, Log, "UInterchangeGenericMeshPipeline: Disabling unnecessary tangent recomputation on Nanite skeletalmesh %ls", *DisplayLabel);
		bRecomputeTangents = false;
	}

	//Common meshes build options
	SkeletalMeshFactoryNode->SetCustomRecomputeNormals(CommonMeshesProperties->bRecomputeNormals);
	SkeletalMeshFactoryNode->SetCustomRecomputeTangents(bRecomputeTangents);
	SkeletalMeshFactoryNode->SetCustomUseMikkTSpace(CommonMeshesProperties->bUseMikkTSpace);
	SkeletalMeshFactoryNode->SetCustomComputeWeightedNormals(CommonMeshesProperties->bComputeWeightedNormals);
	SkeletalMeshFactoryNode->SetCustomUseHighPrecisionTangentBasis(CommonMeshesProperties->bUseHighPrecisionTangentBasis);
	SkeletalMeshFactoryNode->SetCustomUseFullPrecisionUVs(CommonMeshesProperties->bUseFullPrecisionUVs);
	SkeletalMeshFactoryNode->SetCustomUseBackwardsCompatibleF16TruncUVs(CommonMeshesProperties->bUseBackwardsCompatibleF16TruncUVs);
	SkeletalMeshFactoryNode->SetCustomRemoveDegenerates(CommonMeshesProperties->bRemoveDegenerates);
	//Skeletal meshes build options
	SkeletalMeshFactoryNode->SetCustomUseHighPrecisionSkinWeights(bUseHighPrecisionSkinWeights);
	SkeletalMeshFactoryNode->SetCustomThresholdPosition(ThresholdPosition);
	SkeletalMeshFactoryNode->SetCustomThresholdTangentNormal(ThresholdTangentNormal);
	SkeletalMeshFactoryNode->SetCustomThresholdUV(ThresholdUV);
	SkeletalMeshFactoryNode->SetCustomMorphThresholdPosition(MorphThresholdPosition);
	SkeletalMeshFactoryNode->SetCustomBoneInfluenceLimit(BoneInfluenceLimit);
	SkeletalMeshFactoryNode->SetCustomMergeMorphTargetShapeWithSameName(bMergeMorphTargetsWithSameName);

	return SkeletalMeshFactoryNode;
}

void UInterchangeGenericMeshPipeline::PostImportSkeletalMesh(UObject* CreatedAsset, const UInterchangeFactoryBaseNode* FactoryNode)
{
	check(CommonSkeletalMeshesAndAnimationsProperties.IsValid());

	if (!BaseNodeContainer)
	{
		return;
	}

	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(CreatedAsset);
	if (!SkeletalMesh)
	{
		return;
	}

	//If we import only the geometry we do not want to update the skeleton reference pose.
	const bool bImportGeometryOnlyContent = SkeletalMeshImportContentType == EInterchangeSkeletalMeshContentType::Geometry;
	USkeleton* Skeleton = CommonSkeletalMeshesAndAnimationsProperties->IsSkeletonValid() 
							? CommonSkeletalMeshesAndAnimationsProperties->Skeleton.LoadSynchronous() 
							: nullptr;

	if (!bImportGeometryOnlyContent && bUpdateSkeletonReferencePose && Skeleton && SkeletalMesh->GetSkeleton() == Skeleton)
	{
		SkeletalMesh->GetSkeleton()->UpdateReferencePoseFromMesh(SkeletalMesh);
		//TODO: notify editor the skeleton has change
	}
}

void UInterchangeGenericMeshPipeline::PostImportPhysicsAssetImport(UObject* CreatedAsset, const UInterchangeFactoryBaseNode* FactoryNode)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeGenericMeshPipeline::PostImportPhysicsAssetImport);
	LLM_SCOPE_BYNAME(TEXT("Interchange"));
#if WITH_EDITOR
	if (!bCreatePhysicsAsset || !BaseNodeContainer)
	{
		return;
	}

	UPhysicsAsset* CreatedPhysicsAsset = Cast<UPhysicsAsset>(CreatedAsset);
	if (!CreatedPhysicsAsset)
	{
		return;
	}
	if (const UInterchangePhysicsAssetFactoryNode* PhysicsAssetFactoryNode = Cast<const UInterchangePhysicsAssetFactoryNode>(FactoryNode))
	{
		FString SkeletalMeshFactoryNodeUid;
		if (PhysicsAssetFactoryNode->GetCustomSkeletalMeshUid(SkeletalMeshFactoryNodeUid))
		{
			if (const UInterchangeSkeletalMeshFactoryNode* SkeletalMeshFactoryNode = Cast<const UInterchangeSkeletalMeshFactoryNode>(BaseNodeContainer->GetFactoryNode(SkeletalMeshFactoryNodeUid)))
			{
				FSoftObjectPath ReferenceObject;
				SkeletalMeshFactoryNode->GetCustomReferenceObject(ReferenceObject);
				if (ReferenceObject.IsValid())
				{
					if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(ReferenceObject.TryLoad()))
					{
						auto CreateFromSkeletalMeshLambda = [CreatedPhysicsAsset, SkeletalMesh]()
						{
							TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeGenericMeshPipeline::PostImportPhysicsAssetImport::CreateFromSkeletalMeshLambda);
							FPhysAssetCreateParams NewBodyData;
							FText CreationErrorMessage;
							constexpr bool bSetToMesh = true;
							constexpr bool bShowProgress = false;
							if (!FPhysicsAssetUtils::CreateFromSkeletalMesh(CreatedPhysicsAsset, SkeletalMesh, NewBodyData, CreationErrorMessage, bSetToMesh, bShowProgress))
							{
								//TODO: Log an error
							}
						};

						if (!IsInGameThread() && SkeletalMesh->IsCompiling())
						{
							//If the skeletalmesh is compiling we have to stall on the main thread
							Async(EAsyncExecution::TaskGraphMainThread, [CreateFromSkeletalMeshLambda]()
							{
								CreateFromSkeletalMeshLambda();
							});
						}
						else
						{
							CreateFromSkeletalMeshLambda();
						}
					}
				}
			}
		}
	}
#endif //WITH_EDITOR
}

void UInterchangeGenericMeshPipeline::ImplementUseSourceNameForAssetOptionSkeletalMesh(const int32 MeshesImportedNodeCount, const bool bUseSourceNameForAsset, const FString& AssetName)
{
	check(CommonSkeletalMeshesAndAnimationsProperties.IsValid());

	const UClass* SkeletalMeshFactoryNodeClass = UInterchangeSkeletalMeshFactoryNode::StaticClass();
	TArray<FString> SkeletalMeshNodeUids;
	BaseNodeContainer->GetNodes(SkeletalMeshFactoryNodeClass, SkeletalMeshNodeUids);
	if (SkeletalMeshNodeUids.Num() == 0)
	{
		return;
	}
	//If we import only one asset, and bUseSourceNameForAsset is true, we want to rename the asset using the file name.
	const bool bShouldChangeAssetName = ((bUseSourceNameForAsset || !AssetName.IsEmpty()) && MeshesImportedNodeCount == 1);
	const FString SkeletalMeshUid = SkeletalMeshNodeUids[0];
	UInterchangeSkeletalMeshFactoryNode* SkeletalMeshNode = Cast<UInterchangeSkeletalMeshFactoryNode>(BaseNodeContainer->GetFactoryNode(SkeletalMeshUid));
	if (!SkeletalMeshNode)
	{
		return;
	}

	FString DisplayLabelName = SkeletalMeshNode->GetDisplayLabel();
		
	if (bShouldChangeAssetName)
	{
		DisplayLabelName = AssetName.IsEmpty() ? FPaths::GetBaseFilename(SourceDatas.Num() ? SourceDatas[0]->GetFilename() : TEXT("No Source File Specified")) : AssetName;
		SkeletalMeshNode->SetDisplayLabel(DisplayLabelName);
	}

	//Also set the skeleton factory node name
	TArray<FString> LodDataUids;
	SkeletalMeshNode->GetLodDataUniqueIds(LodDataUids);
	if (LodDataUids.Num() > 0)
	{
		//Get the skeleton from the base LOD, skeleton is shared with all LODs
		if (const UInterchangeSkeletalMeshLodDataNode* SkeletalMeshLodDataNode = Cast<const UInterchangeSkeletalMeshLodDataNode>(BaseNodeContainer->GetFactoryNode(LodDataUids[0])))
		{
			//If the user did not specify any skeleton
			if (!CommonSkeletalMeshesAndAnimationsProperties->IsSkeletonValid())
			{
				FString SkeletalMeshSkeletonUid;
				SkeletalMeshLodDataNode->GetCustomSkeletonUid(SkeletalMeshSkeletonUid);
				UInterchangeSkeletonFactoryNode* SkeletonFactoryNode = Cast<UInterchangeSkeletonFactoryNode>(BaseNodeContainer->GetFactoryNode(SkeletalMeshSkeletonUid));
				if (SkeletonFactoryNode)
				{
					const FString SkeletonName = DisplayLabelName + TEXT("_Skeleton");
					SkeletonFactoryNode->SetDisplayLabel(SkeletonName);
				}
			}
		}
	}
	const UClass* PhysicsAssetFactoryNodeClass = UInterchangePhysicsAssetFactoryNode::StaticClass();
	TArray<FString> PhysicsAssetNodeUids;
	BaseNodeContainer->GetNodes(PhysicsAssetFactoryNodeClass, PhysicsAssetNodeUids);
	for (const FString& PhysicsAssetNodeUid : PhysicsAssetNodeUids)
	{
		UInterchangePhysicsAssetFactoryNode* PhysicsAssetFactoryNode = Cast<UInterchangePhysicsAssetFactoryNode>(BaseNodeContainer->GetFactoryNode(PhysicsAssetNodeUid));
		if (!ensure(PhysicsAssetFactoryNode))
		{
			continue;
		}
		FString PhysicsAssetSkeletalMeshUid;
		if (PhysicsAssetFactoryNode->GetCustomSkeletalMeshUid(PhysicsAssetSkeletalMeshUid) && PhysicsAssetSkeletalMeshUid.Equals(SkeletalMeshUid))
		{
			//Rename this asset
			const FString PhysicsAssetName = DisplayLabelName + TEXT("_PhysicsAsset");
			PhysicsAssetFactoryNode->SetDisplayLabel(PhysicsAssetName);
		}
	}
}

/*
* Deprecated functions
*/

UInterchangeSkeletalMeshFactoryNode* UInterchangeGenericMeshPipeline::CreateSkeletalMeshFactoryNode(const FString& RootJointUid, const TMap<int32, TArray<FString>>& MeshUidsPerLodIndex)
{
	return nullptr;
}

UInterchangeSkeletalMeshLodDataNode* UInterchangeGenericMeshPipeline::CreateSkeletalMeshLodDataNode(const FString& NodeName, const FString& NodeUniqueID, const FString& ParentNodeUniqueID)
{
	return nullptr;
}

void UInterchangeGenericMeshPipeline::AddLodDataToSkeletalMesh(const UInterchangeSkeletonFactoryNode* SkeletonFactoryNode, UInterchangeSkeletalMeshFactoryNode* SkeletalMeshFactoryNode, const TMap<int32, TArray<FString>>& NodeUidsPerLodIndex)
{
}

bool UInterchangeGenericMeshPipeline::MakeMeshFactoryNodeUidAndDisplayLabel(const TMap<int32, TArray<FString>>& MeshUidsPerLodIndex, int32 LodIndex, FString& NewMeshUid, FString& DisplayLabel, const FString& BaseMeshUid)
{
	return false;
}