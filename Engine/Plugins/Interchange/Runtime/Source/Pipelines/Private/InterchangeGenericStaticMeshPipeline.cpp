// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeGenericMeshPipeline.h"

#include "InterchangeCommonPipelineDataFactoryNode.h"
#include "InterchangeMeshBundleNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangePipelineLog.h"
#include "InterchangePipelineMeshesUtilities.h"
#include "InterchangeSceneComponentNodes.h"
#include "InterchangeSceneNode.h"
#include "InterchangeStaticMeshFactoryNode.h"
#include "InterchangeStaticMeshLodDataNode.h"
#include "Mesh/InterchangeMeshHelper.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeSourceNode.h"
#include "InterchangeLODDataParser.h"

#include "Async/Async.h"
#include "CoreMinimal.h"
#include "Tasks/Task.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

// Returns true if MeshUid describes a Mesh node that is purely the collision mesh of some other mesh
static bool IsJustCollisionMesh(TObjectPtr<UInterchangePipelineMeshesUtilities> PipelineMeshesUtilities, const UInterchangeBaseNodeContainer& NodeContainer, const FString& MeshUid, const TArray<FString>& MeshUids, bool bImportCollisionAccordingToMeshName)
{
	TTuple<EInterchangeMeshCollision, FString> Tuple = UE::Interchange::CollisionHelper::GetCollisionMeshType(PipelineMeshesUtilities, NodeContainer, MeshUid, MeshUids, bImportCollisionAccordingToMeshName);

	// If MeshUid is a collision mesh but *of itself*, then it is both a collision and a render mesh, and we should return false
	return Tuple.Key != EInterchangeMeshCollision::None && Tuple.Value != MeshUid;
}


static void BuildMeshToCollisionMeshMap(TObjectPtr<UInterchangePipelineMeshesUtilities> PipelineMeshesUtilities, const UInterchangeBaseNodeContainer& NodeContainer, const TArray<FString>& MeshUids, TMap<FString, TArray<FString>>& MeshToCollisionMeshMap, bool bImportCollisionAccordingToMeshName)
{
	for (const FString& MeshUid : MeshUids)
	{
		TTuple<EInterchangeMeshCollision, FString> CollisionType = UE::Interchange::CollisionHelper::GetCollisionMeshType(PipelineMeshesUtilities, NodeContainer, MeshUid, MeshUids, bImportCollisionAccordingToMeshName);
		if (CollisionType.Get<0>() != EInterchangeMeshCollision::None)
		{
			MeshToCollisionMeshMap.FindOrAdd(FString(CollisionType.Get<1>())).Emplace(MeshUid);
		}
	}
}


void UInterchangeGenericMeshPipeline::ExecutePreImportPipelineStaticMesh()
{
	check(CommonMeshesProperties.IsValid());

#if WITH_EDITOR
	//Make sure the generic pipeline will cover all staticmesh build settings when we import
	Async(EAsyncExecution::TaskGraphMainThread, []()
		{
			static bool bVerifyBuildProperties = false;
			if (!bVerifyBuildProperties)
			{
				bVerifyBuildProperties = true;
				TArray<const UClass*> Classes;
				Classes.Add(UInterchangeGenericCommonMeshesProperties::StaticClass());
				Classes.Add(UInterchangeGenericMeshPipeline::StaticClass());
				if (!DoClassesIncludeAllEditableStructProperties(Classes, FMeshBuildSettings::StaticStruct()))
				{
					UE_LOGF(LogInterchangePipeline, Log, "UInterchangeGenericMeshPipeline: The generic pipeline does not cover all static mesh build options.");
				}
			}
		});
#endif

	if (UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(BaseNodeContainer))
	{
		// Keep a single triangle threshold for the entire scene
		SourceNode->SetCustomNaniteTriangleThreshold(NaniteTriangleThreshold);
	}

	if (bImportStaticMeshes && (CommonMeshesProperties->ForceAllMeshAsType == EInterchangeForceMeshType::IFMT_None || CommonMeshesProperties->ForceAllMeshAsType == EInterchangeForceMeshType::IFMT_StaticMesh))
	{
		if (CombineStaticMeshesBehavior != EInterchangeCombineStaticMeshesBehavior::DoNotCombine)
		{
			// Combine all the static meshes
			//#interchange_LODRefactor_Note: This does not add the CollisionMeshes to the list, presumably because all MeshUids should already be part of the list passed to the CreateStaticMeshFactoryNode.
			//However that's not entirely true, as if we have MeshInstances, currently there is nothing making sure that the corresponding CollisionMeshes are part of MeshInstance list.
			TArray<FString> StaticMeshInstancingUids;
			PipelineMeshesUtilities->GetAllStaticMeshInstance(StaticMeshInstancingUids, CombineStaticMeshesBehavior == EInterchangeCombineStaticMeshesBehavior::All);

			auto SeaparateRenderAndCollisionUids = [&](const TArray<FString> AllNodeUids, TArray<FString>& RenderUids, TArray<FString>& CollisionUids)
				{
					RenderUids.Reserve(AllNodeUids.Num());

					for (const FString& NodeUid : AllNodeUids)
					{
						TTuple<EInterchangeMeshCollision, FString> Tuple = UE::Interchange::CollisionHelper::GetCollisionMeshType(PipelineMeshesUtilities, *BaseNodeContainer, NodeUid, AllNodeUids, bImportCollisionAccordingToMeshName);

						if (Tuple.Key != EInterchangeMeshCollision::None)
						{
							CollisionUids.Add(NodeUid);
							if (Tuple.Value == NodeUid)
							{
								RenderUids.Add(NodeUid);
							}
						}
						else
						{
							RenderUids.Add(NodeUid);
						}
					}
				};

			if (StaticMeshInstancingUids.Num() > 0)
			{
				TArray<FString> RenderUids;
				TArray<FString> CollisionUids;
				SeaparateRenderAndCollisionUids(StaticMeshInstancingUids, RenderUids, CollisionUids);

				if (UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode = CreateStaticMeshFactoryNode(RenderUids, CollisionUids))
				{
					StaticMeshFactoryNodes.Add(StaticMeshFactoryNode);
				}
			}
			else
			{
				// If we haven't yet managed to build a factory node, look at static mesh geometry directly.
				TArray<FString> StaticMeshUids;
				PipelineMeshesUtilities->GetAllStaticMeshGeometry(StaticMeshUids);

				if (StaticMeshUids.Num() > 0)
				{
					TArray<FString> RenderUids;
					TArray<FString> CollisionUids;
					SeaparateRenderAndCollisionUids(StaticMeshUids, RenderUids, CollisionUids);

					if (UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode = CreateStaticMeshFactoryNode(RenderUids, CollisionUids))
					{
						StaticMeshFactoryNodes.Add(StaticMeshFactoryNode);
					}
				}
			}
		}
		else
		{
			// Do not combine static meshes

			//As NodeUids can come from MeshInstances, they can be SceneNodes. but also can be: UInterchangeMeshNode/UInterchangeMeshLODContainerNode/UInterchangeGeometricCaches
			auto CreateMeshFactoryNodeUnCombined = [this](const TArray<FString>& NodeUids)->bool
				{
					bool bFoundMeshes = false;

					// Work out which meshes are collision meshes which correspond to another mesh
					// This Maps RenderMeshUid to CollisionMeshUids.
					// Original algorithm added the corresponding CollisionMeshUids to the NodeUids list at LODIndex 0.
					TMap<FString, TArray<FString>> MeshToCollisionMeshMap;
					BuildMeshToCollisionMeshMap(PipelineMeshesUtilities, *BaseNodeContainer, NodeUids, MeshToCollisionMeshMap, bImportCollisionAccordingToMeshName);

					// Now iterate through each mesh UID, creating a new factory for each one
					for (const FString& NodeUid : NodeUids)
					{
						// If this is just a collision mesh, don't add a factory node; it will be added as part of another factory node
						if (IsJustCollisionMesh(PipelineMeshesUtilities, *BaseNodeContainer, NodeUid, NodeUids, bImportCollisionAccordingToMeshName))
						{
							continue;
						}

						TArray<FString> CollisionNodeUids;
						if (const TArray<FString>* CorrespondingCollisionMeshes = MeshToCollisionMeshMap.Find(NodeUid))
						{
							for (const FString& CollisionMesh : *CorrespondingCollisionMeshes)
							{
								CollisionNodeUids.Add(CollisionMesh);
							}
						}

						if (UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode = CreateStaticMeshFactoryNode({ NodeUid }, CollisionNodeUids))
						{
							StaticMeshFactoryNodes.Add(StaticMeshFactoryNode);
							bFoundMeshes = true;
						}
					}
					return bFoundMeshes;
				};

			TArray<FString> NodeUids;

			PipelineMeshesUtilities->GetAllStaticMeshInstance(NodeUids);
			if (!CreateMeshFactoryNodeUnCombined(NodeUids))
			{
				NodeUids.Reset();
				PipelineMeshesUtilities->GetAllStaticMeshGeometry(NodeUids);
				CreateMeshFactoryNodeUnCombined(NodeUids);
			}
		}
	}

	CreateAssemblyPartDependencies();
}

namespace UE::InterchangeStaticMeshFactory::Private
{
	void ApplyNaniteTriangleThreshold(
		const UInterchangeStaticMeshFactoryNode* FactoryNode,
		UStaticMesh* StaticMesh,
		const UInterchangeBaseNodeContainer* NodeContainer
	)
	{
#if WITH_EDITORONLY_DATA
		if (!FactoryNode || !NodeContainer || !StaticMesh || !StaticMesh->GetNaniteSettings().bEnabled)
		{
			return;
		}

		int64 TriangleThreshold = 0;
		if (const UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::GetUniqueInstance(NodeContainer))
		{
			SourceNode->GetCustomNaniteTriangleThreshold(TriangleThreshold);
		}
		if (TriangleThreshold <= 0)
		{
			return;
		}

		const int32 LodIndex = 0;
		FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(LodIndex);
		if (!MeshDescription)
		{
			return;
		}

		if (MeshDescription->Triangles().Num() < TriangleThreshold)
		{
			StaticMesh->GetNaniteSettings().bEnabled = false;
		}
#endif	  // #if WITH_EDITORONLY_DATA
	}
}	 // namespace UE::InterchangeStaticMeshFactory::Private

void UInterchangeGenericMeshPipeline::ExecutePostFactoryPipelineStaticMesh(
	const UInterchangeBaseNodeContainer* InBaseNodeContainer,
	const FString& NodeKey,
	UObject* CreatedAsset,
	bool bIsAReimport
)
{
	UStaticMesh* StaticMesh = Cast<UStaticMesh>(CreatedAsset);
	if (!StaticMesh)
	{
		return;
	}

	UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode = Cast<UInterchangeStaticMeshFactoryNode>(InBaseNodeContainer->GetFactoryNode(NodeKey));
	if (!StaticMeshFactoryNode)
	{
		return;
	}

	// If we're reimporting and being told to leave the properties alone, just early out instead
	if (bIsAReimport)
	{
		EReimportStrategyFlags Strategy = StaticMeshFactoryNode->GetReimportStrategyFlags();
		if (Strategy == EReimportStrategyFlags::ApplyNoProperties)
		{
			return;
		}
	}

	UE::InterchangeStaticMeshFactory::Private::ApplyNaniteTriangleThreshold(StaticMeshFactoryNode, StaticMesh, InBaseNodeContainer);
}

//Note: bCombineStaticMesh and bBakeMesh, both flags play a role in Name and UID generation, in the following manner:
// This gives 4 variations:
//	- 1.) (bCombineStaticMeshes && bBakeMeshes)
//			We use the original SKM uid generation's approach which is 'first node' based.
//	- 2.) (bCombineStaticMeshes && !bBakeMeshes)
//			We use the original SKM uid generation's approach, which 'first node' based, however with the addition that we shall check for the nested assets as well.
//	- 3.) (!bCombineStaticMeshes && bBakeMeshes)
//			Make sure that the Array.Num() == 1, as we are not combining the staticMeshes, we shouldn't have more than 1 in the input.
//			Uid and Name should be generated from the single entry directly.
//	- 4.) (!bCombineStaticMeshes && !bBakeMeshes)
//			Make sure that the Array.Num() == 1, as we are not combining the staticMeshes, we shouldn't have more than 1 in the input.
//			Uid and Name should be generated from the AssetInstanceUid's UID and Name, in other words we shall check for the nest assets as well..
//
//Note2:
//	-AssetType nodes: (UInterchangeMeshNode/UInterchangeMeshLODContainerNode/UInterchangeGeometricCaches)
//	-non-AssetType nodes: (UInterchangeSceneNode/UInterchangeInstancedStaticMeshNode)
//Note3:
//	- bCombineStaticMeshes should probably have a UID and Name coming from the Import file as a whole as there should only be 1 Mesh as we are combining all StaticMeshes to 1 when this option is on.
//Note4:
//	- Worth noting, that if the bBakeMeshes:=true then non-Asset type nodes will generate their respective factory nodes, which might be confusing without the context of pipeline settings.
//		for ep: UInterchangeInstancedStaticMeshComponentNode would generate the same factoryUid for the StaticMeshFactoryNode as the UInterchangeInstancedStaticMeshComponentFactoryNode.
bool UInterchangeGenericMeshPipeline::MakeMeshFactoryNodeUidAndDisplayLabel(const TArray<FString>& NodeUids, FString& OutFactoryNodeUid, FString& OutFactoryNodeName) const
{
	//For short if statements:
	const bool bBakeMeshes = CommonMeshesProperties->bBakeMeshes;

	struct FUidName
	{
		FString Uid;
		FString Name;
		FUidName(const FString& InUid, const FString& InName)
			: Uid(InUid)
			, Name(InName)
		{
		}
	};

	auto CheckNestedness = [this](const FString& UidCandidate, FString& OutUid, FString& OutDisplayLabel) -> bool
		{
			if (const UInterchangeBaseNode* BaseNode = BaseNodeContainer->GetNode(UidCandidate))
			{
				if (const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(BaseNode))
				{
					FString AssetInstanceUid;
					if (SceneNode->GetCustomAssetInstanceUid(AssetInstanceUid))
					{
						if (const UInterchangeBaseNode* AssetBaseNode = BaseNodeContainer->GetNode(AssetInstanceUid))
						{
							OutUid = AssetInstanceUid;
							OutDisplayLabel = AssetBaseNode->GetDisplayLabel();

							return true;
						}
					}
				}
				else if (const UInterchangeInstancedStaticMeshComponentNode* ISMComponentNode = Cast<UInterchangeInstancedStaticMeshComponentNode>(BaseNode))
				{
					FString InstancedAssetUid;
					if (ISMComponentNode->GetCustomInstancedAssetUid(InstancedAssetUid))
					{
						if (const UInterchangeBaseNode* AssetBaseNode = BaseNodeContainer->GetNode(InstancedAssetUid))
						{
							OutUid = InstancedAssetUid;
							OutDisplayLabel = AssetBaseNode->GetDisplayLabel();

							return true;
						}
					}
				}
				else
				{
					//if its not sceneNode/ismcomponentnode its an assetNode (UInterchagneMeshNode/UInterchangeMeshLODContainerNode/UInterchangeGeometricCaches/UInterchangeMeshBundleNode)
					OutUid = UidCandidate;
					OutDisplayLabel = BaseNode->GetDisplayLabel();

					return true;
				}
			}

			return false;
		};

	auto GetFirstNodeUid = [&NodeUids]() -> FString
		{
			FString FirstNodeUid = NodeUids.Num() > 0 ? (NodeUids)[0] : FString();
			for (int32 Index = 1; Index < NodeUids.Num(); ++Index)
			{
				if (NodeUids[Index] < FirstNodeUid)
				{
					FirstNodeUid = NodeUids[Index];
				}
			}

			return FirstNodeUid;
		};

	auto SetUidAndDisplayLabel = [&OutFactoryNodeUid, &OutFactoryNodeName](const FString& Uid, const FString& Name)
		{
			OutFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(Uid);
			OutFactoryNodeName = Name;
		};
	
	if ((CombineStaticMeshesBehavior != EInterchangeCombineStaticMeshesBehavior::DoNotCombine) && bBakeMeshes)
	{
		FString NodeUid = GetFirstNodeUid();

		if (const UInterchangeBaseNode* BaseNode = BaseNodeContainer->GetNode(NodeUid))
		{
			SetUidAndDisplayLabel(NodeUid, BaseNode->GetDisplayLabel());
		}
	}
	else if ((CombineStaticMeshesBehavior != EInterchangeCombineStaticMeshesBehavior::DoNotCombine) && !bBakeMeshes)
	{
		FString Uid;
		FString DisplayLabel;
		if (CheckNestedness(GetFirstNodeUid(), Uid, DisplayLabel))
		{
			SetUidAndDisplayLabel(Uid, DisplayLabel);
		}
	}
	else if ((CombineStaticMeshesBehavior == EInterchangeCombineStaticMeshesBehavior::DoNotCombine) && bBakeMeshes)
	{
		if (!ensure(NodeUids.Num() == 1))
		{
			return false;
		}

		if (const UInterchangeBaseNode* BaseNode = BaseNodeContainer->GetNode(NodeUids[0]))
		{
			SetUidAndDisplayLabel(NodeUids[0], BaseNode->GetDisplayLabel());
		}
	}
	else if ((CombineStaticMeshesBehavior == EInterchangeCombineStaticMeshesBehavior::DoNotCombine) && !bBakeMeshes)
	{
		if (!ensure(NodeUids.Num() == 1))
		{
			return false;
		}

		FString Uid;
		FString DisplayLabel;

		if (CheckNestedness(NodeUids[0], Uid, DisplayLabel))
		{
			SetUidAndDisplayLabel(Uid, DisplayLabel);
		}
	}

	if (OutFactoryNodeUid.Len() == 0)
	{
		//Can happen when input NodeUid is not a node inheriting from UInterchangeBaseNode 
		//Or when the UInterchangeSceneNode does not have a AssetInstanceUid.
		//Neither of these scenarios are expected, so adding an ensure here:
		ensure(false);

		return false;
	}

	return true;
}

void UInterchangeGenericMeshPipeline::ApplyMeshProperties(UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode)
{
	//Set the pipeline import sockets property
	StaticMeshFactoryNode->SetCustomImportSockets(CommonMeshesProperties->bImportSockets);

	if (CommonMeshesProperties->bKeepSectionsSeparate)
	{
		StaticMeshFactoryNode->SetCustomKeepSectionsSeparate(CommonMeshesProperties->bKeepSectionsSeparate);
	}

	switch (CommonMeshesProperties->VertexColorImportOption)
	{
		case EInterchangeVertexColorImportOption::IVCIO_Replace:
		{
			StaticMeshFactoryNode->SetCustomVertexColorReplace(true);
		}
		break;
		case EInterchangeVertexColorImportOption::IVCIO_Ignore:
		{
			StaticMeshFactoryNode->SetCustomVertexColorIgnore(true);
		}
		break;
		case EInterchangeVertexColorImportOption::IVCIO_Override:
		{
			StaticMeshFactoryNode->SetCustomVertexColorOverride(CommonMeshesProperties->VertexOverrideColor);
		}
		break;
	}

	StaticMeshFactoryNode->SetCustomLODGroup(LodGroup);

	//Common meshes build options
	StaticMeshFactoryNode->SetCustomRecomputeNormals(CommonMeshesProperties->bRecomputeNormals);
	StaticMeshFactoryNode->SetCustomRecomputeTangents(CommonMeshesProperties->bRecomputeTangents);
	StaticMeshFactoryNode->SetCustomUseMikkTSpace(CommonMeshesProperties->bUseMikkTSpace);
	StaticMeshFactoryNode->SetCustomComputeWeightedNormals(CommonMeshesProperties->bComputeWeightedNormals);
	StaticMeshFactoryNode->SetCustomUseHighPrecisionTangentBasis(CommonMeshesProperties->bUseHighPrecisionTangentBasis);
	StaticMeshFactoryNode->SetCustomUseFullPrecisionUVs(CommonMeshesProperties->bUseFullPrecisionUVs);
	StaticMeshFactoryNode->SetCustomUseBackwardsCompatibleF16TruncUVs(CommonMeshesProperties->bUseBackwardsCompatibleF16TruncUVs);
	StaticMeshFactoryNode->SetCustomRemoveDegenerates(CommonMeshesProperties->bRemoveDegenerates);

	//Static meshes build options
	StaticMeshFactoryNode->SetCustomBuildReversedIndexBuffer(bBuildReversedIndexBuffer);
	StaticMeshFactoryNode->SetCustomGenerateLightmapUVs(bGenerateLightmapUVs);
	StaticMeshFactoryNode->SetCustomGenerateDistanceFieldAsIfTwoSided(bGenerateDistanceFieldAsIfTwoSided);
	StaticMeshFactoryNode->SetCustomSupportFaceRemap(bSupportFaceRemap);
	StaticMeshFactoryNode->SetCustomMinLightmapResolution(MinLightmapResolution);
	StaticMeshFactoryNode->SetCustomSrcLightmapIndex(SrcLightmapIndex);
	StaticMeshFactoryNode->SetCustomDstLightmapIndex(DstLightmapIndex);
	StaticMeshFactoryNode->SetCustomBuildScale3D(BuildScale3D);
	StaticMeshFactoryNode->SetCustomDistanceFieldResolutionScale(DistanceFieldResolutionScale);
	StaticMeshFactoryNode->SetCustomDistanceFieldReplacementMesh(DistanceFieldReplacementMesh.Get());
	StaticMeshFactoryNode->SetCustomMaxLumenMeshCards(MaxLumenMeshCards);
	StaticMeshFactoryNode->SetCustomBuildNanite(bBuildNanite);
	StaticMeshFactoryNode->SetCustomAutoComputeLODScreenSizes(bAutoComputeLODScreenSizes);
	StaticMeshFactoryNode->SetLODScreenSizes(LODScreenSizes);
}

UInterchangeStaticMeshFactoryNode* UInterchangeGenericMeshPipeline::CreateStaticMeshFactoryNode(const TArray<FString>& NodeUids, const TArray<FString>& CollisionMeshUids)
{
	if (!ensure(CommonMeshesProperties.IsValid()))
	{
		return nullptr;
	}
	if (NodeUids.Num() == 0)
	{
		return nullptr;
	}

	// Create the static mesh factory node, name it according to the first mesh node compositing the meshes
	FString FactoryNodeUid;
	FString FactoryDisplayLabel;
	if (!MakeMeshFactoryNodeUidAndDisplayLabel(NodeUids, FactoryNodeUid, FactoryDisplayLabel))
	{
		// Log an error
		return nullptr;
	}

	UInterchangeFactoryBaseNode* FactoryBaseNode = BaseNodeContainer->GetFactoryNode(FactoryNodeUid);
	UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode = Cast<UInterchangeStaticMeshFactoryNode>(FactoryBaseNode);

	if (StaticMeshFactoryNode)
	{
		return StaticMeshFactoryNode;
	}

	StaticMeshFactoryNode = NewObject<UInterchangeStaticMeshFactoryNode>(BaseNodeContainer, NAME_None);
	if (!ensure(StaticMeshFactoryNode))
	{
		return nullptr;
	}

	StaticMeshFactoryNode->InitializeStaticMeshNode(FactoryNodeUid, FactoryDisplayLabel, UStaticMesh::StaticClass()->GetName(), BaseNodeContainer);
	ApplyMeshProperties(StaticMeshFactoryNode);

	FInterchangeMeshLODDataParser MeshLODDataParser(*BaseNodeContainer, StaticMeshFactoryNode, PipelineMeshesUtilities, this, SourceDatas.Num() ? SourceDatas[0]->GetFilename() : TEXT(""));
	MeshLODDataParser.ProcessNodeUids(NodeUids, CollisionMeshUids);

	return StaticMeshFactoryNode;
}

/*
* Deprecated functions
*/

UInterchangeStaticMeshFactoryNode* UInterchangeGenericMeshPipeline::CreateStaticMeshFactoryNode(const TMap<int32, TArray<FString>>& MeshUidsPerLodIndex, const FString& BaseMeshUid)
{
	return nullptr;
}

UInterchangeStaticMeshLodDataNode* UInterchangeGenericMeshPipeline::FindOrCreateStaticMeshLodDataNode(UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode, const int32 LODIndex)
{
	return nullptr;
}

void UInterchangeGenericMeshPipeline::AddLodDataToStaticMesh(UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode, const TMap<int32, TArray<FString>>& NodeUidsPerLodIndex)
{
}

