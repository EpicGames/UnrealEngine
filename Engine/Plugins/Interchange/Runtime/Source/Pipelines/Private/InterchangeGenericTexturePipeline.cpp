// Copyright Epic Games, Inc. All Rights Reserved. 

#include "InterchangeGenericTexturePipeline.h"

#include "Async/ParallelFor.h"
#include "Engine/Texture.h"
#include "Engine/TextureCube.h"
#include "Engine/TextureLightProfile.h"
#include "ImageUtils.h"
#include "InterchangePipelineLog.h"
#include "InterchangeTexture2DArrayFactoryNode.h"
#include "InterchangeTexture2DArrayNode.h"
#include "InterchangeTexture2DFactoryNode.h"
#include "InterchangeTexture2DNode.h"
#include "InterchangeTextureBlurNode.h"
#include "InterchangeTextureCubeArrayFactoryNode.h"
#include "InterchangeTextureCubeArrayNode.h"
#include "InterchangeTextureCubeFactoryNode.h"
#include "InterchangeTextureCubeNode.h"
#include "InterchangeTextureFactoryNode.h"
#include "InterchangeTextureLightProfileFactoryNode.h"
#include "InterchangeTextureLightProfileNode.h"
#include "InterchangeTextureNode.h"
#include "InterchangeVolumeTextureNode.h"
#include "InterchangeVolumeTextureFactoryNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeGenericTexturePipeline)

#if WITH_EDITOR
#include "NormalMapIdentification.h"
#include "TextureCompiler.h"
#include "UDIMUtilities.h"
#endif //WITH_EDITOR

int32 GInterchangeMinTextureCubeMapDetection = 512;
static FAutoConsoleVariableRef CVarInterchangeMinTextureCubeMapDetection(
	TEXT("Interchange.Textures.MinCubeMapDetection"),
	GInterchangeMinTextureCubeMapDetection,
	TEXT("The minimum size to auto detect that an image is a long-lat cubemap texture during import. Default: 512, Minimum value: 8")
);

static FAutoConsoleVariableSink CVarInterchangeMinTextureCubeMapDetectionSink
(FConsoleCommandDelegate::CreateLambda([]()
 {
	 int32 MinValue = CVarInterchangeMinTextureCubeMapDetection->GetInt();
	 if (MinValue < 8)
	 {
		 CVarInterchangeMinTextureCubeMapDetection->Set(8, ECVF_SetByConsole);
		 UE_LOGF(LogInterchangePipeline, Warning, "Interchange.Textures.MinCubeMapDetection cannot be less than 8. Reset to 8.");
	 }
 }
));

namespace UE::Interchange::Private
{
	UClass* GetDefaultFactoryClassFromTextureNodeClass(UClass* NodeClass)
	{
		if (UInterchangeTexture2DNode::StaticClass() == NodeClass)
		{
			return UInterchangeTexture2DFactoryNode::StaticClass();
		}

		if (UInterchangeTextureCubeNode::StaticClass() == NodeClass)
		{
			return UInterchangeTextureCubeFactoryNode::StaticClass();
		}

		if (UInterchangeTextureCubeArrayNode::StaticClass() == NodeClass)
		{
			return UInterchangeTextureCubeArrayFactoryNode::StaticClass();
		}

		if (UInterchangeTexture2DArrayNode::StaticClass() == NodeClass)
		{
			return UInterchangeTexture2DArrayFactoryNode::StaticClass();
		}

		if (UInterchangeTextureLightProfileNode::StaticClass() == NodeClass)
		{
			return UInterchangeTextureLightProfileFactoryNode::StaticClass();
		}

		if (UInterchangeVolumeTextureNode::StaticClass() == NodeClass)
		{
			return UInterchangeVolumeTextureFactoryNode::StaticClass();
		}

		if(UInterchangeTextureBlurNode::StaticClass() == NodeClass)
		{
			return UInterchangeTexture2DFactoryNode::StaticClass();
		}

		return nullptr;
	}

#if WITH_EDITOR
	void AdjustTextureForNormalMap(UTexture* Texture, FImageView MipToAnalyze, bool bFlipNormalMapGreenChannel)
	{
		if (Texture)
		{
			if (UE::NormalMapIdentification::HandleAssetPostImport(Texture, MipToAnalyze))
			{
				UE_LOGF(LogInterchangePipeline, Display, "Auto-detected normal map");

				if (bFlipNormalMapGreenChannel)
				{
					Texture->bFlipGreenChannel = true;
				}
			}
			// this will rebuild the texture if it changed to normal map
		}
	}
#endif

	TextureAddress ConvertWrap(const EInterchangeTextureWrapMode WrapMode)
	{
		switch (WrapMode)
		{
		case EInterchangeTextureWrapMode::Wrap:
			return TA_Wrap;
		case EInterchangeTextureWrapMode::Clamp:
			return TA_Clamp;
		case EInterchangeTextureWrapMode::Mirror:
			return TA_Mirror;

		default:
			ensureMsgf(false, TEXT("Unkown Interchange Texture Wrap Mode"));
			return TA_Wrap;
		}
	}

	// Compute linear rec709 luminance
	float ComputeLuminance(const FLinearColor& Color)
	{
		return 0.2126f * Color.R + 0.7152f * Color.G + 0.0722f * Color.B;
	}
	
	/** Compute the global average luminance of an image */
	double ComputeGlobalLuminance(const TArrayView64<FLinearColor>& Pixels, int32 Width, int32 Height)
	{
		TArray<double> LocalLuminances;
		LocalLuminances.SetNum(Height);
				
		ParallelFor
		(Height, [&](int32 Y)
		 {
			 double RowSum = 0.0;
			 for (int32 X = 0; X < Width; ++X)
			 {
				 RowSum += ComputeLuminance(Pixels[Y * Width + X]);
			 }
			 LocalLuminances[Y] = RowSum;
		 });

		double GlobalLuminance = 0;
		for (double Luminance : LocalLuminances)
		{
			GlobalLuminance += Luminance;
		}

		GlobalLuminance /= (Width * Height);
		return GlobalLuminance;
	}

	/** 
	 * For equirectangular maps, since there's a horizontal wrap then the edges should be very similar
	 * We both compare color and luminance average differences
	 */
	bool CompareEdgeSimilarity(const TArrayView64<FLinearColor>& Pixels, int32 Width, int32 Height)
	{
		struct FLocalEdgeDiff
		{
			double RGB = 0.0;
			double Luminance = 0.0;
		};

		TArray<FLocalEdgeDiff> EdgeDiffs;
		EdgeDiffs.SetNumZeroed(Height);

		ParallelFor
		(Height, [&](int32 Y)
		 {
			 FLocalEdgeDiff& EdgeDiff= EdgeDiffs[Y];

			 const FLinearColor& Left = Pixels[Y * Width];
			 const FLinearColor& Right = Pixels[Y * Width + (Width - 1)];

			 const float DeltaR = Left.R - Right.R;
			 const float DeltaG = Left.G - Right.G;
			 const float DeltaB = Left.B - Right.B;

			 const float LLeft = ComputeLuminance(Left);
			 const float LRight = ComputeLuminance(Right);

			 EdgeDiff.Luminance += FMath::Abs(LLeft - LRight);
			 EdgeDiff.RGB += FMath::Sqrt(DeltaR * DeltaR + DeltaG * DeltaG + DeltaB * DeltaB);
		 });

		double MeanErrorRGB = 0.0;
		double MeanErrorLuminance = 0.0;

		for (const FLocalEdgeDiff& Pair : EdgeDiffs)
		{
			MeanErrorRGB += Pair.RGB;
			MeanErrorLuminance += Pair.Luminance;
		}

		MeanErrorRGB /= Height;
		MeanErrorLuminance /= Height;

		return MeanErrorRGB < 0.1 && MeanErrorLuminance < 0.1;
	}

	/**
	 * Compute the variance of the luminance using Welford's online algorithm
	 */
	double ComputeLuminanceVarianceWelford(const TArrayView64<FLinearColor>& Pixels, int32 Width, int32 YStart, int32 YEnd)
	{
		struct FLocalVariance
		{
			double Mean = 0.0;
			double M2 = 0.0; // Sum of squares of differences from the current mean
			int64 Count = 0;

			void Accumulate(double Value)
			{
				++Count;
				double Delta = Value - Mean;
				Mean += Delta / Count;
				double Delta2 = Value - Mean;
				M2 += Delta * Delta2;
			}

			void Combine(const FLocalVariance& Other)
			{
				if (Other.Count == 0)
				{
					return;
				}

				double Delta = Other.Mean - Mean;
				int64 NewCount = Count + Other.Count;

				M2 += Other.M2 + Delta * Delta * Count * Other.Count / NewCount;
				Mean += Delta * Other.Count / NewCount;
				Count = NewCount;
			}

			double Variance() const
			{
				return (Count > 1) ? M2 / (Count - 1) : 0.0;
			}
		};


		int32 SampleHeight = YEnd - YStart;
		TArray<FLocalVariance> LocalStats;
		LocalStats.SetNum(SampleHeight);

		const double Epsilon = 1e-6;

		ParallelFor(
			SampleHeight,
			[&Pixels, Width, &LocalStats, YStart, Epsilon](int32 LocalY)
			{
				int32 Y = YStart + LocalY;
				FLocalVariance Stats;

				for (int32 X = 0; X < Width; ++X)
				{
					// use log to flatten the values in case of high disparities
					const double L = FMath::Loge(FMath::Max(ComputeLuminance(Pixels[Y * Width + X]), Epsilon));
					Stats.Accumulate(L);
				}

				LocalStats[LocalY] = Stats;
			});

		// Combine all local stats into a global one
		FLocalVariance GlobalStats;
		for (const FLocalVariance& Stats : LocalStats)
		{
			GlobalStats.Combine(Stats);
		}

		return GlobalStats.Variance();
	}

	/**
	 * Usually for equirectangular textures, the polar regions(top / bottom) appear visually compressed
	 * which results in lower luminance variance compared to the equatorial (middle)
	 * We detect this by comparing the pixels distribution at the poles (10% of the whole image) against the equatorial (80% of the rest of the image)
	 * If we have a pole repartion lesser than 70% for the equatorial then it should be a good candidate
	 */
	bool HasPoleCompression(const TArrayView64<FLinearColor>& Pixels, int32 Width, int32 Height)
	{
		const int32 Pole = FMath::Clamp(Height / 10, 1, Height / 4); // avoid 0 or really small bands
		const int32 MiddleStart = Pole;
		const int32 MiddleEnd = Height - Pole;

		// We compute the global luminance of the image, probably a bit overkill
		// but it can help mitigate high discrepancies between bright and dark areas making it intensity and exposure-independent
		const double Luminance = ComputeGlobalLuminance(Pixels, Width, Height);
		double Luminance2 = FMath::Max(Luminance * Luminance, 1e-6);

		double VarianceTop = ComputeLuminanceVarianceWelford(Pixels, Width, 0, Pole) / Luminance2;
		double VarianceMiddle = ComputeLuminanceVarianceWelford(Pixels, Width, MiddleStart, MiddleEnd) / Luminance2;
		double VarianceBottom = ComputeLuminanceVarianceWelford(Pixels, Width, Height - Pole, Height) / Luminance2;

		// 80% is probably too permissive?
		double VarianceTolerance = VarianceMiddle * 0.8;
		if (VarianceTop < VarianceTolerance && VarianceBottom < VarianceTolerance)
		{
			return true;
		}

		// the first test might be sometimes too restrictive whereas we still have an equirectangular map
		// for example for hdr file with bright light on top (like studios or indoor with neons)
		// the variance at the top can be a bit greater than 80% (probably a bit too permissive) of the variance of the middle
		// by averaging the poles we ensure that at least we're not exceluding a true equirectangular map
		return (0.5 * (VarianceTop + VarianceBottom)) < VarianceTolerance;
	}

	/**
	 * A texture should be a cubemap texture if:
	 * - its ratio is 2:1 (we try to not be as strict as possible, giving a very small marge of error)
	 * - it should wrap horizontally without seams
	 * - if the poles (top/bottom of the image) appear more compressed than the middle 
	 */
	bool IsEquirectangular(const FString& SourceFile)
	{
		using namespace UE::Interchange::Private;

		bool bEquirectangular = false;

		FImage Image, ImageFloat32;
		const TCHAR* Filename = *SourceFile;
		if (FImageUtils::LoadImage(Filename, Image))
		{
			// aspect ratio of equirectangular maps should be 2:1
			const bool bIsAspectRatio_2_1 =
				Image.GetWidth() > 0 &&
				Image.GetHeight() >= GInterchangeMinTextureCubeMapDetection &&
				Image.GetNumPixels() &&
				FMath::IsNearlyEqual(float(Image.GetWidth()) / float(Image.GetHeight()), 2.);

			if (bIsAspectRatio_2_1)
			{
				TArrayView64<FLinearColor> Pixels;
				// for a proper detection we want a full float image format in linear space which is generally the case for .hdr or.exr image
				// but definitely not for jpg or png, and we want to avoid unnecessary conversions
				if (Image.Format == ERawImageFormat::RGBA32F && Image.GammaSpace == EGammaSpace::Linear)
				{
					Pixels = Image.AsRGBA32F();
				}
				else
				{
					Image.CopyTo(ImageFloat32, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
					Pixels = ImageFloat32.AsRGBA32F();
				}
				
				bool bHasEdgeSimilarity = CompareEdgeSimilarity(Pixels, Image.GetWidth(), Image.GetHeight());
				bool bHasPoleCompression = HasPoleCompression(Pixels, Image.GetWidth(), Image.GetHeight());
				bEquirectangular = (bHasEdgeSimilarity && bHasPoleCompression);
			}
		}
		return bEquirectangular;
	}
}

FString UInterchangeGenericTexturePipeline::GetPipelineCategory(UClass* AssetClass)
{
	return TEXT("Textures");
}

void UInterchangeGenericTexturePipeline::AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams)
{
	Super::AdjustSettingsForContext(ContextParams);
#if WITH_EDITOR
	TArray<FString> HideCategories;
	bool bIsObjectATexture = !ContextParams.ReimportAsset ? false : ContextParams.ReimportAsset.IsA(UTexture::StaticClass());
	if( (!bIsObjectATexture && ContextParams.ContextType == EInterchangePipelineContext::AssetReimport)
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomLODImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomLODReimport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetAlternateSkinningImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetAlternateSkinningReimport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomMorphTargetImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomMorphTargetReImport)
	{
		bImportTextures = false;
		HideCategories.Add(UInterchangeGenericTexturePipeline::GetPipelineCategory(nullptr));
	}
	if (UInterchangePipelineBase* OuterMostPipeline = GetMostPipelineOuter())
	{
		for (const FString& HideCategoryName : HideCategories)
		{
			HidePropertiesOfCategory(OuterMostPipeline, this, HideCategoryName);
		}
	}
	
#endif //WITH_EDITOR
#if WITH_EDITORONLY_DATA
	bIsReimport = ContextParams.ReimportAsset != nullptr;
#endif
}

#if WITH_EDITOR

bool UInterchangeGenericTexturePipeline::IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UInterchangeGenericTexturePipeline, bImportTextures))
	{
		return true;
	}
	return Super::IsPropertyChangeNeedRefresh(PropertyChangedEvent);
}

void UInterchangeGenericTexturePipeline::FilterPropertiesFromTranslatedData(UInterchangeBaseNodeContainer* InBaseNodeContainer)
{
	Super::FilterPropertiesFromTranslatedData(InBaseNodeContainer);

	//Filter all material pipeline properties if there is no translated material.
	TArray<FString> TmpTextureNodes;
	InBaseNodeContainer->GetNodes(UInterchangeTextureNode::StaticClass(), TmpTextureNodes);
	if (TmpTextureNodes.Num() == 0)
	{
		//Filter out all Textures properties
		if (UInterchangePipelineBase* OuterMostPipeline = GetMostPipelineOuter())
		{
			HidePropertiesOfCategory(OuterMostPipeline, this, UInterchangeGenericTexturePipeline::GetPipelineCategory(nullptr));
		}
	}
}

void UInterchangeGenericTexturePipeline::GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const
{
	PipelineSupportAssetClasses.Add(UTexture::StaticClass());
}

#endif //WITH_EDITOR

void UInterchangeGenericTexturePipeline::ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath)
{
	if (!InBaseNodeContainer)
	{
		UE_LOGF(LogInterchangePipeline, Warning, "UInterchangeGenericTexturePipeline: Cannot execute pre-import pipeline because InBaseNodeContrainer is null");
		return;
	}

	BaseNodeContainer = InBaseNodeContainer;
	SourceDatas.Empty(InSourceDatas.Num());
	for (const UInterchangeSourceData* SourceData : InSourceDatas)
	{
		SourceDatas.Add(SourceData);
	}
	
	//Find all translated node we need for this pipeline
	BaseNodeContainer->GetNodesOfType<UInterchangeTextureNode>(TextureNodes);

	if (bImportTextures)
	{
		UInterchangeTextureFactoryNode* TextureFactoryNode = nullptr;
		for (UInterchangeTextureNode* TextureNode : TextureNodes)
		{
			TextureFactoryNode = HandleCreationOfTextureFactoryNode(TextureNode);
		}
		//If we have a valid override name
		FString OverrideAssetName = IsStandAlonePipeline() ? DestinationName : FString();
		if (OverrideAssetName.IsEmpty() && IsStandAlonePipeline())
		{
			OverrideAssetName = AssetName;
		}

		const bool bOverrideAssetName = TextureNodes.Num() == 1 && IsStandAlonePipeline() && !OverrideAssetName.IsEmpty();
		if (TextureFactoryNode && bOverrideAssetName)
		{
			TextureFactoryNode->SetAssetName(OverrideAssetName);
			TextureFactoryNode->SetDisplayLabel(OverrideAssetName);
		}
	}
}

void UInterchangeGenericTexturePipeline::ExecutePostFactoryPipeline(const UInterchangeBaseNodeContainer* InBaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport)
{
	//We do not use the provided base container since ExecutePreImportPipeline cache it
	//We just make sure the same one is pass in parameter
	if (!InBaseNodeContainer || !ensure(BaseNodeContainer == InBaseNodeContainer) || !CreatedAsset)
	{
		return;
	}

	const UInterchangeFactoryBaseNode* Node = BaseNodeContainer->GetFactoryNode(NodeKey);
	if (!Node)
	{
		return;
	}

	PostImportTextureAssetImport(CreatedAsset, bIsAReimport);
}

UInterchangeTextureFactoryNode* UInterchangeGenericTexturePipeline::HandleCreationOfTextureFactoryNode(const UInterchangeTextureNode* TextureNode)
{
	UClass* FactoryClass = UE::Interchange::Private::GetDefaultFactoryClassFromTextureNodeClass(TextureNode->GetClass());

	// Handle the Texture2D as a TextureCube if we should
	TOptional<FString> SourceFile = TextureNode->GetPayLoadKey();
#if WITH_EDITORONLY_DATA
	if (FactoryClass == UInterchangeTexture2DFactoryNode::StaticClass())
	{
		bool bForceLongLatCubemap = false;
		const UInterchangeTexture2DNode* Texture2DNode = Cast<const UInterchangeTexture2DNode>(TextureNode);
		if (Texture2DNode && Texture2DNode->GetForceLongLatCubemap(bForceLongLatCubemap) && bForceLongLatCubemap)
		{
			FactoryClass = UInterchangeTextureCubeFactoryNode::StaticClass();
		}
		else if (SourceFile)
		{
			using namespace UE::Interchange::Private;
			if (bDetectLongLatCubemap)
			{
				if(IsEquirectangular(*SourceFile))
				{
					FactoryClass = UInterchangeTextureCubeFactoryNode::StaticClass();
				}
			}
			else
			{
				const FString Extension = FPaths::GetExtension(SourceFile.GetValue()).ToLower();
				if (FileExtensionsToImportAsLongLatCubemap.Contains(Extension))
				{
					FactoryClass = UInterchangeTextureCubeFactoryNode::StaticClass();
				}
			}
		}
	}
#endif

	UInterchangeTextureFactoryNode* InterchangeTextureFactoryNode =  CreateTextureFactoryNode(TextureNode, FactoryClass);

	if (FactoryClass == UInterchangeTexture2DFactoryNode::StaticClass() && InterchangeTextureFactoryNode)
	{
		// Forward the UDIM from the translator to the factory node
		TMap<int32, FString> SourceBlocks;
		UInterchangeTexture2DFactoryNode* Texture2DFactoryNode = static_cast<UInterchangeTexture2DFactoryNode*>(InterchangeTextureFactoryNode);
		if (const UInterchangeTexture2DNode* Texture2DNode = Cast<UInterchangeTexture2DNode>(TextureNode))
		{
			SourceBlocks = Texture2DNode->GetSourceBlocks();

			EInterchangeTextureWrapMode WrapU;
			if (Texture2DNode->GetCustomWrapU(WrapU))
			{
				Texture2DFactoryNode->SetCustomAddressX(UE::Interchange::Private::ConvertWrap(WrapU));
			}

			EInterchangeTextureWrapMode WrapV;
			if (Texture2DNode->GetCustomWrapV(WrapV))
			{
				Texture2DFactoryNode->SetCustomAddressY(UE::Interchange::Private::ConvertWrap(WrapV));
			}
		}

#if WITH_EDITOR
		if (SourceBlocks.IsEmpty() && bImportUDIMs && SourceFile)
		{
			FString PrettyAssetName;
			SourceBlocks = UE::TextureUtilitiesCommon::GetUDIMBlocksFromSourceFile(SourceFile.GetValue(), UE::TextureUtilitiesCommon::DefaultUdimRegexPattern, &PrettyAssetName);
			if (!PrettyAssetName.IsEmpty())
			{
				InterchangeTextureFactoryNode->SetAssetName(PrettyAssetName);
			}
		}
#endif

		if (!SourceBlocks.IsEmpty())
		{
			Texture2DFactoryNode->SetSourceBlocks(MoveTemp(SourceBlocks));
		}
	}

	return InterchangeTextureFactoryNode;
}

UInterchangeTextureFactoryNode* UInterchangeGenericTexturePipeline::CreateTextureFactoryNode(const UInterchangeTextureNode* TextureNode, const TSubclassOf<UInterchangeTextureFactoryNode>& FactorySubclass)
{
	FString DisplayLabel = TextureNode->GetDisplayLabel();
	FString NodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TextureNode->GetUniqueID());
	UInterchangeTextureFactoryNode* TextureFactoryNode = nullptr;
	if (BaseNodeContainer->IsNodeUidValid(NodeUid))
	{
		TextureFactoryNode = Cast<UInterchangeTextureFactoryNode>(BaseNodeContainer->GetFactoryNode(NodeUid));
		if (!ensure(TextureFactoryNode))
		{
			//Log an error
			return nullptr;
		}
	}
	else
	{
		UClass* FactoryClass = FactorySubclass.Get();
		if (!ensure(FactoryClass))
		{
			// Log an error
			return nullptr;
		}

		TextureFactoryNode = NewObject<UInterchangeTextureFactoryNode>(BaseNodeContainer, FactoryClass);
		if (!ensure(TextureFactoryNode))
		{
			return nullptr;
		}
		//Creating a Texture
		TextureFactoryNode->InitializeTextureNode(NodeUid, DisplayLabel, TextureNode->GetDisplayLabel(), BaseNodeContainer);
		TextureFactoryNode->SetCustomTranslatedTextureNodeUid(TextureNode->GetUniqueID());
		TextureFactoryNodes.Add(TextureFactoryNode);

		TextureFactoryNode->AddTargetNodeUid(TextureNode->GetUniqueID());
		TextureNode->AddTargetNodeUid(TextureFactoryNode->GetUniqueID());

		UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(TextureNode, TextureFactoryNode, false);

		if (bAllowNonPowerOfTwo)
		{
			TextureFactoryNode->SetCustomAllowNonPowerOfTwo(bAllowNonPowerOfTwo);
		}
	}

	EInterchangeTextureColorSpace ColorSpace;
	const bool bHasColorSpace = TextureNode->GetCustomColorSpace(ColorSpace);
	if(bHasColorSpace)
	{
		TextureFactoryNode->SetCustomColorSpace(ETextureColorSpace(ColorSpace));
	}

	if (bool bSRGB; TextureNode->GetCustomSRGB(bSRGB))
	{
		TextureFactoryNode->SetCustomSRGB(bSRGB);
		if(bSRGB && !bHasColorSpace)
		{
			TextureFactoryNode->SetCustomColorSpace(ETextureColorSpace::TCS_sRGB);
		}
	}
	if (bool bFlipGreenChannel; TextureNode->GetCustombFlipGreenChannel(bFlipGreenChannel))
	{
		TextureFactoryNode->SetCustombFlipGreenChannel(bFlipGreenChannel);
	}

	using FInterchangeTextureFilterMode = std::underlying_type_t<EInterchangeTextureFilterMode>;
	using FTextureFilter = std::underlying_type_t<TextureFilter>;
	using FCommonTextureFilterModes = std::common_type_t<FInterchangeTextureFilterMode, FTextureFilter>;

	static_assert(FCommonTextureFilterModes(EInterchangeTextureFilterMode::Nearest) == FCommonTextureFilterModes(TextureFilter::TF_Nearest), "EInterchangeTextureFilterMode::Nearest differs from TextureFilter::TF_Nearest");
	static_assert(FCommonTextureFilterModes(EInterchangeTextureFilterMode::Bilinear) == FCommonTextureFilterModes(TextureFilter::TF_Bilinear), "EInterchangeTextureFilterMode::Bilinear differs from TextureFilter::TF_Bilinear");
	static_assert(FCommonTextureFilterModes(EInterchangeTextureFilterMode::Trilinear) == FCommonTextureFilterModes(TextureFilter::TF_Trilinear), "EInterchangeTextureFilterMode::Trilinear differs from TextureFilter::TF_Trilinear");
	static_assert(FCommonTextureFilterModes(EInterchangeTextureFilterMode::Default) == FCommonTextureFilterModes(TextureFilter::TF_Default), "EInterchangeTextureFilterMode::Default differs from TextureFilter::TF_Default");

	if (EInterchangeTextureFilterMode TextureFilter; TextureNode->GetCustomFilter(TextureFilter))
	{

		TextureFactoryNode->SetCustomFilter(uint8(TextureFilter));
	}

#if WITH_EDITORONLY_DATA
	if (bPreferCompressedSourceData)
	{
		TextureFactoryNode->SetCustomPreferCompressedSourceData(true);
	}
#endif // WITH_EDITORONLY_DATA

	return TextureFactoryNode;
}

void UInterchangeGenericTexturePipeline::PostImportTextureAssetImport(UObject* CreatedAsset, bool bIsAReimport)
{
#if WITH_EDITOR

	// this is run on main thread
	check(IsInGameThread());

	UTexture* Texture = Cast<UTexture>(CreatedAsset);
	if (Texture == nullptr)
	{
		return;
	}

	// (Note - as part of the standard interchange import this is called during the object
	// import iteration, _before_ the iteration to call to PostEditChange which is what starts the texture build via UpdateResource,
	// so altering properties here should be safe!)

	if (Texture->IsCompiling())
	{
		ensure(!bIsAReimport);
		FTextureCompilingManager::Get().FinishCompilation(MakeArrayView(&Texture, 1));
	}

	if(bFlipNormalMapGreenChannel && Texture->IsNormalMap())
	{
		Texture->bFlipGreenChannel = true;
	}

	FTextureSource& Source = Texture->Source;

	bool bRunNormapMapDetection = !bIsAReimport && bDetectNormalMapTexture && !Texture->IsNormalMap();

	 // we probably got the info via Init() - if we didn't it's because it's compressed. Here we can decompress, so do it if needed.
	bool bRunChannelScan = !Source.HasLayerColorInfo();

	bool bNeedLockedMip = bRunChannelScan || bRunNormapMapDetection;
	if (!bNeedLockedMip)
	{
		return;
	}

	// Lock the mip outside of everything. We allow nested locks so this just makes sure
	// that we don't decompress the mip data multiple times. Since the mips are all shared
	// this is locking all of them even if it only exposes the one mip.
	FTextureSource::FMipLock LockedMip0(FTextureSource::ELockState::ReadOnly, &Texture->Source, 0);
	if (!LockedMip0.IsValid())
	{
		UE_LOGF(LogInterchangePipeline, Error, "PostImport Texture failed to lock mip data, actions (like normal map detection) not performed on %ls", *Texture->GetPathName());
		return;
	}

	if (bRunChannelScan)
	{
		Source.UpdateChannelLinearMinMax();
	}

	if (bRunNormapMapDetection)
	{
		// AdjustTextureForNormalMap technically only adjusts properties and doesn't kick a texture build, however
		// if it guesses it's a normal map it pops a toast notification that can Revert the change, which does a whole
		// Modify / PostEditChange which theoretically can kick a build before the PostEditChange in the outer interchange
		// import chain if somehow the UI click chain routes before the outer loop calls PostEditChange.
		UE::Interchange::Private::AdjustTextureForNormalMap(Texture, LockedMip0.Image, bFlipNormalMapGreenChannel);
	}

#endif //WITH_EDITOR
}


