// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeLODDataParser.h"

#include "Mesh/InterchangeMeshHelper.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"

#include "InterchangeCommonPipelineDataFactoryNode.h"
#include "InterchangeGenericMeshPipeline.h"
#include "InterchangeMeshNode.h"
#include "InterchangeMeshBundleNode.h"
#include "InterchangeMeshLODContainerNode.h"
#include "InterchangePipelineMeshesUtilities.h"
#include "InterchangeSceneComponentNodes.h"
#include "InterchangeSceneNode.h"
#include "InterchangeSkeletonHelper.h"
#include "InterchangeSkeletalMeshFactoryNode.h"
#include "InterchangeSkeletalMeshLodDataNode.h"
#include "Nodes/InterchangeSourceNode.h"
#include "InterchangeStaticMeshFactoryNode.h"
#include "InterchangeStaticMeshLodDataNode.h"

namespace UE::Interchange
{
	namespace Private
	{
		FString GetNodeName(TObjectPtr<UInterchangePipelineMeshesUtilities> PipelineMeshesUtilities, const UInterchangeBaseNodeContainer& NodeContainer, const FString& NodeUid)
		{
			const UInterchangeBaseNode* BaseNode = NodeContainer.GetNode(NodeUid);
			if (BaseNode)
			{
				FString NodeName = BaseNode->GetDisplayLabel();
				if (BaseNode->IsA<UInterchangeMeshLODContainerNode>() || BaseNode->IsA<UInterchangeMeshNode>())
				{
					if (PipelineMeshesUtilities->IsValidMeshGeometryUid(NodeUid))
					{
						//If this mesh is reference by only one scene node that do not have any children, use the scene node display label
						const FInterchangeMeshGeometry& MeshGeometry = PipelineMeshesUtilities->GetMeshGeometryByUid(NodeUid);
						if (MeshGeometry.UidsOfInstancingNodes.Num() == 1
							&& NodeContainer.GetNodeChildrenCount(MeshGeometry.UidsOfInstancingNodes[0]) == 0)
						{
							if (const UInterchangeBaseNode* InstanceMeshNode = NodeContainer.GetNode(MeshGeometry.UidsOfInstancingNodes[0]))
							{
								return InstanceMeshNode->GetDisplayLabel();
							}
						}
					}
				}

				return NodeName;
			}
			else
			{
				return FString();
			}
		}
	};

	namespace CollisionHelper
	{
		TTuple<EInterchangeMeshCollision, FString> GetCollisionMeshType(
			TObjectPtr<UInterchangePipelineMeshesUtilities> PipelineMeshesUtilities,
			const UInterchangeBaseNodeContainer& NodeContainer,
			const FString& NodeUid,
			const TArray<FString>& AllNodeUids,
			bool bImportCollisionAccordingToMeshName
		)
		{
			EInterchangeMeshCollision CollisionType = EInterchangeMeshCollision::None;

			// Try to find the mesh node we're actually talking about
			const UInterchangeBaseNode* BaseNode = NodeContainer.GetNode(NodeUid);
			const UInterchangeMeshNode* MeshNode = Cast<const UInterchangeMeshNode>(BaseNode);
			if (!MeshNode)
			{
				if (const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(BaseNode))
				{
					FString MeshDependency;
					if (SceneNode->GetCustomAssetInstanceUid(MeshDependency))
					{
						MeshNode = Cast<const UInterchangeMeshNode>(NodeContainer.GetNode(MeshDependency));
					}
				}
			}

			// If we have an explicit collision type on the mesh, we know the mesh node is a collision mesh itself
			if (MeshNode)
			{
				if (MeshNode->GetCustomCollisionType(CollisionType) && CollisionType != EInterchangeMeshCollision::None)
				{
					return { CollisionType, NodeUid };
				}
			}

			if (bImportCollisionAccordingToMeshName)
			{
				FString MeshName = UE::Interchange::Private::GetNodeName(PipelineMeshesUtilities, NodeContainer, NodeUid);

				// Determine if the mesh name is a potential collision mesh
				CollisionType = UE::Interchange::Private::MeshHelper::GetMeshCollisionFromName(MeshName);

				if (CollisionType == EInterchangeMeshCollision::None)
				{
					return { EInterchangeMeshCollision::None, FString() };
				}

				// We have a mesh name with a collision type suffix.
				// However it should only be treated as a collision mesh if its body name corresponds to one of the other meshes.
				// If we get here, we know there is at least one underscore, so we never expect either of these character searches to fail.

				int32 FirstUnderscore = INDEX_NONE;
				verify(MeshName.FindChar(TEXT('_'), FirstUnderscore));

				int32 LastUnderscore = INDEX_NONE;
				verify(MeshName.FindLastChar(TEXT('_'), LastUnderscore));

				auto MatchPredicate = [&PipelineMeshesUtilities, &NodeContainer](FString Body)
					{
						// Generate a predicate to be used by the below Finds.
						return [Body, &PipelineMeshesUtilities, &NodeContainer](const FString& ToCompare) { return Body == UE::Interchange::Private::GetNodeName(PipelineMeshesUtilities, NodeContainer, ToCompare); };
					};

				auto FindPyPredicate = [&AllNodeUids, &MatchPredicate](const FString& RenderMeshName) -> const FString*
					{
						return AllNodeUids.FindByPredicate(MatchPredicate(RenderMeshName));
					};

				// If we find a mesh named the same as the collision mesh (following the collision prefix), we have a match
				// e.g. this will match 'UBX_House' with a mesh called 'House'
				{
					const FString RenderMeshName = MeshName.RightChop(FirstUnderscore + 1);
					const FString* CorrespondingMeshUid = FindPyPredicate(RenderMeshName);
					if (CorrespondingMeshUid)
					{
						return { CollisionType, *CorrespondingMeshUid };
					}
				}

				// Otherwise strip the final underscore suffix from the collision mesh name and look again
				// e.g. this will match 'UBX_House_01' with a mesh called 'House'
				if (FirstUnderscore != LastUnderscore)
				{
					const FString RenderMeshName = MeshName.Mid(FirstUnderscore + 1, LastUnderscore - FirstUnderscore - 1);
					const FString* CorrespondingMeshUid = FindPyPredicate(RenderMeshName);
					if (CorrespondingMeshUid)
					{
						return { CollisionType, *CorrespondingMeshUid };
					}
				}

				// It is possible our target mesh has been renamed away by a name sanitizing process (see UE-248981 and
				// FFbxParser::EnsureNodeNameAreValid). Since it is unusual to have a prefixed collision mesh and no target, let's 
				// assume we actually have a render mesh somewhere and try stripping any nameclash-looking suffixes from mesh names 
				// to see if we have a match
				{
					// Like above start by just stripping the prefix (e.g. 'UBX_House_01' --> 'House_01')
					FString RenderMeshName = MeshName.RightChop(FirstUnderscore + 1);

					auto NameCollisionSearchPredicate = [&RenderMeshName, &PipelineMeshesUtilities, &NodeContainer](const FString& Other)
						{
							FString NodeUid = Other;
							FString MeshName = UE::Interchange::Private::GetNodeName(PipelineMeshesUtilities, NodeContainer, NodeUid);
							if (!MeshName.StartsWith(RenderMeshName))
							{
								return false;
							}

							FString MeshNameSuffix = MeshName.RightChop(RenderMeshName.Len());

							// Check if everything after the render mesh name is just some combination of "ncl" (nameclash), numbers and/or underscores
							MeshNameSuffix = MeshNameSuffix.Replace(TEXT("ncl"), TEXT(""));

							FString LastChar = MeshNameSuffix.Right(1);
							while ((LastChar.IsNumeric() || LastChar == TEXT("_")) && MeshNameSuffix.Len() > 0)
							{
								MeshNameSuffix.LeftChopInline(1, EAllowShrinking::No);
								LastChar = MeshNameSuffix.Right(1);
							}

							// If there's nothing left, everything after the RenderMeshName was nameclash stuff, so this is likely our mesh
							return MeshNameSuffix.Len() == 0;
						};

					const FString* CorrespondingMeshUid = AllNodeUids.FindByPredicate(NameCollisionSearchPredicate);
					if (CorrespondingMeshUid)
					{
						return { CollisionType, *CorrespondingMeshUid };
					}

					// If we still have nothing, try also stripping numbered suffixes from the render mesh name and repeating the search
					// (e.g. 'UBX_House_01' --> 'House')
					RenderMeshName = MeshName.Mid(FirstUnderscore + 1, LastUnderscore - FirstUnderscore - 1);
					CorrespondingMeshUid = AllNodeUids.FindByPredicate(NameCollisionSearchPredicate);
					if (CorrespondingMeshUid)
					{
						return { CollisionType, *CorrespondingMeshUid };
					}
				}
			}

			return { EInterchangeMeshCollision::None, FString() };
		}
	}
}

template <class TMeshFactoryType>
FInterchangeMeshLODDataParser<TMeshFactoryType>::FInterchangeMeshLODDataParser(UInterchangeBaseNodeContainer& InBaseNodeContainer,
	TMeshFactoryType* InMeshFactoryNode,
	TObjectPtr<UInterchangePipelineMeshesUtilities> InPipelineMeshesUtilities,
	UInterchangeGenericMeshPipeline* InGenericMeshPipeline,
	const FString& InSourceAssetFileName,
	const FString& InRootJointNodeUid,
	const FString& InSkeletonFactoryNodeUid, 
	const bool InbUseTimeZeroAsBindPose)
	: MeshFactoryNode(InMeshFactoryNode)
	, MeshFactoryNodeUid(InMeshFactoryNode->GetUniqueID())
	, BaseNodeContainer(InBaseNodeContainer)
	, bBakeMeshes(InGenericMeshPipeline->CommonMeshesProperties->bBakeMeshes)
	, bBakePivotMeshes(InGenericMeshPipeline->CommonMeshesProperties->bBakePivotMeshes)
	, PipelineMeshesUtilities(InPipelineMeshesUtilities)
	, GenericMeshPipeline(InGenericMeshPipeline)
	, SourceAssetFileName(InSourceAssetFileName)
	, SkeletonFactoryNodeUid(InSkeletonFactoryNodeUid)
	, bUseTimeZeroAsBindPose(InbUseTimeZeroAsBindPose)
{

	GlobalOffsetTransform = FTransform::Identity;
	if (UInterchangeCommonPipelineDataFactoryNode* CommonPipelineDataFactoryNode = UInterchangeCommonPipelineDataFactoryNode::GetUniqueInstance(&BaseNodeContainer))
	{
		CommonPipelineDataFactoryNode->GetCustomGlobalOffsetTransform(GlobalOffsetTransform);
	}

	bAlignSkeletalMeshInScene = false;
	if (const UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::GetUniqueInstance(&BaseNodeContainer))
	{
		SourceNode->GetCustomAlignSkeletalMeshInScene(bAlignSkeletalMeshInScene);
	}

	RootJointParentUidAndGlobalInverseMatrix.Reset();
	if (InRootJointNodeUid.Len())
	{
		if (const UInterchangeSceneNode* RootJointNode = Cast<UInterchangeSceneNode>(BaseNodeContainer.GetNode(InRootJointNodeUid)))
		{
			if (const UInterchangeSceneNode* RootJointParentNode = Cast<UInterchangeSceneNode>(BaseNodeContainer.GetNode(RootJointNode->GetParentUid())))
			{
				FTransform RootJointParentGlobalTransform;
				RootJointParentNode->GetCustomGlobalTransform(&BaseNodeContainer, GlobalOffsetTransform, RootJointParentGlobalTransform);

				RootJointParentUidAndGlobalInverseMatrix = TPair<FString, FMatrix>(RootJointNode->GetParentUid(), RootJointParentGlobalTransform.ToMatrixWithScale().Inverse());
			}
		}
		else
		{
			ensure(false);
		}
	}
}

template <class TMeshFactoryType>
void FInterchangeMeshLODDataParser<TMeshFactoryType>::CreateMeshLodDataNode(const int32 LODIndex)
{
	const FString LODDataPrefix = TEXT("\\LodData") + (LODIndex > 0 ? FString::FromInt(LODIndex) : TEXT(""));
	const FString MeshLODDataUniqueID = LODDataPrefix + MeshFactoryNodeUid;

	if (UInterchangeFactoryBaseNode* FactoryBaseNode = BaseNodeContainer.GetFactoryNode(MeshLODDataUniqueID))
	{
		return;
	}

	TLODDataNodeType* MeshLODDataNode = NewObject<TLODDataNodeType>(&BaseNodeContainer, NAME_None);
	if (!ensure(MeshLODDataNode))
	{
		return;
	}

	const FString MeshLODDataName = TEXT("LodData") + FString::FromInt(LODIndex);
	BaseNodeContainer.SetupNode(MeshLODDataNode, MeshLODDataUniqueID, MeshLODDataName, EInterchangeNodeContainerType::FactoryData, MeshFactoryNode->GetUniqueID());

	if constexpr (std::is_same_v<TMeshFactoryType, UInterchangeStaticMeshFactoryNode>)
	{
		MeshLODDataNode->SetOneConvexHullPerUCX(GenericMeshPipeline->bOneConvexHullPerUCX);
		MeshLODDataNode->SetImportCollision(GenericMeshPipeline->bCollision);
		MeshLODDataNode->SetImportCollisionType(GenericMeshPipeline->Collision);
		MeshLODDataNode->SetForceCollisionPrimitiveGeneration(GenericMeshPipeline->bForceCollisionPrimitiveGeneration);
	}
	else if constexpr (std::is_same_v<TMeshFactoryType, UInterchangeSkeletalMeshFactoryNode>)
	{
		MeshLODDataNode->SetCustomSkeletonUid(SkeletonFactoryNodeUid);
	}
	else
	{
		ensure(false);
	}

	MeshFactoryNode->AddLodDataUniqueId(MeshLODDataUniqueID);

	LODIndexToLodDataNodeMap.Add(LODIndex, MeshLODDataNode);
}

/*
* We query by the MappedLODIndex, not by the original LODIndex.
*/
template <class TMeshFactoryType>
FInterchangeMeshLODDataParser<TMeshFactoryType>::TLODDataNodeType* FInterchangeMeshLODDataParser<TMeshFactoryType>::GetMeshLodDataNode(const int32 MappedLODIndex)
{
	if (LODIndexToLodDataNodeMap.Contains(MappedLODIndex))
	{
		return LODIndexToLodDataNodeMap[MappedLODIndex];
	}

	//All required MeshLODDataNodes should have been created in the PreProcess stage.
	ensure(false);

	return nullptr;
}

template <class TMeshFactoryType>
void FInterchangeMeshLODDataParser<TMeshFactoryType>::LogWarning(const FText& Text)
{
	UInterchangeResultWarning_Generic* Message = GenericMeshPipeline->AddMessage<UInterchangeResultWarning_Generic>();
	Message->SourceAssetName = SourceAssetFileName;
	Message->AssetFriendlyName = MeshFactoryNode->GetAssetName();
	Message->AssetType = MeshFactoryNode->GetObjectClass();
	Message->Text = Text;
}

//we might have to call ProcessCollision on every single node of the ProcessNode
//with the logic being:
//additional input argument: bProcessAsCollision
//bCurrentProcessAsCollision = bProcessAsCollision || ProcessColission(...)
//pass onward the bCurrentProcessAsCollision
//if bCurrentProcessAsCollision is true then do not add as a Mesh..
template <class TMeshFactoryType>
void FInterchangeMeshLODDataParser<TMeshFactoryType>::ProcessCollision(const FString& NodeUid, UInterchangeStaticMeshLodDataNode* LodDataNode, const FTransform& CompleteTransform)
{
	//#interchange_CollisionRefactor_TODO: Validate the need for the special use case for mesh name generation: if there is only 1 SceneNode instantiating the mesh, it should be named after/by the Instantiating node.
	const FString MeshName = UE::Interchange::Private::GetNodeName(PipelineMeshesUtilities, BaseNodeContainer, NodeUid);
	const bool HasPotentialMeshCollisionPrefix = UE::Interchange::Private::MeshHelper::HasPotentialMeshCollisionPrefix(MeshName);

    //#interchange_LODRefactor_Note: using NodeUidTransformsPair (previously NodeUids) is LOD Index based, we are not actually checking against all NodeUids for the CollisionMeshType.
    TTuple<EInterchangeMeshCollision, FString> MeshType = UE::Interchange::CollisionHelper::GetCollisionMeshType(PipelineMeshesUtilities, BaseNodeContainer, NodeUid, AllNodeUids, GenericMeshPipeline->bImportCollisionAccordingToMeshName);
    EInterchangeMeshCollision CollisionType = MeshType.Key;
    const FString& RenderMeshUid = MeshType.Value;
    const FString& ColliderMeshUid = NodeUid;

    FTransform TransformToBake = CompleteTransform;

    if (!bBakeMeshes && 
        RenderMeshUid != ColliderMeshUid && 
        CollisionType != EInterchangeMeshCollision::None)
    {
        // Make the bake transform relative to the render mesh if we're a collider mesh and we're not baking the entire scene together
        // This so that if our collider mesh has any transform with respect to the render mesh, we'll bake it in, and display the
        // collider correctly on the component

        const UInterchangeBaseNode* RenderMeshNode = BaseNodeContainer.GetNode(*RenderMeshUid);
        if (const UInterchangeSceneNode* RenderSceneNode = Cast<const UInterchangeSceneNode>(RenderMeshNode))
        {
            FTransform RenderLocalToGlobal = UE::Interchange::Private::MeshHelper::CalculateCompleteSceneNodeTransform(&BaseNodeContainer, RenderSceneNode,
                true, true, //Collisions are always baking global transforms
                GlobalOffsetTransform, bUseTimeZeroAsBindPose, 
				RootJointParentUidAndGlobalInverseMatrix, bCreatingSkeletalMesh, bAlignSkeletalMeshInScene);

            // We want to ultimately produce "TransformToBake = CompleteTransform * RenderLocalToGlobal.Inverse();"
            TransformToBake = CompleteTransform * RenderLocalToGlobal.Inverse();
        }
    }

    switch (CollisionType)
    {
        case EInterchangeMeshCollision::None:
            // If the user is attempting to import a collision mesh using a naming convention but the mesh doesn't match, warn them that it won't be imported as collision.
            if (HasPotentialMeshCollisionPrefix)
            {
                LogWarning(FText::Format(NSLOCTEXT("FInterchangeMeshLODDataParser", "NoMatchingRenderMeshForCustomCollision",
                    "No render mesh found for custom collision '{0}'. The name must follow the pattern 'UCX_[ExactRenderMeshName]', and a matching render mesh with that name must exist in the source file."),
                    FText::FromString(MeshName)));
            }

            break;

        case EInterchangeMeshCollision::Box:
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
            LodDataNode->AddBoxCollisionMeshUids(ColliderMeshUid, RenderMeshUid);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
            LodDataNode->AddBoxCollisionMeshData(FInterchangeLODMeshData(ColliderMeshUid, TransformToBake));
            break;

        case EInterchangeMeshCollision::Sphere:
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
            LodDataNode->AddSphereCollisionMeshUids(ColliderMeshUid, RenderMeshUid);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
            LodDataNode->AddSphereCollisionMeshData(FInterchangeLODMeshData(ColliderMeshUid, TransformToBake));
            break;

        case EInterchangeMeshCollision::Capsule:
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
            LodDataNode->AddCapsuleCollisionMeshUids(ColliderMeshUid, RenderMeshUid);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
            LodDataNode->AddCapsuleCollisionMeshData(FInterchangeLODMeshData(ColliderMeshUid, TransformToBake));
            break;

        case EInterchangeMeshCollision::Convex10DOP_X:
        case EInterchangeMeshCollision::Convex10DOP_Y:
        case EInterchangeMeshCollision::Convex10DOP_Z:
        case EInterchangeMeshCollision::Convex18DOP:
        case EInterchangeMeshCollision::Convex26DOP:
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
            LodDataNode->AddConvexCollisionMeshUids(ColliderMeshUid, RenderMeshUid);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
            LodDataNode->AddConvexCollisionMeshData(FInterchangeLODMeshData(ColliderMeshUid, TransformToBake));
            break;
    }

    
}

//Input argument possibilities for ProcessNode:
//	- InterchangeSceneNode: using AssetInstanceUid, which can be UInterchangeMeshLODContainerNode, UInterchangeMeshNode, UInterchangeGeometryCacheNode(which inherits from UInterchangeMeshNode)
//	- UInterchangeMeshLODContainerNode
//	- UInterchangeMeshNode
//	- UInterchangeGeometryCacheNode(which inherits from UInterchangeMeshNode, so can be handled in the same branch as UInterchangeMeshNode)
//	- UInterchangeInstancedStaticMeshComponentNode
//  - UInterchangeMeshBundleNode
template <class TMeshFactoryType>
void FInterchangeMeshLODDataParser<TMeshFactoryType>::ProcessNode(const UInterchangeBaseNode* BaseNode, const FTransform& MeshTransform, const UInterchangeSceneNode* CurrentTransformNode, const int32 LODIndex, bool bProcessAsSubMesh, bool bProcessAsCollision)
{
	if (!BaseNode)
	{
		return;
	}

	constexpr bool bAddSourceNodeName = true;
	UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(BaseNode, MeshFactoryNode, bAddSourceNodeName);

	if (const UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(BaseNode))
	{
		//This has no further recursion, this is where we fill the actual LODDataNode.
		//All iteration and recursion of ProcessNode should end up in here eventually.

		const FString MeshNodeUid = MeshNode->GetUniqueID();

		MeshFactoryNode->AddTargetNodeUid(MeshNodeUid);

		FTransform CompleteTransformOfCurrentTransformNode = UE::Interchange::Private::MeshHelper::CalculateCompleteSceneNodeTransform(&BaseNodeContainer, CurrentTransformNode,
			bBakeMeshes || bProcessAsCollision, bBakePivotMeshes || bProcessAsCollision, //Collisions are always baking global transforms
			GlobalOffsetTransform, bUseTimeZeroAsBindPose, 
			RootJointParentUidAndGlobalInverseMatrix, bCreatingSkeletalMesh, bAlignSkeletalMeshInScene);
		FTransform CompleteTransform = MeshTransform * CompleteTransformOfCurrentTransformNode;

		FString SceneNodeName = TEXT("");
		FTransform GeometricTransform = FTransform::Identity;
		if (CurrentTransformNode)
		{
			CurrentTransformNode->GetCustomGeometricTransform(GeometricTransform);
			SceneNodeName = CurrentTransformNode->GetDisplayLabel();
		}

		auto AddLODMeshData = [this, &CompleteTransform, &GeometricTransform, &LODIndex, &SceneNodeName, &MeshNodeUid]()
			{
				if (LODIndex == INDEX_NONE)
				{
					//LODIndex==INDEX_NONE means that the MeshUid is not coming from a LODContainer,
					// in that case We need to add the MeshNodeUidTransformsPair to all LODs.

					for (const TPair<int32, TLODDataNodeType*>& LODIndexDataNodePair: LODIndexToLodDataNodeMap)
					{
						if (!LODIndexDataNodePair.Value)
						{
							ensure(false);
							continue;
						}
						LODIndexDataNodePair.Value->AddLODMeshData(FInterchangeLODMeshData(MeshNodeUid, SceneNodeName, CompleteTransform, GeometricTransform));
					}
				}
				else
				{
					int32 MappedLODIndex = 0;
					if (OriginalToCompactLODIndexMap.Contains(LODIndex))
					{
						MappedLODIndex = OriginalToCompactLODIndexMap[LODIndex];
					}
					else
					{
						//This inderictly happens, because of GenericMeshPipeline->CommonMeshesProperties->bImportLods==false
						//As in that case we are only creating MappedLODIndex:=0 scenario.
						//And so, if we are querying a LOD Index that is not part of the mapping, that means we want to discard the given mesh.
						return;
					}

					TLODDataNodeType* LodDataNode = GetMeshLodDataNode(MappedLODIndex);
					if (!LodDataNode)
					{
						ensure(false);
						return;
					}
					LodDataNode->AddLODMeshData(FInterchangeLODMeshData(MeshNodeUid, SceneNodeName, CompleteTransform, GeometricTransform));
				}
			};

		//For StaticMeshes we handle sockets and collisions:
		if constexpr (std::is_same_v<TMeshFactoryType, UInterchangeStaticMeshFactoryNode>)
		{
			if (PipelineMeshesUtilities->IsValidMeshGeometryUid(MeshNodeUid))
			{
				MeshFactoryNode->AddSocketUids(PipelineMeshesUtilities->GetMeshGeometryByUid(MeshNodeUid).AttachedSocketUids);
			}

            if (bProcessAsCollision)
            {
                const int32 LODIndexZero = 0;
                ProcessCollision(MeshNodeUid, GetMeshLodDataNode(LODIndexZero), CompleteTransform);
            }
            else
			{
                //if it is not a Collision, it should be added as a MeshUid.
                AddLODMeshData();
                UniqueMeshUids.Add(MeshNodeUid);
			}
		}
		else
		{
			AddLODMeshData();
			UniqueMeshUids.Add(MeshNodeUid);
		}

		TMap<FString, FString> SlotMaterialDependencies;
		MeshNode->GetSlotMaterialDependencies(SlotMaterialDependencies);
		UE::Interchange::MeshesUtilities::ApplySlotMaterialDependencies(*MeshFactoryNode, SlotMaterialDependencies, BaseNodeContainer, &ExistingLodSlotMaterialDependenciesPerLODIndices.FindOrAdd(LODIndex));

		//update assembly:
		if (!bProcessAsSubMesh)
		{
			GenericMeshPipeline->AddToAssemblyPartMeshUidToFactoryNodeTable(MeshNodeUid, MeshFactoryNode);
			MeshNode->AddTargetNodeUid(MeshFactoryNodeUid);
		}
	}
	else if (const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(BaseNode))
	{
		FString AssetInstanceUid;
		if (SceneNode->GetCustomAssetInstanceUid(AssetInstanceUid))
		{
			ProcessNode(BaseNodeContainer.GetNode(AssetInstanceUid), FTransform::Identity, SceneNode, LODIndex, bProcessAsSubMesh, bProcessAsCollision);
		}

		//#interchange_LODRefactor_Note: Technically based on old implementation we should be calling GetSlotMaterialDependencies on the SceneNode if the AssetInstanceUid does not pan out.
	}
	else if (const UInterchangeMeshLODContainerNode* MeshLODContainerNode = Cast<UInterchangeMeshLODContainerNode>(BaseNode))
	{
		//Note: In theory it is possible to nest MeshLODContainerNodes, 
		//			however the system will parse the nested MeshLODContainerNodes into the LOD Indices marked in its entries.
		//			(In other words: the supersceding MeshLODContainerNode won't dictate the LODIndex for the nested MeshLODContainer)
		TMap<int32, TArray<FInterchangeLODMeshData>> LODMeshesPerLODIndices;
		MeshLODContainerNode->GetLODMeshDataPerLODIndices(LODMeshesPerLODIndices);
		MeshLODContainerNode->ValidateMorphTargets(&BaseNodeContainer, LODMeshesPerLODIndices);

		if constexpr (std::is_same_v<TMeshFactoryType, UInterchangeStaticMeshFactoryNode>)
		{
			if (PipelineMeshesUtilities->IsValidMeshGeometryUid(MeshLODContainerNode->GetUniqueID()))
			{
				MeshFactoryNode->AddSocketUids(PipelineMeshesUtilities->GetMeshGeometryByUid(MeshLODContainerNode->GetUniqueID()).AttachedSocketUids);
			}
		}

		//Process sub LODs:
		for (const TPair<int32, TArray<FInterchangeLODMeshData>>& LODMeshesPerLODIndex : LODMeshesPerLODIndices)
		{
			for (const FInterchangeLODMeshData& NodeUidTransformsPair : LODMeshesPerLODIndex.Value)
			{
				ProcessNode(BaseNodeContainer.GetNode(NodeUidTransformsPair.MeshUid), NodeUidTransformsPair.Transform, CurrentTransformNode, LODMeshesPerLODIndex.Key, true, bProcessAsCollision);
			}
		}

		GenericMeshPipeline->AddToAssemblyPartMeshUidToFactoryNodeTable(MeshLODContainerNode->GetUniqueID(), MeshFactoryNode);
		MeshLODContainerNode->AddTargetNodeUid(MeshFactoryNodeUid);
	}
	else if (const UInterchangeInstancedStaticMeshComponentNode* ISMComponentNode = Cast<UInterchangeInstancedStaticMeshComponentNode>(BaseNode))
	{
		//We query the parent of ISMComponentNode, because the Interchange ComponentNode system is not part of the UInterchangeSceneNode system, especially for the GlobalTransform calculations.
		//For that reason we grab the ISMComponetNode's parent which is a UInterchangeSceneNode, acquire necessary transforms and then apply the ISComponentNode's relevant transforms.
		if (const UInterchangeSceneNode* ISMSceneNode = Cast<UInterchangeSceneNode>(BaseNodeContainer.GetNode(ISMComponentNode->GetParentUid())))
		{
			FString AssetInstanceUid;
			if (ISMComponentNode->GetCustomInstancedAssetUid(AssetInstanceUid))
			{
				if (const UInterchangeBaseNode* AssetBaseNode = BaseNodeContainer.GetNode(AssetInstanceUid))
				{
					if (bBakeMeshes)
					{
						//In this case we want to add all instances to the LODDataNode
						TArray<FTransform> InstanceTransforms;
						ISMComponentNode->GetInstanceTransforms(InstanceTransforms);

						for (const FTransform& InstanceTransform : InstanceTransforms)
						{
							ProcessNode(AssetBaseNode, InstanceTransform, ISMSceneNode, LODIndex, bProcessAsSubMesh, bProcessAsCollision);
						}
					}
					else
					{
						//bBakeMeshes := false, we don't bake the instances, we only use the single occurance
						// => Primary use case is Level import, in which case the instances will be appled to the Instanced Static Mesh Component.
						ProcessNode(AssetBaseNode, FTransform::Identity, ISMSceneNode, LODIndex, bProcessAsSubMesh, bProcessAsCollision);
					}
				}
			}

			ISMComponentNode->AddTargetNodeUid(MeshFactoryNodeUid);
		}
		else
		{
			//Unexpected, ISMComponentNode should always have a SceneNode parent.
			ensure(false);
		}
	}
	else if (const UInterchangeMeshBundleNode* MeshBundleNode = Cast<UInterchangeMeshBundleNode>(BaseNode))
	{
		TMap<FString, TArray<FTransform>> MeshNodeToTransforms;
		MeshBundleNode->GetAllMeshNodesAndTransforms(MeshNodeToTransforms);

		for (const TPair<FString, TArray<FTransform>>& Pair : MeshNodeToTransforms)
		{
			if (const UInterchangeBaseNode* MeshBaseNode = BaseNodeContainer.GetNode(Pair.Key))
			{
				for (const FTransform& Transform : Pair.Value)
				{
					ProcessNode(MeshBaseNode, Transform, CurrentTransformNode, LODIndex, true, bProcessAsCollision);
				}
			}
			
		}

		GenericMeshPipeline->AddToAssemblyPartMeshUidToFactoryNodeTable(MeshBundleNode->GetUniqueID(), MeshFactoryNode);
		MeshBundleNode->AddTargetNodeUid(MeshFactoryNodeUid);
	}
	else
	{
		ensure(false);
	}
}

template <class TMeshFactoryType>
void FInterchangeMeshLODDataParser<TMeshFactoryType>::PreProcess(const TArray<FString>& NodeUids)
{
	TSet<int32> AvailableLODIndices;
	for (const FString& NodeUid : NodeUids)
	{
		FindAllAvailableLODIndices(BaseNodeContainer.GetNode(NodeUid), AvailableLODIndices);
	}

	TArray<int32> AvailableLODIndicesArray = AvailableLODIndices.Array();
	AvailableLODIndicesArray.Sort();

	//If we are *not* importing in LODs, then we only use 1 LOD and LOD Index.
	//	In case the LOD Container uses LODIndex > 0, then we use the smallest original LOD Index as LOD 0.
	//	This also inherently means that whenever we end up on a use case where later on a LOD Index is not part of OriginalToCompactLODIndexMap,
	//		that mesh shall be discarded. (see AddLODMeshData in ProcessNode)
	int32 LODLimit = GenericMeshPipeline->CommonMeshesProperties->bImportLods ? AvailableLODIndicesArray.Num() : FMath::Min(AvailableLODIndicesArray.Num(), 1);
	for (int32 CompactLODIndex = 0; CompactLODIndex < LODLimit; CompactLODIndex++)
	{
		OriginalToCompactLODIndexMap.Add(AvailableLODIndicesArray[CompactLODIndex], CompactLODIndex);
		CreateMeshLodDataNode(CompactLODIndex);
	}

	if (OriginalToCompactLODIndexMap.Num() == 0)
	{
		const int32 LODIndexZero = 0;
		//add LOD Index 0 in case no LOD Indices were identified (aka no MeshLODContainerNode is present)
		OriginalToCompactLODIndexMap.Add(LODIndexZero, LODIndexZero);
		CreateMeshLodDataNode(LODIndexZero);
	}
}

template <class TMeshFactoryType>
void FInterchangeMeshLODDataParser<TMeshFactoryType>::FindAllAvailableLODIndices(const UInterchangeBaseNode* BaseNode, TSet<int32>& AvailableLODIndices)
{
	if (!BaseNode)
	{
		return;
	}

	if (const UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(BaseNode))
	{
		//No further recursion. End of the road.
	}
	else if (const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(BaseNode))
	{
		FString AssetInstanceUid;
		if (SceneNode->GetCustomAssetInstanceUid(AssetInstanceUid))
		{
			FindAllAvailableLODIndices(BaseNodeContainer.GetNode(AssetInstanceUid), AvailableLODIndices);
		}
	}
	else if (const UInterchangeMeshLODContainerNode* MeshLODContainerNode = Cast<UInterchangeMeshLODContainerNode>(BaseNode))
	{
		TMap<int32, TArray<FInterchangeLODMeshData>> LODMeshesPerLODIndices;
		MeshLODContainerNode->GetLODMeshDataPerLODIndices(LODMeshesPerLODIndices);

		//Process sub LODs:
		for (const TPair<int32, TArray<FInterchangeLODMeshData>>& LODMeshesPerLODIndex : LODMeshesPerLODIndices)
		{
			AvailableLODIndices.Add(LODMeshesPerLODIndex.Key);
			for (const FInterchangeLODMeshData& NodeUidTransformsPair : LODMeshesPerLODIndex.Value)
			{
				FindAllAvailableLODIndices(BaseNodeContainer.GetNode(NodeUidTransformsPair.MeshUid), AvailableLODIndices);
			}
		}
	}
	else if (const UInterchangeInstancedStaticMeshComponentNode* ISMComponentNode = Cast<UInterchangeInstancedStaticMeshComponentNode>(BaseNode))
	{
		//We query the parent of ISMComponentNode, because the Interchange ComponentNode system is not part of the UInterchangeSceneNode system, especially for the GlobalTransform calculations.
		//For that reason we grab the ISMComponetNode's parent which is a UInterchangeSceneNode, acquire necessary transforms and then apply the ISComponentNode's relevant transforms.
		if (const UInterchangeSceneNode* ISMSceneNode = Cast<UInterchangeSceneNode>(BaseNodeContainer.GetNode(ISMComponentNode->GetParentUid())))
		{
			FString AssetInstanceUid;
			if (ISMComponentNode->GetCustomInstancedAssetUid(AssetInstanceUid))
			{
				FindAllAvailableLODIndices(BaseNodeContainer.GetNode(AssetInstanceUid), AvailableLODIndices);
			}
		}
	}
	else if (const UInterchangeMeshBundleNode* MeshBundleNode = Cast<UInterchangeMeshBundleNode>(BaseNode))
	{
		TMap<FString, TArray<FTransform>> MeshNodeToTransforms;
		MeshBundleNode->GetAllMeshNodesAndTransforms(MeshNodeToTransforms);

		for (const TPair<FString, TArray<FTransform>>& Pair : MeshNodeToTransforms)
		{
			FindAllAvailableLODIndices(BaseNodeContainer.GetNode(Pair.Key), AvailableLODIndices);
		}
	}
}

template <class TMeshFactoryType>
void FInterchangeMeshLODDataParser<TMeshFactoryType>::ProcessNodeUids(const TArray<FString>& RenderNodeUids, const TArray<FString>& CollisionMeshUids)
{
	TArray<FString> CollisionMeshUidsToProcess = GenericMeshPipeline->bCollision ? CollisionMeshUids : TArray<FString>();

	TSet<FString> AllUniqueNodeUids;
	AllUniqueNodeUids.Append(RenderNodeUids);
	AllUniqueNodeUids.Append(CollisionMeshUidsToProcess);

	//Only add Unique NodeUids to the AllNodeUids list.
	AllNodeUids = AllUniqueNodeUids.Array();

    if (GenericMeshPipeline->bImportCollisionAccordingToMeshName)
    {
        PreProcess(RenderNodeUids);
    }
    else
    {
        PreProcess(AllNodeUids);
    }

	const int32 InitialLODIndex = INDEX_NONE; // this indicates for the processing system that node being processed should be part of all LODs.
	for (const FString& NodeUid : RenderNodeUids)
	{
		ProcessNode(BaseNodeContainer.GetNode(NodeUid), 
            FTransform::Identity/*MeshTransform*/, nullptr /*CurrentNodeTransform*/,
			InitialLODIndex,
            false /*bProcessAsSubmesh*/, false/*bProcessAsCollision*/);
	}

	if (GenericMeshPipeline->bImportCollisionAccordingToMeshName)
	{
		for (const FString& NodeUid : CollisionMeshUidsToProcess)
		{
			ProcessNode(BaseNodeContainer.GetNode(NodeUid),
				FTransform::Identity /*MeshTransform*/, nullptr /*CurrentNodeTransform*/,
				InitialLODIndex,
				false /*bProcessAsSubmesh*/, true/*bProcessAsCollision*/);
		}
	}
	else
	{
		//In this case (where we import Collisision as Render meshes)
		// we need to make sure we are not re-processing nodes that are Render AND Collision nodes at the same time.
		TSet<FString> UniqueRenderNodeUids;
		UniqueRenderNodeUids.Append(RenderNodeUids);

		for (const FString& NodeUid : CollisionMeshUidsToProcess)
		{
			if (UniqueRenderNodeUids.Contains(NodeUid))
			{
				continue;
			}

			ProcessNode(BaseNodeContainer.GetNode(NodeUid),
				FTransform::Identity /*MeshTransform*/, nullptr /*CurrentNodeTransform*/,
				InitialLODIndex,
				false /*bProcessAsSubmesh*/, false/*bProcessAsCollision*/);
		}
	}

    

	UE::Interchange::MeshesUtilities::ReorderSlotMaterialDependencies(*MeshFactoryNode, BaseNodeContainer);
}

template struct FInterchangeMeshLODDataParser<UInterchangeStaticMeshFactoryNode>;
template struct FInterchangeMeshLODDataParser<UInterchangeSkeletalMeshFactoryNode>;
