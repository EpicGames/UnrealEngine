// Copyright Epic Games, Inc. All Rights Reserved. 

#include "InterchangeMaterialXPipeline.h"

#include "InterchangeImportModule.h"
#include "MaterialX/InterchangeMaterialXDefinitions.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangePipelineLog.h"

#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionStaticBoolParameter.h"
#include "Materials/MaterialFunction.h"
#include "Misc/PackageName.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeMaterialXPipeline)

#define MATERIALX_FUNCTIONS_SUBSTRATE_PATH(Name)  \
	constexpr const TCHAR* Name##FunctionsPath = TEXT("/InterchangeAssets/Functions/") TEXT("MX_") TEXT(#Name) TEXT(".") TEXT("MX_") TEXT(#Name);  \
	constexpr const TCHAR* Name##SubstratePath = TEXT("/InterchangeAssets/Substrate/") TEXT("MX_") TEXT(#Name) TEXT(".") TEXT("MX_") TEXT(#Name);

#define MATERIALX_MATERIALFUNCTION_PATH(Name) \
	!MaterialXSettings->bIsSubstrateEnabled ? Name##FunctionsPath : Name##SubstratePath

namespace mx = MaterialX;

namespace
{
	constexpr const TCHAR* OpenPBRSurfaceFunctionsPath = TEXT("/InterchangeAssets/Functions/MX_OpenPBR_Opaque.MX_OpenPBR_Opaque");
	constexpr const TCHAR* OpenPBRSurfaceSubstratePath = TEXT("/Engine/Functions/Substrate/MF_Substrate_OpenPBR_Opaque.MF_Substrate_OpenPBR_Opaque");
	constexpr const TCHAR* OpenPBRTransmissionSurfaceFunctionsPath = TEXT("/InterchangeAssets/Functions/MX_OpenPBR_Translucent.MX_OpenPBR_Translucent");
	constexpr const TCHAR* OpenPBRTransmissionSurfaceSubstratePath = TEXT("/Engine/Functions/Substrate/MF_Substrate_OpenPBR_Translucent.MF_Substrate_OpenPBR_Translucent");
	constexpr const TCHAR* StandardSurfaceFunctionsPath = TEXT("/InterchangeAssets/Functions/MX_StandardSurface.MX_StandardSurface");
	constexpr const TCHAR* StandardSurfaceSubstratePath = TEXT("/Engine/Functions/Substrate/Substrate-StandardSurface-Opaque.Substrate-StandardSurface-Opaque");
	constexpr const TCHAR* TransmissionSurfaceFunctionsPath = TEXT("/InterchangeAssets/Functions/MX_TransmissionSurface.MX_TransmissionSurface");
	constexpr const TCHAR* TransmissionSurfaceSubstratePath = TEXT("/Engine/Functions/Substrate/Substrate-StandardSurface-Translucent.Substrate-StandardSurface-Translucent");
	MATERIALX_FUNCTIONS_SUBSTRATE_PATH(SurfaceUnlit);
	MATERIALX_FUNCTIONS_SUBSTRATE_PATH(Surface);
	constexpr const TCHAR* GltfPBRFunctionsPath = TEXT("/InterchangeAssets/gltf/Substrate/MF_GLTFPBR.MF_GLTFPBR");
	constexpr const TCHAR* GltfPBRSubstratePath = TEXT("/InterchangeAssets/gltf/Substrate/MF_GLTFPBR.MF_GLTFPBR");
	constexpr const TCHAR* UsdPreviewSurfaceFunctionsPath = TEXT("/InterchangeAssets/Functions/MX_UsdPreviewSurface.MX_UsdPreviewSurface");
	constexpr const TCHAR* UsdPreviewSurfaceSubstratePath = TEXT("/Engine/Functions/Substrate/MF_Substrate_UsdPreviewSurface.MF_Substrate_UsdPreviewSurface");
	constexpr const TCHAR* DisplacementFunctionsPath = TEXT("/InterchangeAssets/Functions/MX_Displacement.MX_Displacement");
	constexpr const TCHAR* DisplacementSubstratePath = TEXT("/InterchangeAssets/Functions/MX_Displacement.MX_Displacement");
	
	// will act differently, we have 2 different volumes depending on the setting in the pipeline
	constexpr const TCHAR* SimpleVolumeSlabSubstratePath = TEXT("/InterchangeAssets/Substrate/MX_SimpleVolume.MX_SimpleVolume");
	constexpr const TCHAR* FOGCloudVolumeSlabSubstratePath = TEXT("/InterchangeAssets/Substrate/MX_VolumetricFogCloud.MX_VolumetricFogCloud");
	constexpr const TCHAR* VolumeFunctionsPath = SimpleVolumeSlabSubstratePath;
	constexpr const TCHAR* VolumeSubstratePath = SimpleVolumeSlabSubstratePath;
	
	MATERIALX_FUNCTIONS_SUBSTRATE_PATH(OrenNayarBSDF);
	MATERIALX_FUNCTIONS_SUBSTRATE_PATH(BurleyDiffuseBSDF);
	MATERIALX_FUNCTIONS_SUBSTRATE_PATH(DielectricBSDF);
	MATERIALX_FUNCTIONS_SUBSTRATE_PATH(ConductorBSDF);
	MATERIALX_FUNCTIONS_SUBSTRATE_PATH(SheenBSDF);
	MATERIALX_FUNCTIONS_SUBSTRATE_PATH(SubsurfaceBSDF);
	MATERIALX_FUNCTIONS_SUBSTRATE_PATH(GeneralizedSchlickBSDF);
	MATERIALX_FUNCTIONS_SUBSTRATE_PATH(TranslucentBSDF);
	MATERIALX_FUNCTIONS_SUBSTRATE_PATH(ChiangHairBSDF);
	
	MATERIALX_FUNCTIONS_SUBSTRATE_PATH(UniformEDF);
	MATERIALX_FUNCTIONS_SUBSTRATE_PATH(ConicalEDF);
	MATERIALX_FUNCTIONS_SUBSTRATE_PATH(MeasuredEDF);
	
	MATERIALX_FUNCTIONS_SUBSTRATE_PATH(AbsorptionVDF);
	MATERIALX_FUNCTIONS_SUBSTRATE_PATH(AnisotropicVDF);
}

TMap<FString, EInterchangeMaterialXSettings> UInterchangeMaterialXPipeline::PathToEnumMapping;
#if WITH_EDITOR
TMap<EInterchangeMaterialXSettings, TPair<TSet<FName>, TSet<FName>>> UMaterialXPipelineSettings::SettingsInputsOutputs;
#endif // WITH_EDITOR

UMaterialXPipelineSettings::UMaterialXPipelineSettings()
{	
	using namespace UE::Interchange::Materials;
#if WITH_EDITOR
	if(HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
	{
		bIsSubstrateEnabled = IInterchangeImportModule::IsAvailable() ? IInterchangeImportModule::Get().IsSubstrateEnabled() : false;

		SettingsInputsOutputs = {
			//Surface Shaders
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::OpenPBRSurface),
				{
				// OpenPBRSurface Inputs
				TSet<FName>{
					mx::OpenPBRSurface::Input::BaseWeight,
					mx::OpenPBRSurface::Input::BaseColor,
					mx::OpenPBRSurface::Input::BaseDiffuseRoughness,
					mx::OpenPBRSurface::Input::BaseMetalness,
					mx::OpenPBRSurface::Input::SpecularWeight,
					mx::OpenPBRSurface::Input::SpecularColor,
					mx::OpenPBRSurface::Input::SpecularRoughness,
					mx::OpenPBRSurface::Input::SpecularIOR,
					mx::OpenPBRSurface::Input::SpecularRoughnessAnisotropy,
					mx::OpenPBRSurface::Input::TransmissionWeight,
					mx::OpenPBRSurface::Input::TransmissionColor,
					mx::OpenPBRSurface::Input::TransmissionDepth,
					mx::OpenPBRSurface::Input::TransmissionScatter,
					mx::OpenPBRSurface::Input::TransmissionScatterAnisotropy,
					mx::OpenPBRSurface::Input::TransmissionDispersionScale,
					mx::OpenPBRSurface::Input::TransmissionDispersionAbbeNumber,
					mx::OpenPBRSurface::Input::SubsurfaceWeight,
					mx::OpenPBRSurface::Input::SubsurfaceColor,
					mx::OpenPBRSurface::Input::SubsurfaceRadius,
					mx::OpenPBRSurface::Input::SubsurfaceRadiusScale,
					mx::OpenPBRSurface::Input::SubsurfaceScatterAnisotropy,
					mx::OpenPBRSurface::Input::FuzzWeight,
					mx::OpenPBRSurface::Input::FuzzColor,
					mx::OpenPBRSurface::Input::FuzzRoughness,
					mx::OpenPBRSurface::Input::CoatWeight,
					mx::OpenPBRSurface::Input::CoatColor,
					mx::OpenPBRSurface::Input::CoatRoughness,
					mx::OpenPBRSurface::Input::CoatRoughnessAnisotropy,
					mx::OpenPBRSurface::Input::CoatIOR,
					mx::OpenPBRSurface::Input::CoatDarkening,
					mx::OpenPBRSurface::Input::ThinFilmWeight,
					mx::OpenPBRSurface::Input::ThinFilmThickness,
					mx::OpenPBRSurface::Input::ThinFilmIOR,
					mx::OpenPBRSurface::Input::EmissionLuminance,
					mx::OpenPBRSurface::Input::EmissionColor,
					mx::OpenPBRSurface::Input::GeometryOpacity,
					mx::OpenPBRSurface::Input::GeometryThinWalled,
					mx::OpenPBRSurface::Input::GeometryNormal,
					mx::OpenPBRSurface::Input::GeometryCoatNormal,
					mx::OpenPBRSurface::Input::GeometryTangent,
					mx::OpenPBRSurface::Input::GeometryCoatTangent,
				},
				// OpenPBRSurface Outputs
				!bIsSubstrateEnabled ?
				TSet<FName>{
					PBRMR::Parameters::BaseColor,
					PBRMR::Parameters::Metallic,
					PBRMR::Parameters::Specular,
					PBRMR::Parameters::Roughness,
					PBRMR::Parameters::Anisotropy,
					PBRMR::Parameters::EmissiveColor,
					PBRMR::Parameters::Opacity,
					PBRMR::Parameters::Normal,
					PBRMR::Parameters::Tangent,
					Sheen::Parameters::SheenRoughness,
					Sheen::Parameters::SheenColor,
					Subsurface::Parameters::SubsurfaceColor,
					ClearCoat::Parameters::ClearCoat,
					ClearCoat::Parameters::ClearCoatRoughness,
					ClearCoat::Parameters::ClearCoatNormal
					}	:
				TSet<FName>{
					OpenPBRSurface::SubstrateMaterial::Outputs::FrontMaterial,
					OpenPBRSurface::SubstrateMaterial::Outputs::OpacityMask
					}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::OpenPBRSurfaceTransmission),
				{
					// OpenPBRSurfaceTransmission Inputs
					TSet<FName>{
						mx::OpenPBRSurface::Input::BaseWeight,
						mx::OpenPBRSurface::Input::BaseColor,
						mx::OpenPBRSurface::Input::BaseDiffuseRoughness,
						mx::OpenPBRSurface::Input::BaseMetalness,
						mx::OpenPBRSurface::Input::SpecularWeight,
						mx::OpenPBRSurface::Input::SpecularColor,
						mx::OpenPBRSurface::Input::SpecularRoughness,
						mx::OpenPBRSurface::Input::SpecularIOR,
						mx::OpenPBRSurface::Input::SpecularRoughnessAnisotropy,
						mx::OpenPBRSurface::Input::TransmissionWeight,
						mx::OpenPBRSurface::Input::TransmissionColor,
						mx::OpenPBRSurface::Input::TransmissionDepth,
						mx::OpenPBRSurface::Input::TransmissionScatter,
						mx::OpenPBRSurface::Input::TransmissionScatterAnisotropy,
						mx::OpenPBRSurface::Input::TransmissionDispersionScale,
						mx::OpenPBRSurface::Input::TransmissionDispersionAbbeNumber,
						mx::OpenPBRSurface::Input::SubsurfaceWeight,
						mx::OpenPBRSurface::Input::SubsurfaceColor,
						mx::OpenPBRSurface::Input::SubsurfaceRadius,
						mx::OpenPBRSurface::Input::SubsurfaceRadiusScale,
						mx::OpenPBRSurface::Input::SubsurfaceScatterAnisotropy,
						mx::OpenPBRSurface::Input::FuzzWeight,
						mx::OpenPBRSurface::Input::FuzzColor,
						mx::OpenPBRSurface::Input::FuzzRoughness,
						mx::OpenPBRSurface::Input::CoatWeight,
						mx::OpenPBRSurface::Input::CoatColor,
						mx::OpenPBRSurface::Input::CoatRoughness,
						mx::OpenPBRSurface::Input::CoatRoughnessAnisotropy,
						mx::OpenPBRSurface::Input::CoatIOR,
						mx::OpenPBRSurface::Input::CoatDarkening,
						mx::OpenPBRSurface::Input::ThinFilmWeight,
						mx::OpenPBRSurface::Input::ThinFilmThickness,
						mx::OpenPBRSurface::Input::ThinFilmIOR,
						mx::OpenPBRSurface::Input::EmissionLuminance,
						mx::OpenPBRSurface::Input::EmissionColor,
						mx::OpenPBRSurface::Input::GeometryOpacity,
						mx::OpenPBRSurface::Input::GeometryThinWalled,
						mx::OpenPBRSurface::Input::GeometryNormal,
						mx::OpenPBRSurface::Input::GeometryCoatNormal,
						mx::OpenPBRSurface::Input::GeometryTangent,
						mx::OpenPBRSurface::Input::GeometryCoatTangent,
					},
					// OpenPBRSurfaceTransmission Outputs
					!bIsSubstrateEnabled ?
					TSet<FName>{
						PBRMR::Parameters::BaseColor,
						PBRMR::Parameters::Metallic,
						PBRMR::Parameters::Specular,
						PBRMR::Parameters::Roughness,
						PBRMR::Parameters::Anisotropy,
						PBRMR::Parameters::EmissiveColor,
						PBRMR::Parameters::Opacity,
						PBRMR::Parameters::Normal,
						PBRMR::Parameters::Tangent,
						PBRMR::Parameters::Refraction,
						ThinTranslucent::Parameters::TransmissionColor
					}	:
					TSet<FName>{
						OpenPBRSurface::SubstrateMaterial::Outputs::FrontMaterial,
						OpenPBRSurface::SubstrateMaterial::Outputs::OpacityMask
					}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::StandardSurface),
				{
					// StandardSurface Inputs
					TSet<FName>{
						mx::StandardSurface::Input::Base,
						mx::StandardSurface::Input::BaseColor,
						mx::StandardSurface::Input::DiffuseRoughness,
						mx::StandardSurface::Input::Metalness,
						mx::StandardSurface::Input::Specular,
						mx::StandardSurface::Input::SpecularColor,
						mx::StandardSurface::Input::SpecularRoughness,
						mx::StandardSurface::Input::SpecularIOR,
						mx::StandardSurface::Input::SpecularAnisotropy,
						mx::StandardSurface::Input::SpecularRotation,
						mx::StandardSurface::Input::Transmission,
						mx::StandardSurface::Input::TransmissionColor,
						mx::StandardSurface::Input::TransmissionDepth,
						mx::StandardSurface::Input::TransmissionScatter,
						mx::StandardSurface::Input::TransmissionScatterAnisotropy,
						mx::StandardSurface::Input::TransmissionDispersion,
						mx::StandardSurface::Input::TransmissionExtraRoughness,
						mx::StandardSurface::Input::Subsurface,
						mx::StandardSurface::Input::SubsurfaceColor,
						mx::StandardSurface::Input::SubsurfaceRadius,
						mx::StandardSurface::Input::SubsurfaceScale,
						mx::StandardSurface::Input::SubsurfaceAnisotropy,
						mx::StandardSurface::Input::Sheen,
						mx::StandardSurface::Input::SheenColor,
						mx::StandardSurface::Input::SheenRoughness,
						mx::StandardSurface::Input::Coat,
						mx::StandardSurface::Input::CoatColor,
						mx::StandardSurface::Input::CoatRoughness,
						mx::StandardSurface::Input::CoatAnisotropy,
						mx::StandardSurface::Input::CoatRotation,
						mx::StandardSurface::Input::CoatIOR,
						mx::StandardSurface::Input::CoatNormal,
						mx::StandardSurface::Input::CoatAffectColor,
						mx::StandardSurface::Input::CoatAffectRoughness,
						mx::StandardSurface::Input::ThinFilmThickness,
						mx::StandardSurface::Input::ThinFilmIOR,
						mx::StandardSurface::Input::Emission,
						mx::StandardSurface::Input::EmissionColor,
						mx::StandardSurface::Input::Opacity,
						mx::StandardSurface::Input::ThinWalled,
						mx::StandardSurface::Input::Normal,
						mx::StandardSurface::Input::Tangent,
					},
					// StandardSurface Outputs
					!bIsSubstrateEnabled ?
					TSet<FName>{
						TEXT("Base Color"), // MX_StandardSurface has BaseColor with a whitespace, this should be fixed in further release
						PBRMR::Parameters::Metallic,
						PBRMR::Parameters::Specular,
						PBRMR::Parameters::Roughness,
						PBRMR::Parameters::Anisotropy,
						PBRMR::Parameters::EmissiveColor,
						PBRMR::Parameters::Opacity,
						PBRMR::Parameters::Normal,
						PBRMR::Parameters::Tangent,
						Sheen::Parameters::SheenRoughness,
						Sheen::Parameters::SheenColor,
						Subsurface::Parameters::SubsurfaceColor,
						ClearCoat::Parameters::ClearCoat,
						ClearCoat::Parameters::ClearCoatRoughness,
						ClearCoat::Parameters::ClearCoatNormal
					}	:
					TSet<FName>{
						StandardSurface::SubstrateMaterial::Outputs::Opaque,
						StandardSurface::SubstrateMaterial::Outputs::Opacity
					}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::StandardSurfaceTransmission),
				{
					// StandardSurfaceTransmission Inputs
					TSet<FName>{
						mx::StandardSurface::Input::Base,
						mx::StandardSurface::Input::BaseColor,
						mx::StandardSurface::Input::DiffuseRoughness,
						mx::StandardSurface::Input::Metalness,
						mx::StandardSurface::Input::Specular,
						mx::StandardSurface::Input::SpecularColor,
						mx::StandardSurface::Input::SpecularRoughness,
						mx::StandardSurface::Input::SpecularIOR,
						mx::StandardSurface::Input::SpecularAnisotropy,
						mx::StandardSurface::Input::SpecularRotation,
						mx::StandardSurface::Input::Transmission,
						mx::StandardSurface::Input::TransmissionColor,
						mx::StandardSurface::Input::TransmissionDepth,
						mx::StandardSurface::Input::TransmissionScatter,
						mx::StandardSurface::Input::TransmissionScatterAnisotropy,
						mx::StandardSurface::Input::TransmissionDispersion,
						mx::StandardSurface::Input::TransmissionExtraRoughness,
						mx::StandardSurface::Input::Subsurface,
						mx::StandardSurface::Input::SubsurfaceColor,
						mx::StandardSurface::Input::SubsurfaceRadius,
						mx::StandardSurface::Input::SubsurfaceScale,
						mx::StandardSurface::Input::SubsurfaceAnisotropy,
						mx::StandardSurface::Input::Sheen,
						mx::StandardSurface::Input::SheenColor,
						mx::StandardSurface::Input::SheenRoughness,
						mx::StandardSurface::Input::Coat,
						mx::StandardSurface::Input::CoatColor,
						mx::StandardSurface::Input::CoatRoughness,
						mx::StandardSurface::Input::CoatAnisotropy,
						mx::StandardSurface::Input::CoatRotation,
						mx::StandardSurface::Input::CoatIOR,
						mx::StandardSurface::Input::CoatNormal,
						mx::StandardSurface::Input::CoatAffectColor,
						mx::StandardSurface::Input::CoatAffectRoughness,
						mx::StandardSurface::Input::ThinFilmThickness,
						mx::StandardSurface::Input::ThinFilmIOR,
						mx::StandardSurface::Input::Emission,
						mx::StandardSurface::Input::EmissionColor,
						mx::StandardSurface::Input::Opacity,
						mx::StandardSurface::Input::ThinWalled,
						mx::StandardSurface::Input::Normal,
						mx::StandardSurface::Input::Tangent,
					},
					// StandardSurfaceTransmission Outputs
					!bIsSubstrateEnabled ?
					TSet<FName>{
						PBRMR::Parameters::BaseColor,
						PBRMR::Parameters::Metallic,
						PBRMR::Parameters::Specular,
						PBRMR::Parameters::Roughness,
						PBRMR::Parameters::Anisotropy,
						PBRMR::Parameters::EmissiveColor,
						PBRMR::Parameters::Opacity,
						PBRMR::Parameters::Normal,
						PBRMR::Parameters::Tangent,
						PBRMR::Parameters::Refraction,
						ThinTranslucent::Parameters::TransmissionColor
					}	:
					TSet<FName>{
						StandardSurface::SubstrateMaterial::Outputs::Opaque, // It should be named Translucent, but it got renamed somehow
						StandardSurface::SubstrateMaterial::Outputs::Opacity,
					}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::SurfaceUnlit),
				{
					// SurfaceUnlit Inputs
					TSet<FName>{
						SurfaceUnlit::Parameters::Emission,
						SurfaceUnlit::Parameters::EmissionColor,
						SurfaceUnlit::Parameters::Transmission,
						SurfaceUnlit::Parameters::TransmissionColor,
						SurfaceUnlit::Parameters::Opacity
					},
					// SurfaceUnlit Outputs
					!bIsSubstrateEnabled ?
					TSet<FName>{
						Common::Parameters::EmissiveColor,
						Common::Parameters::Opacity,
						SurfaceUnlit::Outputs::OpacityMask
						} :
					TSet<FName>{
						SurfaceUnlit::Substrate::Outputs::OpacityMask,
						SurfaceUnlit::Substrate::Outputs::SurfaceUnlit
					}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::GltfPBR),
				{
					// GltfPBR Inputs
					TSet<FName>{
						mx::GltfPbr::Input::Alpha,
						mx::GltfPbr::Input::AlphaCutoff,
						mx::GltfPbr::Input::AlphaMode,
						mx::GltfPbr::Input::AnisotropyRotation,
						mx::GltfPbr::Input::AnisotropyStrength,
						mx::GltfPbr::Input::AttenuationColor,
						mx::GltfPbr::Input::AttenuationDistance,
						mx::GltfPbr::Input::BaseColor,
						mx::GltfPbr::Input::Clearcoat,
						mx::GltfPbr::Input::ClearcoatNormal,
						mx::GltfPbr::Input::ClearcoatRoughness,
						mx::GltfPbr::Input::Dispersion,
						mx::GltfPbr::Input::Emissive,
						mx::GltfPbr::Input::EmissiveStrength,
						mx::GltfPbr::Input::Ior,
						mx::GltfPbr::Input::Iridescence,
						mx::GltfPbr::Input::IridescenceIor,
						mx::GltfPbr::Input::IridescenceThickness,
						mx::GltfPbr::Input::Metallic,
						mx::GltfPbr::Input::Normal,
						mx::GltfPbr::Input::Occlusion,
						mx::GltfPbr::Input::Roughness,
						mx::GltfPbr::Input::SheenColor,
						mx::GltfPbr::Input::SheenRoughness,
						mx::GltfPbr::Input::Specular,
						mx::GltfPbr::Input::SpecularColor,
						mx::GltfPbr::Input::Tangent,
						mx::GltfPbr::Input::Thickness,
						mx::GltfPbr::Input::Transmission
					},
					// GltfPbr Outputs					
					TSet<FName>{
						SubstrateMaterial::Parameters::FrontMaterial,
						Common::Parameters::Refraction,
						Common::Parameters::Occlusion
						}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::UsdPreviewSurface),
				{
					// UsdPreviewSurface Inputs
					TSet<FName>{
						UsdPreviewSurface::Parameters::DiffuseColor,
						UsdPreviewSurface::Parameters::EmissiveColor,
						UsdPreviewSurface::Parameters::SpecularColor,
						UsdPreviewSurface::Parameters::Metallic,
						UsdPreviewSurface::Parameters::Roughness,
						UsdPreviewSurface::Parameters::Clearcoat,
						UsdPreviewSurface::Parameters::ClearcoatRoughness,
						UsdPreviewSurface::Parameters::Opacity,
						UsdPreviewSurface::Parameters::OpacityThreshold,
						UsdPreviewSurface::Parameters::IOR,
						UsdPreviewSurface::Parameters::Normal,
						UsdPreviewSurface::Parameters::Displacement,
						UsdPreviewSurface::Parameters::Occlusion
					},
					// UsdPreviewSurface Outputs
					!bIsSubstrateEnabled ?
					TSet<FName>{
						PBRMR::Parameters::BaseColor,
						PBRMR::Parameters::Metallic,
						PBRMR::Parameters::Specular,
						PBRMR::Parameters::Roughness,
						PBRMR::Parameters::EmissiveColor,
						PBRMR::Parameters::Opacity,
						PBRMR::Parameters::Normal,
						Common::Parameters::Refraction,
						Common::Parameters::Occlusion,
						ClearCoat::Parameters::ClearCoat,
						ClearCoat::Parameters::ClearCoatRoughness,
					} :
					TSet<FName>{
						UsdPreviewSurface::SubstrateMaterial::Outputs::FrontMaterial,
						UsdPreviewSurface::SubstrateMaterial::Outputs::Displacement,
						UsdPreviewSurface::SubstrateMaterial::Outputs::Occlusion}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::Surface),
				{
					// Surface Inputs
					TSet<FName>{
						Surface::Parameters::BSDF,
						Surface::Parameters::EDF,
						Surface::Parameters::Opacity
					},
					// Surface Outputs
					bIsSubstrateEnabled ?
					TSet<FName>{Surface::Outputs::Surface} :
					TSet<FName>{Surface::Substrate::Outputs::Surface,Surface::Substrate::Outputs::Opacity}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::Displacement),
				{
					// Displacement Inputs
					TSet<FName>{
						TEXT("displacement"),
						TEXT("scale"),
					},
					// Displacement Outputs
					TSet<FName>{
						TEXT("Displacement"),
						TEXT("Normal"), // Not part of the spec, but we have to recompute them
					}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::Volume),
				{
					// Volume Inputs
					TSet<FName>{
						TEXT("vdf"),
						TEXT("edf"),
					},
					// Volume Outputs
					TSet<FName>{TEXT("Volume")},
				}
			},
			// BSDF
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::OrenNayarDiffuse),
				{
					// OrenNayarDiffuse Inputs
					TSet<FName>{
						TEXT("weight"),
						TEXT("color"),
						TEXT("roughness"),
						TEXT("normal"),
						TEXT("energy_compensation"),
					},
					// OrenNayarDiffuse Outputs
					TSet<FName>{
						TEXT("Output")
					}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::BurleyDiffuse),
				{
					// BurleyDiffuse Inputs
					TSet<FName>{
						TEXT("weight"),
						TEXT("color"),
						TEXT("roughness"),
						TEXT("normal")
					},
					// BurleyDiffuse Outputs
					TSet<FName>{
						TEXT("Output")
					}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::Translucent),
				{
					// Translucent Inputs
					TSet<FName>{
						TEXT("weight"),
						TEXT("color"),
						TEXT("normal")
					},
					// Translucent Outputs
					TSet<FName>{
						TEXT("Output")
					}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::Dielectric),
				{
					// Dielectric Inputs
					TSet<FName>{
						TEXT("weight"),
						TEXT("tint"),
						TEXT("ior"),
						TEXT("roughness"),
						TEXT("thinfilm_thickness"),
						TEXT("thinfilm_ior"),
						TEXT("normal"),
						TEXT("tangent"),
						TEXT("distribution"),
						TEXT("scatter_mode"),
					},
					// Dielectric Outputs
					TSet<FName>{
						TEXT("Output")
					}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::Conductor),
				{
					// Conductor Inputs
					TSet<FName>{
						TEXT("weight"),
						TEXT("ior"),
						TEXT("extinction"),
						TEXT("roughness"),
						TEXT("normal"),
						TEXT("tangent")
					},
					// Conductor Outputs
					TSet<FName>{
						TEXT("Output")
					}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::GeneralizedSchlick),
				{
					// GeneralizedSchlick Inputs
					TSet<FName>{
						TEXT("weight"),
						TEXT("color0"),
						TEXT("color90"),
						TEXT("exponent"),
						TEXT("roughness"),
						TEXT("normal"),
						TEXT("tangent")
					},
					// GeneralizedSchlick Outputs
					TSet<FName>{
						TEXT("Output")
					}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::Subsurface),
				{
					// Subsurface Inputs
					TSet<FName>{
						TEXT("weight"),
						TEXT("color"),
						TEXT("radius"),
						TEXT("anisotropy"),
						TEXT("normal")
					},
					// Subsurface Outputs
					TSet<FName>{
						TEXT("Output")
					}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::ChiangHair),
				{
					// ChiangHair Inputs
					TSet<FName>{
						TEXT("tint_R"),
						TEXT("tint_TT"),
						TEXT("tint_TRT"),
						TEXT("ior"),
						TEXT("roughness_R"),
						TEXT("roughness_TT"),
						TEXT("roughness_TRT"),
						TEXT("cuticle_angle"),
						TEXT("absorption_coefficient"),
						TEXT("normal"),
						TEXT("curve_direction"),
					},
					// ChiangHair Outputs
					TSet<FName>{
						TEXT("Output")
					}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::Sheen),
				{
					// Sheen Inputs
					TSet<FName>{
						TEXT("weight"),
						TEXT("color"),
						TEXT("roughness"),
						TEXT("normal")
					},
					// Sheen Outputs
					TSet<FName>{
						TEXT("Output")
					}
				}
			},
			// EDF
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXEDF::Uniform),
				{
					// Uniform Inputs
					TSet<FName>{
						TEXT("color")
					},
					// Uniform Outputs
					TSet<FName>{
						TEXT("Output")
					}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXEDF::Conical),
				{
					// Conical Inputs
					TSet<FName>{
						TEXT("color"),
						TEXT("normal"),
						TEXT("inner_angle"),
						TEXT("outer_angle"),
					},
					// Conical Outputs
					TSet<FName>{
						TEXT("Output")
					}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXEDF::Measured),
				{
					// Measured Inputs
					TSet<FName>{
						TEXT("color"),
						TEXT("normal"),
						TEXT("file")
					},
					// Measured Outputs
					TSet<FName>{
						TEXT("Output")
					}
				}
			},
			// VDF
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXVDF::Absorption),
				{
					// Absorption Inputs
					TSet<FName>{
						TEXT("absorption")
					},
					// Absorption Outputs
					TSet<FName>{TEXT("Output")}
				}
			},
			{
				UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXVDF::Anisotropic),
				{
					// Anisotropic Inputs
					TSet<FName>{
						TEXT("absorption"),
						TEXT("scattering"),
						TEXT("anisotropy"),
					},
					// Anisotropic Outputs
					TSet<FName>{TEXT("Output")}
				}
			}
		};
	}
#endif // WITH_EDITOR
}

UInterchangeMaterialXPipeline::UInterchangeMaterialXPipeline()
	: MaterialXSettings(UMaterialXPipelineSettings::StaticClass()->GetDefaultObject<UMaterialXPipelineSettings>())
{
	if (HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
	{
#if WITH_EDITOR
		PathToEnumMapping = {
			{MATERIALX_MATERIALFUNCTION_PATH(OpenPBRSurface),		      UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::OpenPBRSurface)},
			{MATERIALX_MATERIALFUNCTION_PATH(OpenPBRTransmissionSurface), UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::OpenPBRSurfaceTransmission)},
			{MATERIALX_MATERIALFUNCTION_PATH(StandardSurface),			  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::StandardSurface)},
			{MATERIALX_MATERIALFUNCTION_PATH(TransmissionSurface),		  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::StandardSurfaceTransmission)},
			{MATERIALX_MATERIALFUNCTION_PATH(SurfaceUnlit),				  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::SurfaceUnlit)},
			{MATERIALX_MATERIALFUNCTION_PATH(Surface),					  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::Surface)},
			{MATERIALX_MATERIALFUNCTION_PATH(Volume),					  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::Volume)},
			{MATERIALX_MATERIALFUNCTION_PATH(GltfPBR),					  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::GltfPBR)},
			{MATERIALX_MATERIALFUNCTION_PATH(UsdPreviewSurface),		  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::UsdPreviewSurface)},
			{MATERIALX_MATERIALFUNCTION_PATH(Displacement),				  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::Displacement)},

			{MATERIALX_MATERIALFUNCTION_PATH(OrenNayarBSDF),			  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::OrenNayarDiffuse)},
			{MATERIALX_MATERIALFUNCTION_PATH(BurleyDiffuseBSDF),		  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::BurleyDiffuse)},
			{MATERIALX_MATERIALFUNCTION_PATH(DielectricBSDF),			  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::Dielectric)},
			{MATERIALX_MATERIALFUNCTION_PATH(ConductorBSDF),			  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::Conductor)},
			{MATERIALX_MATERIALFUNCTION_PATH(SheenBSDF),				  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::Sheen)},
			{MATERIALX_MATERIALFUNCTION_PATH(SubsurfaceBSDF),			  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::Subsurface)},
			{MATERIALX_MATERIALFUNCTION_PATH(GeneralizedSchlickBSDF),	  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::GeneralizedSchlick)},
			{MATERIALX_MATERIALFUNCTION_PATH(TranslucentBSDF),			  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::Translucent)},
			{MATERIALX_MATERIALFUNCTION_PATH(ChiangHairBSDF),			  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::ChiangHair)},

			{MATERIALX_MATERIALFUNCTION_PATH(UniformEDF),				  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXEDF::Uniform)},
			{MATERIALX_MATERIALFUNCTION_PATH(ConicalEDF),				  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXEDF::Conical)},
			{MATERIALX_MATERIALFUNCTION_PATH(MeasuredEDF),				  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXEDF::Measured)},

			{MATERIALX_MATERIALFUNCTION_PATH(AbsorptionVDF),			  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXVDF::Absorption)},
			{MATERIALX_MATERIALFUNCTION_PATH(AnisotropicVDF),			  UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXVDF::Anisotropic)},
		};

		MaterialXSettings->InitPredefinedAssets();
#endif // WITH_EDITOR
	}

	for (const TPair<EInterchangeMaterialXShaders, FSoftObjectPath>& Entry : MaterialXSettings->PredefinedSurfaceShaders)
	{
		PathToEnumMapping.FindOrAdd(Entry.Value.GetAssetPathString(), EInterchangeMaterialXSettings{ TInPlaceType<EInterchangeMaterialXShaders>{}, Entry.Key });
	}

	for(const TPair<EInterchangeMaterialXBSDF, FSoftObjectPath>& Entry : MaterialXSettings->PredefinedBSDF)
	{
		PathToEnumMapping.FindOrAdd(Entry.Value.GetAssetPathString(), EInterchangeMaterialXSettings{ TInPlaceType<EInterchangeMaterialXBSDF>{}, Entry.Key });
	}

	for(const TPair<EInterchangeMaterialXEDF, FSoftObjectPath>& Entry : MaterialXSettings->PredefinedEDF)
	{
		PathToEnumMapping.FindOrAdd(Entry.Value.GetAssetPathString(), EInterchangeMaterialXSettings{ TInPlaceType<EInterchangeMaterialXEDF>{}, Entry.Key });
	}

	for(const TPair<EInterchangeMaterialXVDF, FSoftObjectPath>& Entry : MaterialXSettings->PredefinedVDF)
	{
		PathToEnumMapping.FindOrAdd(Entry.Value.GetAssetPathString(), EInterchangeMaterialXSettings{ TInPlaceType<EInterchangeMaterialXVDF>{}, Entry.Key });
	}
}

void UInterchangeMaterialXPipeline::AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams)
{
	Super::AdjustSettingsForContext(ContextParams);

	if (!MaterialXSettings->AreRequiredPackagesLoaded())
	{
		UE_LOGF(LogInterchangePipeline, Warning, "UInterchangeGenericMaterialPipeline: Some required packages are missing. Material import might be wrong");
	}
}

void UInterchangeMaterialXPipeline::ExecutePipeline(UInterchangeBaseNodeContainer* NodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath)
{
	Super::ExecutePipeline(NodeContainer, InSourceDatas, ContentBasePath);

#if WITH_EDITOR
	// we only override the volume shader if it's not a data-driven path
	if (bVolumetricMaterial)
	{
		PathToEnumMapping.FindOrAdd(FOGCloudVolumeSlabSubstratePath, UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::Volume));
		FSoftObjectPath& VolumePath = MaterialXSettings->PredefinedSurfaceShaders.FindChecked(EInterchangeMaterialXShaders::Volume);
		if (VolumePath.ToString() == SimpleVolumeSlabSubstratePath)
		{
			VolumePath = FOGCloudVolumeSlabSubstratePath;
		}
	}
	else
	{
		PathToEnumMapping.FindOrAdd(SimpleVolumeSlabSubstratePath, UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::Volume));
		FSoftObjectPath& VolumePath = MaterialXSettings->PredefinedSurfaceShaders.FindChecked(EInterchangeMaterialXShaders::Volume);
		if (VolumePath.ToString() == FOGCloudVolumeSlabSubstratePath)
		{
			VolumePath = SimpleVolumeSlabSubstratePath;
		}
	}

	TArray<UInterchangeMaterialFunctionCallExpressionFactoryNode*> DisplacementNodes;
	// only add the StaticBoolParameter if the MX_Displacement hasn't been overridden
	const bool bRecomputeNormalsStaticBool = DisplacementFunctionsPath ==  MaterialXSettings->PredefinedSurfaceShaders.FindChecked(EInterchangeMaterialXShaders::Displacement).ToString();

	auto UpdateMaterialXNodes = [this, NodeContainer, &DisplacementNodes, bRecomputeNormalsStaticBool](const FString& NodeUid, UInterchangeMaterialFunctionCallExpressionFactoryNode* FactoryNode)
	{
		const FString MaterialFunctionMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionMaterialFunctionCall, MaterialFunction).ToString();

		FString FunctionShaderNodeUID = FactoryNode->GetUniqueID();
		FunctionShaderNodeUID.RemoveFromStart(TEXT("Factory_"));

		const UInterchangeFunctionCallShaderNode* FunctionCallShaderNode = Cast<UInterchangeFunctionCallShaderNode>(NodeContainer->GetNode(FunctionShaderNodeUID));
		
		if(int32 EnumType; FunctionCallShaderNode->GetInt32Attribute(UE::Interchange::MaterialX::Attributes::EnumType, EnumType))
		{
			if (int32 EnumValue; FunctionCallShaderNode->GetInt32Attribute(UE::Interchange::MaterialX::Attributes::EnumValue, EnumValue))
			{
				FactoryNode->AddStringAttribute(MaterialFunctionMemberName, MaterialXSettings->GetAssetPathString(MaterialXSettings->ToEnumKey(EnumType, EnumValue)));

				if (bRecomputeNormalsStaticBool &&
					EnumType == UE::Interchange::MaterialX::IndexSurfaceShaders &&
					EnumValue == int32(EInterchangeMaterialXShaders::Displacement))
				{
					DisplacementNodes.Emplace(FactoryNode);
				}
			}
		}
		if(FString MaterialFunctionPath; FactoryNode->GetStringAttribute(MaterialFunctionMemberName, MaterialFunctionPath))
		{
			if (const EInterchangeMaterialXSettings* EnumPtr = PathToEnumMapping.Find(MaterialFunctionPath))
			{
				FactoryNode->AddStringAttribute(MaterialFunctionMemberName, MaterialXSettings->GetAssetPathString(*EnumPtr));
			}
		}
	};

	//Find all translated node we need for this pipeline
	NodeContainer->IterateNodesOfType<UInterchangeMaterialFunctionCallExpressionFactoryNode>(UpdateMaterialXNodes);


	// Create StaticBool expression Factory node
	for(UInterchangeMaterialFunctionCallExpressionFactoryNode* DisplacementExpressionFactoryNode : DisplacementNodes)
	{
		const FString ParentUid = DisplacementExpressionFactoryNode->GetUniqueID();
		const FString InputName = TEXT("bRecomputeNormals");
		const FString MaterialExpressionUid = ParentUid + TEXT("\\") + InputName;

		UInterchangeMaterialExpressionFactoryNode* StaticBoolExpressionFactoryNode = NewObject<UInterchangeMaterialExpressionFactoryNode>(NodeContainer);
		StaticBoolExpressionFactoryNode->SetCustomExpressionClassName(UMaterialExpressionStaticBoolParameter::StaticClass()->GetName());
		NodeContainer->SetupNode(StaticBoolExpressionFactoryNode, MaterialExpressionUid, InputName, EInterchangeNodeContainerType::FactoryData, ParentUid);

		const FName DefaultValueMemberName = GET_MEMBER_NAME_CHECKED(UMaterialExpressionStaticBoolParameter, DefaultValue);
		StaticBoolExpressionFactoryNode->AddBooleanAttribute(DefaultValueMemberName.ToString(), bRecomputeNormals);
		StaticBoolExpressionFactoryNode->AddApplyAndFillDelegates<bool>(DefaultValueMemberName.ToString(), UMaterialExpressionStaticBoolParameter::StaticClass(), DefaultValueMemberName);

		UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(DisplacementExpressionFactoryNode, InputName, StaticBoolExpressionFactoryNode->GetUniqueID());
	}
#endif
}

bool UMaterialXPipelineSettings::AreRequiredPackagesLoaded()
{
	auto ArePackagesLoaded = [&](const auto& ObjectPaths)
	{
		bool bAllLoaded = true;

		for(const auto& Pair : ObjectPaths)
		{
			const FSoftObjectPath& ObjectPath = Pair.Value;

			if(!ObjectPath.ResolveObject())
			{
				FString PackagePath = ObjectPath.GetLongPackageName();
				if(FPackageName::DoesPackageExist(PackagePath))
				{
					UObject* Asset = ObjectPath.TryLoad();
					if(!Asset)
					{
						UE_LOGF(LogInterchangePipeline, Warning, "Couldn't load %ls", *PackagePath);
						bAllLoaded = false;
					}
#if WITH_EDITOR
					else
					{

						using EnumT = decltype(Pair.Key);

						static_assert(std::is_same_v<EnumT, EInterchangeMaterialXShaders> ||
									  std::is_same_v<EnumT, EInterchangeMaterialXBSDF> ||
									  std::is_same_v<EnumT, EInterchangeMaterialXEDF> ||
									  std::is_same_v<EnumT, EInterchangeMaterialXVDF>,
									  "Enum type not supported");

						uint8 EnumType = UE::Interchange::MaterialX::IndexSurfaceShaders;

						if constexpr(std::is_same_v<EnumT, EInterchangeMaterialXBSDF>)
						{
							EnumType = UE::Interchange::MaterialX::IndexBSDF;
						}
						else if constexpr(std::is_same_v<EnumT, EInterchangeMaterialXEDF>)
						{
							EnumType = UE::Interchange::MaterialX::IndexEDF;
						}
						else if constexpr(std::is_same_v<EnumT, EInterchangeMaterialXVDF>)
						{
							EnumType = UE::Interchange::MaterialX::IndexVDF;
						}

						if(FMaterialXSettings::ValueType* Settings = SettingsInputsOutputs.Find(ToEnumKey(EnumType, uint8(Pair.Key))))
						{
							bAllLoaded = !ShouldFilterAssets(Cast<UMaterialFunction>(Asset), Settings->Key, Settings->Value);
						}
					}
#endif // WITH_EDITOR
				}
				else
				{
					UE_LOGF(LogInterchangePipeline, Warning, "Couldn't find %ls", *PackagePath);
					bAllLoaded = false;
				}
			}
		}

		return bAllLoaded;
	};

	// ensure to load the FOGCloudVolumetric Material Function, we only need to load it once compared to the other shaders
	static bool bFOGCloudVolumetricAsset = ArePackagesLoaded(TMap<EInterchangeMaterialXShaders, FSoftObjectPath>{ { { EInterchangeMaterialXShaders::Volume, FSoftObjectPath{ FOGCloudVolumeSlabSubstratePath } } } });
	return ArePackagesLoaded(PredefinedSurfaceShaders) && ArePackagesLoaded(PredefinedBSDF) && ArePackagesLoaded(PredefinedEDF) && ArePackagesLoaded(PredefinedVDF) && bFOGCloudVolumetricAsset;
}

#if WITH_EDITOR
void UMaterialXPipelineSettings::InitPredefinedAssets()
{
	if(bIsSubstrateEnabled)
	{
		TArray<TTuple<EInterchangeMaterialXSettings, FString, FString>> MappingToSubstrate
		{
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::OpenPBRSurface), OpenPBRSurfaceFunctionsPath, OpenPBRSurfaceSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::OpenPBRSurfaceTransmission), OpenPBRTransmissionSurfaceFunctionsPath, OpenPBRTransmissionSurfaceSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::Surface), SurfaceFunctionsPath, SurfaceSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::Volume), VolumeFunctionsPath, VolumeSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::StandardSurface), StandardSurfaceFunctionsPath, StandardSurfaceSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::StandardSurfaceTransmission), TransmissionSurfaceFunctionsPath, TransmissionSurfaceSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::SurfaceUnlit), SurfaceUnlitFunctionsPath, SurfaceUnlitSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::GltfPBR), GltfPBRFunctionsPath, GltfPBRSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::UsdPreviewSurface), UsdPreviewSurfaceFunctionsPath, UsdPreviewSurfaceSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXShaders::Displacement), DisplacementFunctionsPath, DisplacementSubstratePath},

			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::OrenNayarDiffuse), OrenNayarBSDFFunctionsPath, OrenNayarBSDFSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::BurleyDiffuse), BurleyDiffuseBSDFFunctionsPath, BurleyDiffuseBSDFSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::Dielectric), DielectricBSDFFunctionsPath, DielectricBSDFSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::Conductor), ConductorBSDFFunctionsPath, ConductorBSDFSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::Sheen), SheenBSDFFunctionsPath, SheenBSDFSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::Subsurface), SubsurfaceBSDFFunctionsPath, SubsurfaceBSDFSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::GeneralizedSchlick), GeneralizedSchlickBSDFFunctionsPath, GeneralizedSchlickBSDFSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::Translucent), TranslucentBSDFFunctionsPath, TranslucentBSDFSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXBSDF::ChiangHair), ChiangHairBSDFFunctionsPath, ChiangHairBSDFSubstratePath},

			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXEDF::Uniform), UniformEDFFunctionsPath, UniformEDFSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXEDF::Conical), ConicalEDFFunctionsPath, ConicalEDFSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXEDF::Measured), MeasuredEDFFunctionsPath, MeasuredEDFSubstratePath},

			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXVDF::Absorption), AbsorptionVDFFunctionsPath, AbsorptionVDFSubstratePath},
			{UMaterialXPipelineSettings::ToEnumKey(EInterchangeMaterialXVDF::Anisotropic), AnisotropicVDFFunctionsPath, AnisotropicVDFSubstratePath},
		};

		for(const TTuple<EInterchangeMaterialXSettings, FString, FString> & Mapping : MappingToSubstrate)
		{
			const EInterchangeMaterialXSettings& ShadersSettings = Mapping.Get<0>();
			if(FString Path = GetAssetPathString(ShadersSettings); Path == Mapping.Get<1>())
			{
				if(ShadersSettings.IsType<EInterchangeMaterialXShaders>())
				{
					PredefinedSurfaceShaders.Add(ShadersSettings.Get<EInterchangeMaterialXShaders>(), FSoftObjectPath{ Mapping.Get<2>() });
				}
				else if(ShadersSettings.IsType<EInterchangeMaterialXBSDF>())
				{
					PredefinedBSDF.Add(ShadersSettings.Get<EInterchangeMaterialXBSDF>(), FSoftObjectPath{ Mapping.Get<2>() });
				}
				else if(ShadersSettings.IsType<EInterchangeMaterialXEDF>())
				{
					PredefinedEDF.Add(ShadersSettings.Get<EInterchangeMaterialXEDF>(), FSoftObjectPath{ Mapping.Get<2>() });
				}
				else if(ShadersSettings.IsType<EInterchangeMaterialXVDF>())
				{
					PredefinedVDF.Add(ShadersSettings.Get<EInterchangeMaterialXVDF>(), FSoftObjectPath{ Mapping.Get<2>() });
				}
			}
		}
	}
}
#endif // WITH_EDITOR

FString UMaterialXPipelineSettings::GetAssetPathString(EInterchangeMaterialXSettings EnumValue) const
{
	auto FindAssetPathString = [](const auto& PredefinedEnumPath, auto Enum) -> FString
	{
		if(const FSoftObjectPath* ObjectPath = PredefinedEnumPath.Find(Enum))
		{
			return ObjectPath->GetAssetPathString();
		}

		return {};
	};

	if(EnumValue.IsType<EInterchangeMaterialXShaders>())
	{
		return FindAssetPathString(PredefinedSurfaceShaders, EnumValue.Get<EInterchangeMaterialXShaders>());
	}
	else if(EnumValue.IsType<EInterchangeMaterialXBSDF>())
	{
		return FindAssetPathString(PredefinedBSDF, EnumValue.Get<EInterchangeMaterialXBSDF>());
	}
	else if(EnumValue.IsType<EInterchangeMaterialXEDF>())
	{
		return FindAssetPathString(PredefinedEDF, EnumValue.Get<EInterchangeMaterialXEDF>());
	}
	else if(EnumValue.IsType<EInterchangeMaterialXVDF>())
	{
		return FindAssetPathString(PredefinedVDF, EnumValue.Get<EInterchangeMaterialXVDF>());
	}

	return {};
}

#if WITH_EDITOR
bool UMaterialXPipelineSettings::ShouldFilterAssets(UMaterialFunction* Asset, const TSet<FName>& Inputs, const TSet<FName>& Outputs)
{
	int32 InputMatches = 0;
	int32 OutputMatches = 0;

	if (Asset != nullptr)
	{
		TArray<FFunctionExpressionInput> ExpressionInputs;
		TArray<FFunctionExpressionOutput> ExpressionOutputs;
		Asset->GetInputsAndOutputs(ExpressionInputs, ExpressionOutputs);

		for (const FFunctionExpressionInput& ExpressionInput : ExpressionInputs)
		{
			if (Inputs.Find(ExpressionInput.Input.InputName))
			{
				InputMatches++;
			}
		}

		for (const FFunctionExpressionOutput& ExpressionOutput : ExpressionOutputs)
		{
			if (Outputs.Find(ExpressionOutput.Output.OutputName))
			{
				OutputMatches++;
			}
		}
	}

	// we allow at least one input of the same name, but we should have exactly the same outputs
	return !(InputMatches > 0 && OutputMatches == Outputs.Num());
}

EInterchangeMaterialXSettings UMaterialXPipelineSettings::ToEnumKey(uint8 EnumType, uint8 EnumValue)
{
	switch(EnumType)
	{
	case UE::Interchange::MaterialX::IndexBSDF:
		return EInterchangeMaterialXSettings{ TInPlaceType<EInterchangeMaterialXBSDF>{}, EInterchangeMaterialXBSDF{EnumValue} };

	case UE::Interchange::MaterialX::IndexEDF:
		return EInterchangeMaterialXSettings{ TInPlaceType<EInterchangeMaterialXEDF>{}, EInterchangeMaterialXEDF{EnumValue} };

	case UE::Interchange::MaterialX::IndexVDF:
		return EInterchangeMaterialXSettings{ TInPlaceType<EInterchangeMaterialXVDF>{}, EInterchangeMaterialXVDF{EnumValue} };

	default:
		return EInterchangeMaterialXSettings{ TInPlaceType<EInterchangeMaterialXShaders>{}, EInterchangeMaterialXShaders{EnumValue} };
	}
}
#endif // WITH_EDITOR

namespace
{
	static uint8 GetMaterialXSettingsIndexValue(const EInterchangeMaterialXSettings Enum, SIZE_T& Index)
	{
		Index = Enum.GetIndex();
		const uint8* RawValuePointer = 
			Index == UE::Interchange::MaterialX::IndexBSDF ? reinterpret_cast<const uint8*>(Enum.TryGet<EInterchangeMaterialXBSDF>()) :
			Index == UE::Interchange::MaterialX::IndexEDF ? reinterpret_cast<const uint8*>(Enum.TryGet<EInterchangeMaterialXEDF>()) :
			Index == UE::Interchange::MaterialX::IndexVDF ? reinterpret_cast<const uint8*>(Enum.TryGet<EInterchangeMaterialXVDF>()) :
			reinterpret_cast<const uint8*>(Enum.TryGet<EInterchangeMaterialXShaders>());
		return *RawValuePointer;
	}
}

uint32 GetTypeHash(EInterchangeMaterialXSettings Key)
{
	SIZE_T Index;
	const uint8 UnderlyingValue = GetMaterialXSettingsIndexValue(Key, Index);
	return HashCombine(Index, UnderlyingValue);
}

bool operator==(EInterchangeMaterialXSettings Lhs, EInterchangeMaterialXSettings Rhs)
{
	SIZE_T LhsIndex;
	const uint8 LhsUnderlyingValue = GetMaterialXSettingsIndexValue(Lhs, LhsIndex);

	SIZE_T RhsIndex;
	const uint8 RhsUnderlyingValue = GetMaterialXSettingsIndexValue(Rhs, RhsIndex);

	return LhsIndex == RhsIndex && LhsUnderlyingValue == RhsUnderlyingValue;
}

#undef MATERIALX_FUNCTIONS_SUBSTRATE_PATH
#undef MATERIALX_MATERIALFUNCTION_PATH
