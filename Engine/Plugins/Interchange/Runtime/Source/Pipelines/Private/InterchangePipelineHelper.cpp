// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangePipelineHelper.h"

#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "CoreMinimal.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SpecularProfile.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "GeometryCache.h"
#include "GroomAsset.h"
#include "GroomBindingAsset.h"
#include "GroomCache.h"
#include "FoliageType_InstancedStaticMesh.h"
#include "HAL/PlatformApplicationMisc.h"
#include "InterchangePipelineBase.h"
#include "InterchangeSceneImportAsset.h"
#include "LevelSequence.h"
#include "LevelVariantSets.h"
#include "Materials/MaterialInterface.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Nodes/InterchangeSourceNode.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Sound/SoundBase.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"

namespace UE::Interchange::PipelineHelper
{
	void ShowModalDialog(TSharedRef<SInterchangeBaseConflictWidget> ConflictWidget, const FText& Title, const FVector2D& WindowSize)
	{
		TSharedPtr<SWindow> ParentWindow = FGlobalTabmanager::Get()->GetRootWindow();

		const float FbxImportWindowWidth = WindowSize.X > 150.0f ? WindowSize.X : 150.0f;
		const float FbxImportWindowHeight = WindowSize.Y > 50.0f ? WindowSize.Y : 50.0f;
		FVector2D FbxImportWindowSize = FVector2D(FbxImportWindowWidth, FbxImportWindowHeight); // Max window size it can get based on current slate

		FSlateRect WorkAreaRect = FSlateApplicationBase::Get().GetPreferredWorkArea();
		FVector2D DisplayTopLeft(WorkAreaRect.Left, WorkAreaRect.Top);
		FVector2D DisplaySize(WorkAreaRect.Right - WorkAreaRect.Left, WorkAreaRect.Bottom - WorkAreaRect.Top);

		float ScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(DisplayTopLeft.X, DisplayTopLeft.Y);
		FbxImportWindowSize *= ScaleFactor;

		FVector2D WindowPosition = (DisplayTopLeft + (DisplaySize - FbxImportWindowSize) / 2.0f) / ScaleFactor;

		TSharedRef<SWindow> Window = SNew(SWindow)
			.Title(Title)
			.SizingRule(ESizingRule::UserSized)
			.AutoCenter(EAutoCenter::None)
			.ClientSize(FbxImportWindowSize)
			.ScreenPosition(WindowPosition);

		ConflictWidget->SetWidgetWindow(Window);
		Window->SetContent
		(
			ConflictWidget
		);

		// @todo: we can make this slow as showing progress bar later
		FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
	}

	bool FillSubPathFromSourceNode(UInterchangeFactoryBaseNode* FactoryNode, const UInterchangeSourceNode* SourceNode)
	{
		if (!FactoryNode || !SourceNode)
		{
			return false;
		}

		FString Prefix;
		bool bHasPrefix = SourceNode->GetCustomSubPathPrefix(Prefix);

		bool bUseAssetTypeSuffix = false;
		if (bool bAttributeValue = false; SourceNode->GetCustomUseAssetTypeSubPathSuffix(bAttributeValue))
		{
			bUseAssetTypeSuffix = bAttributeValue;
		}

		if (!bUseAssetTypeSuffix && !bHasPrefix)
		{
			// Nothing to do
			return true;
		}

		FString Suffix;
		const UClass* Class = FactoryNode->GetObjectClass();
		if (bUseAssetTypeSuffix && Class)
		{
			if (
				Class->IsChildOf(UMaterialInterface::StaticClass()) ||
				Class->IsChildOf(USpecularProfile::StaticClass())
			)
			{
				Suffix = TEXT("Materials");
			}
			else if (Class->IsChildOf(UStaticMesh::StaticClass()))
			{
				Suffix = TEXT("StaticMeshes");
			}
			else if (
				Class->IsChildOf(UTexture::StaticClass()) ||
				Class->IsChildOf(USparseVolumeTexture::StaticClass())
			)
			{
				Suffix = TEXT("Textures");
			}
			else if (
				Class->IsChildOf(USkeletalMesh::StaticClass()) ||
				Class->IsChildOf(USkeleton::StaticClass()) ||
				Class->IsChildOf(UPhysicsAsset::StaticClass()) ||
				Class->IsChildOf(UAnimSequence::StaticClass())
			)
			{
				Suffix = TEXT("SkeletalMeshes");
			}
			else if (Class->IsChildOf(ULevelSequence::StaticClass()))
			{
				Suffix = TEXT("LevelSequences");
			}
			else if (Class->IsChildOf(UGeometryCache::StaticClass()))
			{
				Suffix = TEXT("GeometryCaches");
			}
			else if (Class->IsChildOf(UGroomAsset::StaticClass()) || Class->IsChildOf(UGroomCache::StaticClass()) || Class->IsChildOf(UGroomBindingAsset::StaticClass()))
			{
				Suffix = TEXT("Grooms");
			}
			else if (Class->IsChildOf(UWorld::StaticClass()))
			{
				Suffix = TEXT("Levels");
			}
			else if (Class->IsChildOf(USoundBase::StaticClass()))
			{
				Suffix = TEXT("Sounds");
			}
			else if (Class->IsChildOf(ULevelVariantSets::StaticClass()))
			{
				Suffix = TEXT("Variants");
			}
			else if (Class->IsChildOf(UFoliageType_InstancedStaticMesh::StaticClass()))
			{
				Suffix = TEXT("Foliage");
			}
			else if (Class->IsChildOf(UInterchangeSceneImportAsset::StaticClass()))
			{
				// No suffix. Always place this outside of the asset type folders
			}
			// TODO: Category for GroomAsset, GroomCache, GroomBindingAsset.
			// Not done yet because base Interchange doesn't support those assets anyway, and it would require
			// another plugin dependency on the HairStrands plugin
			else
			{
				// Fallback to using the class name as the subfolder
				Suffix = Class->GetName();
			}
		}

		return FactoryNode->SetCustomSubPath(FPaths::Combine(Prefix, Suffix));
	}
}
