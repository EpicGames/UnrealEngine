// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangePipelineMeshesUtilities.h"

#include "InterchangeMeshBundleNode.h"
#include "InterchangeMeshFactoryNode.h"
#include "InterchangeMeshLODContainerNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangeSceneComponentNodes.h"
#include "InterchangeJointNode.h"
#include "Mesh/InterchangeMeshHelper.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeSourceNode.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"

DEFINE_LOG_CATEGORY(LogInterchangePipelineMeshesUtilities);

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangePipelineMeshesUtilities)

namespace UE::Private::InterchangeMeshPipeline
{
	void FindNamedLodGroup(UInterchangeBaseNodeContainer* BaseNodeContainer)
	{
		//{ParentNodePtr, LODContainerName} => [{CandidateNode, CandidateLODIndex}..]
		TMap<TPair<const UInterchangeSceneNode*, FString>, TArray<TPair<UInterchangeSceneNode*, int32>>> LODContainerCandidates;
		const FString LodPrefix = TEXT("LOD");
		BaseNodeContainer->IterateNodesOfType<UInterchangeSceneNode>(
			[&BaseNodeContainer, &LodPrefix, &LODContainerCandidates](const FString& NodeUid, UInterchangeSceneNode* SceneNode)
			{
				if (!SceneNode->IsEnabled())
				{
					return;
				}

				FString AssetInstanceUid;
				if (!SceneNode->GetCustomAssetInstanceUid(AssetInstanceUid))
				{
					return;
				}

				if (const UInterchangeMeshLODContainerNode* MeshLODContainerNode = Cast<UInterchangeMeshLODContainerNode>(BaseNodeContainer->GetNode(AssetInstanceUid)))
				{
					return;
				}

				FString SceneNodeName = SceneNode->GetDisplayLabel();

				int32 LODIndex;
				FString LODContainerName;
				FString Error;
				if (UInterchangeMeshLODContainerNode::CheckForLODPattern(SceneNodeName, LODIndex, LODContainerName, Error))
				{
					FString ParentUid = SceneNode->GetParentUid();
					const UInterchangeSceneNode* ConstParentSceneNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(ParentUid));

					LODContainerCandidates.FindOrAdd(TPair<const UInterchangeSceneNode*, FString>(ConstParentSceneNode, LODContainerName)).Add(TPair<UInterchangeSceneNode*, int32>(SceneNode, LODIndex));
				}
				else if (Error.Len())
				{
					if (UE::Interchange::Private::MeshHelper::GetMeshCollisionFromName(SceneNodeName) == EInterchangeMeshCollision::None)
					{
						//Do not warn for presumed Collision nodes: (for ep: UCX_SM_Test_LOD0)
						UE_LOGF(LogInterchangePipelineMeshesUtilities, Warning, "%ls", *Error);
					}
				}
			}
		);

		//{ParentNodePtr, LODContainerName} => {CandidateNode, CandidateLODIndex}
		for (const TPair<TPair<const UInterchangeSceneNode*, FString>, TArray<TPair<UInterchangeSceneNode*, int32>>>& LODContainerCandidate : LODContainerCandidates)
		{
			if (LODContainerCandidate.Value.Num() <= 1)
			{
				//#interchange_LODRefactor_Note: Do not create LODContainer if only 1 node is marked for LODs, as pre-refactor implementation.
				continue;
			}

			const UInterchangeSceneNode* ConstParentSceneNode = LODContainerCandidate.Key.Key;
			const FString LODContainerName = LODContainerCandidate.Key.Value;

			FString MeshLODContainerNodeUid = UInterchangeMeshLODContainerNode::MakeMeshLODContainerUid((ConstParentSceneNode ? ConstParentSceneNode->GetUniqueID() : TEXT("_RootSceneNode_")) + TEXT("_") + LODContainerName);

			if (const UInterchangeMeshLODContainerNode* ExistingLodContainerNode = Cast<UInterchangeMeshLODContainerNode>(BaseNodeContainer->GetNode(MeshLODContainerNodeUid)))
			{
				//If the LODContainerNode already exists, then we presume all LODs have been parsed already and we can skip to the next entry for processing.
				continue;
			}

			UInterchangeMeshLODContainerNode* LODContainerNode = NewObject<UInterchangeMeshLODContainerNode>(BaseNodeContainer);
			BaseNodeContainer->SetupNode(
				LODContainerNode,
				MeshLODContainerNodeUid,
				LODContainerName,
				EInterchangeNodeContainerType::TranslatedAsset
			);

			auto CreateInstantiation = [&](const FString& ParentUid, bool bSetParent = true)
				{
					FString NewSceneNodeUid = ParentUid + TEXT("\\MeshLODContainerInstance\\") + LODContainerName;

					if (Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(NewSceneNodeUid)) == nullptr)
					{
						UInterchangeSceneNode* NewSceneNode = NewObject<UInterchangeSceneNode>(BaseNodeContainer);
						BaseNodeContainer->SetupNode(NewSceneNode, NewSceneNodeUid, LODContainerName, EInterchangeNodeContainerType::TranslatedScene, bSetParent ? ParentUid : TEXT(""));
						NewSceneNode->SetCustomAssetInstanceUid(MeshLODContainerNodeUid);
					}
				};

			if (ConstParentSceneNode)
			{
				CreateInstantiation(ConstParentSceneNode->GetUniqueID());
			}
			else
			{
				CreateInstantiation(FString(TEXT("_RootSceneNode_")), false);
			}

			//Add LODs to new LODContainer
			for (const TPair<UInterchangeSceneNode*, int32>& LOD : LODContainerCandidate.Value)
			{
				if (!LOD.Key)
				{
					continue;
				}

				FString MeshUid;
				if (!LOD.Key->GetCustomAssetInstanceUid(MeshUid))
				{
					continue;
				}

				FTransform LocalTransfrom;
				LOD.Key->GetCustomLocalTransform(LocalTransfrom);
				LODContainerNode->AddMeshForLODIndex(LOD.Value, MeshUid, LocalTransfrom);

				LOD.Key->SetCustomDoNotInstantiateInLevel(true);
			}
		}
	}
}

static bool IsSceneNodeASocket(const UInterchangeSceneNode* SceneNode)
{
	// Generic pipeline determines where a scene node should be considered as a socket from its naming convention.
	// This is no longer decided by the translator.
	FString NodeDisplayName = SceneNode->GetDisplayLabel();
	return NodeDisplayName.StartsWith(UInterchangeMeshFactoryNode::GetMeshSocketPrefix());
}

bool FInterchangeMeshInstance::IsNestedInSkeleton() const
{
	return InstancingNode && InstancingNode->IsA<UInterchangeJointNode>();
}

bool FInterchangeMeshInstance::DoNotInstantiateInLevel() const
{
	if (const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(InstancingNode))
	{
		bool bDoNotInstantiateInLevel = false;
		if (SceneNode && SceneNode->GetCustomDoNotInstantiateInLevel(bDoNotInstantiateInLevel))
		{
			return bDoNotInstantiateInLevel;
		}
	}
	return false;
}

void FInterchangeMeshGeometry::SetupMeshGeometry(const UInterchangeBaseNodeContainer& BaseNodeContainer, const UInterchangeBaseNode* InBaseNode)
{
	BaseNode = InBaseNode;

	SetIsSkinnedMesh(BaseNodeContainer);
	SetHasMorphTargetDependencies(BaseNodeContainer);
	SetHasSkeletonDependecies(BaseNodeContainer);
	SetHasAssemblyPartDependencies(BaseNodeContainer);
	SetAssemblyPartDependencies(BaseNodeContainer);
}

bool FInterchangeMeshGeometry::IsMorphTarget() const
{
	if (const UInterchangeMeshNode* BaseMeshNode = Cast<UInterchangeMeshNode>(BaseNode))
	{
		return BaseMeshNode->IsMorphTarget();
	}

	return false;
}

FString FInterchangeMeshGeometry::GetSkeletonRootUid(const UInterchangeBaseNodeContainer& BaseNodeContainer, const TArray<FString>& SkeletonRootNodeUids) const
{
	//We are not caching this as results depend on input.

	return ParseMeshOrLODContainerNode<FString>(BaseNodeContainer,
		UInterchangeBaseNode::InvalidNodeUid(), StringConflictHandler,
		[this, &SkeletonRootNodeUids, &BaseNodeContainer](const UInterchangeMeshNode* SkinnedMeshNode)
		{
			FString JointNodeUid;
			SkinnedMeshNode->GetSkeletonDependency(0, JointNodeUid);
			while (!JointNodeUid.Equals(UInterchangeBaseNode::InvalidNodeUid()) && !SkeletonRootNodeUids.Contains(JointNodeUid))
			{
				const UInterchangeBaseNode* JointNode = BaseNodeContainer.GetNode(JointNodeUid);
				if (ensureMsgf(JointNode, TEXT("Node set as SkeletonDependency must exist in the container!")))
				{
					JointNodeUid = JointNode->GetParentUid();
				}
				else
				{
					break;
				}
			}

			return JointNodeUid;
		});
}

template <class T>
T FInterchangeMeshGeometry::ParseMeshOrLODContainerNode(const UInterchangeBaseNodeContainer& BaseNodeContainer, T DefaultValue, TFunctionRef<T(T, T)> HandleConflictFunction, TFunctionRef<T(const UInterchangeMeshNode* MeshNodeInFunction)> MeshFunction) const
{
	if (!BaseNode)
	{
		return DefaultValue;
	}

	if (const UInterchangeMeshNode* BaseMeshNode = Cast<UInterchangeMeshNode>(BaseNode))
	{
		return MeshFunction(BaseMeshNode);
	}
	else
	{
		// Both UInterchangeMeshLODContainerNode and UInterchangeMeshBundleNode contain references to child
		// UInterchangeMeshNode UIDs. We collect those UIDs and apply the same per-MeshNode logic for both.
		TArray<FString> ReferencedMeshUidArray;

		if (const UInterchangeMeshLODContainerNode* LODContainerNode = Cast<UInterchangeMeshLODContainerNode>(BaseNode))
		{
			TSet<FString> ReferencedMeshUids;
			LODContainerNode->GetAllReferencedMeshUids(ReferencedMeshUids);
			ReferencedMeshUidArray = ReferencedMeshUids.Array();
		}
		else if (const UInterchangeMeshBundleNode* MeshBundleNode = Cast<UInterchangeMeshBundleNode>(BaseNode))
		{
			MeshBundleNode->GetMeshNodeUids(ReferencedMeshUidArray);
		}

		if (ReferencedMeshUidArray.Num() > 0)
		{
			TOptional<T> OptionalValue;
			for (const FString& ReferencedMeshUid : ReferencedMeshUidArray)
			{
				if (const UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(BaseNodeContainer.GetNode(ReferencedMeshUid)))
				{
					if (OptionalValue.IsSet())
					{
						T CurrentValue = MeshFunction(MeshNode);
						if (OptionalValue != CurrentValue)
						{
							OptionalValue = HandleConflictFunction(OptionalValue.GetValue(), CurrentValue);
						}
					}
					else
					{
						OptionalValue = MeshFunction(MeshNode);
					}
				}
			}

			if (OptionalValue.IsSet())
			{
				return OptionalValue.GetValue();
			}
		}
	}

	return DefaultValue;
}

bool FInterchangeMeshGeometry::BoolConflictHandler(bool bLeft, bool bRight)
{
	return bLeft || bRight;
}

FString FInterchangeMeshGeometry::StringConflictHandler(const FString& Left, const FString& Right)
{
	UE_LOGF(LogInterchangePipelineMeshesUtilities, Warning, "SkeletonRootUid confict found in LODContainer: %ls vs %ls", *Left, *Right);
	return Left;
}

void FInterchangeMeshGeometry::SetIsSkinnedMesh(const UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	bIsSkinnedMesh = ParseMeshOrLODContainerNode<bool>(BaseNodeContainer,
		false, BoolConflictHandler,
		[](const UInterchangeMeshNode* MeshNode)
		{
			return MeshNode->IsSkinnedMesh();
		});
}

void FInterchangeMeshGeometry::SetHasMorphTargetDependencies(const UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	bHasMorphTargetDependencies = ParseMeshOrLODContainerNode<bool>(BaseNodeContainer,
		0, BoolConflictHandler,
		[](const UInterchangeMeshNode* MeshNode)
		{
			return MeshNode->GetMorphTargetDependeciesCount() > 0;
		});
}

void FInterchangeMeshGeometry::SetHasSkeletonDependecies(const UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	bHasSkeletonDependencies = ParseMeshOrLODContainerNode<bool>(BaseNodeContainer,
		0, BoolConflictHandler,
		[](const UInterchangeMeshNode* MeshNode)
		{
			return MeshNode->GetSkeletonDependeciesCount() > 0;
		});
}

void FInterchangeMeshGeometry::SetHasAssemblyPartDependencies(const UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	bHasAssemblyPartDependencies = ParseMeshOrLODContainerNode<bool>(BaseNodeContainer, 
		0, BoolConflictHandler,
		[](const UInterchangeMeshNode* MeshNode)
		{
			return MeshNode->GetAssemblyPartDependenciesCount() > 0;
		});
}

void FInterchangeMeshGeometry::SetAssemblyPartDependencies(const UInterchangeBaseNodeContainer& BaseNodeContainer)
{
	TArray<FString> AssemblyPartUids = ParseMeshOrLODContainerNode<TArray<FString>>(BaseNodeContainer,
		TArray<FString>(),
		[](const TArray<FString>& Left, const TArray<FString>& Right)
		{
			TArray<FString> Combined = Left;
			Combined.Append(Right);
			return Combined;
		},
		[](const UInterchangeMeshNode* MeshNode) -> TArray<FString>
		{
			TArray<FString> Dependencies;
			MeshNode->GetAssemblyPartDependencies(Dependencies);
			return Dependencies;
		});

	//Make sure parts are unique.
	AssemblyPartDependenciesArray = TSet<FString>(AssemblyPartUids).Array();
}

bool FInterchangePipelineMeshesUtilitiesContext::IsStaticMeshInstance(const FInterchangeMeshInstance& MeshInstance, UInterchangeBaseNodeContainer* BaseNodeContainer)
{
	if (MeshInstance.DoNotInstantiateInLevel())
	{
		return false;
	}
	if (MeshInstance.bIsGeometryCache)
	{
		return false;
	}
	return !IsSkeletalMeshInstance(MeshInstance, BaseNodeContainer);
}

bool FInterchangePipelineMeshesUtilitiesContext::IsSkeletalMeshInstance(const FInterchangeMeshInstance& MeshInstance, UInterchangeBaseNodeContainer* BaseNodeContainer)
{
	if (MeshInstance.DoNotInstantiateInLevel())
	{
		return false;
	}
	if (MeshInstance.bIsGeometryCache)
	{
		return false;
	}
	if (ForceMeshType == EInterchangeForceMeshType::IFMT_StaticMesh)
	{
		return false;
	}
	if (bConvertStaticsWithAnimatedTransformToSkeletals && MeshInstance.bHasTransformAnimationTrack)
	{
		return true;
	}
	if (!MeshInstance.bReferenceMorphTarget)
	{
		if (ForceMeshType == EInterchangeForceMeshType::IFMT_SkeletalMesh)
		{
			return true;
		}
		if (MeshInstance.bReferenceSkinnedMesh)
		{
			return true;
		}
		if (bConvertStaticsWithMorphTargetsToSkeletals && MeshInstance.bHasMorphTargets)
		{
			return true;
		}
	}
	if (bConvertStaticsInBoneHierarchyToSkeletals && MeshInstance.IsNestedInSkeleton())
	{
		return true;
	}
	return false;
}

bool FInterchangePipelineMeshesUtilitiesContext::IsGeometryCacheInstance(const FInterchangeMeshInstance& MeshInstance)
{
	if (bIgnoreGeometryCaches)
	{
		return false;
	}
	return MeshInstance.bIsGeometryCache;
}

bool FInterchangePipelineMeshesUtilitiesContext::IsStaticMeshGeometry(const FInterchangeMeshGeometry& MeshGeometry)
{
	if (bQueryGeometryOnlyIfNoInstance && MeshGeometry.UidsOfInstancingNodes.Num() > 0)
	{
		return false;
	}
	if (MeshGeometry.IsMorphTarget())
	{
		return false;
	}
	if (MeshGeometry.BaseNode->IsA<UInterchangeGeometryCacheNode>())
	{
		return false;
	}
	if (ForceMeshType == EInterchangeForceMeshType::IFMT_StaticMesh)
	{
		return true;
	}
	if (bConvertStaticsInBoneHierarchyToSkeletals && MeshGeometry.bIsReferencedBySkeleton)
	{
		return false;
	}
	return !IsSkeletalMeshGeometry(MeshGeometry);
}

bool FInterchangePipelineMeshesUtilitiesContext::IsSkeletalMeshGeometry(const FInterchangeMeshGeometry& MeshGeometry)
{
	if (ForceMeshType == EInterchangeForceMeshType::IFMT_StaticMesh)
	{
		return false;
	}
	if (bQueryGeometryOnlyIfNoInstance && MeshGeometry.UidsOfInstancingNodes.Num() > 0)
	{
		return false;
	}
	if (MeshGeometry.IsMorphTarget())
	{
		return false;
	}
	if (MeshGeometry.BaseNode->IsA<UInterchangeGeometryCacheNode>())
	{
		return false;
	}
	if (ForceMeshType == EInterchangeForceMeshType::IFMT_SkeletalMesh)
	{
		return true;
	}
	if (MeshGeometry.IsSkinnedMesh())
	{
		return true;
	}
	if (bConvertStaticsWithMorphTargetsToSkeletals && MeshGeometry.HasMorphTargetDependencies())
	{
		return true;
	}
	if (bConvertStaticsWithAnimatedTransformToSkeletals && MeshGeometry.bHasTransformAnimationTrack)
	{
		return true;
	}
	return false;
}

bool FInterchangePipelineMeshesUtilitiesContext::IsGeometryCacheGeometry(const FInterchangeMeshGeometry& MeshGeometry)
{
	if (bIgnoreGeometryCaches)
	{
		return false;
	}
	if (bQueryGeometryOnlyIfNoInstance && MeshGeometry.UidsOfInstancingNodes.Num() > 0)
	{
		return false;
	}
	return MeshGeometry.BaseNode->IsA<UInterchangeGeometryCacheNode>();
}

bool IsImpactingMeshRecursive(const UInterchangeSceneNode* AnimatedNode
	, const UInterchangeBaseNodeContainer* InBaseNodeContainer
	, const FString& StaticMeshNodeUid)
{
	FString AssetUid;
	if (AnimatedNode->GetCustomAssetInstanceUid(AssetUid))
	{
		if (StaticMeshNodeUid == AssetUid)
		{
			return true;
		}
	}

	TArray<FString> Children = InBaseNodeContainer->GetNodeChildrenUids(AnimatedNode->GetUniqueID());
	for (const FString& ChildUid : Children)
	{
		if (const UInterchangeSceneNode* ChildSceneNode = Cast<UInterchangeSceneNode>(InBaseNodeContainer->GetNode(ChildUid)))
		{
			if (IsImpactingMeshRecursive(ChildSceneNode, InBaseNodeContainer, StaticMeshNodeUid))
			{
				return true;
			}
		}
	}

	return false;
}

UInterchangePipelineMeshesUtilities* UInterchangePipelineMeshesUtilities::CreateInterchangePipelineMeshesUtilities(UInterchangeBaseNodeContainer* BaseNodeContainer)
{
	if (!ensure(BaseNodeContainer))
	{
		return nullptr;
	}
	UInterchangePipelineMeshesUtilities* PipelineMeshesUtilities = NewObject<UInterchangePipelineMeshesUtilities>(GetTransientPackage(), NAME_None);
	
	//Set the container
	PipelineMeshesUtilities->BaseNodeContainer = BaseNodeContainer;

	UE::Private::InterchangeMeshPipeline::FindNamedLodGroup(BaseNodeContainer);

	TArray<FString> SkeletonRootNodeUids;

	//Find all translated node we need for this pipeline
	BaseNodeContainer->IterateNodes(
		[&PipelineMeshesUtilities, &BaseNodeContainer](const FString& NodeUid, UInterchangeBaseNode* Node)
		{
			if (Node->GetNodeContainerType() == EInterchangeNodeContainerType::TranslatedAsset)
			{
				if (Node->IsA<UInterchangeMeshNode>() || Node->IsA<UInterchangeMeshLODContainerNode>() || Node->IsA<UInterchangeMeshBundleNode>())
				{
					FInterchangeMeshGeometry& MeshGeometry = PipelineMeshesUtilities->MeshGeometriesPerMeshUid.FindOrAdd(NodeUid);
					MeshGeometry.SetupMeshGeometry(*BaseNodeContainer, Node);

					if (MeshGeometry.HasAssemblyPartDependencies())
					{
						PipelineMeshesUtilities->AssemblyMeshUids.Add(NodeUid);
						TArray<FString> AssemblyPartDependencies;
						MeshGeometry.GetAssemblyPartDependencies(AssemblyPartDependencies);
						PipelineMeshesUtilities->AssemblyMeshUids.Append(AssemblyPartDependencies);
					}
				}
			}
		}
	);

	TSet<const UInterchangeSceneNode*> SceneNodesWithTransformAnimationTrackNode;

	BaseNodeContainer->IterateNodesOfType<UInterchangeTransformAnimationTrackNode>(
		[&PipelineMeshesUtilities, &BaseNodeContainer, &SceneNodesWithTransformAnimationTrackNode](const FString& NodeUid, UInterchangeTransformAnimationTrackNode* AnimationNode)
		{
			FString SceneNodeUid;
			if (AnimationNode->GetCustomActorDependencyUid(SceneNodeUid))
			{
				if (const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(SceneNodeUid)))
				{
					SceneNodesWithTransformAnimationTrackNode.Add(SceneNode);
				}
			}
		});

	//returns false if MeshUid is not a Mesh.
	auto ProcessMeshNode = [&PipelineMeshesUtilities, &BaseNodeContainer, &SceneNodesWithTransformAnimationTrackNode](const FString& MeshInstanceUid, const UInterchangeBaseNode* BaseNode, const FString& AssetUid) -> bool
		{
			const UInterchangeBaseNode* AssetNode = BaseNodeContainer->GetNode(AssetUid);

			FInterchangeMeshInstance* MeshInstancePtr = nullptr;

			bool bIsVisible = true;
			if (const UInterchangeSceneNode* SceneNode = Cast<const UInterchangeSceneNode>(BaseNode))
			{
				SceneNode->GetCustomActorVisibility(bIsVisible);
			}

			const bool bIsNestedIntoSkeleton = BaseNode && BaseNode->IsA<UInterchangeJointNode>();

			if (AssetNode && (AssetNode->IsA<UInterchangeMeshNode>() || AssetNode->IsA<UInterchangeMeshLODContainerNode>()))
			{
				FInterchangeMeshGeometry& MeshGeometry = PipelineMeshesUtilities->MeshGeometriesPerMeshUid.FindChecked(AssetUid);
				MeshGeometry.UidsOfInstancingNodes.Add(MeshInstanceUid);
				MeshGeometry.bIsReferencedBySkeleton |= bIsNestedIntoSkeleton;

				FInterchangeMeshInstance& MeshInstance = PipelineMeshesUtilities->MeshInstancesPerMeshInstanceUid.FindOrAdd(MeshInstanceUid);
				MeshInstance.InstancingNode = BaseNode;
				MeshInstance.ReferencingMeshGeometryUids.Add(AssetUid);
				MeshInstance.bIsVisible = bIsVisible;

				if (MeshGeometry.BaseNode->IsA<UInterchangeGeometryCacheNode>())
				{
					MeshInstance.bReferenceSkinnedMesh = false;
					MeshInstance.bReferenceMorphTarget = false;
					MeshInstance.bHasMorphTargets = false;
					MeshInstance.bIsGeometryCache = true;	
				}
				else
				{
					MeshInstance.bReferenceSkinnedMesh |= MeshGeometry.IsSkinnedMesh();
					MeshInstance.bReferenceMorphTarget |= MeshGeometry.IsMorphTarget();
					MeshInstance.bHasMorphTargets |= MeshGeometry.HasMorphTargetDependencies();
				}

				MeshInstancePtr = &MeshInstance;
			}
			//#interchange_LODRefactor_Note: we might be able to use UInterchangeMeshLODContainerNode instead of UInterchangeMeshBundleNode completely?
			else if (const UInterchangeMeshBundleNode* MeshBundle = Cast<UInterchangeMeshBundleNode>(AssetNode))
			{
				FInterchangeMeshInstance& MeshInstance = PipelineMeshesUtilities->MeshInstancesPerMeshInstanceUid.FindOrAdd(MeshInstanceUid);
				MeshInstance.InstancingNode = BaseNode;
				MeshInstance.bReferenceSkinnedMesh = MeshBundle->IsSkinnedMeshBundle();
				MeshInstance.bReferenceMorphTarget = false;
				MeshInstance.bIsGeometryCache = false;
				MeshInstance.bIsVisible = bIsVisible;

				// This is a group of many mesh nodes collapsed together, so we need to update all the FInterchangeMeshGeometry entries
				bool bBundleHasMorphTargets = false;
				TArray<FString> ReferencedMeshNodeUids;
				MeshBundle->GetMeshNodeUids(ReferencedMeshNodeUids);
				for (const FString& ReferenceMeshNodeUid : ReferencedMeshNodeUids)
				{
					FInterchangeMeshGeometry& MeshGeometry = PipelineMeshesUtilities->MeshGeometriesPerMeshUid.FindChecked(ReferenceMeshNodeUid);
					MeshGeometry.UidsOfInstancingNodes.Add(MeshInstanceUid);
					MeshGeometry.bIsReferencedBySkeleton |= bIsNestedIntoSkeleton;
					bBundleHasMorphTargets |= MeshGeometry.HasMorphTargetDependencies();
				}
				MeshInstance.bHasMorphTargets = bBundleHasMorphTargets;

				MeshInstance.ReferencingMeshGeometryUids.Add(AssetUid);

				MeshInstancePtr = &MeshInstance;
			}

			if (MeshInstancePtr == nullptr)
			{
				return false;
			}

			if (AssetNode && (AssetNode->IsA<UInterchangeMeshNode>() || AssetNode->IsA<UInterchangeMeshLODContainerNode>()))
			{
				const FString& StaticMeshNodeUid = AssetNode->GetUniqueID();
				FInterchangeMeshGeometry* MeshGeometry = PipelineMeshesUtilities->MeshGeometriesPerMeshUid.Find(StaticMeshNodeUid);

				for (const UInterchangeSceneNode* AnimatedSceneNode : SceneNodesWithTransformAnimationTrackNode)
				{
					if (IsImpactingMeshRecursive(AnimatedSceneNode, BaseNodeContainer, StaticMeshNodeUid))
					{
						MeshInstancePtr->bHasTransformAnimationTrack = true;
						if (ensure(MeshGeometry)) //We could have used FindChecked above but the since the semantic is not clear let's avoid a crash.
						{
							MeshGeometry->bHasTransformAnimationTrack = true;
						}
						break;
					}
				}
			}

			return true;
		};

	BaseNodeContainer->BreakableIterateNodesOfType<UInterchangeInstancedStaticMeshComponentNode>([&PipelineMeshesUtilities, &BaseNodeContainer, &ProcessMeshNode](const FString& ISMComponentNodeUid, UInterchangeInstancedStaticMeshComponentNode* ISMComponentNode)
		{
			FString AssetInstanceUid;
			if (ISMComponentNode->GetCustomInstancedAssetUid(AssetInstanceUid))
			{
				ProcessMeshNode(ISMComponentNodeUid, ISMComponentNode, AssetInstanceUid);
			}

			return false;
		});

	//Find all translated scene node we need for this pipeline
	bool bHasSockets = false;
	BaseNodeContainer->IterateNodesOfType<UInterchangeSceneNode>(
		[&PipelineMeshesUtilities, &BaseNodeContainer, &SkeletonRootNodeUids, &bHasSockets, &ProcessMeshNode](const FString& NodeUid, const UInterchangeSceneNode* SceneNode)
		{
			if (SceneNode->IsA(UInterchangeJointNode::StaticClass()))
			{
				const UInterchangeSceneNode* ParentJointNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(SceneNode->GetParentUid()));
				if (!ParentJointNode || !ParentJointNode->IsA(UInterchangeJointNode::StaticClass()))
				{
					SkeletonRootNodeUids.Add(SceneNode->GetUniqueID());
				}
			}

			if (IsSceneNodeASocket(SceneNode))
			{
				bHasSockets = true;
			}

			FString AssetInstanceUid;
			if (SceneNode->GetCustomAssetInstanceUid(AssetInstanceUid))
			{
				ProcessMeshNode(NodeUid, SceneNode, AssetInstanceUid);
			}
		}
	);

	// Do a second pass to discover sockets
	if (bHasSockets)
	{
		if (PipelineMeshesUtilities->MeshGeometriesPerMeshUid.Num() == 1)
		{
			//Import of Global Sockets (only done, in case there are only 1 mesh in the source data) :
			TArray<FString> SocketUIDs;
			BaseNodeContainer->IterateNodes(
				[&PipelineMeshesUtilities, &BaseNodeContainer, &SocketUIDs](const FString& NodeUid, const UInterchangeBaseNode* Node)
				{
					if (const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(Node))
					{
						if (IsSceneNodeASocket(SceneNode))
						{
							SocketUIDs.Add(SceneNode->GetUniqueID());
						}
					}
				}
			);
			PipelineMeshesUtilities->MeshGeometriesPerMeshUid.FindArbitraryElement()->Value.AttachedSocketUids = SocketUIDs;
		}
		else
		{
			//Import of Local Sockets:
			BaseNodeContainer->IterateNodesOfType<UInterchangeSceneNode>(
				[&PipelineMeshesUtilities, &BaseNodeContainer](const FString& NodeUid, const UInterchangeSceneNode* SceneNode)
				{
					if (IsSceneNodeASocket(SceneNode))
					{
						FString MeshUid;
						if (!SceneNode->GetCustomAssetInstanceUid(MeshUid))
						{
							const UInterchangeSceneNode* ParentMeshSceneNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(SceneNode->GetParentUid()));
							while (ParentMeshSceneNode)
							{
								if (ParentMeshSceneNode->GetCustomAssetInstanceUid(MeshUid))
								{
									break;
								}

								ParentMeshSceneNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(ParentMeshSceneNode->GetParentUid()));
							}
						}

						if (!MeshUid.IsEmpty())
						{
							FInterchangeMeshGeometry& MeshGeometry = PipelineMeshesUtilities->MeshGeometriesPerMeshUid.FindChecked(MeshUid);
							MeshGeometry.AttachedSocketUids.Add(SceneNode->GetUniqueID());
						}
					}
				}
			);
		}
	}


	//Fill the SkeletonRootUidPerMeshUid data
	for (const TPair<FString, FInterchangeMeshGeometry>& MeshGeometryUidAndMeshGeometry : PipelineMeshesUtilities->MeshGeometriesPerMeshUid)
	{
		const FInterchangeMeshGeometry& MeshGeometry = MeshGeometryUidAndMeshGeometry.Value;
		if (!ensure(MeshGeometry.BaseNode))
		{
			continue;
		}
		if (!MeshGeometry.IsSkinnedMesh() || PipelineMeshesUtilities->SkeletonRootUidPerMeshUid.Contains(MeshGeometry.BaseNode->GetUniqueID()))
		{
			continue;
		}
		const UInterchangeBaseNode* BaseNode = MeshGeometry.BaseNode;
		if (!BaseNode || !MeshGeometry.HasSkeletonDependecies())
		{
			continue;
		}
		//Find the root joint for this MeshGeometry
		FString JointNodeUid = MeshGeometry.GetSkeletonRootUid(*BaseNodeContainer, SkeletonRootNodeUids);
		if (SkeletonRootNodeUids.Contains(JointNodeUid))
		{
			PipelineMeshesUtilities->SkeletonRootUidPerMeshUid.Add(MeshGeometry.BaseNode->GetUniqueID(), JointNodeUid);
		}
	}

	return PipelineMeshesUtilities;
}

void UInterchangePipelineMeshesUtilities::GetAllMeshInstanceUids(TArray<FString>& MeshInstanceUids) const
{
	MeshInstancesPerMeshInstanceUid.GetKeys(MeshInstanceUids);
}

void UInterchangePipelineMeshesUtilities::IterateAllMeshInstance(TFunctionRef<void(const FInterchangeMeshInstance&)> IterationLambda) const
{
	for (const TPair<FString, FInterchangeMeshInstance>& MeshInstanceUidAndMeshInstance : MeshInstancesPerMeshInstanceUid)
	{
		const FInterchangeMeshInstance& MeshInstance = MeshInstanceUidAndMeshInstance.Value;
		IterationLambda(MeshInstance);
	}
}

void UInterchangePipelineMeshesUtilities::GetAllSkinnedMeshInstance(TArray<FString>& MeshInstanceUids) const
{
	MeshInstanceUids.Empty(MeshInstancesPerMeshInstanceUid.Num());
	for (const TPair<FString, FInterchangeMeshInstance>& MeshInstanceUidAndMeshInstance : MeshInstancesPerMeshInstanceUid)
	{
		const FInterchangeMeshInstance& MeshInstance = MeshInstanceUidAndMeshInstance.Value;
		if (CurrentDataContext.IsSkeletalMeshInstance(MeshInstance, BaseNodeContainer))
		{
			MeshInstanceUids.Add(MeshInstanceUidAndMeshInstance.Key);
		}
	}
}

void UInterchangePipelineMeshesUtilities::IterateAllSkinnedMeshInstance(TFunctionRef<void(const FInterchangeMeshInstance&)> IterationLambda) const
{
	for (const TPair<FString, FInterchangeMeshInstance>& MeshInstanceUidAndMeshInstance : MeshInstancesPerMeshInstanceUid)
	{
		const FInterchangeMeshInstance& MeshInstance = MeshInstanceUidAndMeshInstance.Value;
		if (CurrentDataContext.IsSkeletalMeshInstance(MeshInstance, BaseNodeContainer))
		{
			IterationLambda(MeshInstance);
		}
	}
}

void UInterchangePipelineMeshesUtilities::GetAllStaticMeshInstance(TArray<FString>& MeshInstanceUids, const bool bAllowNotVisibles) const
{
	MeshInstanceUids.Reserve(MeshInstancesPerMeshInstanceUid.Num());
	for (const TPair<FString, FInterchangeMeshInstance>& MeshInstanceUidAndMeshInstance : MeshInstancesPerMeshInstanceUid)
	{
		const FInterchangeMeshInstance& MeshInstance = MeshInstanceUidAndMeshInstance.Value;

		if (bAllowNotVisibles || MeshInstance.bIsVisible)
		{
			if (CurrentDataContext.IsStaticMeshInstance(MeshInstance, BaseNodeContainer))
			{
				MeshInstanceUids.Add(MeshInstanceUidAndMeshInstance.Key);
			}
		}
	}
}

void UInterchangePipelineMeshesUtilities::IterateAllStaticMeshInstance(TFunctionRef<void(const FInterchangeMeshInstance&)> IterationLambda) const
{
	for (const TPair<FString, FInterchangeMeshInstance>& MeshInstanceUidAndMeshInstance : MeshInstancesPerMeshInstanceUid)
	{
		const FInterchangeMeshInstance& MeshInstance = MeshInstanceUidAndMeshInstance.Value;
		if (CurrentDataContext.IsStaticMeshInstance(MeshInstance, BaseNodeContainer))
		{
			IterationLambda(MeshInstance);
		}
	}
}

void UInterchangePipelineMeshesUtilities::GetAllGeometryCacheInstance(TArray<FString>& MeshInstanceUids) const
{
	MeshInstanceUids.Empty(MeshInstancesPerMeshInstanceUid.Num());
	for (const TPair<FString, FInterchangeMeshInstance>& MeshInstanceUidAndMeshInstance : MeshInstancesPerMeshInstanceUid)
	{
		const FInterchangeMeshInstance& MeshInstance = MeshInstanceUidAndMeshInstance.Value;
		if (CurrentDataContext.IsGeometryCacheInstance(MeshInstance))
		{
			MeshInstanceUids.Add(MeshInstanceUidAndMeshInstance.Key);
		}
	}
}

void UInterchangePipelineMeshesUtilities::IterateAllGeometryCacheInstance(TFunctionRef<void(const FInterchangeMeshInstance&)> IterationLambda) const
{
	for (const TPair<FString, FInterchangeMeshInstance>& MeshInstanceUidAndMeshInstance : MeshInstancesPerMeshInstanceUid)
	{
		const FInterchangeMeshInstance& MeshInstance = MeshInstanceUidAndMeshInstance.Value;
		if (CurrentDataContext.IsGeometryCacheInstance(MeshInstance))
		{
			IterationLambda(MeshInstance);
		}
	}
}

void UInterchangePipelineMeshesUtilities::GetAllMeshGeometry(TArray<FString>& MeshGeometryUids) const
{
	MeshGeometriesPerMeshUid.GetKeys(MeshGeometryUids);
}

void UInterchangePipelineMeshesUtilities::IterateAllMeshGeometry(TFunctionRef<void(const FInterchangeMeshGeometry&)> IterationLambda) const
{
	for (const TPair<FString, FInterchangeMeshGeometry>& MeshGeometryUidAndMeshGeometry : MeshGeometriesPerMeshUid)
	{
		const FInterchangeMeshGeometry& MeshGeometry = MeshGeometryUidAndMeshGeometry.Value;
		IterationLambda(MeshGeometry);
	}
}

void UInterchangePipelineMeshesUtilities::GetAllSkinnedMeshGeometry(TArray<FString>& MeshGeometryUids) const
{
	MeshGeometryUids.Empty(MeshGeometriesPerMeshUid.Num());
	for (const TPair<FString, FInterchangeMeshGeometry>& MeshGeometryUidAndMeshGeometry : MeshGeometriesPerMeshUid)
	{
		const FInterchangeMeshGeometry& MeshGeometry = MeshGeometryUidAndMeshGeometry.Value;
		if(CurrentDataContext.IsSkeletalMeshGeometry(MeshGeometry))
		{
			MeshGeometryUids.Add(MeshGeometry.BaseNode->GetUniqueID());
		}
	}
}

void UInterchangePipelineMeshesUtilities::IterateAllSkinnedMeshGeometry(TFunctionRef<void(const FInterchangeMeshGeometry&)> IterationLambda) const
{
	for (const TPair<FString, FInterchangeMeshGeometry>& MeshGeometryUidAndMeshGeometry : MeshGeometriesPerMeshUid)
	{
		const FInterchangeMeshGeometry& MeshGeometry = MeshGeometryUidAndMeshGeometry.Value;
		if (CurrentDataContext.IsSkeletalMeshGeometry(MeshGeometry))
		{
			IterationLambda(MeshGeometry);
		}
	}
}

void UInterchangePipelineMeshesUtilities::GetAllStaticMeshGeometry(TArray<FString>& MeshGeometryUids) const
{
	MeshGeometryUids.Empty(MeshGeometriesPerMeshUid.Num());
	for (const TPair<FString, FInterchangeMeshGeometry>& MeshGeometryUidAndMeshGeometry : MeshGeometriesPerMeshUid)
	{
		const FInterchangeMeshGeometry& MeshGeometry = MeshGeometryUidAndMeshGeometry.Value;
		if (CurrentDataContext.IsStaticMeshGeometry(MeshGeometry))
		{
			MeshGeometryUids.Add(MeshGeometry.BaseNode->GetUniqueID());
		}
	}
}

void UInterchangePipelineMeshesUtilities::IterateAllStaticMeshGeometry(TFunctionRef<void(const FInterchangeMeshGeometry&)> IterationLambda) const
{
	for (const TPair<FString, FInterchangeMeshGeometry>& MeshGeometryUidAndMeshGeometry : MeshGeometriesPerMeshUid)
	{
		const FInterchangeMeshGeometry& MeshGeometry = MeshGeometryUidAndMeshGeometry.Value;
		if (CurrentDataContext.IsStaticMeshGeometry(MeshGeometry))
		{
			IterationLambda(MeshGeometry);
		}
	}
}

void UInterchangePipelineMeshesUtilities::GetAllGeometryCacheGeometry(TArray<FString>& MeshGeometryUids) const
{
	MeshGeometryUids.Empty(MeshGeometriesPerMeshUid.Num());
	for (const TPair<FString, FInterchangeMeshGeometry>& MeshGeometryUidAndMeshGeometry : MeshGeometriesPerMeshUid)
	{
		const FInterchangeMeshGeometry& MeshGeometry = MeshGeometryUidAndMeshGeometry.Value;
		if (CurrentDataContext.IsGeometryCacheGeometry(MeshGeometry))
		{
			MeshGeometryUids.Add(MeshGeometry.BaseNode->GetUniqueID());
		}
	}
}

void UInterchangePipelineMeshesUtilities::IterateAllGeometryCacheGeometry(TFunctionRef<void(const FInterchangeMeshGeometry&)> IterationLambda) const
{
	for (const TPair<FString, FInterchangeMeshGeometry>& MeshGeometryUidAndMeshGeometry : MeshGeometriesPerMeshUid)
	{
		const FInterchangeMeshGeometry& MeshGeometry = MeshGeometryUidAndMeshGeometry.Value;
		if (CurrentDataContext.IsGeometryCacheGeometry(MeshGeometry))
		{
			IterationLambda(MeshGeometry);
		}
	}
}

void UInterchangePipelineMeshesUtilities::GetAllMeshGeometryNotInstanced(TArray<FString>& MeshGeometryUids) const
{
	MeshGeometryUids.Empty(MeshGeometriesPerMeshUid.Num());
	for (const TPair<FString, FInterchangeMeshGeometry>& MeshGeometryUidAndMeshGeometry : MeshGeometriesPerMeshUid)
	{
		const FInterchangeMeshGeometry& MeshGeometry = MeshGeometryUidAndMeshGeometry.Value;
		if (MeshGeometry.UidsOfInstancingNodes.Num() == 0)
		{
			MeshGeometryUids.Add(MeshGeometry.BaseNode->GetUniqueID());
		}
	}
}

void UInterchangePipelineMeshesUtilities::IterateAllMeshGeometryNotIntanced(TFunctionRef<void(const FInterchangeMeshGeometry&)> IterationLambda) const
{
	for (const TPair<FString, FInterchangeMeshGeometry>& MeshGeometryUidAndMeshGeometry : MeshGeometriesPerMeshUid)
	{
		const FInterchangeMeshGeometry& MeshGeometry = MeshGeometryUidAndMeshGeometry.Value;
		if (MeshGeometry.UidsOfInstancingNodes.Num() == 0)
		{
			IterationLambda(MeshGeometry);
		}
	}
}

bool UInterchangePipelineMeshesUtilities::IsValidMeshInstanceUid(const FString& MeshInstanceUid) const
{
	return MeshInstancesPerMeshInstanceUid.Contains(MeshInstanceUid);
}

const FInterchangeMeshInstance& UInterchangePipelineMeshesUtilities::GetMeshInstanceByUid(const FString& MeshInstanceUid) const
{
	return MeshInstancesPerMeshInstanceUid.FindChecked(MeshInstanceUid);
}

bool UInterchangePipelineMeshesUtilities::IsValidMeshGeometryUid(const FString& MeshGeometryUid) const
{
	return MeshGeometriesPerMeshUid.Contains(MeshGeometryUid);
}

const FInterchangeMeshGeometry& UInterchangePipelineMeshesUtilities::GetMeshGeometryByUid(const FString& MeshGeometryUid) const
{
	return MeshGeometriesPerMeshUid.FindChecked(MeshGeometryUid);
}

void UInterchangePipelineMeshesUtilities::GetAllMeshInstanceUidsUsingMeshGeometryUid(const FString& MeshGeometryUid, TArray<FString>& MeshInstanceUids) const
{
	const FInterchangeMeshGeometry& MeshGeometry = MeshGeometriesPerMeshUid.FindChecked(MeshGeometryUid);
	MeshInstanceUids = MeshGeometry.UidsOfInstancingNodes;
}

void UInterchangePipelineMeshesUtilities::IterateAllMeshInstanceUsingMeshGeometry(const FString& MeshGeometryUid, TFunctionRef<void(const FInterchangeMeshInstance&)> IterationLambda) const
{
	const FInterchangeMeshGeometry& MeshGeometry = MeshGeometriesPerMeshUid.FindChecked(MeshGeometryUid);
	for (const FString& MeshInstanceUid : MeshGeometry.UidsOfInstancingNodes)
	{
		const FInterchangeMeshInstance& MeshInstance = MeshInstancesPerMeshInstanceUid.FindChecked(MeshInstanceUid);
		IterationLambda(MeshInstance);
	}
}

void UInterchangePipelineMeshesUtilities::GetCombinedSkinnedMeshInstances(TMap<FString, TArray<FString>>& OutMeshInstanceUidsPerSkeletonRootUid,
	bool bUseSingleBoneForConvertedMeshes /*= false*/, bool bVisibleOnly /*= false*/) const
{
	if (!ensure(BaseNodeContainer))
	{
		return;
	}

	bool bAllowSceneRootAsJoint = true;
	if (const UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::GetUniqueInstance(BaseNodeContainer))
	{
		SourceNode->GetCustomAllowSceneRootAsJoint(bAllowSceneRootAsJoint);
	}

	for (const TPair<FString, FInterchangeMeshInstance>& MeshInstanceUidAndMeshInstance : MeshInstancesPerMeshInstanceUid)
	{
		const FInterchangeMeshInstance& MeshInstance = MeshInstanceUidAndMeshInstance.Value;
		if (bVisibleOnly && !MeshInstance.bIsVisible)
		{
			//Skip invisible nodes
			continue;
		}

		if (!CurrentDataContext.IsSkeletalMeshInstance(MeshInstance, BaseNodeContainer))
		{
			continue;
		}

		bool bIsStaticMesh = !MeshInstance.bReferenceSkinnedMesh;
		//Find the root skeleton for this MeshInstance
		FString SkeletonRootUid; // = CurrentDataContext.FindSkeletalMeshSkeletonRootFromMeshInstance(MeshInstance, BaseNodeContainer);

		for (const FString& MeshGeometryUid : MeshInstance.ReferencingMeshGeometryUids)
		{
			if (const FString* SkeletonRootUidPtr = SkeletonRootUidPerMeshUid.Find(MeshGeometryUid))
			{
				if (SkeletonRootUid.IsEmpty())
				{
					SkeletonRootUid = *SkeletonRootUidPtr;
				}
				else if (!SkeletonRootUid.Equals(*SkeletonRootUidPtr))
				{
					//Log an error, this FInterchangeMeshInstance use more then one skeleton root node, we will not add this instance to the combined
					SkeletonRootUid.Empty();
					break;
				}
			}
			else if (bIsStaticMesh)
			{
				//Create a joint from the instance node (the scene node pointing on the mesh).
				const bool bIsNestedIntoSkeleton = MeshInstance.IsNestedInSkeleton();
				SkeletonRootUid = MeshInstanceUidAndMeshInstance.Key;

				if (bIsNestedIntoSkeleton || !MeshInstance.bHasMorphTargets)
				{
					//Find the deepest joint node
					const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(MeshInstanceUidAndMeshInstance.Key));
					if (SceneNode)
					{
						FString ParentUid = SceneNode->GetParentUid();
						FString LastSceneNodeUid = SkeletonRootUid;

						const UInterchangeSceneNode* ParentNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(ParentUid));
						while (ParentNode)
						{
							bool bIsSceneRoot = false;
							ParentNode->GetCustomIsSceneRoot(bIsSceneRoot);
							if (!bAllowSceneRootAsJoint && bIsSceneRoot)
							{
								//Do not include scene root node in Skeleton.
								break;
							}

							if (ParentNode->IsA(UInterchangeJointNode::StaticClass()))
							{
								SkeletonRootUid = ParentUid;
							}
							LastSceneNodeUid = ParentUid;
							ParentUid = ParentNode->GetParentUid();

							ParentNode = Cast<UInterchangeSceneNode>(BaseNodeContainer->GetNode(ParentUid));
						}

						//If we did not find any joint because we have non nested mesh, get the deepest scene node
						if (!bIsNestedIntoSkeleton && SkeletonRootUid.Equals(MeshInstanceUidAndMeshInstance.Key))
						{
							SkeletonRootUid = LastSceneNodeUid;
						}

						//If we want to use a single bone skeleton for the static mesh, we need to create or reuse a dedicated scene node for that
						//By design, only Static Meshes that are not nested in a joint hierarchy can use a single bone skeleton.
						if (!bIsNestedIntoSkeleton && bUseSingleBoneForConvertedMeshes)
						{
							//Make sure to use the SkeletonRootUid found above so all meshes under the same root joint will use the same single bone skeleton too.
							const FString NodeUid = SkeletonRootUid + "_SingleBone_RootNode";

							const UInterchangeBaseNode* SingleBoneRootNode = BaseNodeContainer->GetNode(NodeUid);
							if (!SingleBoneRootNode)
							{
								UInterchangeJointNode* SkeletonRootNode = NewObject<UInterchangeJointNode>(BaseNodeContainer);
								BaseNodeContainer->SetupNode(SkeletonRootNode, NodeUid, "Root", EInterchangeNodeContainerType::TranslatedScene);
								SkeletonRootNode->SetCustomLocalTransform(BaseNodeContainer, FTransform::Identity);
							}

							SkeletonRootUid = NodeUid;
						}
					}
				}
			}
			else
			{
				//every skinned geometry should have a skeleton root node ???
			}
		}
		
		if (SkeletonRootUid.IsEmpty())
		{
			//Skip this MeshInstance
			continue;
		}

		TArray<FString>& MeshInstanceUids = OutMeshInstanceUidsPerSkeletonRootUid.FindOrAdd(SkeletonRootUid);
		MeshInstanceUids.Add(MeshInstanceUidAndMeshInstance.Key);
	}
}

FString UInterchangePipelineMeshesUtilities::GetMeshInstanceSkeletonRootUid(const FString& MeshInstanceUid) const
{
	FString SkeletonRootUid;
	if (IsValidMeshInstanceUid(MeshInstanceUid))
	{
		SkeletonRootUid = GetMeshInstanceSkeletonRootUid(GetMeshInstanceByUid(MeshInstanceUid));
	}
	return SkeletonRootUid;
}

FString UInterchangePipelineMeshesUtilities::GetMeshInstanceSkeletonRootUid(const FInterchangeMeshInstance& MeshInstance) const
{
	FString SkeletonRootUid;
	const int32 BaseLodIndex = 0;
	const UInterchangeBaseNode* BaseNode = MeshInstance.InstancingNode;
	if (const UInterchangeSceneNode* SceneNode = Cast<const UInterchangeSceneNode>(BaseNode))
	{
		FString MeshNodeUid;
		if (SceneNode->GetCustomAssetInstanceUid(MeshNodeUid))
		{
			if (SkeletonRootUidPerMeshUid.Contains(MeshNodeUid))
			{
				SkeletonRootUid = SkeletonRootUidPerMeshUid.FindChecked(MeshNodeUid);
			}
		}
	}
	return SkeletonRootUid;
}

FString UInterchangePipelineMeshesUtilities::GetMeshGeometrySkeletonRootUid(const FString& MeshGeometryUid) const
{
	FString SkeletonRootUid;
	if (IsValidMeshGeometryUid(MeshGeometryUid))
	{
		const FInterchangeMeshGeometry& MeshGeometry = GetMeshGeometryByUid(MeshGeometryUid);
		SkeletonRootUid = GetMeshGeometrySkeletonRootUid(MeshGeometry);
	}
	return SkeletonRootUid;
}

FString UInterchangePipelineMeshesUtilities::GetMeshGeometrySkeletonRootUid(const FInterchangeMeshGeometry& MeshGeometry) const
{
	FString SkeletonRootUid;
	if (SkeletonRootUidPerMeshUid.Contains(MeshGeometry.BaseNode->GetUniqueID()))
	{
		SkeletonRootUid = SkeletonRootUidPerMeshUid.FindChecked(MeshGeometry.BaseNode->GetUniqueID());
	}
	return SkeletonRootUid;
}

bool UInterchangePipelineMeshesUtilities::HasAssemblyMeshDependencies() const
{
	return !AssemblyMeshUids.IsEmpty();
}

bool UInterchangePipelineMeshesUtilities::IsAssemblyMeshUid(const FString& MeshUid) const
{
	return AssemblyMeshUids.Contains(MeshUid);
}
