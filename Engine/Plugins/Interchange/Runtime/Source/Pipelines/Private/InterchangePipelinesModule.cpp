// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangePipelinesModule.h"

#include "CoreMinimal.h"
#include "InterchangePipelineLog.h"
#include "InterchangePipelineObjectVersion.h"
#include "Misc/Guid.h"
#include "Modules/ModuleManager.h"
#include "UObject/DevObjectVersion.h"

DEFINE_LOG_CATEGORY(LogInterchangePipeline);

// Unique serialization id for InterchangePipelineObjectVersion.
const FGuid FFortniteMainInterchangePipelineObjectVersion::GUID(0xB69D2E47, 0xE2A84003, 0xBF7718A4, 0x92C4D899);

static FDevVersionRegistration GRegisterInterchangePipelineObjectVersion(FFortniteMainInterchangePipelineObjectVersion::GUID, FFortniteMainInterchangePipelineObjectVersion::LatestVersion, TEXT("FN-Main-InterchangePipeline"));

class FInterchangePipelinesModule : public IInterchangePipelinesModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override {}
	virtual void ShutdownModule() override {}
};

IMPLEMENT_MODULE(FInterchangePipelinesModule, InterchangePipelines)
