// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeSparseVolumeTexturePipeline.h"

#include "InterchangePipelineHelper.h"
#include "InterchangePipelineLog.h"
#include "InterchangeSparseVolumeTextureFactoryNode.h"
#include "InterchangeVolumeNode.h"
#include "Nodes/InterchangeSourceNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"
#include "Volume/InterchangeVolumeDefinitions.h"

#include "Engine/Texture.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSparseVolumeTexturePipeline)

namespace UE::Interchange::Private
{
	UInterchangeSparseVolumeTextureFactoryNode* CreateTextureFactoryNode(
		const FString& DisplayLabel,
		const FString& NodeUid,
		UInterchangeBaseNodeContainer* BaseNodeContainer
	)
	{
		UInterchangeSparseVolumeTextureFactoryNode* TextureFactoryNode = nullptr;
		if (BaseNodeContainer->IsNodeUidValid(NodeUid))
		{
			TextureFactoryNode = Cast<UInterchangeSparseVolumeTextureFactoryNode>(BaseNodeContainer->GetFactoryNode(NodeUid));
			if (!ensure(TextureFactoryNode))
			{
				// Log an error
				return nullptr;
			}
		}
		else
		{
			const EInterchangeNodeContainerType NodeContainerType = EInterchangeNodeContainerType::FactoryData;

			TextureFactoryNode = NewObject<UInterchangeSparseVolumeTextureFactoryNode>(BaseNodeContainer);
			BaseNodeContainer->SetupNode(TextureFactoryNode, NodeUid, DisplayLabel, NodeContainerType);

			UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(BaseNodeContainer);
			UE::Interchange::PipelineHelper::FillSubPathFromSourceNode(TextureFactoryNode, SourceNode);
		}

		return TextureFactoryNode;
	}

	// SparseVolumeTextures have 8 individual channels, grouped into two RGBA 'textures' called "AttributesA" and "AttributesB",
	// each 'texture' being of a format according to EInterchangeSparseVolumeTextureFormat.
	//
	// The purpose of this function is to figure out some sensible default assignment/distribution of the grids of the provided
	// volume texture across these 8 channels. The idea is that other pipelines (like the USD Pipeline) would later override these
	// with any specific grid-to-SVT channel mapping that the source files specify.
	//
	// Another goal here is to match the default assignment done by the SparseVolumeTextureFactory, so that SVTs imported via
	// Interchange match the ones imported with the legacy factory
	void SetupDefaultOpenVDBGridAssignment(
		UInterchangeSparseVolumeTextureFactoryNode* VolumeFactoryNode,
		UInterchangeBaseNodeContainer* BaseNodeContainer
	)
	{
		// References:
		// - ComputeDefaultOpenVDBGridAssignment function from SparseVolumeTextureFactory.cpp

		using namespace UE::Interchange::Volume;

		if (!VolumeFactoryNode || !BaseNodeContainer)
		{
			return;
		}

		// Get the translated node for this factory node
		const UInterchangeVolumeNode* VolumeNode = nullptr;
		{
			TArray<FString> TargetNodeUids;
			VolumeFactoryNode->GetTargetNodeUids(TargetNodeUids);

			for (int32 TargetNodeIndex = TargetNodeUids.Num() - 1; TargetNodeIndex >= 0; --TargetNodeIndex)
			{
				VolumeNode = Cast<UInterchangeVolumeNode>(BaseNodeContainer->GetNode(TargetNodeUids[TargetNodeIndex]));
				if (VolumeNode)
				{
					break;
				}
			}

			if (!VolumeNode)
			{
				return;
			}
		}

		// Get all the grids contained in the given volume
		TArray<const UInterchangeVolumeGridNode*> GridNodes;
		{
			TArray<FString> GridNodeUids;
			VolumeNode->GetCustomGridDependecies(GridNodeUids);

			for (int32 GridIndex = 0; GridIndex < GridNodeUids.Num(); ++GridIndex)
			{
				const UInterchangeVolumeGridNode* GridNode = Cast<UInterchangeVolumeGridNode>(BaseNodeContainer->GetNode(GridNodeUids[GridIndex]));
				if (GridNode)
				{
					GridNodes.Add(GridNode);
				}
			}

			if (GridNodes.Num() == 0)
			{
				return;
			}
		}

		// Check whether we have a grid named "density" (seems to be common for .vdbs)
		int32 DensityGridIndex = INDEX_NONE;
		int32 NumNonDensity = GridNodes.Num();
		for (int32 Index = 0; Index < GridNodes.Num(); ++Index)
		{
			const UInterchangeVolumeGridNode* Node = GridNodes[Index];
			if (Node->GetDisplayLabel() == DensityGridName)
			{
				DensityGridIndex = Index;
				NumNonDensity--;
				break;
			}
		}

		// We use these to help distribute the grids through the different channels, as we have to iterate through them
		int32 SetterIndex = 0;
		using SetterFunc = decltype(&UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAChannelX);
		static const TArray<SetterFunc> AttributeChannelSetters = {
			&UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAChannelX,
			&UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAChannelY,
			&UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAChannelZ,
			&UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesAChannelW,
			&UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesBChannelX,
			&UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesBChannelY,
			&UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesBChannelZ,
			&UInterchangeSparseVolumeTextureFactoryNode::SetCustomAttributesBChannelW,
		};

		// Optimized density assignment: "density" grid as 8bit unsigned normalized on AttributesA, and everything else on Attributes B.
		// This is only done if we have 0, 1, 2 or 4 non-density grid components (if we have density and 3 non-density we have a total
		// of 4, so since they would fit nicely into a single AttributesA 'texture' we just do that instead)
		const bool bOptimizedDensityAssignment = DensityGridIndex != INDEX_NONE && (NumNonDensity <= 4 && NumNonDensity != 3);
		if (bOptimizedDensityAssignment)
		{
			VolumeFactoryNode->SetCustomAttributesAFormat(EInterchangeSparseVolumeTextureFormat::Unorm8);
			VolumeFactoryNode->SetCustomAttributesAChannelX(DensityGridName + GridNameAndComponentIndexSeparator + TEXT("0"));

			VolumeFactoryNode->SetCustomAttributesBFormat(EInterchangeSparseVolumeTextureFormat::Float16);

			SetterIndex = 4;	// Start at SetCustomAttributesBChannelX() instead, as our AttributesA texture will hold just the density
		}
		else
		{
			VolumeFactoryNode->SetCustomAttributesAFormat(EInterchangeSparseVolumeTextureFormat::Float16);

			VolumeFactoryNode->SetCustomAttributesBFormat(EInterchangeSparseVolumeTextureFormat::Float16);
		}

		// Actually distribute the remaining grid/components across the channels in order
		for (int32 Index = 0; Index < GridNodes.Num(); ++Index)
		{
			if (SetterIndex >= AttributeChannelSetters.Num())
			{
				break;
			}

			if (Index == DensityGridIndex)
			{
				continue;
			}

			const UInterchangeVolumeGridNode* Node = GridNodes[Index];

			int32 GridNumComponents = 0;
			bool bHasComponents = Node->GetCustomNumComponents(GridNumComponents);
			if (!bHasComponents)
			{
				continue;
			}

			// e.g. "temperature_"
			const FString& GridNameAndSeparator = Node->GetDisplayLabel() + GridNameAndComponentIndexSeparator;

			for (int32 GridComponentIndex = 0;															   //
				 GridComponentIndex < GridNumComponents && SetterIndex < AttributeChannelSetters.Num();	   //
				 ++GridComponentIndex, ++SetterIndex)
			{
				// e.g. "temperature_2"
				const FString GridNameAndComponentIndex = GridNameAndSeparator + LexToString(GridComponentIndex);

				const SetterFunc& Setter = AttributeChannelSetters[SetterIndex];
				(VolumeFactoryNode->*Setter)(GridNameAndComponentIndex);
			}
		}
	}

	// Splits something like "tornado_23" into the "tornado_" prefix and 23 suffix
	void SplitNumberedSuffix(const FString& String, FString& OutPrefix, int32& OutSuffix)
	{
		const int32 LastNonDigitIndex = String.FindLastCharByPredicate(
			[](TCHAR Letter)
			{
				return !FChar::IsDigit(Letter);
			}
		);

		// No numbered suffix
		if (LastNonDigitIndex == String.Len() - 1)
		{
			OutPrefix = String;
			OutSuffix = INDEX_NONE;
			return;
		}

		FString NumberSuffixStr;

		// String is all numbers
		if (LastNonDigitIndex == INDEX_NONE)
		{
			NumberSuffixStr = String;
			OutPrefix = {};
		}
		// Some prefix, some numbers
		else
		{
			NumberSuffixStr = String.RightChop(LastNonDigitIndex + 1);
			OutPrefix = String.Left(LastNonDigitIndex + 1);
		}

		int32 Number = INDEX_NONE;
		if (ensure(NumberSuffixStr.IsNumeric()))
		{
			TTypeFromString<int32>::FromString(Number, *NumberSuffixStr);
		}
		OutSuffix = Number;
	}

	// Turns something like "tornado_" or "tornado-" into just "tornado"
	FString RemoveTrailingSeparators(FString String)
	{
		FString LastChar = String.Right(1);
		while ((LastChar == TEXT("-") || LastChar == TEXT("_")) && String.Len() > 1)
		{
			String.LeftChopInline(1, EAllowShrinking::No);
			LastChar = String.Right(1);
		}
		String.Shrink();
		return String;
	}
}	 // namespace UE::Interchange::Private

FString UInterchangeSparseVolumeTexturePipeline::GetPipelineCategory(UClass* AssetClass)
{
	return TEXT("Volumes");
}

void UInterchangeSparseVolumeTexturePipeline::AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams)
{
	Super::AdjustSettingsForContext(ContextParams);

#if WITH_EDITOR
	TArray<FString> HideCategories;
	bool bIsObjectAnSVT = !ContextParams.ReimportAsset ? false : ContextParams.ReimportAsset.IsA(USparseVolumeTexture::StaticClass());
	if ((!bIsObjectAnSVT && ContextParams.ContextType == EInterchangePipelineContext::AssetReimport)
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomLODImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomLODReimport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetAlternateSkinningImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetAlternateSkinningReimport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomMorphTargetImport
		|| ContextParams.ContextType == EInterchangePipelineContext::AssetCustomMorphTargetReImport)
	{
		bImportSparseVolumeTextures = false;
		bImportAnimatedSparseVolumeTextures = false;
		HideCategories.Add(UInterchangeSparseVolumeTexturePipeline::GetPipelineCategory(nullptr));
	}

	if (UInterchangePipelineBase* OuterMostPipeline = GetMostPipelineOuter())
	{
		for (const FString& HideCategoryName : HideCategories)
		{
			HidePropertiesOfCategory(OuterMostPipeline, this, HideCategoryName);
		}
	}
#endif	  // WITH_EDITOR
}

#if WITH_EDITOR
bool UInterchangeSparseVolumeTexturePipeline::IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UInterchangeSparseVolumeTexturePipeline, bImportSparseVolumeTextures))
	{
		return true;
	}

	if (PropertyChangedEvent.GetPropertyName()
		== GET_MEMBER_NAME_CHECKED(UInterchangeSparseVolumeTexturePipeline, bImportAnimatedSparseVolumeTextures))
	{
		return true;
	}

	return Super::IsPropertyChangeNeedRefresh(PropertyChangedEvent);
}

void UInterchangeSparseVolumeTexturePipeline::FilterPropertiesFromTranslatedData(UInterchangeBaseNodeContainer* InBaseNodeContainer)
{
	Super::FilterPropertiesFromTranslatedData(InBaseNodeContainer);

	TArray<FString> TmpTextureNodes;
	InBaseNodeContainer->GetNodes(UInterchangeVolumeNode::StaticClass(), TmpTextureNodes);
	if (TmpTextureNodes.Num() == 0)
	{
		if (UInterchangePipelineBase* OuterMostPipeline = GetMostPipelineOuter())
		{
			HidePropertiesOfCategory(OuterMostPipeline, this, UInterchangeSparseVolumeTexturePipeline::GetPipelineCategory(nullptr));
		}
	}
}

void UInterchangeSparseVolumeTexturePipeline::GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const
{
	PipelineSupportAssetClasses.Add(USparseVolumeTexture::StaticClass());
}
#endif	  // WITH_EDITOR

void UInterchangeSparseVolumeTexturePipeline::ExecutePipeline(
	UInterchangeBaseNodeContainer* InBaseNodeContainer,
	const TArray<UInterchangeSourceData*>& InSourceDatas,
	const FString& ContentBasePath
)
{
	using namespace UE::Interchange::Private;

	if (!bImportSparseVolumeTextures)
	{
		return;
	}

	if (!InBaseNodeContainer)
	{
		return;
	}
	BaseNodeContainer = InBaseNodeContainer;

	// Find all the translated nodes we need for this pipeline
	TArray<UInterchangeVolumeNode*> VolumeNodes;
	BaseNodeContainer->GetNodesOfType<UInterchangeVolumeNode>(VolumeNodes);

	TArray<UInterchangeSparseVolumeTextureFactoryNode*> CreatedFactoryNodes;

	struct FNodeAndAnimationIndex
	{
		const UInterchangeVolumeNode* Node;
		int32 Index;
	};

	// Group up volume nodes by animation ID
	//
	// Note: A volume may show up in multiple animation IDs, but that's supported.
	TSet<UInterchangeVolumeNode*> VolumeNodesWithNoAnimationID;
	TMap<FString, TArray<FNodeAndAnimationIndex>> AnimationIDToVolumeNodes;
	for (UInterchangeVolumeNode* VolumeNode : VolumeNodes)
	{
		FString AnimationID;

		// Animated volume
		if (bImportAnimatedSparseVolumeTextures && VolumeNode->GetCustomAnimationID(AnimationID) && !AnimationID.IsEmpty())
		{
			TArray<int32> AnimationIndices;
			VolumeNode->GetCustomFrameIndicesInAnimation(AnimationIndices);

			for (const int32 Index : AnimationIndices)
			{
				FNodeAndAnimationIndex& NewEntry = AnimationIDToVolumeNodes.FindOrAdd(AnimationID).Emplace_GetRef();
				NewEntry.Node = VolumeNode;
				NewEntry.Index = Index;
			}
		}
		// Static volume
		else
		{
			VolumeNodesWithNoAnimationID.Add(VolumeNode);
		}
	}

	// Create static factory nodes for ungrouped volume nodes (no animation id)
	for (const UInterchangeVolumeNode* VolumeNode : VolumeNodesWithNoAnimationID)
	{
		const FString FactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(VolumeNode->GetUniqueID());

		UInterchangeSparseVolumeTextureFactoryNode* FactoryNode = CreateTextureFactoryNode(
			VolumeNode->GetDisplayLabel(),
			FactoryNodeUid,
			BaseNodeContainer
		);
		if (!FactoryNode)
		{
			continue;
		}

		CreatedFactoryNodes.Add(FactoryNode);

		const bool bAddSourceNodeName = false;
		UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(VolumeNode, FactoryNode, bAddSourceNodeName);

		FactoryNode->AddTargetNodeUid(VolumeNode->GetUniqueID());
		VolumeNode->AddTargetNodeUid(FactoryNode->GetUniqueID());

		SetupDefaultOpenVDBGridAssignment(FactoryNode, BaseNodeContainer);
	}

	// Create animated factory nodes for each animation ID
	for (TPair<FString, TArray<FNodeAndAnimationIndex>>& Pair : AnimationIDToVolumeNodes)
	{
		const FString& AnimationID = Pair.Key;
		TArray<FNodeAndAnimationIndex>& NodeAndIndices = Pair.Value;
		if (NodeAndIndices.Num() == 0)
		{
			continue;
		}

		// Sort them according to their animation indices
		NodeAndIndices.Sort(
			[](const FNodeAndAnimationIndex& LHS, const FNodeAndAnimationIndex& RHS)
			{
				if (LHS.Index == RHS.Index)
				{
					// Fallback for a consistent order in case the animation IDs collide
					return LHS.Node->GetUniqueID() < RHS.Node->GetUniqueID();
				}

				return LHS.Index < RHS.Index;
			}
		);

		const UInterchangeVolumeNode* FirstVolume = NodeAndIndices[0].Node;

		FString FileName;
		bool bSuccess = FirstVolume->GetCustomFileName(FileName);
		if (!bSuccess || FileName.IsEmpty())
		{
			continue;
		}
		FileName = FPaths::GetBaseFilename(FileName);				// e.g. "tornado_223"

		FString Prefix;												// e.g. "tornado_"
		int32 NumberSuffix;											// e.g. 223
		SplitNumberedSuffix(FileName, Prefix, NumberSuffix);
		FString DisplayLabel = RemoveTrailingSeparators(Prefix);	// e.g. "tornado"

		// If the volume name is purely a number or something else weird (e.g. MaterialEggs scenes) then the above
		// sanitizing process has likely produced a fully empty string. Here let's try using one of the grid names
		// as the SVT asset name, if we can find one
		if (DisplayLabel.IsEmpty())
		{
			TArray<FString> GridNodeUids;
			FirstVolume->GetCustomGridDependecies(GridNodeUids);

			for (int32 GridIndex = 0; GridIndex < GridNodeUids.Num(); ++GridIndex)
			{
				const UInterchangeVolumeGridNode* GridNode = Cast<UInterchangeVolumeGridNode>(BaseNodeContainer->GetNode(GridNodeUids[GridIndex]));
				if (GridNode)
				{
					DisplayLabel = GridNode->GetDisplayLabel();
					if (!DisplayLabel.IsEmpty())
					{
						break;
					}
				}
			}
		}
		// Just give up and use the "Volume" string as the asset name instead, if that still didn't work
		if (DisplayLabel.IsEmpty())
		{
			DisplayLabel = TEXT("Volume");
		}

		const FString FactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(FirstVolume->GetUniqueID());

		UInterchangeSparseVolumeTextureFactoryNode* FactoryNode = CreateTextureFactoryNode(DisplayLabel, FactoryNodeUid, BaseNodeContainer);
		if (!FactoryNode)
		{
			continue;
		}
		CreatedFactoryNodes.Add(FactoryNode);

		const bool bAddSourceNodeName = false;
		UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(FirstVolume, FactoryNode, bAddSourceNodeName);

		// Providing the animationID is required to have the factory treat this node as an actual volume animation
		FactoryNode->SetCustomAnimationID(AnimationID);

		TSet<const UInterchangeVolumeNode*> AddedNodes;
		for (const FNodeAndAnimationIndex& NodeAndIndex : NodeAndIndices)
		{
			// We may have multiple FNodeAndAnimationIndex for the same node, if the same volume frame shows up
			// multiple times in an animation. We don't want to add it as a target multiple times though
			if (AddedNodes.Contains(NodeAndIndex.Node))
			{
				continue;
			}
			AddedNodes.Add(NodeAndIndex.Node);

			FactoryNode->AddTargetNodeUid(NodeAndIndex.Node->GetUniqueID());
			NodeAndIndex.Node->AddTargetNodeUid(FactoryNode->GetUniqueID());
		}

		SetupDefaultOpenVDBGridAssignment(FactoryNode, BaseNodeContainer);
	}

	// Set an override asset name if we have exactly one factory node
	if (CreatedFactoryNodes.Num() == 1)
	{
		FString OverrideAssetName = IsStandAlonePipeline() ? DestinationName : FString();
		if (OverrideAssetName.IsEmpty() && IsStandAlonePipeline())
		{
			OverrideAssetName = AssetName;
		}

		UInterchangeSparseVolumeTextureFactoryNode* FactoryNode = CreatedFactoryNodes[0];

		const bool bOverrideAssetName = IsStandAlonePipeline() && !OverrideAssetName.IsEmpty();
		if (FactoryNode && bOverrideAssetName)
		{
			FactoryNode->SetAssetName(OverrideAssetName);
			FactoryNode->SetDisplayLabel(OverrideAssetName);
		}
	}
}
