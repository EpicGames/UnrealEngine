// Copyright Epic Games, Inc. All Rights Reserved. 

#include "InterchangeglTFPipeline.h"

#include "Gltf/InterchangeGLTFMaterial.h"

#include "InterchangeGenericMaterialPipeline.h"
#include "InterchangeImportModule.h"
#include "InterchangeManager.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeShaderGraphNode.h"

#include "InterchangePipelineHelper.h"
#include "InterchangePipelineLog.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "Nodes/InterchangeSourceNode.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeglTFPipeline)

const TArray<FString> UGLTFPipelineSettings::ExpectedMaterialInstanceIdentifiers = {TEXT("M_GLTF"), // Substrate material that replaces all of the material instances below
																					TEXT("MI_Default_Opaque"), TEXT("MI_Default_Mask"), TEXT("MI_Default_Blend"), 
																					TEXT("MI_Unlit_Opaque"), TEXT("MI_Unlit_Mask"), TEXT("MI_Unlit_Blend"), 
																					TEXT("MI_ClearCoat_Opaque"), TEXT("MI_ClearCoat_Mask"), TEXT("MI_ClearCoat_Blend"),
																					TEXT("MI_Sheen_Opaque"), TEXT("MI_Sheen_Mask"), TEXT("MI_Sheen_Blend"), 
																					TEXT("MI_Transmission"), 
																					TEXT("MI_SpecularGlossiness_Opaque"), TEXT("MI_SpecularGlossiness_Mask"), TEXT("MI_SpecularGlossiness_Blend"), 
																					TEXT("MI_Default_Opaque_DS"), TEXT("MI_Default_Mask_DS"), TEXT("MI_Default_Blend_DS"), 
																					TEXT("MI_Unlit_Opaque_DS"), TEXT("MI_Unlit_Mask_DS"), TEXT("MI_Unlit_Blend_DS"), 
																					TEXT("MI_ClearCoat_Opaque_DS"), TEXT("MI_ClearCoat_Mask_DS"), TEXT("MI_ClearCoat_Blend_DS"), 
																					TEXT("MI_Sheen_Opaque_DS"), TEXT("MI_Sheen_Mask_DS"), TEXT("MI_Sheen_Blend_DS"), 
																					TEXT("MI_Transmission_DS"), 
																					TEXT("MI_SpecularGlossiness_Opaque_DS"), TEXT("MI_SpecularGlossiness_Mask_DS"), TEXT("MI_SpecularGlossiness_Blend_DS")};

TArray<FString> UGLTFPipelineSettings::ValidateMaterialInstancesAndParameters() const
{
	TArray<FString> NotCoveredIdentifiersParameters;

	//Check if all Material variations are covered:
	TArray<FString> ExpectedIdentifiers = ExpectedMaterialInstanceIdentifiers;
	TArray<FString> IdentifiersUsed;
	MaterialParents.GetKeys(IdentifiersUsed);
	for (const FString& Identifier : IdentifiersUsed)
	{
		ExpectedIdentifiers.Remove(Identifier);
	}
	for (const FString& ExpectedIdentifier : ExpectedIdentifiers)
	{
		NotCoveredIdentifiersParameters.Add(TEXT("[") + ExpectedIdentifier + TEXT("]: MaterialInstance not found for Identifier."));
	}

	auto ValidateParameters = [this, &NotCoveredIdentifiersParameters]<typename T>(const FString& Identifier, const FSoftObjectPath& ParentPath)
	{
		TSet<FString> ExpectedParameters = GenerateExpectedParametersList(Identifier);

		if (T* ParentMaterialInstance = Cast<T>(ParentPath.TryLoad()))
		{
			TArray<FGuid> ParameterIds;
			TArray<FMaterialParameterInfo> ScalarParameterInfos;
			TArray<FMaterialParameterInfo> VectorParameterInfos;
			TArray<FMaterialParameterInfo> TextureParameterInfos;
			ParentMaterialInstance->GetAllScalarParameterInfo(ScalarParameterInfos, ParameterIds);
			ParentMaterialInstance->GetAllVectorParameterInfo(VectorParameterInfos, ParameterIds);
			ParentMaterialInstance->GetAllTextureParameterInfo(TextureParameterInfos, ParameterIds);

			for (const FMaterialParameterInfo& ParameterInfo : ScalarParameterInfos)
			{
				ExpectedParameters.Remove(ParameterInfo.Name.ToString());
			}
			for (const FMaterialParameterInfo& ParameterInfo : VectorParameterInfos)
			{
				ExpectedParameters.Remove(ParameterInfo.Name.ToString());
			}
			for (const FMaterialParameterInfo& ParameterInfo : TextureParameterInfos)
			{
				ExpectedParameters.Remove(ParameterInfo.Name.ToString());
			}
		}

		for (const FString& ExpectedParameter : ExpectedParameters)
		{
			NotCoveredIdentifiersParameters.Add(TEXT("[") + Identifier + TEXT("]: Does not cover expected parameter: ") + ExpectedParameter + TEXT("."));
		}
	};

	// For Substrate we only need to validate the first Material
	if (IInterchangeImportModule::Get().IsSubstrateEnabled())
	{
		const FSoftObjectPath& SubstrateMaterialPath = MaterialParents[TEXT("M_GLTF")];
		ValidateParameters.operator()<UMaterial>({}, SubstrateMaterialPath);
	}
	else
	{
		for (const TPair<FString, FSoftObjectPath>& MaterialParent : MaterialParents)
		{
			if (MaterialParent.Key == TEXT("M_GLTF"))
			{
				continue;
			}

			ValidateParameters.operator()<UMaterialInstance>(MaterialParent.Key, MaterialParent.Value);
		}
	}

	return NotCoveredIdentifiersParameters;
}

TSet<FString> UGLTFPipelineSettings::GenerateExpectedParametersList(const FString& Identifier) const
{
	using namespace UE::Interchange::GLTFMaterials;

	TSet<FString> ExpectedParameters;

	auto AddTextureAndRelated = [&ExpectedParameters](const FString& TextureName)
		{
			ExpectedParameters.Add(TextureName);
			ExpectedParameters.Add(TextureName + Inputs::PostFix::OffsetScale);
			ExpectedParameters.Add(TextureName + Inputs::PostFix::Rotation);
			ExpectedParameters.Add(TextureName + Inputs::PostFix::TexCoord);
			ExpectedParameters.Add(TextureName + Inputs::PostFix::TilingMethod);
		};

	if (IInterchangeImportModule::Get().IsSubstrateEnabled())
	{
		AddTextureAndRelated(Inputs::NormalTexture);
		ExpectedParameters.Add(Inputs::NormalScale);

		AddTextureAndRelated(Inputs::EmissiveTexture);
		ExpectedParameters.Add(Inputs::EmissiveFactor);
		ExpectedParameters.Add(Inputs::EmissiveStrength);

		AddTextureAndRelated(Inputs::OcclusionTexture);
		ExpectedParameters.Add(Inputs::OcclusionStrength);

		ExpectedParameters.Add(Inputs::IOR);

		AddTextureAndRelated(Inputs::SpecularTexture);
		ExpectedParameters.Add(Inputs::SpecularFactor);

		AddTextureAndRelated(Inputs::SpecularColorTexture);
		ExpectedParameters.Add(Inputs::SpecularColorFactor);

		AddTextureAndRelated(Inputs::BaseColorTexture);
		ExpectedParameters.Add(Inputs::BaseColorFactor);

		AddTextureAndRelated(Inputs::MetallicRoughnessTexture);
		ExpectedParameters.Add(Inputs::MetallicFactor);
		ExpectedParameters.Add(Inputs::RoughnessFactor);

		AddTextureAndRelated(Inputs::ClearCoatTexture);
		ExpectedParameters.Add(Inputs::ClearCoatFactor);

		AddTextureAndRelated(Inputs::ClearCoatRoughnessTexture);
		ExpectedParameters.Add(Inputs::ClearCoatRoughnessFactor);

		AddTextureAndRelated(Inputs::ClearCoatNormalTexture);
		ExpectedParameters.Add(Inputs::ClearCoatNormalScale);

		AddTextureAndRelated(Inputs::SheenColorTexture);
		ExpectedParameters.Add(Inputs::SheenColorFactor);

		AddTextureAndRelated(Inputs::SheenRoughnessTexture);
		ExpectedParameters.Add(Inputs::SheenRoughnessFactor);

		AddTextureAndRelated(Inputs::TransmissionTexture);
		ExpectedParameters.Add(Inputs::TransmissionFactor);

		AddTextureAndRelated(Inputs::DiffuseTexture);
		ExpectedParameters.Add(Inputs::DiffuseFactor);

		AddTextureAndRelated(Inputs::SpecularGlossinessTexture);
		ExpectedParameters.Add(Inputs::SpecFactor);
		ExpectedParameters.Add(Inputs::GlossinessFactor);

		AddTextureAndRelated(Inputs::ThicknessTexture);
		ExpectedParameters.Add(Inputs::ThicknessFactor);
		ExpectedParameters.Add(Inputs::AttenuationDistance);
		ExpectedParameters.Add(Inputs::AttenuationColor);
	}
	else
	{
		if (Identifier.Contains(TEXT("_Unlit")))
		{
			AddTextureAndRelated(Inputs::BaseColorTexture);
			ExpectedParameters.Add(Inputs::BaseColorFactor);

			return ExpectedParameters;
		}

		//Generic ones:
		{
			AddTextureAndRelated(Inputs::NormalTexture);
			ExpectedParameters.Add(Inputs::NormalScale);

			if (!Identifier.Contains(TEXT("Transmission")))
			{
				AddTextureAndRelated(Inputs::EmissiveTexture);
				ExpectedParameters.Add(Inputs::EmissiveFactor);
				ExpectedParameters.Add(Inputs::EmissiveStrength);
			}

			AddTextureAndRelated(Inputs::OcclusionTexture);
			ExpectedParameters.Add(Inputs::OcclusionStrength);

			if (!Identifier.Contains(TEXT("SpecularGlossiness")))
			{
				ExpectedParameters.Add(Inputs::IOR);

				AddTextureAndRelated(Inputs::SpecularTexture);
				ExpectedParameters.Add(Inputs::SpecularFactor);
			}
		}

		//Based on ShadingModel:

		if (Identifier.Contains(TEXT("Default")))
		{
			//MetalRoughness Specific:

			AddTextureAndRelated(Inputs::BaseColorTexture);
			ExpectedParameters.Add(Inputs::BaseColorFactor);

			AddTextureAndRelated(Inputs::MetallicRoughnessTexture);
			ExpectedParameters.Add(Inputs::MetallicFactor);
			ExpectedParameters.Add(Inputs::RoughnessFactor);
		}
		else if (Identifier.Contains(TEXT("ClearCoat")))
		{
			AddTextureAndRelated(Inputs::ClearCoatTexture);
			ExpectedParameters.Add(Inputs::ClearCoatFactor);

			AddTextureAndRelated(Inputs::ClearCoatRoughnessTexture);
			ExpectedParameters.Add(Inputs::ClearCoatRoughnessFactor);

			AddTextureAndRelated(Inputs::ClearCoatNormalTexture);
			ExpectedParameters.Add(Inputs::ClearCoatNormalScale);
		}
		else if (Identifier.Contains(TEXT("Sheen")))
		{
			AddTextureAndRelated(Inputs::SheenColorTexture);
			ExpectedParameters.Add(Inputs::SheenColorFactor);

			AddTextureAndRelated(Inputs::SheenRoughnessTexture);
			ExpectedParameters.Add(Inputs::SheenRoughnessFactor);
		}
		else if (Identifier.Contains(TEXT("Transmission")))
		{
			AddTextureAndRelated(Inputs::TransmissionTexture);
			ExpectedParameters.Add(Inputs::TransmissionFactor);
		}
		else if (Identifier.Contains(TEXT("SpecularGlossiness")))
		{
			AddTextureAndRelated(Inputs::DiffuseTexture);
			ExpectedParameters.Add(Inputs::DiffuseFactor);

			AddTextureAndRelated(Inputs::SpecularGlossinessTexture);
			ExpectedParameters.Add(Inputs::SpecFactor);
			ExpectedParameters.Add(Inputs::GlossinessFactor);
		}
	}
	return ExpectedParameters;
}

UInterchangeMaterialInstanceFactoryNode* UGLTFPipelineSettings::BuildMaterialInstance(UInterchangeBaseNodeContainer* NodeContainer, const UInterchangeShaderGraphNode* ShaderGraphNode, const FString& OldFactoryNodeUId)
{
	using namespace UE::Interchange::GLTFMaterials;

	if (!ShaderGraphNode)
	{
		return nullptr;
	}

	FString MaterialFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(ShaderGraphNode->GetUniqueID());

	// Check the search location of the material, if no search location is specified, attribute "SearchLocation" is not set,
	// then we shouldn't replace the material factory node and just take the existing one
	if (UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode = Cast<UInterchangeBaseMaterialFactoryNode>(NodeContainer->GetFactoryNode(MaterialFactoryNodeUid)))
	{
		if (MaterialFactoryNode->HasAttribute(UE::Interchange::FAttributeKey(TEXT("SearchLocation"))))
		{
			return nullptr;
		}
	}

	TArray<UE::Interchange::FAttributeKey> AttributeKeys;
	ShaderGraphNode->GetAttributeKeys(AttributeKeys);

	TMap<FString, UE::Interchange::FAttributeKey> GltfAttributeKeys;
	for (const UE::Interchange::FAttributeKey& AttributeKey : AttributeKeys)
	{
		if (AttributeKey.ToString().Contains(InterchangeGltfMaterialAttributeIdentifier))
		{
			GltfAttributeKeys.Add(AttributeKey.ToString().Replace(*InterchangeGltfMaterialAttributeIdentifier, TEXT(""), ESearchCase::CaseSensitive), AttributeKey);
		}
	}
	if (GltfAttributeKeys.Num() == 0)
	{
		return nullptr;
	}

	FString ParentIdentifier;
	if (!ShaderGraphNode->GetStringAttribute(*(InterchangeGltfMaterialAttributeIdentifier + TEXT("ParentIdentifier")), ParentIdentifier))
	{
		return nullptr;
	}
	GltfAttributeKeys.Remove(TEXT("ParentIdentifier"));

	bool bSubstrate = IInterchangeImportModule::Get().IsSubstrateEnabled();
	bool bTwoSided = ParentIdentifier.Contains(TEXT("DS"), ESearchCase::CaseSensitive, ESearchDir::FromEnd) ? true : false;
	EBlendMode BlendMode = 
		ParentIdentifier.Contains(TEXT("Blend"), ESearchCase::CaseSensitive, ESearchDir::FromEnd)        ||
		ParentIdentifier.Contains(TEXT("Mask"), ESearchCase::CaseSensitive, ESearchDir::FromEnd)         ||
		ParentIdentifier.Contains(TEXT("Transmission"), ESearchCase::CaseSensitive, ESearchDir::FromEnd) ?
		EBlendMode::BLEND_TranslucentColoredTransmittance                                                :
		EBlendMode::BLEND_Opaque;
	bool bSpecGloss = ParentIdentifier.Contains(TEXT("SpecularGlossiness"), ESearchCase::CaseSensitive) ? true : false;

	FString Parent;
	bool bUseSubstrateMaterial = bSubstrate && !ParentIdentifier.Contains(TEXT("Unlit"));
	if (bUseSubstrateMaterial)
	{
		ParentIdentifier = TEXT("M_GLTF");
	}

	if (const FSoftObjectPath* ObjectPath = MaterialParents.Find(ParentIdentifier))
	{
		Parent = ObjectPath->GetAssetPathString();
	}
	else
	{
		UE_LOGF(LogInterchangePipeline, Warning, "[Interchange] Failed to load MaterialParent for ParentIdentifier: %ls", *ParentIdentifier);
		return nullptr;
	}

	FString MaterialFactoryNodeName = ShaderGraphNode->GetDisplayLabel();

	// Transfer properties from the Material Factory Node to Material Instance Factory Node
	EReimportStrategyFlags ReimportStrategy = EReimportStrategyFlags::ApplyNoProperties;
	bool bShouldForceReimport = false;

	if (UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode = Cast<UInterchangeBaseMaterialFactoryNode>(NodeContainer->GetFactoryNode(OldFactoryNodeUId)))
	{
		// Retrieve Attributes to Transfer
		ReimportStrategy = MaterialFactoryNode->GetReimportStrategyFlags();
		bShouldForceReimport = MaterialFactoryNode->ShouldForceNodeReimport();
	}

	if (UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode = NewObject<UInterchangeMaterialInstanceFactoryNode>(NodeContainer))
	{
		NodeContainer->SetupAndReplaceFactoryNode(MaterialInstanceFactoryNode, MaterialFactoryNodeUid, MaterialFactoryNodeName, EInterchangeNodeContainerType::FactoryData, OldFactoryNodeUId);

		// Transfer the Attributes
		if (bShouldForceReimport)
		{
			MaterialInstanceFactoryNode->SetForceNodeReimport();
		}
		MaterialInstanceFactoryNode->SetReimportStrategyFlags(ReimportStrategy);

		MaterialInstanceFactoryNode->SetCustomParent(Parent);

		UInterchangeEditorUtilitiesBase* EditorUtilities = UInterchangeManager::GetInterchangeManager().GetEditorUtilities();
		const UClass* MaterialClass = (EditorUtilities && EditorUtilities->IsRuntimeOrPIE()) ? UMaterialInstanceDynamic::StaticClass() : UMaterialInstanceConstant::StaticClass();
		MaterialInstanceFactoryNode->SetCustomInstanceClassName(MaterialClass->GetPathName());
		
		if (bUseSubstrateMaterial)
		{
			MaterialInstanceFactoryNode->SetCustomBlendMode(BlendMode);
			MaterialInstanceFactoryNode->SetCustomTwoSided(bTwoSided);
			MaterialInstanceFactoryNode->AddBooleanAttribute(UInterchangeShaderPortsAPI::MakeInputValueKey(TEXT("bHasDiffuseSpecGloss")), bSpecGloss);
		}

		for (const TPair<FString, UE::Interchange::FAttributeKey>& GltfAttributeKey : GltfAttributeKeys)
		{
			UE::Interchange::EAttributeTypes AttributeType = ShaderGraphNode->GetAttributeType(GltfAttributeKey.Value);

			FString InputValueKey = UInterchangeShaderPortsAPI::MakeInputValueKey(GltfAttributeKey.Key);

			//we are only using 4 attribute types for now:
			switch (AttributeType)
			{
			case UE::Interchange::EAttributeTypes::Bool:
			{
				bool Value;
				if (ShaderGraphNode->GetBooleanAttribute(GltfAttributeKey.Value.Key, Value))
				{
					MaterialInstanceFactoryNode->AddBooleanAttribute(InputValueKey, Value);
				}
			}
			break;
			case UE::Interchange::EAttributeTypes::Float:
			{
				float Value;
				if (ShaderGraphNode->GetFloatAttribute(GltfAttributeKey.Value.Key, Value))
				{
					MaterialInstanceFactoryNode->AddFloatAttribute(InputValueKey, Value);
				}
			}
			break;
			case UE::Interchange::EAttributeTypes::LinearColor:
			{
				FLinearColor Value;
				if (ShaderGraphNode->GetLinearColorAttribute(GltfAttributeKey.Value.Key, Value))
				{
					MaterialInstanceFactoryNode->AddLinearColorAttribute(InputValueKey, Value);
				}
			}
			break;
			case UE::Interchange::EAttributeTypes::String:
			{
				FString TextureUid;
				if (ShaderGraphNode->GetStringAttribute(GltfAttributeKey.Value.Key, TextureUid))
				{
					FString FactoryTextureUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TextureUid);

					MaterialInstanceFactoryNode->AddStringAttribute(InputValueKey, FactoryTextureUid);
					MaterialInstanceFactoryNode->AddFactoryDependencyUid(FactoryTextureUid);
				}
			}
			break;
			default:
				break;
			}
		}

		return MaterialInstanceFactoryNode;
	}

	return nullptr;
}

UInterchangeGLTFPipeline::UInterchangeGLTFPipeline()
	: GLTFPipelineSettings(UGLTFPipelineSettings::StaticClass()->GetDefaultObject<UGLTFPipelineSettings>())
{
}

void UInterchangeGLTFPipeline::AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams)
{
	Super::AdjustSettingsForContext(ContextParams);

	TArray<FString> MaterialInstanceIssues = GLTFPipelineSettings->ValidateMaterialInstancesAndParameters();
	for (const FString& MaterialInstanceIssue : MaterialInstanceIssues)
	{
		UE_LOGF(LogInterchangePipeline, Warning, "%ls", *MaterialInstanceIssue);
	}
}

void UInterchangeGLTFPipeline::ExecutePipeline(UInterchangeBaseNodeContainer* NodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath)
{
	Super::ExecutePipeline(NodeContainer, InSourceDatas, ContentBasePath);

	if (GLTFPipelineSettings)
	{
		TMap<FString, const UInterchangeShaderGraphNode*> MaterialFactoryNodeUidsToShaderGraphNodes;
		auto FindGLTFShaderGraphNode = [&MaterialFactoryNodeUidsToShaderGraphNodes, &NodeContainer](const FString& NodeUid, UInterchangeFactoryBaseNode* /*Material or MaterialInstance*/ FactoryNode)
		{
			TArray<FString> TargetNodeUids;
			FactoryNode->GetTargetNodeUids(TargetNodeUids);

			for (const FString& TargetNodeUid : TargetNodeUids)
			{

				if (const UInterchangeShaderGraphNode* ShaderGraphNode = Cast<UInterchangeShaderGraphNode>(NodeContainer->GetNode(TargetNodeUid)))
				{
					FString ParentIdentifier;
					if (ShaderGraphNode->GetStringAttribute(*(InterchangeGltfMaterialAttributeIdentifier + TEXT("ParentIdentifier")), ParentIdentifier))
					{
						MaterialFactoryNodeUidsToShaderGraphNodes.Add(NodeUid, ShaderGraphNode);
						break;
					}
				}
			}
		};
		NodeContainer->IterateNodesOfType<UInterchangeMaterialFactoryNode>([&MaterialFactoryNodeUidsToShaderGraphNodes, &NodeContainer, &FindGLTFShaderGraphNode](const FString& NodeUid, UInterchangeMaterialFactoryNode* MaterialFactoryNode)
			{
				FindGLTFShaderGraphNode(NodeUid, MaterialFactoryNode);
			});

		NodeContainer->IterateNodesOfType<UInterchangeMaterialInstanceFactoryNode>([&MaterialFactoryNodeUidsToShaderGraphNodes, &NodeContainer, &FindGLTFShaderGraphNode](const FString& NodeUid, UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode)
			{
				FindGLTFShaderGraphNode(NodeUid, MaterialInstanceFactoryNode);
			});

		for (const TPair<FString, const UInterchangeShaderGraphNode*>& ShaderGraphNode : MaterialFactoryNodeUidsToShaderGraphNodes)
		{
			UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode = GLTFPipelineSettings->BuildMaterialInstance(NodeContainer, ShaderGraphNode.Value, ShaderGraphNode.Key);

			if (MaterialInstanceFactoryNode)
			{
				UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(NodeContainer);
				UE::Interchange::PipelineHelper::FillSubPathFromSourceNode(MaterialInstanceFactoryNode, SourceNode);
			}
		}
	}
}

