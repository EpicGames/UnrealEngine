// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeGenericAssetsPipelineSharedSettings.h"
#include "InterchangePipelineBase.h"
#include "InterchangeSourceData.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeGenericAnimationPipeline.generated.h"

#define UE_API INTERCHANGEPIPELINES_API

class UInterchangeAnimationTrackSetNode;
class UInterchangeMeshNode;
class UInterchangeSkeletalAnimationTrackNode;

/** Animation length type when importing. */
UENUM(BlueprintType)
enum class EInterchangeAnimationRange : uint8
{
	/** This option imports the range of frames based on timeline definition in the source. */
	Timeline				UMETA(DisplayName = "Source Timeline"),
	/** This option imports the range of frames that have animation. */
	Animated				UMETA(DisplayName = "Animated Time"),
	/** This option imports the range of frames specified by "FrameImportRange". */
	SetRange				UMETA(DisplayName = "Set Range"),
	MAX,
};

UENUM(BlueprintType)
enum class EInterchangeFrameAlignment : uint8
{
	None            UMETA(DisplayName = "None"),
	SnapToFloor     UMETA(DisplayName = "Snap To Floor"),
	SnapToClosest   UMETA(DisplayName = "Snap To Closest Frame Boundary"),
	SnapToCeiling   UMETA(DisplayName = "Snap To Ceiling")
};

UCLASS(MinimalAPI, BlueprintType, hidedropdown)
class UInterchangeGenericAnimationPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

public:

#if WITH_EDITOR
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif

	UE_API virtual void Serialize(FArchive& Ar) override;

	static UE_API FString GetPipelineCategory(UClass* AssetClass);

	//Common SkeletalMeshes And Animations Properties Settings Pointer
	UPROPERTY(Transient)
	TWeakObjectPtr<UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties> CommonSkeletalMeshesAndAnimationsProperties;

	//Common Meshes Properties Settings Pointer
	UPROPERTY(Transient)
	TWeakObjectPtr<UInterchangeGenericCommonMeshesProperties> CommonMeshesProperties;

	//////	ANIMATION_CATEGORY Properties //////
	/** If enabled, import all animation assets found in the source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animations")
	bool bImportAnimations = true;

	/** Import bone transform tracks. If false, this will discard any bone transform tracks.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animations", meta = (EditCondition = "bImportAnimations"))
	bool bImportBoneTracks = true;

	/** Determines which animation range to import: the range defined at export, the range of frames with animation, or a manually defined range. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animations", meta = (EditCondition = "bImportAnimations && bImportBoneTracks", DisplayName = "Animation Length"))
	EInterchangeAnimationRange AnimationRange = EInterchangeAnimationRange::Timeline;

	/** The frame range used when the Animation Length setting is set to Set Range. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animations", meta = (UIMin = 0, ClampMin = 0))
	FInt32Interval FrameImportRange = FInt32Interval(0, 0);

	/** If enabled, samples all imported animation data at the default rate of 30 FPS. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animations", meta = (EditCondition = "bImportAnimations && bImportBoneTracks", ToolTip = "If enabled, samples all animation curves to 30 FPS"))
	bool bUse30HzToBakeBoneAnimation = false;

	/** Use this option to specify a sample rate for the imported animation, a value of 0 use the best matching sample rate. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animations", meta = (EditCondition = "bImportAnimations && bImportBoneTracks && !bUse30HzToBakeBoneAnimation", ToolTip = "Sample fbx animation data at the specified sample rate, 0 find automaticaly the best sample rate", ClampMin = 0, UIMin = 0, ClampMax = 48000, UIMax = 60))
	int32 CustomBoneAnimationSampleRate = 0;

	/** Frame alignment settings*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animations", meta = (EditCondition = "bImportAnimations && bImportBoneTracks"))
	EInterchangeFrameAlignment FrameAlignment = EInterchangeFrameAlignment::None;

	UPROPERTY()
	bool bSnapToClosestFrameBoundary_DEPRECATED = false;

	/** If enabled, import node attributes as either Animation Curves or Animation Attributes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animations", meta = (SubCategory = "Curves", EditCondition = "bImportAnimations", DisplayName = "Import Attributes as Curves or Animation Attributes"))
	bool bImportCustomAttribute = true;

	/** Determines whether to automatically add curve metadata to an animation's skeleton. If this setting is disabled, curve metadata will be added to skeletal meshes for morph targets, but no metadata entry will be created for general curves. */
	UPROPERTY()
	bool bAddCurveMetadataToSkeleton_DEPRECATED = true;

	/** Set the material curve type for all custom attributes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animations", meta = (SubCategory = "Curves", EditCondition = "bImportAnimations && bImportCustomAttribute", DisplayName = "Set Material Curve Type"))
	bool bSetMaterialDriveParameterOnCustomAttribute = false;

	/** Set the Material Curve Type for custom attributes that have the specified suffixes. This setting is not used if the Set Material Curve Type setting is enabled.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animations", meta = (SubCategory = "Curves", EditCondition = "bImportAnimations && bImportCustomAttribute && bSetMaterialDriveParameterOnCustomAttribute", DisplayName = "Material Curve Suffixes"))
	TArray<FString> MaterialCurveSuffixes = {TEXT("_mat")};

	/** When importing custom attributes as curves, remove redundant keys. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animations", meta = (SubCategory = "Curves", EditCondition = "bImportAnimations && bImportCustomAttribute", DisplayName = "Remove Redundant Keys"))
	bool bRemoveCurveRedundantKeys = false;

	/** When importing a custom attribute or morph target as a curve, only import if it has a value other than zero. This avoids adding extra curves to evaluate. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animations", meta = (SubCategory = "Curves", EditCondition = "bImportAnimations && bImportCustomAttribute", DisplayName = "Do not import curves with only 0 values"))
	bool bDoNotImportCurveWithZero = false;

	/** If enabled, all previous node attributes imported as Animation Attributes will be deleted when doing a reimport. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animations", meta = (SubCategory = "Curves", EditCondition = "bImportAnimations && bImportCustomAttribute", DisplayName = "Delete existing Animation Attributes"))
	bool bDeleteExistingNonCurveCustomAttributes = false;

	/** If enabled, all previous node attributes imported as Animation Curves will be deleted when doing a reimport. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animations", meta = (SubCategory = "Curves", EditCondition = "bImportAnimations && bImportCustomAttribute", DisplayName = "Delete existing Animation Curves"))
	bool bDeleteExistingCustomAttributeCurves = false;

	/** If enabled, all previous morph target curves will be deleted when doing a reimport. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animations", meta = (SubCategory = "Curves", EditCondition = "bImportAnimations && bImportCustomAttribute", DisplayName = "Delete existing Morph Target Curves"))
	bool bDeleteExistingMorphTargetCurves = false;

	/** Name of the source animation that was imported. This is used to reimport correct animation from the translated source. */
	UPROPERTY()
	FString SourceAnimationName;

	UE_API virtual void AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams) override;

#if WITH_EDITOR
	
	UE_API virtual bool IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const override;
	
	UE_API virtual void GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const override;

#endif //WITH_EDITOR

	UE_API void ImplementUseSourceNameForAssetOptionAnimSequence(bool bUseSourceNameForAsset, const FString& AssetName);

protected:

	UE_API virtual void ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath) override;

	virtual bool CanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask) override
	{
		return true;
	}

private:

	UE_API void CreateLevelSequenceFactoryNode(UInterchangeAnimationTrackSetNode& Node);

	UE_API void CreateAnimSequenceFactoryNode(UInterchangeSkeletalAnimationTrackNode& Node);

	// Set as a property to carry this value over during a duplicate
	UPROPERTY()
	bool bSceneImport = false;

	UInterchangeBaseNodeContainer* BaseNodeContainer = nullptr;
	TArray<const UInterchangeSourceData*> SourceDatas;

};


#undef UE_API
