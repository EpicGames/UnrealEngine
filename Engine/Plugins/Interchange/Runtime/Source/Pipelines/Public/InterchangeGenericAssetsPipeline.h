// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeGenericAssetsPipelineSharedSettings.h"
#include "InterchangePipelineBase.h"
#include "InterchangeSourceData.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STreeView.h"

#include "InterchangeGenericAssetsPipeline.generated.h"

#define UE_API INTERCHANGEPIPELINES_API

class STableViewBase;
class ITableRow;
class UInterchangeGenericAnimationPipeline;
class UInterchangeGenericGroomPipeline;
class UInterchangeGenericMaterialPipeline;
class UInterchangeGenericMeshPipeline;
class UInterchangeGenericTexturePipeline;
class USkeletalMesh;
class USkeleton;
class UStaticMesh;

struct FReferenceSkeleton;
struct FMeshBoneInfo;

/**
 * This pipeline is the generic option for all types of meshes. It should be called before specialized mesh pipelines like the generic static mesh or skeletal mesh pipelines.
 * All import options that are shared between mesh types should be added here.
 *
 */
UCLASS(MinimalAPI, BlueprintType, editinlinenew)
class UInterchangeGenericAssetsPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

public:
	UE_API UInterchangeGenericAssetsPipeline();
	
	//////	COMMON_CATEGORY Properties //////
	
	/** The name of the pipeline that will be display in the import dialog. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common", meta = (StandAlonePipelineProperty = "True", PipelineInternalEditionData = "True"))
	FString PipelineDisplayName;

	/* Set the reimport strategy. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common", meta = (AdjustPipelineAndRefreshDetailOnChange = "True"))
	EReimportStrategyFlags ReimportStrategy = EReimportStrategyFlags::ApplyNoProperties;

	/** If enabled, and the Asset Name setting is empty, and there is only one asset and one source, the imported asset is given the same name as the source data. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common")
	bool bUseSourceNameForAsset = true;

	/** Create an additional Content folder inside of the chosen import directory, and name it after the imported scene */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common")
	bool bSceneNameSubFolder = false;

	/** Group the assets according to their type into additional Content folders created on the import directory (/Materials, /StaticMeshes, /SkeletalMeshes, etc.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common")
	bool bAssetTypeSubFolders = false;

	/** If set, and there is only one asset and one source, the imported asset is given this name. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common", meta = (StandAlonePipelineProperty = "True"))
	FString AssetName;

	/** Translation offset applied to meshes and animations. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common", meta = (DisplayName = "Offset Translation"))
	FVector ImportOffsetTranslation;

	/** Rotation offset applied to meshes and animations. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common", meta = (DisplayName = "Offset Rotation"))
	FRotator ImportOffsetRotation;

	/** Uniform scale offset applied to meshes and animations. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common", meta = (DisplayName = "Offset Uniform Scale"))
	float ImportOffsetUniformScale = 1.0f;

	//////	COMMON_MESHES_CATEGORY Properties //////
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Instanced, Category = "Common Meshes")
	TObjectPtr<UInterchangeGenericCommonMeshesProperties> CommonMeshesProperties;
		
	//////  COMMON_SKELETAL_ANIMATIONS_CATEGORY //////
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Instanced, Category = "Common Skeletal Meshes and Animations")
	TObjectPtr<UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties> CommonSkeletalMeshesAndAnimationsProperties;

	//////	MESHES_CATEGORY Properties //////
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Instanced, Category = "Meshes")
	TObjectPtr<UInterchangeGenericMeshPipeline> MeshPipeline;

	//////	ANIMATIONS_CATEGORY Properties //////
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Instanced, Category = "Animation")
	TObjectPtr<UInterchangeGenericAnimationPipeline> AnimationPipeline;

	//////	MATERIALS_CATEGORY Properties //////
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Instanced, Category = "Materials")
	TObjectPtr<UInterchangeGenericMaterialPipeline> MaterialPipeline;

	//////	GROOMS_CATEGORY Properties //////
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Instanced, Category = "Grooms")
	TObjectPtr<UInterchangeGenericGroomPipeline> GroomPipeline;

	/** Internal property to cache the name of the sub-folder prefix for use during reimport */
	UPROPERTY(meta = (PipelineInternalEditionData = "True"))
	FString SceneNameFolderPrefix;

	UE_API virtual void PreDialogCleanup(const FName PipelineStackName) override;

	UE_API virtual void Serialize(FArchive& Ar) override;

	UE_API virtual void PostLoad() override;

	UE_API virtual bool IsSettingsAreValid(TOptional<FText>& OutInvalidReason) const override;


	UE_API virtual void AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams) override;
#if WITH_EDITOR
	UE_API virtual void FilterPropertiesFromTranslatedData(UInterchangeBaseNodeContainer* InBaseNodeContainer) override;

	UE_API virtual bool IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const override;

	UE_API virtual void GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const override;
#endif //WITH_EDITOR

	UE_API virtual TArray<FInterchangeConflictInfo> GetConflictInfos(UObject* ReimportObject, UInterchangeBaseNodeContainer* InBaseNodeContainer, UInterchangeSourceData* SourceData) override;

	UE_API virtual void ShowConflictDialog(const FGuid& ConflictUniqueId) override;

	virtual bool IsScripted() override
	{
		return false;
	}

#if WITH_EDITOR
	UE_API virtual bool GetPropertyPossibleValues(const FName PropertyPath, TArray<FString>& PossibleValues) override;
#endif

	struct FSkeletonJoint : public TSharedFromThis<FSkeletonJoint>
	{
		FString JointName;
		bool bAdded = false;
		bool bRemoved = false;
		bool bMatch = false;
		bool bConflict = false;
		bool bChildConflict = false;
		TSharedPtr<FSkeletonJoint> Parent;
		TArray<TSharedPtr<FSkeletonJoint>> Children;
	};

	//We need to store the adjusted import only animation boolean to restore it in PreDialogCleanup
	UPROPERTY(meta = (AlwaysResetToDefault = "True"))
	bool bImportOnlyAnimationAdjusted = false;

protected:

	UE_API virtual void ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath) override;
	UE_API virtual void ExecutePostFactoryPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport) override;
	UE_API virtual void ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport) override;

	virtual bool CanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask) override
	{
		//We cannot run asynchronously because of the two following issues
		// Post Translator Task: material pipeline is loading assets (Parent Material)
		// Post Import Task: physics asset need to create a scene preview to be created
		return false;
	}

	UE_API virtual void SetReimportSourceIndex(UClass* ReimportObjectClass, const int32 SourceFileIndex) override;

	//virtual bool ExecuteExportPipeline(UInterchangeBaseNodeContainer* BaseNodeContainer) override;
private:

#if WITH_EDITOR
	UE_API void CreateMaterialConflict(UStaticMesh* StaticMesh, USkeletalMesh* SkeletalMesh, UInterchangeBaseNodeContainer* TransientBaseNodeContainer);
	UE_API void InternalRecursiveFillJointsFromReferenceSkeleton(TSharedPtr<FSkeletonJoint> ParentJoint, TMap<FString, TSharedPtr<FSkeletonJoint>>& Joints, const int32 BoneIndex, const FReferenceSkeleton& ReferenceSkeleton);
	UE_API void InternalRecursiveFillJointsFromNodeContainer(TSharedPtr<FSkeletonJoint> ParentJoint, TMap<FString, TSharedPtr<FSkeletonJoint>>& Joints, const FString& JoinUid, const UInterchangeBaseNodeContainer* BaseNodeContainer, const bool bConvertStaticToSkeletalActive);
	UE_API void CreateSkeletonConflict(USkeleton* SpecifiedSkeleton, USkeletalMesh* SkeletalMesh, UInterchangeBaseNodeContainer* TransientBaseNodeContainer);
#endif

	/**
	 * Implement pipeline option bUseSourceNameForAsset
	 */
	UE_API void ImplementUseSourceNameForAssetOption(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas);

	/**
	 * Adds the user defined attributes (UInterchangeUserDefinedAttributesAPI) to the package meta data (FMetaData) for WITH_EDITORONLY_DATA, and add UAssetUserData for AActors.
	 */
	UE_API void AddMetaData(UObject* CreatedAsset, const UInterchangeBaseNode* Node);

	struct FMaterialConflictData
	{
		FGuid ConflictUniqueId;
		TArray<FString> AssetMaterialNames;
		TArray<FString> ImportMaterialNames;
		TArray<int32> MatchMaterialIndexes;
		UObject* ReimportObject = nullptr;
		const FText DialogTitle = NSLOCTEXT("UInterchangeGenericAssetsPipeline", "GetConflictInfos_MaterialTitle", "Material Conflicts");

		void Reset()
		{
			ConflictUniqueId.Invalidate();
			AssetMaterialNames.Empty();
			ImportMaterialNames.Empty();
			MatchMaterialIndexes.Empty();
			ReimportObject = nullptr;
		}
	};
	FMaterialConflictData MaterialConflictData;

	struct FSkeletonConflictData
	{
		FGuid ConflictUniqueId;
		TMap<FString, TSharedPtr<FSkeletonJoint>> Joints;
		UObject* ReimportObject = nullptr;
		const FText DialogTitle = NSLOCTEXT("UInterchangeGenericAssetsPipeline", "GetConflictInfos_SkeletonTitle", "Skeleton Conflicts");

		void Reset()
		{
			ConflictUniqueId.Invalidate();
			Joints.Empty();
			ReimportObject = nullptr;
		}
	};
	FSkeletonConflictData SkeletonConflictData;

	//Make sure we notify the user only once for metadata attribute key name too long
	bool bHasNotify_MetaDataAttributeKeyNameTooLong = false;
};

class SInterchangeGenericAssetMaterialConflictWidget : public SInterchangeBaseConflictWidget
{
public:

	struct FListItem
	{
		FString ImportName;
		int32 bMatched = INDEX_NONE;
		FString AssetMatchedName;
		FString AssetName;
	};

	static const FName NAME_Import;
	static const FName NAME_Asset;

	static const FSlateColor SlateColorFullConflict;
	static const FSlateColor SlateColorSubConflict;

	SLATE_BEGIN_ARGS(SInterchangeGenericAssetMaterialConflictWidget)
		: _AssetMaterialNames()
		, _ImportMaterialNames()
		, _MatchMaterialIndexes()
		, _ReimportObject(nullptr)
		{}
		SLATE_ARGUMENT(TArray<FString>, AssetMaterialNames)
		SLATE_ARGUMENT(TArray<FString>, ImportMaterialNames)
		SLATE_ARGUMENT(TArray<int32>, MatchMaterialIndexes)
		SLATE_ARGUMENT(UObject*, ReimportObject)

	SLATE_END_ARGS()
	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	FReply OnDone();

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

protected:
	
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FListItem> Item, const TSharedRef<STableViewBase>& OwnerTable);

	TArray<FString> AssetMaterialNames;
	TArray<FString> ImportMaterialNames;
	TArray<int32> MatchMaterialIndexes;
	UObject* ReimportObject = nullptr;

	TArray<TSharedPtr<FListItem>> RowItems;
	TSharedPtr<SListView<TSharedPtr<FListItem>>> MaterialList;
};

enum class EInterchangeSkeletonCompareSection : uint8
{
	Skeleton = 0,
	References,
	Count
};

class SInterchangeGenericAssetSkeletonConflictWidget : public SInterchangeBaseConflictWidget
{
	using FExpansionPredicate = TDelegate<bool(TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint>)>;

public:

	SLATE_BEGIN_ARGS(SInterchangeGenericAssetSkeletonConflictWidget)
		: _AssetReferencingSkeleton()
		, _Joints()
		, _ReimportObject(nullptr)
		{}

		SLATE_ARGUMENT(TArray<TSharedPtr<FString>>, AssetReferencingSkeleton)
		SLATE_ARGUMENT(TArray<TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint>>, Joints)
		SLATE_ARGUMENT(UObject*, ReimportObject)

	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	FReply OnDone()
	{
		if (WidgetWindow.IsValid())
		{
			WidgetWindow->RequestDestroyWindow();
		}
		return FReply::Handled();
	}


	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			return OnDone();
		}
		return FReply::Unhandled();
	}

	SInterchangeGenericAssetSkeletonConflictWidget()
	{}

private:
	TArray<TSharedPtr<FString>> AssetReferencingSkeleton;
	TArray<TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint>> Joints;
	UObject* ReimportObject;

	//////////////////////////////////////////////////////////////////////////
	// Recursive Expansion
	void OnSetExpansionRecursive(TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> SkeletonJoint, bool bShouldBeExpanded, const FExpansionPredicate Predicate = FExpansionPredicate());
	//////////////////////////////////////////////////////////////////////////

	//////////////////////////////////////////////////////////////////////////
	//Collapse generic
	bool bShowSectionFlag[(uint8)EInterchangeSkeletonCompareSection::Count];
	FReply OnExpandToConflict();
	FReply SetSectionVisible(EInterchangeSkeletonCompareSection SectionIndex);
	EVisibility IsSectionVisible(EInterchangeSkeletonCompareSection SectionIndex);
	const FSlateBrush* GetCollapsableArrow(EInterchangeSkeletonCompareSection SectionIndex) const;
	//////////////////////////////////////////////////////////////////////////


	//////////////////////////////////////////////////////////////////////////
	// Skeleton Data
	TSharedPtr<STreeView<TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint>>> CompareTree;

	//Construct slate
	TSharedPtr<SWidget> ConstructSkeletonComparison();
	TSharedPtr<SWidget> ConstructSkeletonReference();
	//Slate events
	TSharedRef<ITableRow> OnGenerateRowCompareTreeView(TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> RowData, const TSharedRef<STableViewBase>& Table);
	void OnGetChildrenRowCompareTreeView(TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> InParent, TArray< TSharedPtr<UInterchangeGenericAssetsPipeline::FSkeletonJoint> >& OutChildren);
	TSharedRef<ITableRow> OnGenerateRowAssetReferencingSkeleton(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
