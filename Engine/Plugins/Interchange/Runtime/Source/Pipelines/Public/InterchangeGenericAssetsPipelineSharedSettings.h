// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/Skeleton.h"
#include "InterchangePipelineBase.h"
#include "Math/Color.h"
#include "Math/Rotator.h"
#include "Math/Vector.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"

#include "InterchangeGenericAssetsPipelineSharedSettings.generated.h"

#define UE_API INTERCHANGEPIPELINES_API

class UInterchangeSkeletonFactoryNode;

/** Enumerates the options for importing all meshes as one type. */
UENUM(BlueprintType)
enum class EInterchangeForceMeshType : uint8
{
	/** Import both static meshes and skeletal meshes from the source without converting them. */
	IFMT_None UMETA(DisplayName = "None"),
	/** Import all meshes from the source as static meshes. */
	IFMT_StaticMesh UMETA(DisplayName = "Static Mesh"),
	/** Import all meshes from the source as skeletal meshes. */
	IFMT_SkeletalMesh UMETA(DisplayName = "Skeletal Mesh"),

	IFMT_MAX
};

UENUM(BlueprintType)
enum class EInterchangeVertexColorImportOption : uint8
{
	/** Import the mesh using the vertex colors from the translated source. */
	IVCIO_Replace UMETA(DisplayName = "Replace"),
	/** Ignore vertex colors from the translated source. In case of a reimport, keep the existing mesh vertex colors. */
	IVCIO_Ignore UMETA(DisplayName = "Ignore"),
	/** Override all vertex colors with the specified color. */
	IVCIO_Override UMETA(DisplayName = "Override"),

	IVCIO_MAX
};


UCLASS(MinimalAPI, BlueprintType, hidedropdown)
class UInterchangeGenericCommonMeshesProperties : public UInterchangePipelineBase
{
	GENERATED_BODY()
public:

	static FString GetPipelineCategory(UClass* AssetClass)
	{
		return TEXT("Common Meshes");
	}

	//////	COMMON_MESHES_CATEGORY Properties //////

	/**
	 * If set, imports all meshes in the source as either static meshes or skeletal meshes.
	 * For skeletal meshes the conversion will happen only if there is no skinned meshes.
	 * Mixing rigid skeletal mesh with skinned mesh is not good and will result in multiple skeletal meshes.
	 */
 	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes", meta = (SubCategory = "Conversion"))
	EInterchangeForceMeshType ForceAllMeshAsType = EInterchangeForceMeshType::IFMT_None;

	/** If enabled, and some static mesh transforms are animated, the pipeline will convert the static mesh into a rigid skeletal mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes", meta = (SubCategory = "Conversion", editcondition = "ForceAllMeshAsType == EInterchangeForceMeshType::IFMT_None"))
	bool bConvertStaticsWithAnimatedTransformToSkeletals = true;

	/** If enabled, all static meshes that have morph targets will be imported as skeletal meshes instead. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes", meta = (SubCategory = "Conversion", editcondition = "ForceAllMeshAsType == EInterchangeForceMeshType::IFMT_None"))
	bool bConvertStaticsWithMorphTargetsToSkeletals = false;

	/** If enabled, all static meshes nested in bone hierarchies will be imported as skeletal meshes instead. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes", meta = (SubCategory = "Conversion", editcondition = "ForceAllMeshAsType == EInterchangeForceMeshType::IFMT_None"))
	bool bConvertStaticsInBoneHierarchyToSkeletals = true;

	/**
	* If enabled, converted static meshes to skeletal meshes will use a skeleton with a unique bone at the geometry origin.
	* Otherwise, the bone structure is determined by the model hierarchy.
	* This setting is only used if Force All Meshes As Type setting is set to "Skeletal Mesh".
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes", meta = (SubCategory = "Conversion", editcondition = "ForceAllMeshAsType == EInterchangeForceMeshType::IFMT_SkeletalMesh"))
	bool bSingleBoneSkeleton = false;

	/** If enabled, any existing LODs for meshes are imported. This setting is only used if the Bake Meshes setting is also enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes")
	bool bImportLods = true;

	/** If enabled, meshes are baked with the scene instance hierarchy transform. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes")
	bool bBakeMeshes = true;
	
	/** If enabled, the inverse node rotation pivot will be apply to the mesh vertices. The pivot from the DCC will then be the origin of the mesh.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes", meta = (editcondition = "!bBakeMeshes"))
	bool bBakePivotMeshes = false;
	
	/** If checked, sections with matching materials are kept separate and will not get combined. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes")
	bool bKeepSectionsSeparate = false;
	
	/** Specify how vertex colors should be imported. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes")
	EInterchangeVertexColorImportOption VertexColorImportOption = EInterchangeVertexColorImportOption::IVCIO_Replace;

	/** Specify an override color for use when the Vertex Color Import Option setting is set to Override. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes")
	FColor VertexOverrideColor;

	/**
	 * If checked, import sockets
	 * StaticMesh, naming convention, SOCKET_MeshName_xx where "MeshName" should match the mesh you want to add socket to. The xx part is to add a unique id if many socket on the same mesh exist.
	 * SkeletalMesh, Naming convention, any leaf scene node under the skeleton root with a name starting with "SOCKET_" prefix.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes")
	bool bImportSockets = true;

	/** If enabled, normals in the imported mesh are ignored and recomputed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes", meta = (SubCategory = "Build"))
	bool bRecomputeNormals = true;

	/** If enabled, tangents in the imported mesh are ignored and recomputed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes", meta = (SubCategory = "Build"))
	bool bRecomputeTangents = true;

	/** If enabled, tangents are recomputed using MikkTSpace. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes", meta = (SubCategory = "Build", editcondition = "bRecomputeTangents"))
	bool bUseMikkTSpace = true;

	/** If enabled, normals are computed using the surface area and the corner angle of the triangle as a ratio. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes", meta = (SubCategory = "Build", editcondition = "bRecomputeNormals"))
	bool bComputeWeightedNormals = false;

	/** If true, tangents are stored at 16-bit vs 8-bit precision. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes", meta = (SubCategory = "Build"))
	bool bUseHighPrecisionTangentBasis = false;

	/** If true, UVs are stored at full floating-point precision. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes", meta = (SubCategory = "Build"))
	bool bUseFullPrecisionUVs = false;

	/** If enabled, UVs are converted to 16-bit by a legacy truncation process instead of the default rounding process. This may avoid differences when reimporting older content. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes", meta = (SubCategory = "Build"))
	bool bUseBackwardsCompatibleF16TruncUVs = false;

	/** If true, degenerate triangles are removed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Meshes", meta = (SubCategory = "Build"))
	bool bRemoveDegenerates = false;

	UE_DEPRECATED(5.8, "Use bConvertStaticsWithAnimatedTransformToSkeletals instead")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "bAutoDetectMeshType is deprecated. Use bConvertStaticsWithAnimatedTransformToSkeletals instead."))
	bool bAutoDetectMeshType_DEPRECATED = true;

#if WITH_EDITOR
	virtual bool IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const override
	{
		static const TSet<FName> NeedRefreshProperties =
		{
			GET_MEMBER_NAME_CHECKED(UInterchangeGenericCommonMeshesProperties, ForceAllMeshAsType),
			GET_MEMBER_NAME_CHECKED(UInterchangeGenericCommonMeshesProperties, bConvertStaticsWithAnimatedTransformToSkeletals),
			GET_MEMBER_NAME_CHECKED(UInterchangeGenericCommonMeshesProperties, bConvertStaticsWithMorphTargetsToSkeletals),
			GET_MEMBER_NAME_CHECKED(UInterchangeGenericCommonMeshesProperties, bConvertStaticsInBoneHierarchyToSkeletals),
			GET_MEMBER_NAME_CHECKED(UInterchangeGenericCommonMeshesProperties, bSingleBoneSkeleton)
		};

		if (NeedRefreshProperties.Contains(PropertyChangedEvent.GetPropertyName()))
		{
			return true;
		}
		return Super::IsPropertyChangeNeedRefresh(PropertyChangedEvent);
	}
#endif //WITH_EDITOR
};

UCLASS(MinimalAPI, BlueprintType, hidedropdown)
class UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties : public UInterchangePipelineBase
{
	GENERATED_BODY()
public:

	static FString GetPipelineCategory(UClass* AssetClass)
	{
		return TEXT("Common Skeletal Meshes and Animations");
	}

	/** If enabled, only animations are imported from the source. You must also set a valid skeleton. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Skeletal Meshes and Animations")
	bool bImportOnlyAnimations = false;

	/** If enabled, the pipeline will try to find a skeleton asset at the import location if available and set the skeleton property to use that asset. A new skeleton will be created if no skeleton asset is found. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Skeletal Meshes and Animations")
	bool bTryAutoSelectSkeleton = false;

	/** Skeleton to use for imported asset. When importing a skeletal mesh, leaving this as "None" will create a new skeleton. When importing an animation, this must be specified. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Skeletal Meshes and Animations", meta = (EditCondition = "!bTryAutoSelectSkeleton"))
	TSoftObjectPtr<USkeleton> Skeleton;

	/** If enabled, skinned meshes will be rebind using the joints transform pose at frame 0 instead of being import with the bind pose. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Skeletal Meshes and Animations", meta=(DisplayName="Use T0 As Ref Pose"))
	bool bUseT0AsRefPose = false;

	/** Determines whether to automatically add curve metadata to a skeleton. If this setting is disabled, curve metadata will be added to skeletal meshes for morph targets, but no metadata entry will be created for general curves. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Skeletal Meshes and Animations")
	bool bAddCurveMetadataToSkeleton = true;

	UE_DEPRECATED(5.8, "Use UInterchangeGenericCommonMeshesProperties::bConvertStaticsInBoneHierarchyToSkeletals instead")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "bImportMeshesInBoneHierarchy is deprecated. Use UInterchangeGenericCommonMeshesProperties::bConvertStaticsInBoneHierarchyToSkeletals instead."))
	bool bImportMeshesInBoneHierarchy_DEPRECATED = true;

	UE_DEPRECATED(5.8, "Use UInterchangeGenericCommonMeshesProperties::bConvertStaticsWithMorphTargetsToSkeletals instead")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "bConvertStaticsWithMorphTargetsToSkeletals is deprecated. Use UInterchangeGenericCommonMeshesProperties::bConvertStaticsWithMorphTargetsToSkeletals instead."))
	bool bConvertStaticsWithMorphTargetsToSkeletals_DEPRECATED = false;

#if WITH_EDITOR
	virtual bool IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const override
	{
		static const TSet<FName> NeedRefreshProperties =
		{
			GET_MEMBER_NAME_CHECKED(UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties, Skeleton),
			GET_MEMBER_NAME_CHECKED(UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties, bImportOnlyAnimations),
			GET_MEMBER_NAME_CHECKED(UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties, bTryAutoSelectSkeleton),
		};

		if (NeedRefreshProperties.Contains(PropertyChangedEvent.GetPropertyName()))
		{
			return true;
		}

		return Super::IsPropertyChangeNeedRefresh(PropertyChangedEvent);
	}

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

#endif //WITH_EDITOR
	virtual bool IsSettingsAreValid(TOptional<FText>& OutInvalidReason) const override
	{
		if (bImportOnlyAnimations && !IsSkeletonValid())
		{
			OutInvalidReason = NSLOCTEXT("UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties", "SkeletonMustBeSpecified", "When importing only animations, a valid skeleton must be set.");
			return false;
		}
		return Super::IsSettingsAreValid(OutInvalidReason);
	}

	/** Create a UInterchangeSkeletonFactoryNode */
	UE_API UInterchangeSkeletonFactoryNode* CreateSkeletonFactoryNode(UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& RootJointUid, const FString& SourceFileName/*Used for Warning generation*/);

	UE_API bool IsSkeletonValid() const;
};

#undef UE_API
