// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "InterchangePipelineBase.h"
#include "InterchangeAudioSoundWaveFactoryNode.h"
#include "InterchangeAudioSoundWaveNode.h"

#include "InterchangeGenericAudioPipeline.generated.h"

#define UE_API INTERCHANGEPIPELINES_API

/** Basic pipeline for importing sound wave assets. */
UCLASS(MinimalAPI, BlueprintType, EditInlineNew)
class UInterchangeGenericAudioPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

public:
	static UE_API FString GetPipelineCategory(UClass* AssetClass);

	/** The name of the pipeline that will be display in the import dialog. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Audio", meta = (StandAlonePipelineProperty = "True", PipelineInternalEditionData = "True"))
	FString PipelineDisplayName;

	/** If enabled, imports all sounds found in the source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Audio")
	bool bImportSounds = true;

	virtual bool IsScripted() override
	{
		return false;
	}

public:
	UE_API virtual void AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams) override;
	UE_API virtual void ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath) override;

#if WITH_EDITOR
	UE_API virtual bool IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const override;
	
	UE_API virtual void FilterPropertiesFromTranslatedData(UInterchangeBaseNodeContainer* InBaseNodeContainer) override;

	UE_API virtual void GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const override;
#endif //WITH_EDITOR

protected:
	virtual bool CanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask) override
	{
		return false;
	}

	/**
	 * Creates a SoundWaveFactoryNode for a given SoundWaveNode, if one doesn't already exist.
	 * Initializes the factory node, sets the custom directory path, and sets the target nodes
	 * on both the factory and sound wave nodes.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange")
	UE_API UInterchangeAudioSoundWaveFactoryNode* CreateSoundWaveFactoryNode(const UInterchangeAudioSoundWaveNode* SoundWaveNode);

private:
	UPROPERTY(Transient)
	TObjectPtr<UInterchangeBaseNodeContainer> BaseNodeContainer;

	UPROPERTY(Transient)
	TArray<const TObjectPtr<UInterchangeSourceData>> SourceDatas;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UInterchangeAudioSoundWaveNode>> SoundWaveNodes;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UInterchangeAudioSoundWaveFactoryNode>> SoundWaveFactoryNodes;
};

#undef UE_API
