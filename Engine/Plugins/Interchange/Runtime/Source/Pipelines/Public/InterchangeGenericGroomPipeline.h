// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangePipelineBase.h"

#include "GroomAssetInterpolation.h"
#include "InterchangeGroomCacheFactoryNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeGenericGroomPipeline.generated.h"

#define UE_API INTERCHANGEPIPELINES_API

UCLASS(MinimalAPI, BlueprintType, editinlinenew)
class UInterchangeGenericGroomPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

public:
	static UE_API FString GetPipelineCategory(UClass* AssetClass);

	/** The name of the pipeline that will be displayed in the import dialog. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grooms", meta = (StandAlonePipelineProperty = "True", PipelineInternalEditionData = "True"))
	FString PipelineDisplayName;

	/** If enabled, allow the import of groom-type assets. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grooms")
	bool bEnableGroomTypesImport = false;

	/** If enabled, import all groom assets found in the source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grooms", meta = (EditCondition = bEnableGroomTypesImport))
	bool bImportGrooms = true;

	/** If enabled, import all groom bindings found in the source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grooms", meta = (EditCondition = bEnableGroomTypesImport))
	bool bImportGroomBindings = true;

	/** Settings that will be applied to all hair groups in the groom asset */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grooms")
	FHairGroupsInterpolation GroupInterpolationSettings;

	/** If enabled, import all groom caches found in the source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grooms", meta = (SubCategory = "Caches", EditCondition = bEnableGroomTypesImport))
	bool bImportGroomCaches = true;

	/** If the groom asset is not imported, provide an existing one against which the groom cache will be validated */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grooms", meta = (SubCategory = "Caches", EditCondition = "bImportGrooms == false", MetaClass = "/Script/HairStrandsCore.GroomAsset"))
	FSoftObjectPath GroomAsset;

	/** Groom Cache types to import */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grooms", meta = (SubCategory = "Caches", EditCondition = bImportGroomCaches))
	EInterchangeGroomCacheImportType ImportGroomCacheType = EInterchangeGroomCacheImportType::All;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grooms", meta = (SubCategory = "Caches"))
	bool bOverrideTimeRange = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grooms", meta = (SubCategory = "Caches", EditCondition = bOverrideTimeRange))
	int32 FrameStart = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grooms", meta = (SubCategory = "Caches", EditCondition = bOverrideTimeRange))
	int32 FrameEnd = 1;

	/** Number of points used for the rbf interpolation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Grooms", meta = (SubCategory = "Bindings", ClampMin = "0", ClampMax = "100"))
	int32 NumInterpolationPoints = 100;

protected:

	UE_API virtual void ExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath) override;

public:
#if WITH_EDITOR
	UE_API virtual void GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const override;
	UE_API virtual void FilterPropertiesFromTranslatedData(UInterchangeBaseNodeContainer* InBaseNodeContainer) override;
	UE_API virtual bool IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const override;
#endif

	UE_API virtual bool IsSettingsAreValid(TOptional<FText>& OutInvalidReason) const override;
};

#undef UE_API