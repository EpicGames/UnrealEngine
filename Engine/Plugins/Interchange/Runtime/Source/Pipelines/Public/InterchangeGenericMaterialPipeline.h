// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeCustomVersion.h"
#include "InterchangePipelineBase.h"
#include "InterchangeSourceData.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeGenericMaterialPipeline.generated.h"

#define UE_API INTERCHANGEPIPELINES_API

class UInterchangeBaseMaterialFactoryNode;
class UInterchangeFunctionCallShaderNode;
class UInterchangeGenericTexturePipeline;
class UInterchangeMaterialExpressionFactoryNode;
class UInterchangeMaterialFactoryNode;
class UInterchangeMaterialFunctionFactoryNode;
class UInterchangeMaterialInstanceFactoryNode;
class UInterchangeResult;
class UInterchangeShaderGraphNode;
class UInterchangeShaderNode;
class UInterchangeSparseVolumeTexturePipeline;
class UInterchangeSpecularProfileNode;
class UInterchangeTextureNode;
class UMaterialFunction;

namespace UE::Interchange::Materials::HashUtils
{
	class FDuplicateMaterialHelper;
}

UENUM(BlueprintType)
enum class EInterchangeMaterialImportOption : uint8
{
	/** Import all materials from the source as material assets. */
	ImportAsMaterials,
	/** Import all materials from the source as material instance assets. */
	ImportAsMaterialInstances,
};

UENUM(BlueprintType)
enum class EInterchangeMaterialSearchLocation : uint8
{
	/** Search for existing material in local import folder only. */
	Local,
	/** Search for existing material recursively from parent folder. */
	UnderParent,
	/** Search for existing material recursively from root folder. */
	UnderRoot,
	/** Search for existing material in all assets folders. */
	AllAssets,
	/** Do not search for existing existing materials. Deprecated replaced with boolean bReuseExistingMaterials */
	DoNotSearch  UE_DEPRECATED(5.8, "Moved as a boolean, bReuseExistingMaterials in UInterchangeGenericMaterialPipeline") UMETA(Hidden, Deprecated),
};

UCLASS(MinimalAPI, BlueprintType, editinlinenew)
class UInterchangeGenericMaterialPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

public:
	UE_API UInterchangeGenericMaterialPipeline();

	// Begin UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	// End UObject interface

	static UE_API FString GetPipelineCategory(UClass* AssetClass);

	/** The name of the pipeline that will be display in the import dialog. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Materials", Meta = (StandAlonePipelineProperty = "True", PipelineInternalEditionData = "True"))
	FString PipelineDisplayName;

	/** If enabled, and there is only one asset and one source, the imported asset will be given this name. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Materials", Meta = (StandAlonePipelineProperty = "True"))
	FString AssetName;

	/** If enabled, imports the material assets found in the sources. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Materials")
	bool bImportMaterials = true;

	/**
	 * When enabled, if the name of a material in the imported source matches a material
	 * that already exists in the project, the imported assets will reuse that existing material.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Materials", Meta = (EditCondition = "bImportMaterials"))
	bool bReuseExistingMaterials = true;

	/** Determines where to look for matching materials.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Materials", Meta = (EditCondition = "bImportMaterials && bReuseExistingMaterials"))
	EInterchangeMaterialSearchLocation SearchLocation = EInterchangeMaterialSearchLocation::Local;

	/**
	 * When enabled, a new material asset will be created for each material
	 * in the imported source that has not been matched to an existing material.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Materials", Meta = (EditCondition = "bImportMaterials"))
	bool bCreateNewMaterials = true;

	/** Determines which kind of material asset to create for all new materials. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Materials", Meta=(DisplayName="New Material Asset Type", EditCondition = "bImportMaterials && bCreateNewMaterials"))
	EInterchangeMaterialImportOption MaterialImport = EInterchangeMaterialImportOption::ImportAsMaterials;

	/** If enabled, material instances are created from a specified parent material. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Materials", Meta=(DisplayName = "Create MaterialInstance From Parent", EditCondition="bImportMaterials && bCreateNewMaterials && MaterialImport==EInterchangeMaterialImportOption::ImportAsMaterialInstances"))
	bool bCreateMaterialInstanceForParent = false;

	/** Optional material used as the parent when importing materials as instances. If no parent material is specified, one will be automatically selected during the import process. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Materials", Meta= (EditCondition="bImportMaterials && bCreateNewMaterials && MaterialImport==EInterchangeMaterialImportOption::ImportAsMaterialInstances && bCreateMaterialInstanceForParent", AllowedClasses="/Script/Engine.MaterialInterface"))
	FSoftObjectPath ParentMaterial;

	/**
	 * If enabled and duplicated materials are detected, for each set of duplicated materials,
	 * a reference material along with respective material instances are created.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Materials", Meta = (EditCondition = "bImportMaterials"))
	bool bIdentifyDuplicateMaterials = false;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Instanced, Category = "Textures")
	TObjectPtr<UInterchangeGenericTexturePipeline> TexturePipeline;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Instanced, Category = "Volumes")
	TObjectPtr<UInterchangeSparseVolumeTexturePipeline> SparseVolumeTexturePipeline;

	/**
	 * If enabled, it will override the displacement center set by shader graph nodes, if any
	 * Note: Some MaterialX Surface may have their preferred displacement center specified.
	 *       In that case, enabling this option may affect the rendering result. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Materials", Meta = (DisplayName = "Override Displacement Center", EditCondition = "bImportMaterials"))
	bool bOverrideDisplacement = false;

	/** Set the value of the displacement center. If enabled it will also override any displacement center value set by shader graph nodes*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Materials", Meta = (EditCondition = "bOverrideDisplacement && bImportMaterials", ClampMin = "0", DisplayName = "Displacement Center"))
	float OverrideDisplacementCenter = 0.5f;

	/** BEGIN UInterchangePipelineBase overrides */
	UE_API virtual void PreDialogCleanup(const FName PipelineStackName) override;
	UE_API virtual bool IsSettingsAreValid(TOptional<FText>& OutInvalidReason) const override;
	UE_API virtual void AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams) override;
#if WITH_EDITOR
	UE_API virtual void FilterPropertiesFromTranslatedData(UInterchangeBaseNodeContainer* InBaseNodeContainer) override;
	UE_API virtual bool IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const override;

	UE_API virtual void GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const override;
#endif //WITH_EDITOR

protected:
	UE_API virtual void ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath) override;
	UE_API virtual void ExecutePostFactoryPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport) override;
	UE_API virtual void ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport) override;
	UE_API virtual void SetReimportSourceIndex(UClass* ReimportObjectClass, const int32 SourceFileIndex) override;
	/** END UInterchangePipelineBase overrides */

	UPROPERTY()
	TObjectPtr<UInterchangeBaseNodeContainer> BaseNodeContainer;

	TArray<const UInterchangeSourceData*> SourceDatas;
	
private:
	UE_API UInterchangeBaseMaterialFactoryNode* CreateBaseMaterialFactoryNode(const UInterchangeBaseNode* MaterialNode, TSubclassOf<UInterchangeBaseMaterialFactoryNode> NodeType, bool bAddMaterialInstanceSuffix = false);
	UE_API UInterchangeMaterialFactoryNode* CreateMaterialFactoryNode(const UInterchangeShaderGraphNode* ShaderGraphNode);
	UE_API UInterchangeMaterialFunctionFactoryNode* CreateMaterialFunctionFactoryNode(const UInterchangeShaderGraphNode* FunctionCallShaderNode);
	UE_API UInterchangeMaterialInstanceFactoryNode* CreateMaterialInstanceFactoryNode(const UInterchangeShaderGraphNode* ShaderGraphNode);
	UE_API void CreateSpecularProfileFactoryNode(const UInterchangeSpecularProfileNode* SpecularProfileNode);
	
	/** True if the shader graph has a clear coat input. */
	UE_API bool HasClearCoat(const UInterchangeShaderGraphNode* ShaderGraphNode) const;
	UE_DEPRECATED(5.3, "Deprecated. Use HasClearCoat.")
	bool IsClearCoatModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const
	{
		return HasClearCoat(ShaderGraphNode);
	}

	/** True if the shader graph has a sheen color input. */
	UE_API bool HasSheen(const UInterchangeShaderGraphNode* ShaderGraphNode) const;
	UE_DEPRECATED(5.3, "Deprecated. Use HasSheen.")
	bool IsSheenModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const
	{
		return HasSheen(ShaderGraphNode);
	}

	/** True if the shader graph has a subsurface color input. */
	UE_API bool HasSubsurface(const UInterchangeShaderGraphNode* ShaderGraphNode) const;
	UE_DEPRECATED(5.3, "Deprecated. Use HasSubsurface.")
	bool IsSubsurfaceModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const
	{
		return HasSubsurface(ShaderGraphNode);
	}

	/** True if the shader graph has a transmission color input. */
	UE_API bool HasThinTranslucency(const UInterchangeShaderGraphNode* ShaderGraphNode) const;
	UE_DEPRECATED(5.3, "Deprecated. Use HasThinTranslucency.")
	bool IsThinTranslucentModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const
	{
		return HasThinTranslucency(ShaderGraphNode);
	}

	/** True if the shader graph has a base color input (Metallic/Roughness model). */
	UE_API bool IsMetalRoughModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const;
	UE_DEPRECATED(5.3, "Deprecated. Use IsMetalRoughModel and IsSpecGlossModel to identify the correct PBR model.")
	bool IsPBRModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const
	{
		return IsMetalRoughModel(ShaderGraphNode);
	}

	/** True if the shader graph has diffuse color and specular inputs. */
	UE_API bool IsPhongModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const;

	/** True if the shader graph has a diffuse color input. */
	UE_API bool IsLambertModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const;

	/** True if the shader graph has the surface unlit's shader type name. */
	UE_API bool IsSurfaceUnlitModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const;

	/** True if the shader graph has an unlit color input. */
	UE_API bool IsUnlitModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const;

	/** True if the shader graph has specular color and glossiness scalar inputs. */
	UE_API bool IsSpecGlossModel(const UInterchangeShaderGraphNode* ShaderGraphNode) const;

	UE_API bool HandlePhongModel(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode);
	UE_API bool HandleLambertModel(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode);
	UE_API bool HandleMetalRoughnessModel(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode);
	UE_API bool HandleClearCoat(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode);
	UE_API bool HandleSubsurface(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode);
	UE_API bool HandleSheen(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode);
	UE_API bool HandleThinTranslucent(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode);
	UE_API void HandleCommonParameters(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode);
	UE_API bool HandleBxDFInput(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode);
	UE_API bool HandleUnlitModel(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode);
	UE_API bool HandleSpecGlossModel(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode);
	UE_API bool HandleSubstrate(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialFactoryNode* MaterialFactoryNode);

	UE_API void HandleFlattenNormalNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* FlattenNormalFactoryNode);
	UE_API void HandleNormalFromHeightMapNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* NormalFromHeightMapFactoryNode);
	UE_API void HandleMakeFloat3Node(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* MakeFloat3FactoryNode);
	UE_API void HandleTextureNode(const UInterchangeTextureNode* TextureNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TextureBaseFactoryNode, const FString& ExpressionClassName, bool bIsAParameter);
	UE_API void HandleTextureObjectNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TextureObjectFactoryNode);
	UE_API void HandleTextureSampleNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TextureSampleFactoryNode);
	UE_API void HandleTextureSampleBlurNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TextureSampleFactoryNode);
	UE_API void HandleTextureCoordinateNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode*& TextureSampleFactoryNode);
	UE_API void HandleLerpNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* LerpFactoryNode);
	UE_API void HandleMaskNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* MaskFactoryNode);
	UE_API void HandleRotatorNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* RotatorFactoryNode);
	UE_API void HandleRotateAboutAxisNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* RotateAboutAxisFactoryNode);
	UE_API void HandleTimeNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TimeFactoryNode);
	UE_API void HandleTransformPositionNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TransformPositionFactoryNode);
	UE_API void HandleTransformVectorNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TransformVectorFactoryNode);
	UE_API void HandleNoiseNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* NoiseFactoryNode);
	UE_API void HandleVectorNoiseNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* NoiseFactoryNode);
	UE_API void HandleSwitchNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* SwitchFactoryNode);
	UE_API void HandleSlabBSDFNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* SlabBSDFFactoryNode);
    UE_API void HandleLuminanceNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* LuminanceFactoryNode);
    UE_API void HandleTexturePropertyNode(const UInterchangeShaderNode* ShaderNode, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TexturePropertyFactoryNode);
	UE_API void HandleTrigonometryNode(const UInterchangeShaderNode* ShaderNode, UClass* StaticClass, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, UInterchangeMaterialExpressionFactoryNode* TrigonometryFactoryNode);

	UE_API void HandleScalarParameterNode(const UInterchangeShaderNode* ShaderNode, UInterchangeMaterialExpressionFactoryNode* ScalarParameterFactoryNode);
	UE_API void HandleVectorParameterNode(const UInterchangeShaderNode* ShaderNode, UInterchangeMaterialExpressionFactoryNode* VectorParameterFactoryNode);
	UE_API void HandleStaticBooleanParameterNode(const UInterchangeShaderNode* ShaderNode, UInterchangeMaterialExpressionFactoryNode* StaticBooleanParameterFactoryNode);

	UE_API UInterchangeMaterialExpressionFactoryNode* CreateMaterialExpressionForShaderNode(UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, const UInterchangeShaderNode* ShaderNode, const FString& ParentUid);
	UE_API TTuple<UInterchangeMaterialExpressionFactoryNode*, FString> CreateMaterialExpressionForInput(UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode, const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid);

	UE_API UInterchangeMaterialExpressionFactoryNode* CreateExpressionNode(const FString& ExpressionName, const FString& ParentUid, UClass* MaterialExpressionClass);
	
	UE_API UInterchangeMaterialExpressionFactoryNode* HandleFloatInput(const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid, bool bIsAParameter);
	UE_API UInterchangeMaterialExpressionFactoryNode* CreateConstantExpression(const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid);
	UE_API UInterchangeMaterialExpressionFactoryNode* CreateScalarParameterExpression(const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid);
	UE_API UInterchangeMaterialExpressionFactoryNode* HandleLinearColorInput(const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid, bool bIsAParameter);
	UE_API UInterchangeMaterialExpressionFactoryNode* CreateConstant3VectorExpression(const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid);
	UE_API UInterchangeMaterialExpressionFactoryNode* CreateVectorParameterExpression(const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid);
	UE_API UInterchangeMaterialExpressionFactoryNode* CreateStaticBooleanParameterExpression(const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid);
	UE_API UInterchangeMaterialExpressionFactoryNode* CreateVector2ParameterExpression(const UInterchangeShaderNode* ShaderNode, const FString& InputName, const FString& ParentUid);
	UE_API UInterchangeMaterialExpressionFactoryNode* CreateFunctionCallExpression(const UInterchangeShaderNode* ShaderNode, const FString& MaterialExpressionUid, UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode);

	/**
	 * Visits a given shader node and its connections to find its strongest value.
	 * The goal is to simplify a branch of a node graph to a single value, to be used for material instancing.
	 */
	UE_API void VisitShaderGraphNode(const UInterchangeShaderGraphNode* ShaderGraphNode, UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode) const;
	UE_API void VisitShaderNode(const UInterchangeShaderNode* ShaderNode, UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode, TSet<const UInterchangeShaderNode*> & VisitedNodes) const;
	UE_API void VisitShaderInput(const UInterchangeShaderNode* ShaderNode, UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode, const FString& InputName, TSet<const UInterchangeShaderNode*> & VisitedNodes) const;

	UE_API void VisitScalarParameterNode(const UInterchangeShaderNode* ShaderNode, UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode) const;
	UE_API void VisitTextureSampleNode(const UInterchangeShaderNode* ShaderNode, UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode) const;
	UE_API void VisitVectorParameterNode(const UInterchangeShaderNode* ShaderNode, UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode) const;

	UE_API FString GetTextureUidAttributeFromShaderNode(const UInterchangeShaderNode* ShaderNode, FName ParameterName, bool& OutIsAParameter) const;
	UE_API FString CreateInputKey(const FString& InputName, bool bIsAParameter) const;
private:
	friend class UE::Interchange::Materials::HashUtils::FDuplicateMaterialHelper;

	enum class EMaterialInputType : uint8
	{
		Unknown,
		Color,
		Vector,
		Scalar
	};
	friend FString LexToString(UInterchangeGenericMaterialPipeline::EMaterialInputType);

	struct FMaterialCreationContext
	{
		EMaterialInputType InputTypeBeingProcessed = EMaterialInputType::Color;
	} MaterialCreationContext;

	struct FMaterialExpressionCreationContext
	{
		FString OutputName; // The name of the output we will be connecting from
	};

	TArray<FMaterialExpressionCreationContext> MaterialExpressionCreationContextStack;

	using FParameterMaterialInputType = TTuple<FString, EMaterialInputType>;

	const UInterchangeBaseNode * AttributeStorageNode = nullptr;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void UpdateSearchLocationProperty()
	{
		// Only process DoNotSearch case since bReuseExistingMaterials is set to true by default
		if (SearchLocation == EInterchangeMaterialSearchLocation::DoNotSearch)
		{
			bReuseExistingMaterials = false;
			SearchLocation = EInterchangeMaterialSearchLocation::Local;
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

#undef UE_API
