// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeGenericAssetsPipelineSharedSettings.h"
#include "InterchangeMeshDefinitions.h"
#include "InterchangePipelineBase.h"
#include "InterchangeSourceData.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "InterchangeSkeletalMeshFactoryNode.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeGenericMeshPipeline.generated.h"

#define UE_API INTERCHANGEPIPELINES_API

class UInterchangeGenericAssetsPipeline;
class UInterchangeGeometryCacheFactoryNode;
class UInterchangeMeshNode;
class UInterchangePipelineMeshesUtilities;
class UInterchangeSceneNode;
class UInterchangeSkeletalMeshFactoryNode;
class UInterchangeSkeletalMeshLodDataNode;
class UInterchangeSkeletonFactoryNode;
class UInterchangeStaticMeshFactoryNode;
class UInterchangeStaticMeshLodDataNode;
class UPhysicsAsset;
class UInterchangeMeshLODContainerNode;

UENUM(BlueprintType)
enum class EInterchangeCombineStaticMeshesBehavior : uint8
{
	All UMETA(DisplayName="Combine All", Tooltip="Combine all the meshes into a single Static Mesh"),
	
	VisibleOnly UMETA(DisplayName = "Combine Visible Meshes Only", Tooltip="Skip adding the hidden meshes to the combined Static Mesh"),

	DoNotCombine UMETA(DisplayName = "Do Not Combine Static Meshes", Tooltip = "All meshes will be imported as individual Static Mesh assets")
};

UENUM(BlueprintType)
enum class EInterchangeCombineSkeletalMeshesBehavior : uint8
{
	BySkeleton UMETA(DisplayName="Combine All That Use Same Skeleton", Tooltip="Combine all the meshes that use a specific Skeleton as a single Skeletal Mesh"),
	
	BySkeletonVisibleOnly UMETA(DisplayName = "Combine All Visible That Use Same Skeleton", Tooltip="Combine all visible meshes that use a specific Skeleton as a single Skeletal Mesh"),

	DoNotCombine UMETA(DisplayName = "Do Not Combine Skeletal Meshes", Tooltip = "All meshes will be imported as individual Skeletal Mesh assets")
};

/* Hide drop down will make sure the class is not showing in the class picker */
UCLASS(MinimalAPI, BlueprintType, hidedropdown)
class UInterchangeGenericMeshPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

public:
	//IInterchangeGenericPipelineCategoryInterface
	static UE_API FString GetPipelineCategory(UClass* AssetClass);

	//Common Meshes Properties Settings Pointer
	UPROPERTY(Transient)
	TWeakObjectPtr<UInterchangeGenericCommonMeshesProperties> CommonMeshesProperties;

	//Common SkeletalMeshes And Animations Properties Settings Pointer
	UPROPERTY(Transient)
	TWeakObjectPtr<UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties> CommonSkeletalMeshesAndAnimationsProperties;
	
	//////	STATIC_MESHES_CATEGORY Properties //////

	/** If enabled, imports all static mesh assets found in the sources. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes")
	bool bImportStaticMeshes = true;

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage = "bCombineStaticMeshes is no longer used. Use CombineStaticMeshesBehavior"))
	bool bCombineStaticMeshes_DEPRECATED = false;

	/** Specifies the behavior for combining static meshes into a single static mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta=(DisplayName = "Combine Static Meshes"))
	EInterchangeCombineStaticMeshesBehavior CombineStaticMeshesBehavior = EInterchangeCombineStaticMeshesBehavior::DoNotCombine;

	/** The LOD group that will be assigned to this mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta=(DisplayName = "LOD Group"), meta = (ReimportRestrict = "true"))
	FName LodGroup = NAME_None;

	/** If enabled, LOD Screen Sizes would be auto-computed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta=(ReimportRestrict = "true"))
	bool bAutoComputeLODScreenSizes = true;

	/** This setting is only used if the Auto Compute LOD Screen Sizes setting is disabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta=(EditCondition = "!bAutoComputeLODScreenSizes", DisplayName = "LOD Screen Sizes", ReimportRestrict="true"))
	TArray<float> LODScreenSizes;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Collision instead."))
	bool bImportCollision_DEPRECATED = true;

	/** If enabled, custom collision will be imported. If enabled and there is no custom collision, a generic collision will be automatically generated.
	 * If disabled, no collision will be created or imported.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (DisplayName = "Import Collisions", SubCategory = "Collision"))
	bool bCollision = true;

	/**
	 * If enabled, meshes with certain prefixes will be imported as collision primitives for the mesh with the corresponding unprefixed name.
	 * 
	 * Supported prefixes are:
	 * UBX_ Box collision
	 * UCP_ Capsule collision
	 * USP_ Sphere collision
	 * UCX_ Convex collision
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (DisplayName = "Import Collisions According To Mesh Name", SubCategory = "Collision", editcondition = "bCollision == true"))
	bool bImportCollisionAccordingToMeshName = true;

	/** If enabled, each UCX collision mesh will be imported as a single convex hull. If disabled, a UCX mesh will be decomposed into its separate pieces and a convex hull generated for each. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (SubCategory = "Collision", editcondition = "bCollision == true && bImportCollisionAccordingToMeshName"))
	bool bOneConvexHullPerUCX = true;

	/** Type used to generate a collision when no custom collisions are present in the file. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (ScriptName = "FallbackCollisionType", DisplayName = "Fallback Collision Type", SubCategory = "Collision", editcondition = "bCollision == true"))
	EInterchangeMeshCollision Collision = EInterchangeMeshCollision::Convex18DOP;
	
	/** Sets whether to generate collision shapes even if the provided mesh data doesn't match the requested collision shape very well */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (SubCategory = "Collision", editcondition = "bCollision == true", PipelineInternalEditionData = "True"))
	bool bForceCollisionPrimitiveGeneration = false;

	//////	Static Meshes Build settings Properties //////

	/**
	 * If enabled, imported meshes will be rendered by Nanite at runtime. Make sure your meshes and materials meet the requirements for Nanite.
	 * See also NaniteTriangleThreshold
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (SubCategory = "Build"))
	bool bBuildNanite = true;

	/**
	 * Minimum triangle count a mesh needs to have in order to get Nanite enabled for it when BuildNanite is enabled.
	 * When zero it means Nanite will always be enabled for all meshes when BuildNanite is enabled.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (SubCategory = "Build", EditCondition = "bBuildNanite"))
	int64 NaniteTriangleThreshold = 0;

	/** If enabled, builds a reversed index buffer for each static mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (SubCategory = "Build"))
	bool bBuildReversedIndexBuffer = false;
	
	/** If enabled, generates lightmap UVs for each static mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (SubCategory = "Build"))
	bool bGenerateLightmapUVs = false;
	
	/** 
	 * Determines whether to generate the distance field treating every triangle hit as a front face.  
	 * When enabled, prevents the distance field from being discarded due to the mesh being open, but also lowers distance field ambient occlusion quality.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta=(SubCategory = "Build", DisplayName="Two-Sided Distance Field Generation"))
	bool bGenerateDistanceFieldAsIfTwoSided = false;
	
	/* If enabled, imported static meshes are set up for use with physical material masks. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (SubCategory = "Build", DisplayName="Enable Physical Material Mask"))
	bool bSupportFaceRemap = false;
	
	/* When generating lightmaps, determines the amount of padding used to pack UVs. Set this value to the lowest-resolution lightmap you expect to use with the imported meshes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (SubCategory = "Build"))
	int32 MinLightmapResolution = 64;
	
	/* Specifies the index of the UV channel that will be used as the source when generating lightmaps. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (SubCategory = "Build", DisplayName="Source Lightmap Index"))
	int32 SrcLightmapIndex = 0;
	
	/* Specifies the index of the UV channel that will store generated lightmap UVs. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (SubCategory = "Build", DisplayName="Destination Lightmap Index"))
	int32 DstLightmapIndex = 1;
	
	/** The local scale applied when building the mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (SubCategory = "Build", DisplayName="Build Scale"))
	FVector BuildScale3D = FVector(1.0);
	
	/** 
	 * Scale to apply to the mesh when allocating the distance field volume texture.
	 * The default scale is 1, which assumes that the mesh will be placed unscaled in the world.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (SubCategory = "Build"))
	float DistanceFieldResolutionScale = 1.0f;
	
	/**
	 * If set, replaces the distance field for all imported meshes with the distance field of the specified Static Mesh.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (SubCategory = "Build"))
	TSoftObjectPtr<class UStaticMesh> DistanceFieldReplacementMesh = nullptr;
	
	/** 
	 * The maximum number of Lumen mesh cards to generate for this mesh.
	 * More cards means that the surface will have better coverage, but will result in increased runtime overhead.
	 * Set this to 0 to disable mesh card generation for this mesh.
	 * The default is 12.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Meshes", meta = (SubCategory = "Build"))
	int32 MaxLumenMeshCards = 12;

	//////	SKELETAL_MESHES_CATEGORY Properties //////

	/** If enabled, imports all skeletal mesh assets found in the sources. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Skeletal Meshes")
	bool bImportSkeletalMeshes = true;

	/** Determines what types of information are imported for skeletal meshes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Skeletal Meshes", meta = (DisplayName = "Import Content Type"))
	EInterchangeSkeletalMeshContentType SkeletalMeshImportContentType;
	
	/** The value of the content type during the last import. This cannot be edited and is set only on successful import or reimport. */
	UPROPERTY()
	EInterchangeSkeletalMeshContentType LastSkeletalMeshImportContentType;

	/** Specifies the behavior for combining skeletal meshes into a single skeletal mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Skeletal Meshes", meta=(DisplayName = "Combine Skeletal Meshes"))
	EInterchangeCombineSkeletalMeshesBehavior CombineSkeletalMeshesBehavior = EInterchangeCombineSkeletalMeshesBehavior::BySkeleton;

	/** If enabled, imports all morph target shapes found in the source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Skeletal Meshes")
	bool bImportMorphTargets = true;

	/** If enabled, all morph target shapes with the same name will be merge together. Turn it to false if you want to control those morph with different values. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Skeletal Meshes", meta = (EditCondition = "bImportMorphTargets"))
	bool bMergeMorphTargetsWithSameName = true;

	/** If enabled, imports per-vertex attributes from the FBX file. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Skeletal Meshes", meta = (ToolTip = "If enabled, creates named vertex attributes for secondary vertex color data."))
	bool bImportVertexAttributes = false;

	/** Enable this option to update the reference pose of the Skeleton (of the mesh). The reference pose of the mesh is always updated.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Skeletal Meshes")
	bool bUpdateSkeletonReferencePose = false;

	/** If enabled, create new PhysicsAsset if one doesn't exist. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Skeletal Meshes")
	bool bCreatePhysicsAsset = true;

	/** If set, use the specified PhysicsAsset. If not set and the Create Physics Asset setting is not enabled, the importer will not generate or set any physics asset. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Skeletal Meshes", meta = (editcondition = "!bCreatePhysicsAsset"))
	TSoftObjectPtr<UPhysicsAsset> PhysicsAsset;

	/** If enabled, imported skin weights use 16 bits instead of 8 bits. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Skeletal Meshes", meta = (SubCategory = "Build"))
	bool bUseHighPrecisionSkinWeights = false;

	/** Threshold value that is used to decide whether two vertex positions are equal. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Skeletal Meshes", meta = (SubCategory = "Build"))
	float ThresholdPosition = 0.00002f;
	
	/** Threshold value that is used to decide whether two normals, tangents, or bi-normals are equal. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Skeletal Meshes", meta = (SubCategory = "Build"))
	float ThresholdTangentNormal = 0.00002f;
	
	/** Threshold value that is used to decide whether two UVs are equal. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Skeletal Meshes", meta = (SubCategory = "Build"))
	float ThresholdUV = 0.0009765625f;
	
	/** Threshold to compare vertex position equality when computing morph target deltas. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Skeletal Meshes", meta = (SubCategory = "Build"))
	float MorphThresholdPosition = 0.015f;

	/**
	 * The maximum number of bone influences to allow each vertex in this mesh to use.
	 * If set higher than the limit determined by the project settings, it has no effect.
	 * If set to 0, the value is taken from the DefaultBoneInfluenceLimit project setting.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Skeletal Meshes", meta = (SubCategory = "Build"))
	int32 BoneInfluenceLimit = 0;

	//////	GEOMETRY_CACHES_CATEGORY Properties //////

	/** If enabled, imports all geometry cache assets found in the sources. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Geometry Caches")
	bool bImportGeometryCaches = true;

	/** Whether or not to merge all vertex animation into one track */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Geometry Caches")
	bool bFlattenTracks = true;

	/** Precision used for compressing vertex positions (lower = better result but less compression, higher = more lossy compression but smaller size) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Geometry Caches", meta = (ClampMin = "0.000001", ClampMax = "1000", UIMin = "0.0001", UIMax = "10"))
	float CompressedPositionPrecision = 0.01f;

	/** Bit-precision used for compressing texture coordinates (hight = better result but less compression, lower = more lossy compression but smaller size) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Geometry Caches", meta = (ClampMin = "1", ClampMax = "31", UIMin = "4", UIMax = "16"))
	int32 CompressedTextureCoordinatesNumberOfBits = 10;

	/** If enabled, override the imported animation range. Otherwise, the imported range is automatically set to the range of non-empty animated frames */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Geometry Caches", meta = (SubCategory = "Sampling"))
	bool bOverrideTimeRange = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Geometry Caches", meta = (SubCategory = "Sampling", editcondition = "bOverrideTimeRange == true"))
	int32 FrameStart = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Geometry Caches", meta = (SubCategory = "Sampling", editcondition = "bOverrideTimeRange == true"))
	int32 FrameEnd = 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Geometry Caches", AdvancedDisplay)
	EInterchangeMotionVectorsHandling MotionVectors = EInterchangeMotionVectorsHandling::NoMotionVectors;

	/**
	 * Force the preprocessor to only do optimization once instead of when the preprocessor decides. This may lead to some problems with certain meshes but makes sure motion
	 * blur always works if the topology is constant. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Geometry Caches", AdvancedDisplay)
	bool bApplyConstantTopologyOptimizations = false;

	/**
	 * Store the imported vertex numbers. This lets you know the vertex numbers inside the DCC.
	 * The values of each vertex number will range from 0 to 7 for a cube. Even if the number of positions might be 24.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Geometry Caches", AdvancedDisplay)
	bool bStoreImportedVertexNumbers = false;

	/** Optimizes index buffers for each unique frame, to allow better cache coherency on the GPU. Very costly and time-consuming process, recommended to OFF. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Geometry Caches", AdvancedDisplay)
	bool bOptimizeIndexBuffers = false;

	UE_API virtual void AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams) override;

	UE_API virtual void PreDialogCleanup(const FName PipelineStackName) override;

#if WITH_EDITOR
	UE_API virtual bool IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const override;
	UE_API virtual bool GetPropertyPossibleValues(const FName PropertyPath, TArray<FString>& PossibleValues) override;

	UE_API virtual void GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const override;
#endif

	static UE_API UInterchangePipelineMeshesUtilities* CreateMeshPipelineUtilities(UInterchangeBaseNodeContainer* InBaseNodeContainer
		, const UInterchangeGenericMeshPipeline* Pipeline);

protected:
	UE_API virtual void ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath) override;

	UE_API virtual void ExecutePostFactoryPipeline(const UInterchangeBaseNodeContainer* InBaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport) override;

	UE_API virtual void ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* InBaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport) override;

	virtual bool CanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask) override
	{
		return true;
	}

	UE_API virtual void SetReimportSourceIndex(UClass* ReimportObjectClass, const int32 SourceFileIndex) override;

#if WITH_EDITOR
	/**
	 * This function return true if all UPROPERTYs of the @Struct exist in the provided @Classes.
	 * @Struct UPROPERTY tested must be: not transient, editable
	 * 
	 * @param Classes - The array of UClass that should contains the Struct properties
	 * @param Struct - The struct that has the referenced properties
	 * 
	 */
	static UE_API bool DoClassesIncludeAllEditableStructProperties(const TArray<const UClass*>& Classes, const UStruct* Struct);
#endif

private:

	/* Meshes utilities, to parse the translated graph and extract the meshes informations. */
	TObjectPtr<UInterchangePipelineMeshesUtilities> PipelineMeshesUtilities = nullptr;

	/** Table to access the corresponding factory node for the given mesh uid*/
	TMap<FString, UInterchangeMeshFactoryNode*> AssemblyPartMeshUidToFactoryNodeTable;

	/**
	* Update the MeshUid to MeshFactoryNode table with the given node uids
	* @param NodeUidsPerLodIndex - The NodeUids can be a UInterchangeSceneNode or a UInterchangeMeshNode.
	*/
	UE_API void UpdateAssemblyPartDependencyTable(UInterchangeMeshFactoryNode* MeshFactoryNode, const TMap<int32, TArray<FString>>& NodeUidsPerLodIndex);

	/**
	* Creates the final dependencies between nanite assembly parts
	*/
	UE_API void CreateAssemblyPartDependencies();

	/************************************************************************/
	/* Skeletal mesh API BEGIN                                              */

	/**
	 * This function will create any skeletalmesh we need to create according to the pipeline options
	 */
	UE_API void ExecutePreImportPipelineSkeletalMesh();

	/*
	* This function will create the SkeletalMeshFactoryNode and all necessary UInterchangeSkeletalMeshLodDataNode nodes.
	*	It will also process/parse all input nodes into the created UInterchangeSkeletalMeshLodDataNode nodes, and will also generate the Transforms that should be used for Mesh acquisitions.
	* @param NodeUids - NodeUid Can be Mesh(Mesh/LODContainer/Bundle) or MeshInstance (SceneNode instantiating a Mesh).
	*/
	UE_API UInterchangeSkeletalMeshFactoryNode* CreateSkeletalMeshFactoryNode(const FString& RootJointUid, const TArray<FString>& NodeUids);

	/**
	 * This function will finish creating the skeletalmesh asset
	 */
	UE_API void PostImportSkeletalMesh(UObject* CreatedAsset, const UInterchangeFactoryBaseNode* FactoryNode);

	/**
	 * This function will finish creating the physics asset with the skeletalmesh render data
	 */
	UE_API void PostImportPhysicsAssetImport(UObject* CreatedAsset, const UInterchangeFactoryBaseNode* FactoryNode);
public:
	
	/** Specialize for skeletalmesh */
	UE_API void ImplementUseSourceNameForAssetOptionSkeletalMesh(const int32 MeshesImportedNodeCount, const bool bUseSourceNameForAsset, const FString& AssetName);

private:
	/* Skeletal mesh API END                                                */
	/************************************************************************/


	/************************************************************************/
	/* Static mesh API BEGIN                                              */

	/**
	 * This function will create any skeletalmesh we need to create according to the pipeline options
	 */
	UE_API void ExecutePreImportPipelineStaticMesh();

	UE_API void ExecutePostFactoryPipelineStaticMesh(const UInterchangeBaseNodeContainer* InBaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport);

	/** Static mesh factory assets nodes */
	TArray<UInterchangeStaticMeshFactoryNode*> StaticMeshFactoryNodes;

	/**
	* Applies Common mesh properties and LODGroup settings.
	*/
	UE_API void ApplyMeshProperties(UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode);

	/**
	 * This function can create a UInterchangeStaticMeshFactoryNode
	 * @param NodeUids: All primary MeshUid references for which the Factory Node is created.
	 *						In case a UInterchangeMeshLODContainerNode the containing UInterchangeMeshNodes won't be part of given Array.
	 *						NodeUid Can be Mesh(Mesh/LODContainer/Bundle) or MeshInstance (SceneNode instantiating a Mesh).
	 */
	UE_API UInterchangeStaticMeshFactoryNode* CreateStaticMeshFactoryNode(const TArray<FString>& NodeUids, const TArray<FString>& CollisionMeshUids = {});

	/**
	 * Return a reasonable UID and display label for a new mesh factory node.
	 * * @param NodeUids: NodeUid Can be Mesh(Mesh/LODContainer/Bundle) or MeshInstance (SceneNode instantiating a Mesh).
	 */
	UE_API bool MakeMeshFactoryNodeUidAndDisplayLabel(const TArray<FString>& NodeUids, FString& OutFactoryNodeUid, FString& OutFactoryDisplayLabel) const;

	/* Static mesh API END                                                */
	/************************************************************************/

	/**
	 * Fill all reference parameter from the translated node found in the container
	 */
	UE_API void GetMeshesInformationFromTranslatedData(const UInterchangeBaseNodeContainer* InBaseNodeContainer
		, bool& bContainStaticMesh
		, bool& bContainSkeletalMesh
		, bool& bContainGeometryCache) const;

	/************************************************************************/
	/* Geometry cache API BEGIN                                             */
	
	/**
	 * This function will create any geometry cache we need to create according to the pipeline options
	 */
	UE_API void ExecutePreImportPipelineGeometryCache();

	/** Geometry cache factory assets nodes */
	TArray<UInterchangeGeometryCacheFactoryNode*> GeometryCacheFactoryNodes;

	/**
	 * This function can create a UInterchangeGeometryCacheFactoryNode
	 * @param MeshUids - The MeshUids can represent a SceneNode pointing on a MeshNode or directly a MeshNode
	 */
	UE_API UInterchangeGeometryCacheFactoryNode* CreateGeometryCacheFactoryNode(const TArray<FString>& MeshUids);

	/**
	 * This function add all mesh nodes to the geometry cache.
	 * @param NodeUids - Can be a UInterchangeSceneNode or a UInterchangeMeshNode.
	 */
	UE_API void AddMeshesToGeometryCache(UInterchangeGeometryCacheFactoryNode* GeometryCacheFactoryNode, const TArray<FString>& NodeUids);

	/* Geometry cache API END                                               */
	/************************************************************************/

public:

	UE_API virtual void PostLoad() override;

	UE_API void AddToAssemblyPartMeshUidToFactoryNodeTable(const FString& MeshNodeUid, UInterchangeMeshFactoryNode* MeshFactoryNode);

	/*
	* Deprecated SkeletalMesh functions
	*/
	
	/**
	 * This function can create a UInterchangeSkeletalMeshFactoryNode
	 * @param MeshUidsPerLodIndex - The MeshUids can represent a SceneNode pointing on a MeshNode or directly a MeshNode
	 */
	UE_DEPRECATED(5.8, "Deprecated. Use CreateSkeletalMeshFactoryNode(const FString& RootJointUid, const TArray<FString>& NodeUids) instead.")
	UE_API UInterchangeSkeletalMeshFactoryNode* CreateSkeletalMeshFactoryNode(const FString& RootJointUid, const TMap<int32, TArray<FString>>& MeshUidsPerLodIndex);

	/** This function can create a UInterchangeSkeletalMeshLodDataNode which represent the LOD data need by the factory to create a lod mesh */
	UE_DEPRECATED(5.8, "Deprecated. Use CreateSkeletalMeshFactoryNode(const FString& RootJointUid, const TArray<FString>& NodeUids) instead. It will also take care of LodDataNode generations.")
	UE_API UInterchangeSkeletalMeshLodDataNode* CreateSkeletalMeshLodDataNode(const FString& NodeName, const FString& NodeUniqueID, const FString& ParentNodeUniqueID);

	/**
	 * This function add all lod data node to the skeletal mesh.
	 * @param NodeUidsPerLodIndex - The NodeUids can be a UInterchangeSceneNode or a UInterchangeMeshNode. The scene node can bake each instance of the mesh versus the mesh node will import only the modelled mesh.
	 */
	UE_DEPRECATED(5.8, "Deprecated. Use CreateSkeletalMeshFactoryNode(const FString& RootJointUid, const TArray<FString>& NodeUids) instead. It will also take care of LodData parsing to SkeletalMeshes")
	UE_API void AddLodDataToSkeletalMesh(const UInterchangeSkeletonFactoryNode* SkeletonFactoryNode, UInterchangeSkeletalMeshFactoryNode* SkeletalMeshFactoryNode, const TMap<int32, TArray<FString>>& NodeUidsPerLodIndex);

	/*
	* Deprecated StaticMesh functions
	*/

	UE_DEPRECATED(5.8, "Deprecated. Use CreateStaticMeshFactoryNode(const TArray<FString>& MeshNodeOrMeshInstanceNodeUids, const TArray<FString>& CollisionMeshUids = {}) instead.")
	UE_API UInterchangeStaticMeshFactoryNode* CreateStaticMeshFactoryNode(const TMap<int32, TArray<FString>>& MeshUidsPerLodIndex, const FString& BaseMeshUid = TEXT(""));

	UE_DEPRECATED(5.8, "Deprecated. Use CreateStaticMeshFactoryNode(const TArray<FString>& MeshNodeOrMeshInstanceNodeUids, const TArray<FString>& CollisionMeshUids = {}) instead. It will also take care of LodDataNode generations.")
	UE_API UInterchangeStaticMeshLodDataNode* FindOrCreateStaticMeshLodDataNode(UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode, const int32 LODIndex);

	UE_DEPRECATED(5.8, "Deprecated. Use CreateStaticMeshFactoryNode(const TArray<FString>& MeshNodeOrMeshInstanceNodeUids, const TArray<FString>& CollisionMeshUids = {}) instead. It will also take care of LodData parsing to StaticMeshes.")
	UE_API void AddLodDataToStaticMesh(UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode, const TMap<int32, TArray<FString>>& NodeUidsPerLodIndex);

	UE_DEPRECATED(5.8, "Deprecated. Use MakeMeshFactoryNodeUidAndDisplayLabel(const TArray<FString>& MeshNodeOrMeshInstancingNodeUids, FString& OutFactoryNodeUid, FString& OutFactoryDisplayLabel) instead.")
	UE_API bool MakeMeshFactoryNodeUidAndDisplayLabel(const TMap<int32, TArray<FString>>& MeshUidsPerLodIndex, int32 LodIndex, FString& NewMeshUid, FString& DisplayLabel, const FString& BaseMeshUid);

protected:
	UInterchangeBaseNodeContainer* BaseNodeContainer = nullptr;
	TArray<const UInterchangeSourceData*> SourceDatas;

};


#undef UE_API
