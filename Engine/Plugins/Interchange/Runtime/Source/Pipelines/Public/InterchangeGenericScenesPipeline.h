// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangePipelineBase.h"
#include "InterchangeSourceData.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeGenericScenesPipeline.generated.h"

#define UE_API INTERCHANGEPIPELINES_API

class ALevelInstance;
class FPreviewScene;
class UInterchangeActorFactoryNode;
class UInterchangeLevelFactoryNode;
class UInterchangeStaticMeshFactoryNode;
class UInterchangeLevelInstanceActorFactoryNode;
class UInterchangeSceneNode;
class UInterchangeSceneComponentNode;
class UInterchangeSceneVariantSetsNode;
class UInterchangeSceneImportAssetFactoryNode;
class UWorld;

UENUM(BlueprintType)
enum class EInterchangeSceneHierarchyType : uint8
{
	CreateLevelActors UMETA(DisplayName = "Create level actors", ToolTip = "Create actors in the current editor world for all scene nodes in the source hierarchy."),
	CreateLevelInstanceActor UMETA(DisplayName = "Create a level instance actor", ToolTip = "Create a level instance actor referencing a new/existing world containing all scene nodes in the source hierarchy."),
	CreatePackedActor UMETA(DisplayName = "Create a packed level actor", ToolTip = "Create a packed level actor blueprint which packed all meshes from a new/existing world containing all scene nodes in the source hierarchy.")
};

UCLASS(MinimalAPI, BlueprintType, editinlinenew)
class UInterchangeGenericLevelPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

public:
	UE_API virtual ~UInterchangeGenericLevelPipeline();

	/** The name of the pipeline that will be display in the import dialog. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Scene", meta = (StandAlonePipelineProperty = "True", PipelineInternalEditionData = "True"))
	FString PipelineDisplayName;

	/* Set the reimport strategy when reimporting into the level. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Scene", meta = (SubCategory = "Actors properties", AdjustPipelineAndRefreshDetailOnChange = "True"))
	EReimportStrategyFlags ReimportPropertyStrategy = EReimportStrategyFlags::ApplyNoProperties;

	/* Choose how you want to import the hierarchy. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Scene")
	EInterchangeSceneHierarchyType SceneHierarchyType = EInterchangeSceneHierarchyType::CreateLevelActors;

	/* If enabled, deletes actors that were not part of the translation when reimporting into a level. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Scene", meta = (SubCategory = "Reimport Actors"))
	bool bDeleteMissingActors = false;

	/* If enabled, respawns actors that were deleted in the editor prior to a reimport. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Scene", meta = (SubCategory = "Reimport Actors"))
	bool bForceReimportDeletedActors = false;

	/* If enabled, recreates assets that were deleted in the editor prior to reimporting into a level. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Scene", meta = (SubCategory = "Reimport Assets"))
	bool bForceReimportDeletedAssets = false;

	/* If enabled, deletes assets that were not part of the translation when reimporting into a level. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Scene", meta = (SubCategory = "Reimport Assets"))
	bool bDeleteMissingAssets = false;

	/* If enabled, HierarchicalInstancedStaticMeshComponents will be generated on import instead of InstancedStaticMeshComponents. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Scene", meta = (SubCategory = "Component properties", DisplayName = "Use Hierarchical ISM Components"))
	bool bUseHierarchicalISMComponents = false;

	/** BEGIN UInterchangePipelineBase overrides */
	UE_API virtual void AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams) override;

protected:

	UE_API virtual void ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath) override;
	UE_API virtual void ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport) override;
	UE_API virtual void ExecutePostBroadcastPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport) override;

	virtual bool CanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask) override
	{
		return true;
	}
	/** END UInterchangePipelineBase overrides */

	/**
	 * PreImport step called for each translated SceneNode.
	 */
	UE_API virtual void ExecuteSceneNodePreImport(const FTransform& GlobalOffsetTransform, const UInterchangeSceneNode* SceneNode);

	/**
	 * PreImport step called for each translated SceneVariantSetNode.
	 */
	UE_API virtual void ExecuteSceneVariantSetNodePreImport(const UInterchangeSceneVariantSetsNode& SceneVariantSetNode);

	/**
	 * Return a new Actor Factory Node to be used for the given SceneNode.
	 */
	UE_API virtual UInterchangeActorFactoryNode* CreateActorFactoryNode(const UInterchangeBaseNode* TranslatedAssetNode) const;

	UE_DEPRECATED(5.8, "Use CreateActorFactoryNode(const UInterchangeBaseNode* TranslatedAssetNode) instead")
	virtual UInterchangeActorFactoryNode* CreateActorFactoryNode(const UInterchangeSceneNode* SceneNode, const UInterchangeBaseNode* TranslatedAssetNode) const
	{
		return CreateActorFactoryNode(TranslatedAssetNode);
	}

	/**
	 * Return a new Component Factory Node.
	 */
	UE_API virtual void CreateComponentFactoryNode(const UInterchangeSceneComponentNode* TranslatedAssetNode) const;

	/**
	 * Use to set up the given factory node's attributes after its initialization.
	 */
	UE_API virtual void SetUpFactoryNode(UInterchangeActorFactoryNode* ActorFactoryNode, const UInterchangeSceneNode* SceneNode, const UInterchangeBaseNode* TranslatedAssetNode, bool bIsRootJoint = false) const;

	UE_API virtual void PopulateSceneNodesPreImport(TArray<UInterchangeSceneNode*>& SceneNodes,
		TArray<UInterchangeSceneComponentNode*>& SceneComponentNodes) const;

	/**
	 * Use to set up the given factory node's component's attributes after its initialization.
	 */
	UE_API virtual void SetUpComponents(UInterchangeActorFactoryNode* ActorFactoryNode, const UInterchangeSceneNode* SceneNode) const;

	UE_API virtual void SetUpComponentDependencies(UInterchangeActorFactoryNode* ActorFactoryNode, const TArray<FString>& SceneComponentUids) const;
	
	/** Disable this option to not convert Standard(Perspective) to Physical Cameras*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Skeletal Meshes and Animations")
	bool bUsePhysicalInsteadOfStandardPerspectiveCamera = true;

protected:
	UInterchangeBaseNodeContainer* BaseNodeContainer = nullptr;
#if WITH_EDITORONLY_DATA
	/*
	 * Factory node created by the pipeline to later create the SceneImportAsset
	 * This factory node must be unique and depends on all the other factory nodes
	 * and its factory must be called after all factories
	 * Note that this factory node is not created at runtime. Therefore, the reimport
	 * of a level will not work at runtime
	 */
	UInterchangeSceneImportAssetFactoryNode* SceneImportFactoryNode = nullptr;
	UInterchangeLevelFactoryNode* LevelFactoryNode = nullptr;
	UInterchangeLevelInstanceActorFactoryNode* LevelInstanceActorFactoryNode = nullptr;

	struct FPostPipelineImportData
	{
		void AddLevelInstanceActor(ALevelInstance* LevelInstanceActor, UWorld* ReferenceWorld);
	private:

		TSet<UWorld*> Worlds;
		TMap<ALevelInstance*, UWorld*> ReferenceWorldPerLevelInstanceToUpdates;
	};

	FPostPipelineImportData PostPipelineImportData;
#endif

	UE_API void CacheActiveJointUids();
	TSet<FString> Cached_ActiveJointUids;

	FPreviewScene* PreviewScene = nullptr;

private:
	void CreateActorFactoryNode(const FTransform& GlobalOffsetTransform, const UInterchangeSceneNode* SceneNode, const FString& SkeletalMeshFactoryNodeUid);
};


#undef UE_API
