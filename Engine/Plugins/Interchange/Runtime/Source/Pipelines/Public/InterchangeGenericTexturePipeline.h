// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangePipelineBase.h"
#include "InterchangeSourceData.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeGenericTexturePipeline.generated.h"

#define UE_API INTERCHANGEPIPELINES_API

class UInterchangeTextureFactoryNode;
class UInterchangeTextureNode;

UCLASS(MinimalAPI, BlueprintType, editinlinenew)
class UInterchangeGenericTexturePipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

public:
	static UE_API FString GetPipelineCategory(UClass* AssetClass);

	/** The name of the pipeline that will be display in the import dialog. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Textures", meta = (StandAlonePipelineProperty = "True", PipelineInternalEditionData = "True"))
	FString PipelineDisplayName;

	/** If enabled, imports all texture assets found in the source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Textures")
	bool bImportTextures = true;

	/** If set, and there is only one asset and one source, the imported asset will be given this name. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Textures", meta=(StandAlonePipelineProperty = "True"))
	FString AssetName;

#if WITH_EDITORONLY_DATA
	/** 
	 * If enabled, tests each newly imported texture to see if it is a normal map.
	 * If it is, the SRGB, Compression Settings, and LOD Group properties of that texture will be adjusted.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Textures", Meta=(EditCondition="bImportTextures"))
	bool bDetectNormalMapTexture = true;

	/** If enabled, the texture's green channel will be inverted for normal maps. This setting is only used if the Detect Normal Map Texture setting is also enabled or if the texture has been imported as a Normal map. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Textures", Meta=(EditCondition="bImportTextures"))
	bool bFlipNormalMapGreenChannel = false;

	/** If enabled, imports textures as UDIMs if they are named using a UDIM naming pattern. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Textures", Meta=(EditCondition="bImportTextures"))
	bool bImportUDIMs = true;

	/** Detect if a texture should be imported as long/lat cubemap. The minimum size to detect a cubemap texture is a 512 pixels height. Use the CVar Interchange.Textures.MinCubeMapDetection if you want to control that setting.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Textures", Meta = (EditCondition = "bImportTextures && (bIsReimport == false)"))
	bool bDetectLongLatCubemap = false;

	/** Specify the file types that should be imported as long/lat cubemaps. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Textures", Meta=(EditCondition="bImportTextures && (bDetectLongLatCubemap == false)"))
	TSet<FString> FileExtensionsToImportAsLongLatCubemap = {"hdr"};

	/** 
	 * If enabled, the translator compresses the source data payload whenever possible. This generally results in smaller assets.
	 * However, some operations like the texture build might be slower, because the source data first needs to be decompressed.
	 * If disabled, the translator leaves the decision to the factory or the pipelines.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Textures", Meta=(EditCondition="bImportTextures"))
	bool bPreferCompressedSourceData = false;

#endif

	/** If enabled, textures that have a non-power-of-two resolution are imported. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Textures", Meta=(EditCondition="bImportTextures"))
	bool bAllowNonPowerOfTwo = false;

public:
	UE_API virtual void AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams) override;
	UE_API virtual void ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath) override;
	UE_API virtual void ExecutePostFactoryPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport) override;

#if WITH_EDITOR

	UE_API virtual bool IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const override;

	UE_API virtual void FilterPropertiesFromTranslatedData(UInterchangeBaseNodeContainer* InBaseNodeContainer) override;

	UE_API virtual void GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const override;

#endif //WITH_EDITOR

protected:
	UE_API UInterchangeTextureFactoryNode* HandleCreationOfTextureFactoryNode(const UInterchangeTextureNode* TextureNode);

	UE_API UInterchangeTextureFactoryNode* CreateTextureFactoryNode(const UInterchangeTextureNode* TextureNode, const TSubclassOf<UInterchangeTextureFactoryNode>& FactorySubclass);

	UE_API void PostImportTextureAssetImport(UObject* CreatedAsset, bool bIsAReimport);

private:
	UPROPERTY()
	TObjectPtr<UInterchangeBaseNodeContainer> BaseNodeContainer;

	TArray<const UInterchangeSourceData*> SourceDatas;
	
	/** Texture translated assets nodes */
	TArray<UInterchangeTextureNode*> TextureNodes;

	/** Texture factory assets nodes */
	TArray<UInterchangeTextureFactoryNode*> TextureFactoryNodes;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bIsReimport = false;
#endif
};

#undef UE_API
