// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeMeshDefinitions.h"

class FString;
class UInterchangeBaseNode;
class UInterchangeBaseNodeContainer;
class UInterchangeGenericMeshPipeline;
class UInterchangePipelineMeshesUtilities;
class UInterchangeSceneNode;
class UInterchangeStaticMeshFactoryNode;
class UInterchangeStaticMeshLodDataNode;
class UInterchangeSkeletalMeshLodDataNode;


namespace UE::Interchange::CollisionHelper
{
	TTuple<EInterchangeMeshCollision, FString> GetCollisionMeshType(
		TObjectPtr<UInterchangePipelineMeshesUtilities> PipelineMeshesUtilities,
		const UInterchangeBaseNodeContainer& NodeContainer,
		const FString& NodeUid,
		const TArray<FString>& AllNodeUids,
		bool bImportCollisionAccordingToMeshName
	);
}

//Helper struct that processes an array of NodeUids into LODContainers.
//Template to handle StaticMesh and SkeletalMeshes in one go.
//Difference between SKM and SM is the Collision and SkeletonUID setting for the LODDataNode.
template <class TMeshFactoryType>
struct FInterchangeMeshLODDataParser
{
private:
	using TLODDataNodeType = std::conditional_t<
		std::is_same_v<TMeshFactoryType, UInterchangeStaticMeshFactoryNode>,
		UInterchangeStaticMeshLodDataNode,
		UInterchangeSkeletalMeshLodDataNode
	>;

	TMeshFactoryType* MeshFactoryNode; //The one we are setting up
	const FString MeshFactoryNodeUid;

	UInterchangeBaseNodeContainer& BaseNodeContainer;
	const bool bBakeMeshes;
	const bool bBakePivotMeshes;
	TObjectPtr<UInterchangePipelineMeshesUtilities> PipelineMeshesUtilities;
	UInterchangeGenericMeshPipeline* GenericMeshPipeline;
	FString SourceAssetFileName; //Used for warnings
	const FString SkeletonFactoryNodeUid; //To Set for UInterchangeSkeletalMeshLodDataNode
	bool bUseTimeZeroAsBindPose;

	//
	FTransform GlobalOffsetTransform;
	TMap<int32, TLODDataNodeType*> LODIndexToLodDataNodeMap;
	TMap<int32, TMap<FString, FString>> ExistingLodSlotMaterialDependenciesPerLODIndices;
	TSet<FString> UniqueMeshUids; /*All MeshUids processed (including all LODs).*/
	TMap<int32, int32> OriginalToCompactLODIndexMap;

	/**
	* Used for calculating skeletalMesh's relative transform to RootJoineNode's parent (aka Skeleton Transform).
	* This is needed, because GetTimeZeroGlobalTransform/GetBindPoseGlobalTransform on SceneNodes fallback onto LocalTransforms, if BindPose/T0 is not present.
	* Inherently this means that we get the Transform's relative to the Root SceneNode, regardless if they are Joint or not,
	*	To fix for this behavior we apply the RootJoint's Parent's Global Transform inverse when BakeMesh==false, to acquire 'SkeletonTransform'.
	*	   (See @UE::Interchange::Private::MeshHelper::CalculateCompleteSceneNodeTransform)
	*	If it is not set, the CalculateCompleteScenNodeTransform will try to identify the RootJoint's ParentNode and acquire described transform for application in there.
	*	   (Based on Parent hierarchy and UInterchangeJointNode classification)
	* The FString is the RootJointParent's UID, this is used for identifying, if the the instancing Node is part of the child tree/hierarchy of the RootJointParent.
	*	(See @UE::Interchange::Private::MeshHelper::CalculateCompleteSceneNodeTransform)
	*/
	TOptional<TPair<FString, FMatrix>> RootJointParentUidAndGlobalInverseMatrix;
	bool bCreatingSkeletalMesh = !std::is_same_v<TMeshFactoryType, UInterchangeStaticMeshFactoryNode>;
	bool bAlignSkeletalMeshInScene;

	TArray<FString> AllNodeUids; //Used for collission processing, combination of NodeUids and CollisionNodeUids

public:

	explicit FInterchangeMeshLODDataParser(UInterchangeBaseNodeContainer& InBaseNodeContainer,
		TMeshFactoryType* InStaticMeshFactoryNode,
		TObjectPtr<UInterchangePipelineMeshesUtilities> InPipelineMeshesUtilities,
		UInterchangeGenericMeshPipeline* InGenericMeshPipeline,
		const FString& InSourceAssetFileName,
		const FString& InRootJointNodeUid = TEXT("") /*To calculate Relative Transform for UInterchangeJointNodes*/,
		const FString& InSkeletonFactoryNodeUid = TEXT(""),
		const bool InbUseTimeZeroAsBindPose = false);

	/*
	* Note Regarding CollisionMeshUids:
	*			-If bImportCollision:=false	=> we ignore the CollisionMeshUids array
	*			-If bImportCollision:=true:
	*					--If bImportCollisionAccordingToMeshName:=true	=> we add the CollisionMeshUids as Collisions
	*					--If bImportCollisionAccordingToMeshName:=false	=> we add the CollisionMeshUids as RenderMeshes
	*/
	void ProcessNodeUids(const TArray<FString>& RenderNodeUids, const TArray<FString>& CollisionMeshUids = {});

	TSet<FString> GetUniqueMeshUids() const { return UniqueMeshUids; }

private:

	void CreateMeshLodDataNode(const int32 LODIndex);
	TLODDataNodeType* GetMeshLodDataNode(const int32 MappedLODIndex); //Mapped via OriginalToCompactLODIndexMap.
	void LogWarning(const FText& Text);

	void ProcessCollision(const FString& NodeUid, UInterchangeStaticMeshLodDataNode* LodDataNode, const FTransform& Transform);

	/*
	* bProcessAsSubMesh:= such as Mesh of a MeshLODContainerNode/MeshBundle/ISMComponentNode
	*						we track these, as the ones flagged as sub mesh do not need to be configured via TargetNodes and the AssemblyParts do not need to be set either.
	*/
	void ProcessNode(const UInterchangeBaseNode* BaseNode, const FTransform& MeshTransform, const UInterchangeSceneNode* CurrentTransformNode, const int32 LODIndex, bool bProcessAsSubMesh, bool bProcessAsCollision);

	//Parses All nodes to identify used LODIndices, sets up OriginalToCompactLODIndexMap, and creates all MeshLodDataNodes.
	void PreProcess(const TArray<FString>& NodeUids);
	void FindAllAvailableLODIndices(const UInterchangeBaseNode* BaseNode, TSet<int32>& AvailableLODIndices);
};