// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InterchangePipelineBase.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "Engine/DeveloperSettings.h"
#include "MaterialX/InterchangeMaterialXDefinitions.h"

#include "InterchangeMaterialXPipeline.generated.h"

#define UE_API INTERCHANGEPIPELINES_API

class UMaterialFunction;
class UMaterialInterface;

using EInterchangeMaterialXSettings = TVariant<EInterchangeMaterialXShaders, EInterchangeMaterialXBSDF, EInterchangeMaterialXEDF, EInterchangeMaterialXVDF>;

uint32 INTERCHANGEPIPELINES_API GetTypeHash(EInterchangeMaterialXSettings Key);

bool INTERCHANGEPIPELINES_API operator==(EInterchangeMaterialXSettings Lhs, EInterchangeMaterialXSettings Rhs);

UCLASS(MinimalAPI, config = Interchange, meta = (DisplayName = "MaterialX Settings", ToolTip = "Interchange MaterialX Pipeline Settings"))
class UMaterialXPipelineSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UMaterialXPipelineSettings();

	UE_API bool AreRequiredPackagesLoaded();

	UE_API FString GetAssetPathString(EInterchangeMaterialXSettings EnumType) const;

	template<typename EnumT>
	FString GetAssetPathString(EnumT EnumValue) const
	{
		static_assert(std::is_same_v<EnumT, EInterchangeMaterialXShaders> ||
					  std::is_same_v<EnumT, EInterchangeMaterialXBSDF> ||
					  std::is_same_v<EnumT, EInterchangeMaterialXEDF> ||
					  std::is_same_v<EnumT, EInterchangeMaterialXVDF>,
					  "Enum type not supported");

		return GetAssetPathString(EInterchangeMaterialXSettings{ TInPlaceType<EnumT>{}, EnumValue });
	}

	UPROPERTY(EditAnywhere, config, Category = "MaterialXPredefined | Surface Shaders", meta = (DisplayName = "MaterialX Predefined Surface Shaders"))
	TMap<EInterchangeMaterialXShaders, FSoftObjectPath> PredefinedSurfaceShaders;

	UPROPERTY(EditAnywhere, config, Category = "MaterialXPredefined | BSDF", meta = (DisplayName = "MaterialX Predefined BSDF"))
	TMap<EInterchangeMaterialXBSDF, FSoftObjectPath> PredefinedBSDF;

	UPROPERTY(EditAnywhere, config, Category = "MaterialXPredefined | EDF", meta = (DisplayName = "MaterialX Predefined EDF"))
	TMap<EInterchangeMaterialXEDF, FSoftObjectPath> PredefinedEDF;

	UPROPERTY(EditAnywhere, config, Category = "MaterialXPredefined | VDF", meta = (DisplayName = "MaterialX Predefined VDF"))
	TMap<EInterchangeMaterialXVDF, FSoftObjectPath> PredefinedVDF;

#if WITH_EDITOR
	/** Init the Predefined with Substrate assets, since the default value is set in BaseInterchange.ini and we have no way in the config file to conditionally init a property*/
	UE_API void InitPredefinedAssets();

private:
	friend class FInterchangeMaterialXPipelineSettingsCustomization;
	friend class UInterchangeMaterialXPipeline;

	using FMaterialXSettings = TMap<EInterchangeMaterialXSettings, TPair<TSet<FName>, TSet<FName>>>;

	static UE_API bool ShouldFilterAssets(UMaterialFunction* Asset, const TSet<FName>& Inputs, const TSet<FName>& Outputs);

	static UE_API EInterchangeMaterialXSettings ToEnumKey(uint8 EnumType, uint8 EnumValue);

	template<typename EnumT>
	static EInterchangeMaterialXSettings ToEnumKey(EnumT EnumValue)
	{
		static_assert(std::is_same_v<EnumT, EInterchangeMaterialXShaders> ||
					  std::is_same_v<EnumT, EInterchangeMaterialXBSDF> ||
					  std::is_same_v<EnumT, EInterchangeMaterialXEDF> ||
					  std::is_same_v<EnumT, EInterchangeMaterialXVDF>,
					  "Enum type not supported");

		return EInterchangeMaterialXSettings{ TInPlaceType<EnumT>{}, EnumValue };
	}

	/** The key is a combination of the index in the variant + the corresponding enum */
	static UE_API FMaterialXSettings SettingsInputsOutputs;

	bool bIsSubstrateEnabled{ false };
#endif // WITH_EDITOR
};

UCLASS(MinimalAPI, config = Interchange, BlueprintType)
class UInterchangeMaterialXPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

	UE_API UInterchangeMaterialXPipeline();

public:
	TObjectPtr<UMaterialXPipelineSettings> MaterialXSettings;

	/** If true, handle Volume Distribution Functions as volumetric materials, i.e an internal Substrate Volumetric FOG-Cloud BSDF is created, otherwise a simple volume Slab BSDF is used. */
	UPROPERTY(EditAnywhere, config, Category = "Volume")
	bool bVolumetricMaterial = true;

	/** Derive the normals from height for displacement shaders, this option is best suited with procedural normals (for example shader graphs created specifically for offline-renderers).
	  * Otherwise this option should be disabled if there's already a normal map plugged to the normal input of the surfaceshader.
	  */
	UPROPERTY(EditAnywhere, config, Category = "Displacement", meta = (DisplayName = "Normals From Height"))
	bool bRecomputeNormals = true;

protected:
	UE_API virtual void AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams) override;
	UE_API virtual void ExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath) override;

	virtual bool CanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask) override
	{
		// This pipeline creates UObjects and assets. Not safe to execute outside of main thread.
		return true;
	}

private:

	static UE_API TMap<FString, EInterchangeMaterialXSettings> PathToEnumMapping;
};

#undef UE_API
