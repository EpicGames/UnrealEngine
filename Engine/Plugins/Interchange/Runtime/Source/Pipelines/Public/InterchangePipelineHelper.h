// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangePipelineBase.h"

class UInterchangeFactoryBaseNode;
class UInterchangeSourceNode;

namespace UE::Interchange::PipelineHelper
{
	void ShowModalDialog(TSharedRef<SInterchangeBaseConflictWidget> ConflictWidget, const FText& Title, const FVector2D& WindowSize);

	// Fills in the FactoryNode's SubPath custom attribute with the SourceNode's sub path prefixes and suffixes
	INTERCHANGEPIPELINES_API bool FillSubPathFromSourceNode(UInterchangeFactoryBaseNode* FactoryNode, const UInterchangeSourceNode* SourceNode);
}
