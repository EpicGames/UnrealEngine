// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeMaterialFactoryNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "InterchangeGenericAssetsPipelineSharedSettings.h"

#include "InterchangePipelineMeshesUtilities.generated.h"

#define UE_API INTERCHANGEPIPELINES_API

DECLARE_LOG_CATEGORY_EXTERN(LogInterchangePipelineMeshesUtilities, Log, All);

class UInterchangeBaseNodeContainer;
class UInterchangeMeshNode;
class UInterchangeSceneNode;

/*
* A mesh instance is a description of a translated scene node that points to a translated mesh asset.
* A mesh instance that points to a mesh node will have only LOD 0 and will point to one scene mesh node.
*/
USTRUCT(BlueprintType)
struct FInterchangeMeshInstance
{
	GENERATED_BODY()

	FInterchangeMeshInstance()
		: bReferenceSkinnedMesh(false)
		, bReferenceMorphTarget(false)
		, bHasMorphTargets(false)
		, bIsGeometryCache(false)
		, bHasTransformAnimationTrack(false)
		, bIsVisible(false)
		, InstancingNode(nullptr)
	{
	}

	UPROPERTY(EditAnywhere, Category = "Interchange | Pipeline | MeshesInstance")
	bool bReferenceSkinnedMesh;

	UPROPERTY(EditAnywhere, Category = "Interchange | Pipeline | MeshesInstance")
	bool bReferenceMorphTarget;

	UPROPERTY(EditAnywhere, Category = "Interchange | Pipeline | MeshesInstance")
	bool bHasMorphTargets;

	UPROPERTY(EditAnywhere, Category = "Interchange | Pipeline | MeshesInstance")
	bool bIsGeometryCache;

	UPROPERTY(EditAnywhere, Category = "Interchange | Pipeline | MeshesInstance")
	bool bHasTransformAnimationTrack;

	UPROPERTY(EditAnywhere, Category = "Interchange | Pipeline | MeshesInstance")
	bool bIsVisible;

	//It is the node that is instantiating a Mesh, for which the FInterchangeMeshInstance 'instance' is created.
	//Instancing node can be:
	//	- UInterchangeSceneNode
	//	- UInterchangeGeometryCacheNode
	//	- UInterchangeInstancedStaticMeshComponentNode
	UPROPERTY(EditAnywhere, Category = "Interchange | Pipeline | MeshesInstance")
	TObjectPtr<const UInterchangeBaseNode> InstancingNode; 

	/**
	 * All mesh geometry referenced by this MeshInstance.
	 */
	UPROPERTY(EditAnywhere, Category = "Interchange | Pipeline | MeshesInstance")
	TArray<FString> ReferencingMeshGeometryUids;

	//Returns true if this mesh instance is instanced from a Joint node (i.e. part of a skeleton)
	bool IsNestedInSkeleton() const;

	//Returns true if the associated SceneNode is flagged as DoNotInstantiateInLevel.
	bool DoNotInstantiateInLevel() const;
};

/*
* A mesh geometry is a description of a translated mesh asset node that defines a geometry.
*/
USTRUCT(BlueprintType)
struct FInterchangeMeshGeometry
{
	GENERATED_BODY()

	FInterchangeMeshGeometry()
		: BaseNode(nullptr)
		, bIsReferencedBySkeleton(false)
		, bHasTransformAnimationTrack(false)
		, bIsMorphTarget(false)
		, bIsSkinnedMesh(false)
		, bHasMorphTargetDependencies(false)
		, bHasSkeletonDependencies(false)
		, bHasAssemblyPartDependencies(false)
		, AssemblyPartDependenciesArray()
		, AllUnderlyingMeshNodeUidsArray()
	{
	}

	/**
	 * The UInterchangeMeshNode or UInterchangeMeshLODContainerNode pointer represented by this structure.
	 */
	UPROPERTY(EditAnywhere, Category = "Interchange | Pipeline | MeshesGeometry")
	TObjectPtr<const UInterchangeBaseNode> BaseNode = nullptr;

	/**
	 * All instancing Node Uids.
	 */
	UPROPERTY(EditAnywhere, Category = "Interchange | Pipeline | MeshesGeometry")
	TArray<FString> UidsOfInstancingNodes; //That also identified the FInterchangeMeshInstance.

	/**
	 * A list of all scene nodes that represent sockets attached to this mesh.
	 */
	UPROPERTY(EditAnywhere, Category = "Interchange | Pipeline | MeshesGeometry")
	TArray<FString> AttachedSocketUids;

	/*
	* Indicates that at least 1 of the Instancing Node Uids is a Joint (aka part of a skeleton) or not.
	*/
	UPROPERTY(EditAnywhere, Category = "Interchange | Pipeline | MeshesGeometry")
	bool bIsReferencedBySkeleton;

	/*
	* Indicates that at least 1 of the Instancing Node Uids has a transform animation.
	*/
	UPROPERTY(EditAnywhere, Category = "Interchange | Pipeline | MeshesGeometry")
	bool bHasTransformAnimationTrack;

	void SetupMeshGeometry(const UInterchangeBaseNodeContainer& BaseNodeContainer, const UInterchangeBaseNode* InBaseNode);

	bool IsMorphTarget() const;
	bool IsSkinnedMesh() const { return bIsSkinnedMesh; };
	bool HasMorphTargetDependencies() const { return bHasMorphTargetDependencies; };
	bool HasSkeletonDependecies() const { return bHasSkeletonDependencies; };
	bool HasAssemblyPartDependencies() const { return bHasAssemblyPartDependencies; };
	void GetAssemblyPartDependencies(TArray<FString>& AssemblyPartDependencies) const { AssemblyPartDependencies = AssemblyPartDependenciesArray; };
	
	FString GetSkeletonRootUid(const UInterchangeBaseNodeContainer& BaseNodeContainer, const TArray<FString>& SkeletonRootNodeUids) const;

private:

	/**
	* Helper function to parse/process the above public functions on a UInterchangeMeshnode or UInterchangeMeshLODContainerNode.
	*/
	template <class T>
	T ParseMeshOrLODContainerNode(const UInterchangeBaseNodeContainer& BaseNodeContainer, T DefaultValue, TFunctionRef<T(T, T)> HandleConflictFunction, TFunctionRef<T(const UInterchangeMeshNode* MeshNode)> MeshFunction) const;

	static bool BoolConflictHandler(bool bLeft, bool bRight);
	/*
	* Used for handling SkeletonRootUid conflicts, which shouldn't happen.
	*/
	static FString StringConflictHandler(const FString& Left, const FString& Right);


	/**
	* Helper functions to set values
	*/
	void SetIsSkinnedMesh(const UInterchangeBaseNodeContainer& BaseNodeContainer);
	void SetHasMorphTargetDependencies(const UInterchangeBaseNodeContainer& BaseNodeContainer);
	void SetHasSkeletonDependecies(const UInterchangeBaseNodeContainer& BaseNodeContainer);
	void SetHasAssemblyPartDependencies(const UInterchangeBaseNodeContainer& BaseNodeContainer);
	void SetAssemblyPartDependencies(const UInterchangeBaseNodeContainer& BaseNodeContainer);

	//Values:
	bool bIsMorphTarget;
	bool bIsSkinnedMesh;
	bool bHasMorphTargetDependencies;
	bool bHasSkeletonDependencies;
	bool bHasAssemblyPartDependencies;
	TArray<FString> AssemblyPartDependenciesArray;
	TArray<FString> AllUnderlyingMeshNodeUidsArray;
};

/*
* Represents the context UInterchangePipelineMeshesUtilities will use when the client queries data.
*/
USTRUCT(BlueprintType)
struct FInterchangePipelineMeshesUtilitiesContext
{
	GENERATED_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FInterchangePipelineMeshesUtilitiesContext() = default;
	FInterchangePipelineMeshesUtilitiesContext(const FInterchangePipelineMeshesUtilitiesContext&) = default;
	FInterchangePipelineMeshesUtilitiesContext(FInterchangePipelineMeshesUtilitiesContext&&) = default;
	FInterchangePipelineMeshesUtilitiesContext& operator=(const FInterchangePipelineMeshesUtilitiesContext&) = default;
	FInterchangePipelineMeshesUtilitiesContext& operator=(FInterchangePipelineMeshesUtilitiesContext&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Whether to force import all meshes as Static Meshes or Skeletal Meshes.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | Pipeline | MeshesContext")
	EInterchangeForceMeshType ForceMeshType = EInterchangeForceMeshType::IFMT_None;

	/**
	 * If enabled, all static meshes that have a transform animation will be imported as skeletal meshes instead.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | Pipeline | MeshesContext")
	bool bConvertStaticsWithAnimatedTransformToSkeletals = false;

	/**
	 * If enabled, all static meshes that have morph targets will be imported as skeletal meshes instead.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | Pipeline | MeshesContext", meta = (DisplayName = "Convert Static Meshes with Morph Targets to Skeletal Meshes"))
	bool bConvertStaticsWithMorphTargetsToSkeletals = false;

	/**
	* If enabled, all static meshes nested in bone hierarchies will be imported as skeletal meshes instead.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | Pipeline | MeshesContext")
	bool bConvertStaticsInBoneHierarchyToSkeletals = true;

	/**
	 * When querying geometry, this flag will not add MeshGeometry if there is a scene node that points to a geometry.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | Pipeline | MeshesContext")
	bool bQueryGeometryOnlyIfNoInstance = true;

	/**
	 * If enabled, all geometry caches will be ignored. The mesh utility will not return any geometry caches.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | Pipeline | MeshesContext")
	bool bIgnoreGeometryCaches = false;

	UE_DEPRECATED(5.8, "Use bConvertStaticsInBoneHierarchyToSkeletals instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use bConvertStaticsInBoneHierarchyToSkeletals instead."))
	bool bImportMeshesInBoneHierarchy_DEPRECATED = true;

	UE_DEPRECATED(5.8, "Use ForceMeshType instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use ForceMeshType instead."))
	bool bConvertStaticMeshToSkeletalMesh_DEPRECATED = false;

	UE_DEPRECATED(5.8, "Use ForceMeshType instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use ForceMeshType instead."))
	bool bConvertSkeletalMeshToStaticMesh_DEPRECATED = false;

	UE_DEPRECATED(5.8, "Use ForceMeshType instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use ForceMeshType instead."))
	bool bIgnoreStaticMeshes_DEPRECATED = false;

	bool IsStaticMeshInstance(const FInterchangeMeshInstance& MeshInstance, UInterchangeBaseNodeContainer* BaseNodeContainer);
	bool IsSkeletalMeshInstance(const FInterchangeMeshInstance& MeshInstance, UInterchangeBaseNodeContainer* BaseNodeContainer);
	bool IsGeometryCacheInstance(const FInterchangeMeshInstance& MeshInstance);
	bool IsStaticMeshGeometry(const FInterchangeMeshGeometry& MeshGeometry);
	bool IsSkeletalMeshGeometry(const FInterchangeMeshGeometry& MeshGeometry);
	bool IsGeometryCacheGeometry(const FInterchangeMeshGeometry& MeshGeometry);

	UE_DEPRECATED(5.8, "Use IsSkeletalMeshInstance(const FInterchangeMeshInstance& MeshInstance, UInterchangeBaseNodeContainer* BaseNodeContainer) instead.")
	inline bool IsSkeletalMeshInstance(const FInterchangeMeshInstance& MeshInstance, UInterchangeBaseNodeContainer* BaseNodeContainer, bool& bOutIsStaticMeshNestedInSkeleton)
	{
		bOutIsStaticMeshNestedInSkeleton = false;
		return IsSkeletalMeshInstance(MeshInstance, BaseNodeContainer);
	}
};

/**/
UCLASS(MinimalAPI, BlueprintType)
class UInterchangePipelineMeshesUtilities : public UObject
{
	GENERATED_BODY()
public:
	/**
	* Create an instance of UInterchangePipelineMeshesUtilities.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	static UE_API UInterchangePipelineMeshesUtilities* CreateInterchangePipelineMeshesUtilities(UInterchangeBaseNodeContainer* BaseNodeContainer);

	/**
	* Get the unique IDs of all mesh instances.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	UE_API void GetAllMeshInstanceUids(TArray<FString>& MeshInstanceUids) const;

	/**
	* Iterate over all mesh instances.
	*/
	UE_API void IterateAllMeshInstance(TFunctionRef<void(const FInterchangeMeshInstance&)> IterationLambda) const;

	/**
	* Get the unique IDs of all skinned mesh instances.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	UE_API void GetAllSkinnedMeshInstance(TArray<FString>& MeshInstanceUids) const;

	/**
	* Iterate over all skinned mesh instances.
	*/
	UE_API void IterateAllSkinnedMeshInstance(TFunctionRef<void(const FInterchangeMeshInstance&)> IterationLambda) const;

	/**
	* Get the unique IDs of all static mesh instances.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	UE_API void GetAllStaticMeshInstance(TArray<FString>& MeshInstanceUids, const bool bAllowNotVisibles = true) const;

	/**
	* Iterate over all static mesh instances.
	*/
	UE_API void IterateAllStaticMeshInstance(TFunctionRef<void(const FInterchangeMeshInstance&)> IterationLambda) const;

	/**
	* Get the unique IDs of all geometry cache instances.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	UE_API void GetAllGeometryCacheInstance(TArray<FString>& MeshInstanceUids) const;

	/**
	* Iterate over all geometry cache instances.
	*/
	UE_API void IterateAllGeometryCacheInstance(TFunctionRef<void(const FInterchangeMeshInstance&)> IterationLambda) const;

	/**
	* Get the unique IDs of all mesh geometry.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	UE_API void GetAllMeshGeometry(TArray<FString>& MeshGeometryUids) const;
		
	/**
	* Iterate over all mesh geometry.
	*/
	UE_API void IterateAllMeshGeometry(TFunctionRef<void(const FInterchangeMeshGeometry&)> IterationLambda) const;

	/**
	* Get the unique IDs of all skinned mesh geometry.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	UE_API void GetAllSkinnedMeshGeometry(TArray<FString>& MeshGeometryUids) const;

	/**
	* Iterate over all skinned mesh geometry.
	*/
	UE_API void IterateAllSkinnedMeshGeometry(TFunctionRef<void(const FInterchangeMeshGeometry&)> IterationLambda) const;

	/**
	* Get the unique IDs of all static mesh geometry.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	UE_API void GetAllStaticMeshGeometry(TArray<FString>& MeshGeometryUids) const;

	/**
	* Iterate over all static mesh geometry.
	*/
	UE_API void IterateAllStaticMeshGeometry(TFunctionRef<void(const FInterchangeMeshGeometry&)> IterationLambda) const;

	/**
	* Get the unique IDs of all geometry cache geometry.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	UE_API void GetAllGeometryCacheGeometry(TArray<FString>& MeshGeometryUids) const;

	/**
	* Iterate over all geometry cache geometry.
	*/
	UE_API void IterateAllGeometryCacheGeometry(TFunctionRef<void(const FInterchangeMeshGeometry&)> IterationLambda) const;

	/**
	* Get the unique IDs of all non-instanced mesh geometry.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	UE_API void GetAllMeshGeometryNotInstanced(TArray<FString>& MeshGeometryUids) const;

	/**
	* Iterate over all non-instanced mesh geometry.
	*/
	UE_API void IterateAllMeshGeometryNotIntanced(TFunctionRef<void(const FInterchangeMeshGeometry&)> IterationLambda) const;

	/**
	* Return true if there is an existing FInterchangeMeshInstance that matches the MeshInstanceUid key.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	UE_API bool IsValidMeshInstanceUid(const FString& MeshInstanceUid) const;

	/**
	* Get the instanced mesh from the unique ID.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	UE_API const FInterchangeMeshInstance& GetMeshInstanceByUid(const FString& MeshInstanceUid) const;

	/**
	* Return true if there is an existing FInterchangeMeshGeometry that matches the MeshInstanceUid key.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	UE_API bool IsValidMeshGeometryUid(const FString& MeshGeometryUid) const;

	/**
	* Get the geometry mesh from the unique ID.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	UE_API const FInterchangeMeshGeometry& GetMeshGeometryByUid(const FString& MeshGeometryUid) const;

	/**
	* Get all instanced mesh UIDs that use the mesh geometry unique ID.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	UE_API void GetAllMeshInstanceUidsUsingMeshGeometryUid(const FString& MeshGeometryUid, TArray<FString>& MeshInstanceUids) const;

	/**
	* Iterate over all instanced mesh UIDs that use the mesh geometry unique ID.
	*/
	UE_API void IterateAllMeshInstanceUsingMeshGeometry(const FString& MeshGeometryUid, TFunctionRef<void(const FInterchangeMeshInstance&)> IterationLambda) const;

	/**
	* Return a list of skinned FInterchangeMeshInstance UIDs that can be combined together.
	* We cannot create a skinned mesh that has multiple skeleton root nodes. This function returns a combined MeshInstance per skeleton roots.
	* 
	* @Param bUseSingleBoneForConvertedMeshes - If true, the skeleton roots assigned to non-skinned meshes will be using a single bone. Otherwise, the skeleton will follow the hierarchy scene node hierarchy.
	* @Param bVisibleOnly - If true, invisible meshes will be ignored
	*/
	UE_API void GetCombinedSkinnedMeshInstances(TMap<FString, TArray<FString>>& OutMeshInstanceUidsPerSkeletonRootUid,
		bool bUseSingleBoneForConvertedMeshes = false, bool bVisibleOnly = false) const;
	
	/**
	* Return the skeleton root node UID. This is the UID for a UInterchangeSceneNode that has a "Joint" specialized type.
	* Return an empty string if the MeshInstanceUid parameter points to nothing.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	UE_API FString GetMeshInstanceSkeletonRootUid(const FString& MeshInstanceUid) const;

	UE_API FString GetMeshInstanceSkeletonRootUid(const FInterchangeMeshInstance& MeshInstance) const;

	/**
	* Return the skeleton root node UID. This is the UID for a UInterchangeSceneNode that has a "Joint" specialized type.
	* Return an empty string if the MeshGeometryUid parameter points to nothing.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	UE_API FString GetMeshGeometrySkeletonRootUid(const FString& MeshGeometryUid) const;

	UE_API FString GetMeshGeometrySkeletonRootUid(const FInterchangeMeshGeometry& MeshGeometry) const;

	/**
    * Returns true if there are nanite assembly mesh-to-mesh dependencies specified by the input meshes.
    */
	UE_API bool HasAssemblyMeshDependencies() const;

	/**
	* Returns true if the given mesh uid is the base or part mesh of a nanite assembly.
	*/
	UE_API bool IsAssemblyMeshUid(const FString& MeshUid) const;


	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline | Meshes")
	void SetContext(const FInterchangePipelineMeshesUtilitiesContext& Context) const
	{
		CurrentDataContext = Context;
	}

protected:
	TMap<FString, FInterchangeMeshGeometry> MeshGeometriesPerMeshUid;
	TMap<FString, FInterchangeMeshInstance> MeshInstancesPerMeshInstanceUid;
	TMap<FString, FString> SkeletonRootUidPerMeshUid;
	TSet<FString> AssemblyMeshUids;

	UInterchangeBaseNodeContainer* BaseNodeContainer;

	mutable FInterchangePipelineMeshesUtilitiesContext CurrentDataContext;
};

namespace UE::Interchange::MeshesUtilities
{

	/**
	 * Applies material slot dependencies stored in SlotMaterialDependencies to FactoryNode.
	 * If the caller want to support bKeepSectionSeparate feature it must provide a valid ExistingSlotMaterialDependenciesPtr.
	 */
	template<class T>
	void ApplySlotMaterialDependencies(T& FactoryNode
		, const TMap<FString, FString>& SlotMaterialDependencies
		, const UInterchangeBaseNodeContainer& NodeContainer
		, TMap<FString, FString> *ExistingSlotMaterialDependenciesPtr)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::MeshesUtilities::ApplySlotMaterialDependencies);

		bool bKeepSectionsSeparate = false;
		int32 IndexCounter = 0; //Only use when bKeepSectionsSeparate is true
		if (ExistingSlotMaterialDependenciesPtr)
		{
			FactoryNode.GetCustomKeepSectionsSeparate(bKeepSectionsSeparate);
			IndexCounter = ExistingSlotMaterialDependenciesPtr->Num();
		}
		for (const TPair<FString, FString>& SlotMaterialDependency : SlotMaterialDependencies)
		{
			FString NewSlotName = SlotMaterialDependency.Key;
			if (bKeepSectionsSeparate && ExistingSlotMaterialDependenciesPtr)
			{
				if (ExistingSlotMaterialDependenciesPtr->Contains(NewSlotName))
				{
					NewSlotName += TEXT("_Section") + FString::FromInt(IndexCounter);
				}
				ExistingSlotMaterialDependenciesPtr->Add(NewSlotName, SlotMaterialDependency.Value);
				IndexCounter++;
			}
			// If the dependency value corresponds to a translated node, convert it to the matching factory node UID.
			// Otherwise pass it through as-is (it may be a content path to an existing material)
			if (NodeContainer.GetNode(SlotMaterialDependency.Value))
			{
				const FString MaterialFactoryNodeUid = UInterchangeBaseMaterialFactoryNode::GetMaterialFactoryNodeUidFromMaterialNodeUid(SlotMaterialDependency.Value);
				FactoryNode.SetSlotMaterialDependencyUid(NewSlotName, MaterialFactoryNodeUid);
				if (UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode = Cast<UInterchangeBaseMaterialFactoryNode>(NodeContainer.GetFactoryNode(MaterialFactoryNodeUid)))
				{
					bool IsMaterialImportEnabled = true;
					MaterialFactoryNode->GetCustomIsMaterialImportEnabled(IsMaterialImportEnabled);
					MaterialFactoryNode->SetEnabled(IsMaterialImportEnabled);

					// Create a factory dependency so Material asset are imported before the static mesh asset
					FactoryNode.AddFactoryDependencyUid(MaterialFactoryNodeUid);
				}
			}
			else
			{
				// Content path to an existing material.
				// Preload it so that it can be found via ResolveObject from background threads later
				if (IsInGameThread())
				{
					FSoftObjectPath ContentPath{SlotMaterialDependency.Value};
					if (ContentPath.IsValid())
					{
						ContentPath.TryLoad();
					}
				}
				FactoryNode.SetSlotMaterialDependencyUid(NewSlotName, SlotMaterialDependency.Value);
			}
		}

		if (ExistingSlotMaterialDependenciesPtr)
		{
			ExistingSlotMaterialDependenciesPtr->Append(SlotMaterialDependencies);
		}
	}

	template<class T>
	void ReorderSlotMaterialDependencies(T& FactoryNode, const UInterchangeBaseNodeContainer& NodeContainer)
	{
		TMap<FString, FString> SlotMaterialDependencies;
		FactoryNode.GetSlotMaterialDependencies(SlotMaterialDependencies);

		//Empty all slot dependencies, they will be added backin the correct order
		FactoryNode.ResetSlotMaterialDependencies();

		struct FOrderHelper
		{
			int32 Index;
			FString Name;
			FOrderHelper(int32 InIndex, const FString& InName)
				: Index(InIndex)
				, Name(InName)
			{

			}
		};

		TArray<FOrderHelper> KeyReorder;
		KeyReorder.Reserve(SlotMaterialDependencies.Num());

		TArray<FString> MissingSuffixMaterialNames;
		MissingSuffixMaterialNames.Reserve(SlotMaterialDependencies.Num());
		//Reorder material using the skinXX workflow
		for (const TPair<FString, FString>& SlotMaterialDependency : SlotMaterialDependencies)
		{
			const FString& MaterialName = SlotMaterialDependency.Key;
			if (MaterialName.Len() > 6)
			{
				int32 Offset = MaterialName.Find(TEXT("_SKIN"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
				if (Offset != INDEX_NONE)
				{
					// Chop off the material name so we are left with the number in _SKINXX
					FString SkinXXNumber = MaterialName.Right(MaterialName.Len() - (Offset + 1)).RightChop(4);
					if (SkinXXNumber.IsNumeric())
					{
						int32 TmpIndex = FPlatformString::Atoi(*SkinXXNumber);
						if (TmpIndex >= 0)
						{
							KeyReorder.Add(FOrderHelper(TmpIndex, MaterialName));
							continue;
						}
					}
				}
			}
			
			MissingSuffixMaterialNames.Add(MaterialName);
		}

		KeyReorder.Sort([](const FOrderHelper& A, const FOrderHelper& B)
			{
				if (A.Index == B.Index)
				{
					return A.Name < B.Name;
				}
				else
				{
					return A.Index < B.Index;
				}
			});

		//The map is MaterialName, MaterialUid
		TMap<FString, FString> ReorderedSlotMaterialDependencies;
		ReorderedSlotMaterialDependencies.Reserve(SlotMaterialDependencies.Num());

		for (const FOrderHelper& Element : KeyReorder)
		{
			const FString& SlotMaterialUid = SlotMaterialDependencies.FindChecked(Element.Name);
			ReorderedSlotMaterialDependencies.Add(Element.Name, SlotMaterialUid);
		}

		//Add the missing suffix material
		for (const FString& MaterialName : MissingSuffixMaterialNames)
		{
			const FString& SlotMaterialUid = SlotMaterialDependencies.FindChecked(MaterialName); 
			ReorderedSlotMaterialDependencies.Add(MaterialName, SlotMaterialUid);
		}

		check(ReorderedSlotMaterialDependencies.Num() == SlotMaterialDependencies.Num());

		for (const TPair<FString, FString>& SlotMaterialDependency : ReorderedSlotMaterialDependencies)
		{
			FactoryNode.SetSlotMaterialDependencyUid(SlotMaterialDependency.Key, SlotMaterialDependency.Value);
		}
	}
}

#undef UE_API
