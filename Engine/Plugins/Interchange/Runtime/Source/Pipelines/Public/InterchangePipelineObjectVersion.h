// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Guid.h"

struct FFortniteMainInterchangePipelineObjectVersion
{
	// Not instantiable.
	FFortniteMainInterchangePipelineObjectVersion() = delete;

	enum Type
	{
		InitialVersion,

		// Interchange Move generic animation pipeline property bAddCurveMetadataToSkeleton to the generic shared skeletalmesh and animation pipeline
		InterchangeAddCurveMetadataToSkeletonPropertyMove,

		InterchangeAddFrameAlignmentGenericAnimationPipelineConfig,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	INTERCHANGEPIPELINES_API const static FGuid GUID;
};
