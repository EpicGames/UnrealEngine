// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangePipelineBase.h"
#include "InterchangeSourceData.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeSparseVolumeTexturePipeline.generated.h"

#define UE_API INTERCHANGEPIPELINES_API

class UInterchangeTextureFactoryNode;
class UInterchangeTextureNode;

UCLASS(MinimalAPI, BlueprintType, editinlinenew)
class UInterchangeSparseVolumeTexturePipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

public:
	static UE_API FString GetPipelineCategory(UClass* AssetClass);

	UPROPERTY(
		EditAnywhere,
		BlueprintReadWrite,
		Category = "Volumes",
		meta = (StandAlonePipelineProperty = "True", PipelineInternalEditionData = "True")
	)
	FString PipelineDisplayName;

	/** If enabled, imports all sparse volume texture assets found in the source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Volumes")
	bool bImportSparseVolumeTextures = true;

	/**
	 * If enabled, will attempt to import volume nodes corresponding to numbered files in the same folder as individual frames
	 * of an animated SparseVolumeTextures.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Volumes")
	bool bImportAnimatedSparseVolumeTextures = true;

	/** If set, and there is only one asset and one source, the imported asset will be given this name. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Volumes", meta = (StandAlonePipelineProperty = "True"))
	FString AssetName;

public:
	UE_API virtual void AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams) override;
	UE_API virtual void ExecutePipeline(
		UInterchangeBaseNodeContainer* InBaseNodeContainer,
		const TArray<UInterchangeSourceData*>& InSourceDatas,
		const FString& ContentBasePath
	) override;
#if WITH_EDITOR
	UE_API virtual bool IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const override;
	UE_API virtual void FilterPropertiesFromTranslatedData(UInterchangeBaseNodeContainer* InBaseNodeContainer) override;
	UE_API virtual void GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const override;
#endif	  // WITH_EDITOR

private:
	UPROPERTY()
	TObjectPtr<UInterchangeBaseNodeContainer> BaseNodeContainer;
};

#undef UE_API
