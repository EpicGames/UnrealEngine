// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InterchangePipelineBase.h"
#include "InterchangeSourceData.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "Engine/DeveloperSettings.h"

#include "InterchangeglTFPipeline.generated.h"

#define UE_API INTERCHANGEPIPELINES_API

class UInterchangeBaseNodeContainer;
class UInterchangeMaterialInstanceFactoryNode;
class UInterchangeShaderGraphNode;

UCLASS(MinimalAPI, config = Interchange, meta = (DisplayName = "glTF Settings", ToolTip = "Interchange settings for glTF conversions."))
class UGLTFPipelineSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, config, Category = "PredefinedglTFMaterialLibrary", meta = (DisplayName = "Predefined glTF Material Library"))
	TMap<FString, FSoftObjectPath> MaterialParents;

	UE_API TArray<FString> ValidateMaterialInstancesAndParameters() const;

	UE_API UInterchangeMaterialInstanceFactoryNode* BuildMaterialInstance(UInterchangeBaseNodeContainer* NodeContainer, const UInterchangeShaderGraphNode* ShaderGraphNode, const FString& OldFactoryNodeUId);

	bool IsMaterialParentsEditible() { return bMaterialParentsEditible; }
	void SetMaterialParentsEditible(bool bEditible) { bMaterialParentsEditible = bEditible; }
private:
	UE_API TSet<FString> GenerateExpectedParametersList(const FString& Identifier) const;

	static UE_API const TArray<FString> ExpectedMaterialInstanceIdentifiers; //Default MaterialInstance' identifiers

	//Helper for the Settings Customizer to decide if the ParentMaterials should be editable or not.
	//Should be editable from Project Settings, and should NOT be editable from the Import.
	//It is set by the Pipeline's Customizer.
	//Equals to Pipeline->CanEditPropertiesStates()
	bool bMaterialParentsEditible = true;
};

UCLASS(MinimalAPI, BlueprintType)
class UInterchangeGLTFPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

	UE_API UInterchangeGLTFPipeline();

public:
	
	/** The name of the pipeline that will be display in the import dialog. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common", meta = (StandAlonePipelineProperty = "True", PipelineInternalEditionData = "True"))
	FString PipelineDisplayName;

	TObjectPtr<UGLTFPipelineSettings> GLTFPipelineSettings;

protected:
	UE_API virtual void AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams) override;
	UE_API virtual void ExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath) override;

	virtual bool CanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask) override
	{
		// This pipeline creates UObjects and assets. Not safe to execute outside of main thread.
		return true;
	}
};

#undef UE_API
